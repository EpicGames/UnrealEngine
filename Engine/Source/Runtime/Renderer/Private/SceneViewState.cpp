// Copyright Epic Games, Inc. All Rights Reserved.

/*=======================================================================================
	SceneViewState.cpp: FSceneViewState and GetGPUSizeBytes implementation functions.
=======================================================================================*/

#include "SceneViewState.h"
#include "Lumen/LumenSceneData.h"
#include "LogRenderer.h"
#include "ScenePrivate.h"
#include "LightComponentId.h"
#include "SceneViewStateSystemMemory.h"
#include "VirtualShadowMaps/VirtualShadowMapCacheManager.h"
#include "RenderTargetPool.h"
#include "Algo/RemoveIf.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/Package.h"

DECLARE_GPU_STAT_NAMED(SystemMemoryBackup, TEXT("SystemMemoryBackup"));
DECLARE_GPU_STAT_NAMED(SystemMemoryRestore, TEXT("SystemMemoryRestore"));

static uint64 GetTextureGPUSizeBytes(const FTextureRHIRef& Target, bool bLogSizes)
{
	uint64 Size = Target.IsValid() ? Target->GetDesc().CalcMemorySizeEstimate() : 0;
	if (bLogSizes && Size)
	{
		UE_LOGF(LogRenderer, Log, "LogSizes\tTexture\t0x%p\t%ls\t%llu", Target.GetReference(), *Target->GetName().ToString(), Size);
	}
	return Size;
}

static uint64 GetRenderTargetGPUSizeBytes(const TRefCountPtr<IPooledRenderTarget>& Target, bool bLogSizes)
{
	uint64 Size = Target.IsValid() ? Target->ComputeMemorySize() : 0;
	if (bLogSizes && Size)
	{
		UE_LOGF(LogRenderer, Log, "LogSizes\tRenderTarget\t0x%p\t%ls\t%llu", Target.GetReference(), Target->GetDesc().DebugName, Size);
	}
	return Size;
}

static uint64 GetBufferGPUSizeBytes(const TRefCountPtr<FRDGPooledBuffer>& Buffer, bool bLogSizes)
{
	uint64 Size = Buffer.IsValid() ? Buffer->GetSize() : 0;
	if (bLogSizes && Size)
	{
		const TCHAR* Name = Buffer->GetName();
		UE_LOGF(LogRenderer, Log, "LogSizes\tBuffer\t0x%p\t%ls\t%llu", Buffer.GetReference(), Name ? Name : TEXT("UNKNOWN"), Size);
	}
	return Size;
}

static uint64 GetGPUSizeBytes(const TRefCountPtr<IPooledRenderTarget>& Target, bool bLogSizes)
{
	uint64 Size = Target.IsValid() ? Target->ComputeMemorySize() : 0;
	if (bLogSizes && Size)
	{
		UE_LOGF(LogRenderer, Log, "LogSizes\tRenderTarget\t0x%p\t%ls\t%llu", Target.GetReference(), Target->GetDesc().DebugName, Size);
	}
	return Size;
}

static uint64 GetGPUSizeBytes(const TRefCountPtr<FRDGPooledBuffer>& Buffer, bool bLogSizes)
{
	uint64 Size = Buffer.IsValid() ? Buffer->GetSize() : 0;
	if (bLogSizes && Size)
	{
		const TCHAR* Name = Buffer->GetName();
		UE_LOGF(LogRenderer, Log, "LogSizes\tBuffer\t0x%p\t%ls\t%llu", Buffer.GetReference(), Name ? Name : TEXT("UNKNOWN"), Size);
	}
	return Size;
}

static uint64 GetTextureReadbackGPUSizeBytes(const FRHIGPUTextureReadback* TextureReadback, bool bLogSizes)
{
	uint64 Size = TextureReadback ? TextureReadback->GetGPUSizeBytes() : 0;
	if (bLogSizes && Size)
	{
		UE_LOGF(LogRenderer, Log, "LogSizes\tTextureReadback\t0x%p\t%ls\t%llu", TextureReadback, *TextureReadback->GetName().ToString(), Size);
	}
	return Size;
}

static uint64 GetBufferReadbackGPUSizeBytes(const FRHIGPUBufferReadback* BufferReadback, bool bLogSizes)
{
	uint64 Size = BufferReadback ? BufferReadback->GetGPUSizeBytes() : 0;
	if (bLogSizes && Size)
	{
		UE_LOGF(LogRenderer, Log, "LogSizes\tBufferReadback\t%p\t%ls\t%llu", BufferReadback, *BufferReadback->GetName().ToString(), Size);
	}
	return Size;
}

uint64 FHZBOcclusionTester::GetGPUSizeBytes(bool bLogSizes) const
{
	return ResultsReadback.IsValid() ? GetTextureReadbackGPUSizeBytes(ResultsReadback.Get(), bLogSizes) : 0;
}

uint64 FPersistentSkyAtmosphereData::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize = 0;
	for (int32 VolumeIndex = 0; VolumeIndex < UE_ARRAY_COUNT(CameraAerialPerspectiveVolumes); VolumeIndex++)
	{
		TotalSize += GetRenderTargetGPUSizeBytes(CameraAerialPerspectiveVolumes[VolumeIndex], bLogSizes);
		TotalSize += GetRenderTargetGPUSizeBytes(CameraAerialPerspectiveVolumesMieOnly[VolumeIndex], bLogSizes);
		TotalSize += GetRenderTargetGPUSizeBytes(CameraAerialPerspectiveVolumesRayOnly[VolumeIndex], bLogSizes);
	}
	return TotalSize;
}

uint64 FSceneViewState::FEyeAdaptationManager::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize = 0;
	for (int32 BufferIndex = 0; BufferIndex < UE_ARRAY_COUNT(ExposureBufferData); BufferIndex++)
	{
		TotalSize += GetBufferGPUSizeBytes(ExposureBufferData[BufferIndex], bLogSizes);
	}
	for (FRHIGPUBufferReadback* ReadbackBuffer : ExposureReadbackBuffers)
	{
		TotalSize += GetBufferReadbackGPUSizeBytes(ReadbackBuffer, bLogSizes);
	}
	return TotalSize;
}

uint64 FTemporalAAHistory::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize = 0;
	for (int32 TargetIndex = 0; TargetIndex < kRenderTargetCount; TargetIndex++)
	{
		TotalSize += GetRenderTargetGPUSizeBytes(RT[TargetIndex], bLogSizes);
	}
	return TotalSize;
}

uint64 FTSRHistory::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize =
		GetRenderTargetGPUSizeBytes(ColorArray, bLogSizes) +
		GetRenderTargetGPUSizeBytes(MetadataArray, bLogSizes) +
		GetRenderTargetGPUSizeBytes(GuideArray, bLogSizes) +
		GetRenderTargetGPUSizeBytes(MoireArray, bLogSizes);

	if (CoverageArray.IsValid())
	{
		TotalSize += GetRenderTargetGPUSizeBytes(CoverageArray, bLogSizes);
	}

	for (const TRefCountPtr <IPooledRenderTarget>& Texture : DistortingDisplacementTextures)
	{
		if (Texture.IsValid())
		{
			TotalSize += GetRenderTargetGPUSizeBytes(Texture, bLogSizes);
		}
	}

	return TotalSize;
}

uint64 FScreenSpaceDenoiserHistory::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize = 0;
	for (int32 TargetIndex = 0; TargetIndex < RTCount; TargetIndex++)
	{
		TotalSize += GetRenderTargetGPUSizeBytes(RT[TargetIndex], bLogSizes);
	}
	TotalSize += GetRenderTargetGPUSizeBytes(TileClassification, bLogSizes);
	return TotalSize;
}

uint64 FPreviousViewInfo::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize =
		GetRenderTargetGPUSizeBytes(DepthBuffer, bLogSizes) +
		GetRenderTargetGPUSizeBytes(GBufferA, bLogSizes) +
		GetRenderTargetGPUSizeBytes(GBufferB, bLogSizes) +
		GetRenderTargetGPUSizeBytes(GBufferC, bLogSizes) +
		GetRenderTargetGPUSizeBytes(HZB, bLogSizes) +
		GetRenderTargetGPUSizeBytes(NaniteHZB, bLogSizes) +
		GetRenderTargetGPUSizeBytes(DistortingDisplacementTexture, bLogSizes) +
		GetRenderTargetGPUSizeBytes(CompressedDepthViewNormal, bLogSizes) +
		GetRenderTargetGPUSizeBytes(CompressedOpaqueDepth, bLogSizes) +
		GetRenderTargetGPUSizeBytes(CompressedOpaqueShadingModel, bLogSizes) +
		GetRenderTargetGPUSizeBytes(ScreenSpaceRayTracingInput, bLogSizes) +
		TemporalAAHistory.GetGPUSizeBytes(bLogSizes) +
		TSRHistory.GetGPUSizeBytes(bLogSizes) +
		GetRenderTargetGPUSizeBytes(HalfResTemporalAAHistory, bLogSizes) +
		DOFSetupHistory.GetGPUSizeBytes(bLogSizes) +
		SSRHistory.GetGPUSizeBytes(bLogSizes) +
		WaterSSRHistory.GetGPUSizeBytes(bLogSizes) +
		RoughRefractionHistory.GetGPUSizeBytes(bLogSizes) +
		FSSSHistory.GetGPUSizeBytes(bLogSizes) +
		HairHistory.GetGPUSizeBytes(bLogSizes) +
#if UE_ENABLE_DEBUG_DRAWING
		CompositePrimitiveDepthHistory.GetGPUSizeBytes(bLogSizes) +
#endif
		CustomSSRInput.GetGPUSizeBytes(bLogSizes) +
		ReflectionsHistory.GetGPUSizeBytes(bLogSizes) +
		WaterReflectionsHistory.GetGPUSizeBytes(bLogSizes) +
		AmbientOcclusionHistory.GetGPUSizeBytes(bLogSizes) +
		GetRenderTargetGPUSizeBytes(GTAOHistory.RT, bLogSizes) +
		DiffuseIndirectHistory.GetGPUSizeBytes(bLogSizes) +
		SkyLightHistory.GetGPUSizeBytes(bLogSizes) +
		ReflectedSkyLightHistory.GetGPUSizeBytes(bLogSizes) +
		PolychromaticPenumbraHarmonicsHistory.GetGPUSizeBytes(bLogSizes) +
		GetRenderTargetGPUSizeBytes(MobileBloomSetup_EyeAdaptation, bLogSizes) +
		GetRenderTargetGPUSizeBytes(VisualizeMotionVectors, bLogSizes);

	for (const TPair<FLightComponentId, TSharedPtr<FScreenSpaceDenoiserHistory>>& ShadowHistoryIt : ShadowHistories)
	{
		if (ShadowHistoryIt.Value.IsValid())
		{
			TotalSize += ShadowHistoryIt.Value->GetGPUSizeBytes(bLogSizes);
		}
	}

	if (ThirdPartyTemporalUpscalerHistory)
	{
		TotalSize += ThirdPartyTemporalUpscalerHistory->GetGPUSizeBytes();
		if (bLogSizes)
		{
			UE_LOGF(LogRenderer, Log, "LogSizes\tThirdPartyTemporalUpscaler\t%ls\t%llu", ThirdPartyTemporalUpscalerHistory->GetDebugName(), ThirdPartyTemporalUpscalerHistory->GetGPUSizeBytes());
		}
	}

	if (MegaFiltersHistory)
	{
		TotalSize += MegaFiltersHistory->GetGPUSizeBytes();
		if (bLogSizes)
		{
			UE_LOGF(LogRenderer, Log, "LogSizes\tMegaFiltersHistory\t%ls\t%llu", MegaFiltersHistory->GetDebugName(), MegaFiltersHistory->GetGPUSizeBytes());
		}
	}

	return TotalSize;
}

/** FLumenViewState GPU size queries */
uint64 FScreenProbeGatherTemporalState::GetGPUSizeBytes(bool bLogSizes) const
{
	return
		GetRenderTargetGPUSizeBytes(DiffuseIndirectHistoryRT, bLogSizes) +
		GetRenderTargetGPUSizeBytes(RoughSpecularIndirectHistoryRT, bLogSizes) +
		GetRenderTargetGPUSizeBytes(FastUpdateMode_NumFramesAccumulatedHistoryRT, bLogSizes) +
		GetRenderTargetGPUSizeBytes(HistoryScreenProbeSceneDepth, bLogSizes) +
		GetRenderTargetGPUSizeBytes(HistoryScreenProbeTranslatedWorldPosition, bLogSizes) +
		GetRenderTargetGPUSizeBytes(ProbeHistoryScreenProbeRadiance, bLogSizes) +
		GetRenderTargetGPUSizeBytes(ImportanceSamplingHistoryScreenProbeRadiance, bLogSizes);
}

uint64 FReflectionTemporalState::GetGPUSizeBytes(bool bLogSizes) const
{
	return
		GetRenderTargetGPUSizeBytes(SpecularAndSecondMomentHistory, bLogSizes) +
		GetRenderTargetGPUSizeBytes(NumFramesAccumulatedHistory, bLogSizes);
}

uint64 FRadianceCacheState::GetGPUSizeBytes(bool bLogSizes) const
{
	return
		GetRenderTargetGPUSizeBytes(RadianceProbeIndirectionTexture, bLogSizes) +
		GetRenderTargetGPUSizeBytes(RadianceProbeAtlasTexture, bLogSizes) +
		GetRenderTargetGPUSizeBytes(SkyVisibilityProbeAtlasTexture, bLogSizes) +
		GetRenderTargetGPUSizeBytes(FinalRadianceAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(FinalSkyVisibilityAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(FinalIrradianceAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(ProbeOcclusionAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(DepthProbeAtlasTexture, bLogSizes) +
		GetBufferGPUSizeBytes(ProbeAllocator, bLogSizes) +
		GetBufferGPUSizeBytes(ProbeFreeListAllocator, bLogSizes) +
		GetBufferGPUSizeBytes(ProbeFreeList, bLogSizes) +
		GetBufferGPUSizeBytes(ProbeLastUsedFrame, bLogSizes) +
		GetBufferGPUSizeBytes(ProbeLastTracedFrame, bLogSizes) +
		GetBufferGPUSizeBytes(ProbeWorldOffset, bLogSizes) +
		GetBufferGPUSizeBytes(ProbeInterpolationMisses, bLogSizes) +
		GetBufferGPUSizeBytes(ProbeAdaptiveIndices, bLogSizes);
}

uint64 FLumenViewState::GetGPUSizeBytes(bool bLogSizes) const
{
	return
		ScreenProbeGatherState.GetGPUSizeBytes(bLogSizes) +
		ReflectionState.GetGPUSizeBytes(bLogSizes) +
		TranslucentReflectionState.GetGPUSizeBytes(bLogSizes) +
		WaterReflectionState.GetGPUSizeBytes(bLogSizes) +
		GetRenderTargetGPUSizeBytes(TranslucencyVolume0, bLogSizes) +
		GetRenderTargetGPUSizeBytes(TranslucencyVolume1, bLogSizes) +
		RadianceCacheState.GetGPUSizeBytes(bLogSizes) +
		TranslucencyVolumeRadianceCacheState.GetGPUSizeBytes(bLogSizes);
}

/** FLumenSceneData GPU size queries */
uint64 FLumenSurfaceCacheFeedback::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize = 0;
	for (const FRHIGPUBufferReadback* ReadbackBuffer : ReadbackBuffers)
	{
		TotalSize += GetBufferReadbackGPUSizeBytes(ReadbackBuffer, bLogSizes);
	}
	return TotalSize;
}

uint64 FLumenSceneData::GetGPUSizeBytes(bool bLogSizes) const
{
	return
		GetBufferGPUSizeBytes(CardBuffer, bLogSizes) +
		CardUploadBuffer.GetNumBytes() +
		GetBufferGPUSizeBytes(MeshCardsBuffer, bLogSizes) +
		MeshCardsUploadBuffer.GetNumBytes() +
		GetBufferGPUSizeBytes(HeightfieldBuffer, bLogSizes) +
		HeightfieldUploadBuffer.GetNumBytes() +
		GetBufferGPUSizeBytes(SceneInstanceIndexToMeshCardsIndexBuffer, bLogSizes) +
		SceneInstanceIndexToMeshCardsIndexUploadBuffer.GetNumBytes() +
		GetBufferGPUSizeBytes(CardPageBuffer, bLogSizes) +
		CardPageUploadBuffer.GetNumBytes() +
		GetBufferGPUSizeBytes(CardPageLastUsedBuffer, bLogSizes) +
		GetBufferGPUSizeBytes(CardPageHighResLastUsedBuffer, bLogSizes) +
		GetRenderTargetGPUSizeBytes(AlbedoAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(NormalAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(EmissiveAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(DepthAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(DirectLightingAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(DiffuseLightingAndSecondMomentHistoryAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(NumFramesAccumulatedHistoryAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(IndirectLightingAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(RadiosityNumFramesAccumulatedAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(FinalLightingAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(RadiosityTraceRadianceAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(RadiosityTraceHitDistanceAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(RadiosityProbeSHRedAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(RadiosityProbeSHGreenAtlas, bLogSizes) +
		GetRenderTargetGPUSizeBytes(RadiosityProbeSHBlueAtlas, bLogSizes) +
		SurfaceCacheFeedback.GetGPUSizeBytes(bLogSizes) +
		GetBufferGPUSizeBytes(PageTableBuffer, bLogSizes) +
		PageTableUploadBuffer.GetNumBytes();
}

uint64 FMegaLightsViewState::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 Out = 0;
	Out += GBuffer.GetGPUSizeBytes(bLogSizes);
	Out += HairStrands.GetGPUSizeBytes(bLogSizes);
	Out += FrontLayerTranslucency.GetGPUSizeBytes(bLogSizes);
	Out += GetBufferGPUSizeBytes(PrevLightPower, bLogSizes);
	return Out;
}

uint64 FMegaLightsViewState::FResources::GetGPUSizeBytes(bool bLogSizes) const
{
	return
		GetRenderTargetGPUSizeBytes(DiffuseLightingHistory, bLogSizes)
		+ GetRenderTargetGPUSizeBytes(SpecularLightingHistory, bLogSizes)
		+ GetRenderTargetGPUSizeBytes(LightingMomentsHistory, bLogSizes)
		+ GetRenderTargetGPUSizeBytes(NumFramesAccumulatedHistory, bLogSizes)
		+ GetBufferGPUSizeBytes(VisibleLightHashHistory, bLogSizes)
		+ GetBufferGPUSizeBytes(HiddenLightHashHistory, bLogSizes)
		+ GetBufferGPUSizeBytes(VolumeVisibleLightHashHistory, bLogSizes)
		+ GetBufferGPUSizeBytes(VolumeHiddenLightHashHistory, bLogSizes)
		+ GetBufferGPUSizeBytes(TranslucencyVolume0VisibleLightHashHistory, bLogSizes)
		+ GetBufferGPUSizeBytes(TranslucencyVolume1VisibleLightHashHistory, bLogSizes)
		+ GetBufferGPUSizeBytes(TranslucencyVolume0HiddenLightHashHistory, bLogSizes)
		+ GetBufferGPUSizeBytes(TranslucencyVolume1HiddenLightHashHistory, bLogSizes)
		+ GetBufferGPUSizeBytes(FrozenLightData, bLogSizes)
		+ GetBufferGPUSizeBytes(GeneralPurposeState, bLogSizes);
}

uint64 FStochasticLightingViewState::GetGPUSizeBytes(bool bLogSizes) const
{
	return
		GetRenderTargetGPUSizeBytes(SceneDepthHistory, bLogSizes) +
		GetRenderTargetGPUSizeBytes(SceneNormalHistory, bLogSizes) +
		GetRenderTargetGPUSizeBytes(SceneClosureHistory, bLogSizes) +
		GetRenderTargetGPUSizeBytes(FrontLayerTranslucencyDepthHistory, bLogSizes) +
		GetRenderTargetGPUSizeBytes(FrontLayerTranslucencyNormalHistory, bLogSizes);
}

uint64 FTranslucencyLightingViewState::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize = 0;

	for (int32 Index = 0; Index < TVC_MAX; Index++)
	{
		TotalSize += GetRenderTargetGPUSizeBytes(HistoryAmbient[Index], bLogSizes);
		TotalSize += GetRenderTargetGPUSizeBytes(HistoryDirectional[Index], bLogSizes);

		TotalSize += GetRenderTargetGPUSizeBytes(HistoryMark[Index], bLogSizes);
		TotalSize += GetRenderTargetGPUSizeBytes(HistorySpecularDirection[Index], bLogSizes);
	}

	return TotalSize;
}

uint64 FPersistentGlobalDistanceFieldData::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize =
		GetBufferGPUSizeBytes(PageFreeListAllocatorBuffer, bLogSizes) +
		GetBufferGPUSizeBytes(PageFreeListBuffer, bLogSizes) +
		GetRenderTargetGPUSizeBytes(PageAtlasTexture, bLogSizes) +
		GetRenderTargetGPUSizeBytes(CoverageAtlasTexture, bLogSizes) +
		GetRenderTargetGPUSizeBytes(PageTableCombinedTexture, bLogSizes) +
		GetRenderTargetGPUSizeBytes(MipTexture, bLogSizes);

	for (int32 GDFIndex = 0; GDFIndex < UE_ARRAY_COUNT(PageTableLayerTextures); GDFIndex++)
	{
		TotalSize += GetRenderTargetGPUSizeBytes(PageTableLayerTextures[GDFIndex], bLogSizes);
	}
	return TotalSize;
}

uint64 FVolumetricRenderTargetViewStateData::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize = 0;
	for (uint32 TargetIndex = 0; TargetIndex < kRenderTargetCount; TargetIndex++)
	{
		TotalSize += GetRenderTargetGPUSizeBytes(VolumetricReconstructRT[TargetIndex], bLogSizes);
		TotalSize += GetRenderTargetGPUSizeBytes(VolumetricReconstructRTDepth[TargetIndex], bLogSizes);
	}
	TotalSize += GetRenderTargetGPUSizeBytes(VolumetricTracingRT, bLogSizes);
	TotalSize += GetRenderTargetGPUSizeBytes(VolumetricTracingRTDepth, bLogSizes);
	return TotalSize;
}

uint64 FTemporalRenderTargetState::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize = 0;
	for (int32 TargetIndex = 0; TargetIndex < UE_ARRAY_COUNT(RenderTargets); TargetIndex++)
	{
		TotalSize += GetRenderTargetGPUSizeBytes(RenderTargets[TargetIndex], bLogSizes);
	}
	return TotalSize;
}

uint64 FGlintShadingLUTsStateData::GetGPUSizeBytes(bool bLogSizes) const
{
	return GetTextureGPUSizeBytes(RHIGlintShadingLUTs, bLogSizes);
}

uint64 FVirtualShadowMapArrayFrameData::GetGPUSizeBytes(bool bLogSizes) const
{
	return
		::GetGPUSizeBytes(PageTable, bLogSizes) +
		::GetGPUSizeBytes(PageFlags, bLogSizes) +
		::GetGPUSizeBytes(ProjectionData, bLogSizes) +
		::GetGPUSizeBytes(UncachedPageRectBounds, bLogSizes) +
		::GetGPUSizeBytes(AllocatedPageRectBounds, bLogSizes);
};

uint64 FVirtualShadowMapArrayCacheManager::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize = PrevBuffers.GetGPUSizeBytes(bLogSizes);
	TotalSize += GetRenderTargetGPUSizeBytes(PhysicalPagePool, bLogSizes);
	TotalSize += GetRenderTargetGPUSizeBytes(HZBPhysicalPagePoolArray, bLogSizes);
	TotalSize += GetBufferGPUSizeBytes(PhysicalPageMetaData, bLogSizes);
	TotalSize += GetBufferGPUSizeBytes(AccumulatedStatsBuffer, bLogSizes);
	TotalSize += GetBufferReadbackGPUSizeBytes(GPUBufferReadback, bLogSizes);
	return TotalSize;
}

uint64 FSceneViewState::GetGPUSizeBytes(bool bLogSizes) const
{
	uint64 TotalSize = 0;

	// Todo, not currently computing GPU memory usage for queries or sampler states.  Are these important?  Should be small...
	//  ShadowOcclusionQueryMaps
	//  OcclusionQueryPool
	//  PrimitiveOcclusionQueryPool
	//  PlanarReflectionOcclusionHistories
	//  MaterialTextureBilinearWrapedSamplerCache
	//  MaterialTextureBilinearClampedSamplerCache

	TotalSize += HZBOcclusionTests.GetGPUSizeBytes(bLogSizes);
	TotalSize += PersistentSkyAtmosphereData.GetGPUSizeBytes(bLogSizes);
	TotalSize += EyeAdaptationManager.GetGPUSizeBytes(bLogSizes);
	TotalSize += GetRenderTargetGPUSizeBytes(CombinedLUTRenderTarget, bLogSizes);
	TotalSize += PrevFrameViewInfo.GetGPUSizeBytes(bLogSizes);
	TotalSize += LightShaftOcclusionHistory.GetGPUSizeBytes(bLogSizes);
	for (const TPair<FLightComponentId, TUniquePtr<FTemporalAAHistory>>& LightShaftBloomIt : LightShaftBloomHistoryRTs)
	{
		if (LightShaftBloomIt.Value.IsValid())
		{
			TotalSize += LightShaftBloomIt.Value->GetGPUSizeBytes(bLogSizes);
		}
	}
	TotalSize += GetRenderTargetGPUSizeBytes(DistanceFieldAOHistoryRT, bLogSizes);
	TotalSize += GetRenderTargetGPUSizeBytes(DistanceFieldIrradianceHistoryRT, bLogSizes);
	TotalSize += GetRenderTargetGPUSizeBytes(SubsurfaceScatteringQualityHistoryRT, bLogSizes);
	TotalSize += Lumen.GetGPUSizeBytes(bLogSizes);
	TotalSize += MegaLights.GetGPUSizeBytes(bLogSizes);
	TotalSize += TranslucencyLighting.GetGPUSizeBytes(bLogSizes);
	TotalSize += GetRenderTargetGPUSizeBytes(BloomFFTKernel.Spectral, bLogSizes);
	TotalSize += GetBufferGPUSizeBytes(BloomFFTKernel.ConstantsBuffer, bLogSizes);
	TotalSize += GetBufferGPUSizeBytes(FilmGrainCache.ConstantsBuffer, bLogSizes);
#if RHI_RAYTRACING
	TotalSize += GetBufferGPUSizeBytes(SkyLightVisibilityRaysBuffer, bLogSizes);
#endif
	TotalSize += GetRenderTargetGPUSizeBytes(LightScatteringHistory, bLogSizes);
	TotalSize += GetRenderTargetGPUSizeBytes(PrevLightScatteringConservativeDepthTexture, bLogSizes);
	if (GlobalDistanceFieldData.IsValid())
	{
		TotalSize += GlobalDistanceFieldData->GetGPUSizeBytes(bLogSizes);
	}
	TotalSize += VolumetricCloudRenderTarget.GetGPUSizeBytes(bLogSizes);
	for (int32 LightIndex = 0; LightIndex < UE_ARRAY_COUNT(VolumetricCloudShadowRenderTarget); LightIndex++)
	{
		TotalSize += VolumetricCloudShadowRenderTarget[LightIndex].GetGPUSizeBytes(bLogSizes);
	}
	TotalSize += GetBufferGPUSizeBytes(HairStrandsViewStateData.VoxelFeedbackBuffer, bLogSizes);
	TotalSize += GetBufferGPUSizeBytes(ShaderPrintStateData.EntryBuffer, bLogSizes);
	TotalSize += GetBufferGPUSizeBytes(ShaderPrintStateData.StateBuffer, bLogSizes);

	// Per-view Lumen scene data is stored in a map in the FScene
	if (Scene && bLumenSceneDataAdded)
	{
		FLumenSceneDataKey ByViewKey = { GetViewKey(), (uint32)INDEX_NONE };
		FLumenSceneData** SceneData = Scene->PerViewOrGPULumenSceneData.Find(ByViewKey);

		if (SceneData)
		{
			TotalSize += (*SceneData)->GetGPUSizeBytes(bLogSizes);
		}
	}

	return TotalSize;
}

void FSceneViewState::AddLumenSceneData(FSceneInterface* InScene, float InSurfaceCacheResolution)
{
	check(InScene);
	if (!Scene)
	{
		Scene = (FScene*)InScene;

		// Modification of scene structure needs to happen on render thread
		ENQUEUE_RENDER_COMMAND(SceneViewStateAdd)(
			[RenderScene = Scene, RenderViewState = this](FRHICommandListBase&)
			{
				RenderScene->ViewStates.Add(RenderViewState);
			});
	}

	if (Scene == InScene && Scene->DefaultLumenSceneData)
	{
		// Don't allocate if one already exists
		if (!bLumenSceneDataAdded)
		{
			bLumenSceneDataAdded = true;
			LumenSurfaceCacheResolution = InSurfaceCacheResolution;

			FLumenSceneData* SceneData = new FLumenSceneData(Scene->DefaultLumenSceneData->bTrackAllPrimitives);
			SceneData->bViewSpecific = true;
			SceneData->SurfaceCacheResolution = FMath::Clamp(InSurfaceCacheResolution, 0.5f, 1.0f);

			// Need to add reference to Lumen scene data in render thread
			ENQUEUE_RENDER_COMMAND(LinkLumenSceneData)(
				[this, SceneData](FRHICommandListBase&)
				{
					SceneData->CopyInitialData(*Scene->DefaultLumenSceneData);

					// Key shouldn't already exist in Scene, because the bLumenSceneDataAdded flag should only allow it to be added once.
					FLumenSceneDataKey ByViewKey = { GetViewKey(), (uint32)INDEX_NONE };
					check(Scene->PerViewOrGPULumenSceneData.Find(ByViewKey) == nullptr);

					Scene->PerViewOrGPULumenSceneData.Emplace(ByViewKey, SceneData);
				});
		}
		else if (LumenSurfaceCacheResolution != InSurfaceCacheResolution)
		{
			LumenSurfaceCacheResolution = InSurfaceCacheResolution;

			ENQUEUE_RENDER_COMMAND(ChangeLumenSceneDataQuality)(
				[this, InSurfaceCacheResolution](FRHICommandListBase&)
				{
					FLumenSceneDataKey ByViewKey = { GetViewKey(), (uint32)INDEX_NONE };
					FLumenSceneData** SceneData = Scene->PerViewOrGPULumenSceneData.Find(ByViewKey);

					check(SceneData);

					(*SceneData)->SurfaceCacheResolution = FMath::Clamp(InSurfaceCacheResolution, 0.5f, 1.0f);
				});
		}
	}
}

void FSceneViewState::RemoveLumenSceneData(FSceneInterface* InScene)
{
	check(InScene);
	if (Scene == InScene && bLumenSceneDataAdded)
	{
		bLumenSceneDataAdded = false;

		ENQUEUE_RENDER_COMMAND(RemoveLumenSceneData)(
			[this](FRHICommandListBase&)
			{
				FLumenSceneDataKey ByViewKey = { GetViewKey(), (uint32)INDEX_NONE };
				FLumenSceneData** SceneData = Scene->PerViewOrGPULumenSceneData.Find(ByViewKey);

				check(SceneData);
				delete* SceneData;

				Scene->PerViewOrGPULumenSceneData.Remove(ByViewKey);
			});
	}
}

bool FSceneViewState::HasLumenSceneData() const
{
	return bLumenSceneDataAdded;
}

static bool SystemMemoryBackupTextureSupported(const FRHITextureDesc& Desc)
{
	// Long term, it might be useful to support array textures and mips, but it would require multiple readbacks.  On high end systems most
	// likely to do very high resolution rendering, system memory limits are hit before GPU memory limits, so it works for now.
	return (Desc.Dimension == ETextureDimension::Texture2D || (Desc.Dimension == ETextureDimension::Texture2DArray && Desc.ArraySize == 1)) && Desc.NumMips == 1 && Desc.NumSamples == 1;
}

static void SystemMemoryBackupTextureBegin(FRHICommandListImmediate& RHICmdList, FSceneViewStateSystemMemoryMirror& SystemMemoryMirror, FSceneViewState& ViewState, TRefCountPtr<IPooledRenderTarget>& Texture)
{
	if (Texture.IsValid())
	{
		FRHITexture* TextureRHI = Texture->GetRHI();

		if (TextureRHI && SystemMemoryBackupTextureSupported(TextureRHI->GetDesc()))
		{
			int64 StructureOffset = (uint8*)&Texture - (uint8*)&ViewState;
			TArray<FSceneViewStateSystemMemoryTexture>& TextureMirrorArray = SystemMemoryMirror.TextureMirrors.FindOrAdd(StructureOffset);

			// Enable Dynamic so staging buffers are cached (except depth stencil textures, which use a PF_R32_FLOAT format intermediate, with flag added to that below).
			FRHITextureDesc Desc = TextureRHI->GetDesc();
			if (Desc.Format != PF_DepthStencil)
			{
				Desc.Flags |= ETextureCreateFlags::Dynamic;
			}

			int32 MatchingIndex;
			for (MatchingIndex = 0; MatchingIndex < TextureMirrorArray.Num(); MatchingIndex++)
			{
				if (TextureMirrorArray[MatchingIndex].Desc == Desc)
				{
					break;
				}
			}

			if (MatchingIndex == TextureMirrorArray.Num())
			{
				FSceneViewStateSystemMemoryTexture& TextureMirror = TextureMirrorArray.AddDefaulted_GetRef();
				TextureMirror.Desc = Desc;
				TextureMirror.DebugName = Texture->GetDesc().DebugName;
				TextureMirror.Readback = MakeUnique<FRHIGPUTextureReadback>(TextureMirror.DebugName);
			}

			FSceneViewStateSystemMemoryTexture& TextureMirror = TextureMirrorArray[MatchingIndex];

			if (Desc.Format == PF_DepthStencil)
			{
				// Depth stencil doesn't support readback -- need to copy through an intermediate float texture.  Also, we are
				// only copying the depth, not stencil, assuming previous frame stencil isn't used.
				FRHITextureDesc TemporaryTextureDesc = TextureMirror.Desc;
				TemporaryTextureDesc.Flags = ETextureCreateFlags::Dynamic;
				TemporaryTextureDesc.Format = PF_R32_FLOAT;

				TRefCountPtr<IPooledRenderTarget> TemporaryTexture;
				GRenderTargetPool.FindFreeElement(RHICmdList, TemporaryTextureDesc, TemporaryTexture, TextureMirror.DebugName);

				// Ensure texture isn't destroyed until commands finish
				SystemMemoryMirror.TemporaryTextures.Add(TemporaryTexture);

				RHICmdList.CopyTexture(Texture->GetRHI(), TemporaryTexture->GetRHI(), {});
				TextureMirror.Readback->EnqueueCopy(RHICmdList, TemporaryTexture->GetRHI(), FIntVector(0, 0, 0), 0, FIntVector(0, 0, 0));
			}
			else
			{
				TextureMirror.Readback->EnqueueCopy(RHICmdList, TextureRHI, FIntVector(0, 0, 0), 0, FIntVector(0, 0, 0));
			}
		}
	}
}

static void SystemMemoryBackupTextureEnd(FRHICommandListImmediate& RHICmdList, FSceneViewStateSystemMemoryMirror& SystemMemoryMirror, FSceneViewState& ViewState, TRefCountPtr<IPooledRenderTarget>& Texture)
{
	if (Texture.IsValid())
	{
		FRHITexture* TextureRHI = Texture->GetRHI();

		if (TextureRHI && SystemMemoryBackupTextureSupported(TextureRHI->GetDesc()))
		{
			int64 StructureOffset = (uint8*)&Texture - (uint8*)&ViewState;
			TArray<FSceneViewStateSystemMemoryTexture>& TextureMirrorArray = SystemMemoryMirror.TextureMirrors.FindOrAdd(StructureOffset);

			// Enable Dynamic so staging buffers are cached (except depth stencil textures, which use a PF_R32_FLOAT format intermediate, with the array element not having the flag set).
			FRHITextureDesc Desc = TextureRHI->GetDesc();
			if (Desc.Format != PF_DepthStencil)
			{
				Desc.Flags |= ETextureCreateFlags::Dynamic;
			}

			int32 MatchingIndex;
			for (MatchingIndex = 0; MatchingIndex < TextureMirrorArray.Num(); MatchingIndex++)
			{
				if (TextureMirrorArray[MatchingIndex].Desc == Desc)
				{
					break;
				}
			}
			check(MatchingIndex < TextureMirrorArray.Num());

			FSceneViewStateSystemMemoryTexture& TextureMirror = TextureMirrorArray[MatchingIndex];
			int32 SrcPitchInPixels;
			void* TextureBuffer = TextureMirror.Readback->Lock(SrcPitchInPixels);

			// Align destination width to block size.  Depth is copied through a 32-bit float temporary.
			EPixelFormat CopyFormat = Desc.Format == PF_DepthStencil ? PF_R32_FLOAT : Desc.Format;
			const FPixelFormatInfo& FormatInfo = GPixelFormats[CopyFormat];
			int32 DestPitchInPixels = (Desc.Extent.X + FormatInfo.BlockSizeX - 1) / FormatInfo.BlockSizeX * FormatInfo.BlockSizeX;

			// Allocate storage
			SIZE_T ImageSize = CalcTextureMipMapSize(DestPitchInPixels, Desc.Extent.Y, CopyFormat, 0);
			TArray<uint8>& ImageBuffer = TextureMirror.Instances.FindOrAdd(ViewState.GetViewKey());
			ImageBuffer.SetNumUninitialized(ImageSize);

			// Compute stride in bytes
			int32 SrcStrideInBytes = SrcPitchInPixels / FormatInfo.BlockSizeX * FormatInfo.BlockBytes;
			int32 DestStrideInBytes = DestPitchInPixels / FormatInfo.BlockSizeX * FormatInfo.BlockBytes;

			CopyTextureData2D(TextureBuffer, ImageBuffer.GetData(), Desc.Extent.Y, CopyFormat, SrcStrideInBytes, DestStrideInBytes);

			TextureMirror.Readback->Unlock();

			Texture = nullptr;
		}
	}
}

// Uses Lock / Unlock, rather than UpdateTexture2D, in case we want to extend the function to support array textures in the future (used by TSR).
// UpdateTexture2D only works on the first array element.
static void SystemMemoryUpdateTexture(FRHICommandListImmediate& RHICmdList, FRHITexture* TextureRHI, const FRHITextureDesc& Desc, const uint8* CopySrc)
{
	FIntPoint Extent = Desc.Extent;
	const FPixelFormatInfo& FormatInfo = GPixelFormats[Desc.Format];

	// Align source width to block size
	uint32 SourcePitchInPixels = (Extent.X + FormatInfo.BlockSizeX - 1) / FormatInfo.BlockSizeX * FormatInfo.BlockSizeX;
	uint32 WidthInBlocks = (Extent.X + FormatInfo.BlockSizeX - 1) / FormatInfo.BlockSizeX;
	uint32 HeightInBlocks = (Extent.Y + FormatInfo.BlockSizeY - 1) / FormatInfo.BlockSizeY;
	uint32 SourcePitchInBytes = SourcePitchInPixels / FormatInfo.BlockSizeX * FormatInfo.BlockBytes;

	FRHILockTextureArgs LockArgs = FRHILockTextureArgs::Lock2D(TextureRHI, 0, EResourceLockMode::RLM_WriteOnly, false, false);
	FRHILockTextureResult LockResult = RHICmdList.LockTexture(LockArgs);

	uint8* CopyDst = (uint8*)LockResult.Data;
	for (uint32 BlockRow = 0; BlockRow < HeightInBlocks; BlockRow++)
	{
		FMemory::Memcpy(CopyDst, CopySrc, WidthInBlocks * FormatInfo.BlockBytes);
		CopySrc += SourcePitchInBytes;
		CopyDst += LockResult.Stride;
	}

	RHICmdList.UnlockTexture(LockArgs);
}

static void SystemMemoryRestoreTexture(FRHICommandListImmediate& RHICmdList, FSceneViewStateSystemMemoryMirror& SystemMemoryMirror, FSceneViewState& ViewState, TRefCountPtr<IPooledRenderTarget>& Texture)
{
	int64 StructureOffset = (uint8*)&Texture - (uint8*)&ViewState;
	TArray<FSceneViewStateSystemMemoryTexture>* TextureMirrorArray = SystemMemoryMirror.TextureMirrors.Find(StructureOffset);

	if (TextureMirrorArray)
	{
		int32 MatchingIndex;
		for (MatchingIndex = 0; MatchingIndex < TextureMirrorArray->Num(); MatchingIndex++)
		{
			if ((*TextureMirrorArray)[MatchingIndex].Instances.Contains(ViewState.GetViewKey()))
			{
				break;
			}
		}

		if (MatchingIndex < TextureMirrorArray->Num())
		{
			const FRHITextureDesc& Desc = (*TextureMirrorArray)[MatchingIndex].Desc;

			GRenderTargetPool.FindFreeElement(RHICmdList, Desc, Texture, (*TextureMirrorArray)[MatchingIndex].DebugName);

			TArray<uint8>* ImageBuffer = (*TextureMirrorArray)[MatchingIndex].Instances.Find(ViewState.GetViewKey());
			check(ImageBuffer);

			if (Desc.Format == PF_DepthStencil)
			{
				// For depth stencil, we only copy depth, assuming previous frame stencil isn't used
				FRHITextureDesc TemporaryTextureDesc = (*TextureMirrorArray)[MatchingIndex].Desc;
				TemporaryTextureDesc.Flags = ETextureCreateFlags::Dynamic;
				TemporaryTextureDesc.Format = PF_R32_FLOAT;

				TRefCountPtr<IPooledRenderTarget> TemporaryTexture;
				GRenderTargetPool.FindFreeElement(RHICmdList, TemporaryTextureDesc, TemporaryTexture, (*TextureMirrorArray)[MatchingIndex].DebugName);

				SystemMemoryUpdateTexture(RHICmdList, TemporaryTexture->GetRHI(), TemporaryTextureDesc, ImageBuffer->GetData());
				RHICmdList.CopyTexture(TemporaryTexture->GetRHI(), Texture->GetRHI(), {});
			}
			else
			{
				SystemMemoryUpdateTexture(RHICmdList, Texture->GetRHI(), Desc, ImageBuffer->GetData());
			}
		}
	}
}

static void SystemMemoryForEachTexture(FRHICommandListImmediate& RHICmdList, FSceneViewStateSystemMemoryMirror& SystemMemoryMirror, FSceneViewState& ViewState,
	void (*TextureFunction)(FRHICommandListImmediate&, FSceneViewStateSystemMemoryMirror&, FSceneViewState&, TRefCountPtr<IPooledRenderTarget>&))
{
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.PrevFrameViewInfo.DepthBuffer);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.PrevFrameViewInfo.GBufferA);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.PrevFrameViewInfo.GBufferB);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.PrevFrameViewInfo.GBufferC);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.PrevFrameViewInfo.DistortingDisplacementTexture);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.PrevFrameViewInfo.CompressedDepthViewNormal);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.PrevFrameViewInfo.CompressedOpaqueDepth);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.PrevFrameViewInfo.CompressedOpaqueShadingModel);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.PrevFrameViewInfo.ScreenSpaceRayTracingInput);

	// NOTE:  not bothering to cache the numerous Temporal AA related render targets from PrevFrameViewInfo, as TAA is not supported with tiled rendering,
	//        which is the use case for system memory mirroring of view state.

	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ScreenProbeGatherState.DiffuseIndirectHistoryRT);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ScreenProbeGatherState.BackfaceDiffuseIndirectHistoryRT);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ScreenProbeGatherState.RoughSpecularIndirectHistoryRT);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ScreenProbeGatherState.FastUpdateMode_NumFramesAccumulatedHistoryRT);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ScreenProbeGatherState.ShortRangeAOHistoryRT);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ScreenProbeGatherState.HistoryScreenProbeSceneDepth);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ScreenProbeGatherState.HistoryScreenProbeTranslatedWorldPosition);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ScreenProbeGatherState.ProbeHistoryScreenProbeRadiance);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ScreenProbeGatherState.ImportanceSamplingHistoryScreenProbeRadiance);

	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ReflectionState.SpecularAndSecondMomentHistory);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.ReflectionState.NumFramesAccumulatedHistory);

	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.TranslucentReflectionState.SpecularAndSecondMomentHistory);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.TranslucentReflectionState.NumFramesAccumulatedHistory);

	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.WaterReflectionState.SpecularAndSecondMomentHistory);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.WaterReflectionState.NumFramesAccumulatedHistory);

	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.RadianceCacheState.RadianceProbeAtlasTexture);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.RadianceCacheState.SkyVisibilityProbeAtlasTexture);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.RadianceCacheState.FinalRadianceAtlas);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.RadianceCacheState.FinalSkyVisibilityAtlas);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.RadianceCacheState.DepthProbeAtlasTexture);

	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.TranslucencyVolumeRadianceCacheState.RadianceProbeAtlasTexture);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.TranslucencyVolumeRadianceCacheState.SkyVisibilityProbeAtlasTexture);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.TranslucencyVolumeRadianceCacheState.FinalRadianceAtlas);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.TranslucencyVolumeRadianceCacheState.FinalSkyVisibilityAtlas);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.Lumen.TranslucencyVolumeRadianceCacheState.DepthProbeAtlasTexture);

	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.StochasticLighting.SceneDepthHistory);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.StochasticLighting.SceneNormalHistory);
	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.StochasticLighting.SceneClosureHistory);

	TextureFunction(RHICmdList, SystemMemoryMirror, ViewState, ViewState.BloomFFTKernel.Spectral);
}

void FSceneViewState::SystemMemoryMirrorBackup(FSceneViewStateSystemMemoryMirror* SystemMemoryMirror)
{
	ENQUEUE_RENDER_COMMAND(ViewStateSystemMemoryBackup)([&](FRHICommandListImmediate& RHICmdList)
	{
		{
			RHI_BREADCRUMB_EVENT_STAT(RHICmdList, SystemMemoryBackup, "SystemMemoryBackup");
			SystemMemoryForEachTexture(RHICmdList, *SystemMemoryMirror, *this, SystemMemoryBackupTextureBegin);
		}

		constexpr bool bFlushResources = true;
		RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

		// Clear out any temporary textures used to copy depth
		SystemMemoryMirror->TemporaryTextures.Reset();

		SystemMemoryForEachTexture(RHICmdList, *SystemMemoryMirror, *this, SystemMemoryBackupTextureEnd);
	});

	FlushRenderingCommands();
}

void FSceneViewState::SystemMemoryMirrorRestore(FSceneViewStateSystemMemoryMirror* SystemMemoryMirror)
{
	ENQUEUE_RENDER_COMMAND(ViewStateSystemMemoryRestore)([&](FRHICommandListImmediate& RHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, SystemMemoryRestore, "SystemMemoryRestore");
		SystemMemoryForEachTexture(RHICmdList, *SystemMemoryMirror, *this, SystemMemoryRestoreTexture);
	});

	FlushRenderingCommands();
}

void FSceneViewState::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(MIDPool);
}

UMaterialInstanceDynamic* FSceneViewState::GetReusableMID(class UMaterialInterface* InSource)
{		
	check(IsInGameThread());
	check(InSource);

	// 0 or MID (MaterialInstanceDynamic) pointer
	auto InputAsMID = Cast<UMaterialInstanceDynamic>(InSource);

	// fixup MID parents as this is not allowed, take the next MIC or Material.
	UMaterialInterface* ParentOfTheNewMID = InputAsMID ? ToRawPtr(InputAsMID->Parent) : InSource;

	// this is not allowed and would cause an error later in the code
	check(!ParentOfTheNewMID->IsA(UMaterialInstanceDynamic::StaticClass()));

	UMaterialInstanceDynamic* NewMID = 0;

	if(MIDUsedCount < (uint32)MIDPool.Num())
	{
		NewMID = MIDPool[MIDUsedCount];

		if(NewMID->Parent != ParentOfTheNewMID)
		{
			// create a new one
			// garbage collector will remove the old one
			// this should not happen too often
			NewMID = UMaterialInstanceDynamic::Create(ParentOfTheNewMID, 0);
			MIDPool[MIDUsedCount] = NewMID;
		}

		// reusing an existing object means we need to clear out the Vector and Scalar parameters
		NewMID->ClearParameterValues();
	}
	else
	{
		NewMID = UMaterialInstanceDynamic::Create(ParentOfTheNewMID, 0);
		check(NewMID);

		MIDPool.Add(NewMID);
	}

	if(InputAsMID)
	{
		// parent is an MID so we need to copy the MID Vector and Scalar parameters over
		NewMID->CopyInterpParameters(InputAsMID);
	}

	check(!FApp::CanEverRender() || NewMID->GetRenderProxy());
	MIDUsedCount++;
	return NewMID;
}

void FSceneViewState::ClearMIDPool(FStringView MidParentRootPath)
{
	check(IsInGameThread());
	if (MidParentRootPath.IsEmpty())
	{
		MIDPool.Empty();
		return;
	}

	FNameBuilder PackagePath;
	const int32 RemoveNum = Algo::RemoveIf(MIDPool, [&PackagePath, MidParentRootPath](UMaterialInstanceDynamic* MID) -> bool
	{
		if (MID->Parent)
		{
			MID->Parent->GetPackage()->GetFName().ToString(PackagePath);
			return FStringView(PackagePath).StartsWith(MidParentRootPath);
		}
		return false;
	});

	MIDPool.SetNum(RemoveNum);
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void FSceneViewState::ActivateFrozenViewMatrices(FSceneView& SceneView)
{
	auto* ViewState = static_cast<FSceneViewState*>(SceneView.State);
	if (ViewState->bIsFrozen)
	{
		check(ViewState->bIsFrozenViewMatricesCached);

		Swap(SceneView.ViewMatrices, ViewState->CachedViewMatrices);
		ViewState->bIsFrozenViewMatricesCached = false;
	}
}

void FSceneViewState::RestoreUnfrozenViewMatrices(FSceneView& SceneView)
{
	auto* ViewState = static_cast<FSceneViewState*>(SceneView.State);
	if (ViewState->bIsFrozen)
	{
		check(!ViewState->bIsFrozenViewMatricesCached);

		Swap(SceneView.ViewMatrices, ViewState->CachedViewMatrices);
		ViewState->bIsFrozenViewMatricesCached = true;
	}
}
#endif
