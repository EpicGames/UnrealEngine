// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneManagement.h"
#include "RenderResource.h"
#include "LightComponentId.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "RHIFwd.h"
#include "RenderGraphFwd.h"
#include "RenderGraphDefinitions.h"
#include "SceneRendering.h"
#include "Substrate/Substrate.h"
#include "Lumen/LumenViewState.h"
#include "MegaLights/MegaLightsViewState.h"
#include "StochasticLighting/StochasticLightingViewState.h"
#include "TranslucentLightingViewState.h"
#include "VolumetricRenderTargetViewStateData.h"
#include "HairStrands/HairStrandsData.h"
#include "Substrate/Glint/GlintShadingLUTs.h"

#include "SceneViewOcclusionHistory.h"
#include "SceneViewStateData.h"

class FProjectedShadowInfo;
class FScene;
class UCurveFloat;
class UMaterialInstanceDynamic;
class UMaterialInterface;
template<typename T>
struct TObjectPtr;

namespace DiaphragmDOF
{
	class FDOFViewState;
}

class FOrthoVoxelGridUniformBufferParameters;
class FFrustumVoxelGridUniformBufferParameters;
class FAdaptiveVolumetricShadowMapUniformBufferParameters;
class FBeerShadowMapUniformBufferParameters;
struct FPathTracingState;
class FLandscapeRayTracingStateList;
class FForwardLightingViewResources;
struct FPersistentGlobalDistanceFieldData;
class FSceneViewFamily;
class FViewInfo;
class FReferenceCollector;
struct FSceneViewStateSystemMemoryMirror;

/**
 * The scene manager's private implementation of persistent view state.
 * This class is associated with a particular camera across multiple frames by the game thread.
 * The game thread calls FRendererModule::AllocateViewState to create an instance of this private implementation.
 */
class FSceneViewState : public FSceneViewStateInterface, public FRenderResource
{
public:

	class FProjectedShadowKey
	{
	public:

		inline bool operator == (const FProjectedShadowKey &Other) const
		{
			return ((PrimitiveId == Other.PrimitiveId) && 
					(LightComponentId == Other.LightComponentId) && 
					(ShadowSplitIndex == Other.ShadowSplitIndex) && 
					(bTranslucentShadow == Other.bTranslucentShadow));
		}

		FProjectedShadowKey(const FProjectedShadowInfo& ProjectedShadowInfo);

		FProjectedShadowKey(FPrimitiveComponentId InPrimitiveId, const FLightComponentId& InLightComponentId, int32 InSplitIndex, bool bInTranslucentShadow)
			: PrimitiveId(InPrimitiveId)
			, LightComponentId(InLightComponentId)
			, ShadowSplitIndex(InSplitIndex)
			, bTranslucentShadow(bInTranslucentShadow)
		{
		}

		friend inline uint32 GetTypeHash(const FSceneViewState::FProjectedShadowKey& Key)
		{
			return HashCombineFast(GetTypeHash(Key.LightComponentId), GetTypeHash(Key.PrimitiveId));
		}

	private:
		FPrimitiveComponentId PrimitiveId;
		FLightComponentId LightComponentId;
		int32 ShadowSplitIndex;
		bool bTranslucentShadow;
	};

	uint32 UniqueID;

	/**
	 * Cube map captures share an origin, allowing them to share things like global distance fields and Lumen scene data.  Otherwise,
	 * this will just be the same as UniqueID.
	 */
	uint32 ShareOriginUniqueID;

	/**
	 * The scene pointer may be NULL -- it's filled in by certain API calls that require a FSceneViewState and FScene to know about each other,
	 * Whenever a ViewState and Scene get linked, this pointer is set, and a pointer to the ViewState is added to an array in the Scene.
	 * The linking is necessary in cases where incremental FScene updates need to be reflected in cached data stored in FSceneViewState.
	 */
	FScene* Scene;

	typedef TMap<FSceneViewState::FProjectedShadowKey, FRHIPooledRenderQuery> ShadowKeyOcclusionQueryMap;
	TArray<ShadowKeyOcclusionQueryMap, TInlineAllocator<FOcclusionQueryHelpers::MaxBufferedOcclusionFrames> > ShadowOcclusionQueryMaps;

	/** The view's occlusion query pool. */
	FRenderQueryPoolRHIRef OcclusionQueryPool;
	FFrameBasedOcclusionQueryPool PrimitiveOcclusionQueryPool;

	FHZBOcclusionTester HZBOcclusionTests;
	FOcclusionFeedback OcclusionFeedback;

	FPersistentSkyAtmosphereData PersistentSkyAtmosphereData;

	/** Storage to which compressed visibility chunks are uncompressed at runtime. */
	TArray<uint8> DecompressedVisibilityChunk;

	/** Cached visibility data from the last call to ResolvePrecomputedVisibilityData. */
	const TArray<uint8>* CachedVisibilityChunk;
	int32 CachedVisibilityHandlerId;
	int32 CachedVisibilityBucketIndex;
	int32 CachedVisibilityChunkIndex;

	uint32		PendingPrevFrameNumber;
	uint32		PrevFrameNumber;
	double		LastRenderTime;
	float		MotionBlurTimeScale;
	float		MotionBlurTargetDeltaTime;
	FMatrix44f	PrevViewMatrixForOcclusionQuery;
	FVector		PrevViewOriginForOcclusionQuery;

	// A counter incremented once each time this view is rendered.
	uint32 OcclusionFrameCounter;

	/** For this view, the set of primitives that are currently fading, either in or out. */
	FPrimitiveFadingStateMap PrimitiveFadingStates;

	TMap<int32, FIndividualOcclusionHistory> PlanarReflectionOcclusionHistories;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Are we currently in the state of freezing rendering? (1 frame where we gather what was rendered) */
	uint32 bIsFreezing : 1;

	/** Is rendering currently frozen? */
	uint32 bIsFrozen : 1;

	/** True if the CachedViewMatrices is holding frozen view matrices, otherwise false */
	uint32 bIsFrozenViewMatricesCached : 1;

	/** The set of primitives that were rendered the frame that we froze rendering */
	TSet<FPrimitiveComponentId> FrozenPrimitives;

	/** The cache view matrices at the time of freezing or the cached debug fly cam's view matrices. */
	FViewMatrices CachedViewMatrices;
#endif

	/** HLOD persistent fading and visibility state */
	FHLODVisibilityState HLODVisibilityState;
	TMap<FPrimitiveComponentId, FHLODSceneNodeVisibilityState> HLODSceneNodeVisibilityStates;

	/** The current frame PreExposure */
	float PreExposure;

	/** Whether to get the last exposure from GPU */
	bool bUpdateLastExposure;

private:
	
	// to implement eye adaptation / auto exposure changes over time
	class FEyeAdaptationManager
	{
	public:
		FEyeAdaptationManager();

		void SafeRelease();

		/** Get the last frame exposure value (used to compute pre-exposure) */
		float GetLastExposure() const { return LastExposure; }

		/** Get the last frame average local exposure approximation value (used to compute pre-exposure) */
		float GetLastAverageLocalExposure() const { return LastAverageLocalExposure; }

		/** Get the last frame average scene luminance (used for exposure compensation curve) */
		float GetLastAverageSceneLuminance() const { return LastAverageSceneLuminance; }

		const TRefCountPtr<FRDGPooledBuffer>& GetCurrentBuffer() const
		{
			return GetBuffer(CurrentBufferIndex);
		}

		const TRefCountPtr<FRDGPooledBuffer>& GetCurrentBuffer(FRDGBuilder& GraphBuilder)
		{
			return GetOrCreateBuffer(GraphBuilder, CurrentBufferIndex);
		}

		void SwapBuffers();
		void UpdateLastExposureFromBuffer();
		void EnqueueExposureBufferReadback(FRDGBuilder& GraphBuilder);

		uint64 GetGPUSizeBytes(bool bLogSizes) const;

	private:
		const TRefCountPtr<FRDGPooledBuffer>& GetBuffer(uint32 BufferIndex) const;
		const TRefCountPtr<FRDGPooledBuffer>& GetOrCreateBuffer(FRDGBuilder& GraphBuilder, uint32 BufferIndex);

		FRHIGPUBufferReadback* GetLatestReadbackBuffer();

		// TODO: Do we need to double buffer?
		// - for readback we copy data to readback buffers
		// - do we ever need to access prev frame exposure AFTER current frame exposure has been calculated?
		// - should at least make it more explicit/safe by having GetCurrentBuffer() and GetPreviousBuffer()
		//		and assert if current is accessed too early in frame.
		static const int32 NUM_BUFFERS = 2;

		static const int32 EXPOSURE_BUFFER_SIZE_IN_VECTOR4 = 2;

		int32 CurrentBufferIndex = 0;

		float LastExposure = 0;
		float LastAverageLocalExposure = 1.0f;
		float LastAverageSceneLuminance = 0; // 0 means invalid. Used for Exposure Compensation Curve.

		TRefCountPtr<FRDGPooledBuffer> ExposureBufferData[NUM_BUFFERS];
		TArray<FRHIGPUBufferReadback*> ExposureReadbackBuffers;

		static const uint32 MAX_READBACK_BUFFERS = 4;
		uint32 ReadbackBuffersWriteIndex = 0;
		uint32 ReadbackBuffersNumPending = 0;

	} EyeAdaptationManager;

	struct {
		FTextureRHIRef Texture;
		UCurveFloat* Curve = nullptr;
#if WITH_EDITORONLY_DATA
		TWeakObjectPtr<UCurveFloat> Curve_GameThread = nullptr;
		FDelegateHandle CurveUpdateDelegateHandle;
		uint32 Version = 0;
#endif
	} ExposureCompensationCurveLUT;

	FSubstrateViewDebugData SubstrateViewDebugData;

	// The LUT used by tonemapping. In stereo this is only computed and stored by the Left Eye.
	TRefCountPtr<IPooledRenderTarget> CombinedLUTRenderTarget;

	// The inner LUT used by the LUT used by tonemapping.
	TRefCountPtr<IPooledRenderTarget> InnerLUTRenderTarget;

	// LUT is only valid after it has been computed, not on allocation of the RT
	bool bValidTonemappingLUT = false;


	// used by the Postprocess Material Blending system to avoid recreation and garbage collection of MIDs
	TArray<TObjectPtr<UMaterialInstanceDynamic>> MIDPool;
	uint32 MIDUsedCount;

	// counts up by one each frame, warped in 0..3 range, ResetViewState() puts it back to 0
	int32 DistanceFieldTemporalSampleIndex;

	// whether this view is a stereo counterpart to a primary view
	bool bIsStereoView;

	// The whether or not round-robin occlusion querying is enabled for this view
	bool bRoundRobinOcclusionEnabled;

public:
	
	// if TemporalAA is on this cycles through 0..TemporalAASampleCount-1, ResetViewState() puts it back to 0
	int32 TemporalAASampleIndex;

	// Counts up by one each frame, ResetViewState() puts it back to 0. Should be used as the seed for Halton
	// and other per-render changes. Use OutputFrameIndex as the seed if the effect should be per output frame.
	// Under normal rendering FrameIndex == OutputFrameIndex and there is no distinction between per-render and
	// per-frame, but when accumulating multiple samples then FrameIndex will be unique for each accumulation sample
	// rendered, but OutputFrameIndex will only increment per multi-sample accumulated output frame.
	// 
	// Can be overwritten by OverrideFrameIndexValue.
	uint32 FrameIndex;

	// Counts up by one each frame, ResetViewState() puts it back to zero. See FrameIndex for more details. This should
	// equal FrameIndex in normal scenarios, but can differ when accumulating multiple samples to produce one output frame.
	//
	// Can be overwritten by OverrideOutputFrameIndexValue
	uint32 OutputFrameIndex;
	
	/** Informations of to persist for the next frame's FViewInfo::PrevViewInfo.
	 *
	 * Under normal use case (temporal histories are not frozen), this gets cleared after setting FViewInfo::PrevViewInfo
	 * after being copied to FViewInfo::PrevViewInfo. New temporal histories get directly written to it.
	 *
	 * When temporal histories are frozen (pause command, or r.Test.FreezeTemporalHistories), this keeps it's values, and the currently
	 * rendering FViewInfo should not update it. Refer to FViewInfo::bStatePrevViewInfoIsReadOnly.
	 */
	FPreviousViewInfo PrevFrameViewInfo;

	// Temporal AA result for light shafts of last frame
	FTemporalAAHistory LightShaftOcclusionHistory;
	// Temporal AA result for light shafts of last frame
	TMap<FLightComponentId, TUniquePtr<FTemporalAAHistory> > LightShaftBloomHistoryRTs;

	FIntRect DistanceFieldAOHistoryViewRect;
	TRefCountPtr<IPooledRenderTarget> DistanceFieldAOHistoryRT;
	TRefCountPtr<IPooledRenderTarget> DistanceFieldIrradianceHistoryRT;

	// Burley Subsurface scattering variance texture from the last frame.
	TRefCountPtr<IPooledRenderTarget> SubsurfaceScatteringQualityHistoryRT;

	FLumenViewState Lumen;
	FMegaLightsViewState MegaLights;

	// Shared by Lumen and Mega Lights
	FStochasticLightingViewState StochasticLighting;

	FTranslucencyLightingViewState TranslucencyLighting;

	VisualizeLightShape::FViewState VisualizeLightShape;

	// Heterogeneous Volumes cached data stores
	TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters> OrthoVoxelGridUniformBuffer = nullptr;
	TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters> FrustumVoxelGridUniformBuffer = nullptr;

	FAdaptiveVolumetricShadowMapUniformBufferParameters* AdaptiveVolumetricCameraMapUniformBufferParameters = nullptr;
	TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> AdaptiveVolumetricCameraMapUniformBuffer = nullptr;
	TMap<FLightSceneId, TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters>> AdaptiveVolumetricShadowMapUniformBufferMap;

	TMap<FLightSceneId, TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters>> BeerShadowMapUniformBufferMap;

	// Pre-computed filter in spectral (i.e. FFT) domain along with data to determine if we need to up date it
	struct {
		/// @cond DOXYGEN_WARNINGS
		void SafeRelease()
		{
			PhysicalRHI = nullptr;
			Spectral.SafeRelease();
			ConstantsBuffer.SafeRelease();
		}
		/// @endcond

		// The 2d fourier transform of the physical space texture.
		TRefCountPtr<IPooledRenderTarget> Spectral;
		TRefCountPtr<FRDGPooledBuffer> ConstantsBuffer;

		// The physical space source texture
		FRHITexture* PhysicalRHI = nullptr;

		// The Scale * 100 = percentage of the image space that the physical kernel represents.
		// e.g. Scale = 1 indicates that the physical kernel image occupies the same size 
		// as the image to be processed with the FFT convolution.
		float Scale = 0.f;

		// The size of the viewport for which the spectral kernel was calculated. 
		FIntPoint ImageSize;

		// Mip level of the physical space source texture used when caching the spectral space texture.
		uint32 PhysicalMipLevel;
	} BloomFFTKernel;

	// Film grain
	struct
	{
		/// @cond DOXYGEN_WARNINGS
		void SafeRelease()
		{
			TextureRHI = nullptr;
			ConstantsBuffer.SafeRelease();
		}
		/// @endcond
		// Cache View.FilmGrainTexture in TextureRHI and compare it every frame in case a new one is set (and rebuild constant buffers if this is the case)
		FRHITexture* TextureRHI = nullptr;
		TRefCountPtr<FRDGPooledBuffer> ConstantsBuffer;
	} FilmGrainCache;

	TPimplPtr<DiaphragmDOF::FDOFViewState> DepthOfFieldState;

	// Cached material texture samplers
	float MaterialTextureCachedMipBias;
	float LandscapeCachedMipBias;
	FSamplerStateRHIRef MaterialTextureBilinearWrapedSamplerCache;
	FSamplerStateRHIRef MaterialTextureBilinearClampedSamplerCache;
	FSamplerStateRHIRef LandscapeWeightmapSamplerCache;

#if RHI_RAYTRACING
	// Invalidates cached results related to the path tracer so accumulated rendering can start over
	void PathTracingInvalidate(bool InvalidateAnimationStates = true);
	virtual uint32 GetPathTracingSampleIndex() const override;
	virtual uint32 GetPathTracingSampleCount() const override;

	// Keeps track of the internal path tracer state
	TPimplPtr<FPathTracingState> PathTracingState;
	uint32 PathTracingInvalidationCounter = 0;

	// Ray Traced Sky Light Sample Direction Data
	TRefCountPtr<FRDGPooledBuffer> SkyLightVisibilityRaysBuffer;
	FIntVector SkyLightVisibilityRaysDimensions;

	// List of landscape ray tracing state associated with this view, so it can be cleaned up if the view gets deleted.
	TPimplPtr<FLandscapeRayTracingStateList> LandscapeRayTracingStates;

	virtual void SetLandscapeRayTracingStates(TPimplPtr<FLandscapeRayTracingStateList>&& InLandscapeRayTracingStates) final { LandscapeRayTracingStates = MoveTemp(InLandscapeRayTracingStates); }
	virtual FLandscapeRayTracingStateList* GetLandscapeRayTracingStates() const final { return LandscapeRayTracingStates.Get(); }
#endif // RHI_RAYTRACING

	TUniquePtr<FForwardLightingViewResources> ForwardLightingResources;

	float LightScatteringHistoryPreExposure;
	FVector2f PrevLightScatteringViewGridUVToViewRectVolumeUV;
	FVector2f VolumetricFogPrevViewGridRectUVToResourceUV;
	FVector2f VolumetricFogPrevUVMax;
	FVector2f VolumetricFogPrevUVMaxForTemporalBlend;
	FIntVector VolumetricFogPrevResourceGridSize;
	TRefCountPtr<IPooledRenderTarget> LightScatteringHistory;
	TRefCountPtr<IPooledRenderTarget> PrevLightScatteringConservativeDepthTexture;

	/** Potentially shared to save memory in cases where multiple view states share a common origin, such as cube map capture faces. */
	TRefCountPtr<FPersistentGlobalDistanceFieldData> GlobalDistanceFieldData;

	// Sequencer state for view management
	ESequencerState SequencerState;

	FTemporalLODState TemporalLODState;

	FSceneViewState* SplitScreenDebugViewState = nullptr;

	FVolumetricRenderTargetViewStateData VolumetricCloudRenderTarget;
	FTemporalRenderTargetState VolumetricCloudShadowRenderTarget[NUM_ATMOSPHERE_LIGHTS];
	FMatrix VolumetricCloudShadowmapPreviousTranslatedWorldToLightClipMatrix[NUM_ATMOSPHERE_LIGHTS];
	FVector VolumetricCloudShadowmapPreviousAtmosphericLightPos[NUM_ATMOSPHERE_LIGHTS];
	FVector VolumetricCloudShadowmapPreviousAnchorPoint[NUM_ATMOSPHERE_LIGHTS];
	FVector VolumetricCloudShadowmapPreviousAtmosphericLightDir[NUM_ATMOSPHERE_LIGHTS];

	virtual FRDGTextureRef GetVolumetricCloudTexture(FRDGBuilder& GraphBuilder) override
	{
		return VolumetricCloudRenderTarget.GetDstVolumetricReconstructRT(GraphBuilder);
	}

	virtual FVector2f GetVolumetricCloudTextureUVScale() const override
	{
		return VolumetricCloudRenderTarget.GetDstVolumetricReconstructUVScale();
	}

	virtual FVector2f GetVolumetricCloudTextureUVMax() const override
	{
		return VolumetricCloudRenderTarget.GetDstVolumetricReconstructUVMax();
	}

	FHairStrandsViewStateData HairStrandsViewStateData;

	FShaderPrintStateData ShaderPrintStateData;

	bool bLumenSceneDataAdded;
	float LumenSurfaceCacheResolution;

	// call after OnFrameRenderingSetup()
	virtual uint32 GetCurrentTemporalAASampleIndex() const
	{
		return TemporalAASampleIndex;
	}

	// Returns the index of the frame with a desired power of two modulus.
	inline uint32 GetFrameIndex(uint32 Pow2Modulus) const
	{
		check(FMath::IsPowerOfTwo(Pow2Modulus));
		return FrameIndex & (Pow2Modulus - 1);
	}

	// Returns 32bits frame index.
	inline uint32 GetFrameIndex() const
	{
		return FrameIndex;
	}

	// Returns the index of the output frame with a desired power of two modulus.
	inline uint32 GetOutputFrameIndex(uint32 Pow2Modulus) const
	{
		check(FMath::IsPowerOfTwo(Pow2Modulus));
		return OutputFrameIndex & (Pow2Modulus - 1);
	}

	// Returns 32bits output frame index. Matches GetFrameIndex unless using multi-sample accumulation.
	inline uint32 GetOutputFrameIndex() const
	{
		return OutputFrameIndex;
	}

	// to make rendering more deterministic
	virtual void ResetViewState()
	{
		TemporalAASampleIndex = 0;
		FrameIndex = 0;
		OutputFrameIndex = 0;
		DistanceFieldTemporalSampleIndex = 0;
		PreExposure = 1.f;

		ResetVolumetricFogState();

		ReleaseRHI();
	}

	void SetupDistanceFieldTemporalOffset(const FSceneViewFamily& Family)
	{
		if (!Family.bWorldIsPaused)
		{
			DistanceFieldTemporalSampleIndex++;
		}

		if(DistanceFieldTemporalSampleIndex >= 4)
		{
			DistanceFieldTemporalSampleIndex = 0;
		}
	}

	uint32 GetDistanceFieldTemporalSampleIndex() const
	{
		return DistanceFieldTemporalSampleIndex;
	}

	void ResetVolumetricFogState();

	/** Default constructor. */
	FSceneViewState(ERHIFeatureLevel::Type FeatureLevel, FSceneViewState* ShareOriginTarget);

private:
	// May only be called via Destroy (ensuring it happens on the RT)
	virtual ~FSceneViewState();

public:
	// called every frame after the view state was updated
	void UpdateLastRenderTime(const FSceneViewFamily& Family)
	{
		LastRenderTime = Family.Time.GetRealTimeSeconds();
	}

	// InScene is passed in, as the Scene pointer in the class itself may be null, if it was allocated without a scene.
	void TrimHistoryRenderTargets(const FScene* InScene);

	/**
	 * Calculates and stores the scale factor to apply to motion vectors based on the current game
	 * time and view post process settings.
	 */
	void UpdateMotionBlurTimeScale(const FViewInfo& View);

	/** 
	 * Called every frame after UpdateLastRenderTime, sets up the information for the lagged temporal LOD transition
	 */
	void UpdateTemporalLODTransition(const FViewInfo& View)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (bIsFrozen)
		{
			return;
		}
#endif

		TemporalLODState.UpdateTemporalLODTransition(View, LastRenderTime);
	}

	/** 
	 * Returns an array of visibility data for the given view, or NULL if none exists. 
	 * The data bits are indexed by VisibilityId of each primitive in the scene.
	 * This method decompresses data if necessary and caches it based on the bucket and chunk index in the view state.
	 * InScene is passed in, as the Scene pointer in the class itself may be null, if it was allocated without a scene.
	 */
	const uint8* ResolvePrecomputedVisibilityData(FViewInfo& View, const FScene* InScene);

	/**
	 * Cleans out old entries from the primitive occlusion history, and resets unused pending occlusion queries.
	 * @param MinHistoryTime - The occlusion history for any primitives which have been visible and unoccluded since
	 *							this time will be kept.  The occlusion history for any primitives which haven't been
	 *							visible and unoccluded since this time will be discarded.
	 * @param MinQueryTime - The pending occlusion queries older than this will be discarded.
	 */
	void TrimOcclusionHistory(float CurrentTime, float MinHistoryTime, float MinQueryTime, int32 FrameNumber);

	inline void UpdateRoundRobin(const bool bUseRoundRobin)
	{
		bRoundRobinOcclusionEnabled = bUseRoundRobin;
	}

	inline bool IsRoundRobinEnabled() const
	{
		return bRoundRobinOcclusionEnabled;
	}

	/**
	 * Checks whether a shadow is occluded this frame.
	 * @param Primitive - The shadow subject.
	 * @param Light - The shadow source.
	 */
	bool IsShadowOccluded(FSceneViewState::FProjectedShadowKey ShadowKey, int32 NumBufferedFrames) const;

	FRDGPooledBuffer* GetCurrentEyeAdaptationBuffer() const override final
	{
		FRDGPooledBuffer* Buffer = EyeAdaptationManager.GetCurrentBuffer().GetReference();
		check(bValidEyeAdaptationBuffer && Buffer);
		return Buffer;
	}

	FRDGPooledBuffer* GetCurrentEyeAdaptationBuffer(FRDGBuilder& GraphBuilder)
	{
		bValidEyeAdaptationBuffer = true;
		return EyeAdaptationManager.GetCurrentBuffer(GraphBuilder).GetReference();
	}

	void SwapEyeAdaptationBuffers()
	{
		EyeAdaptationManager.SwapBuffers();
	}

	void UpdateEyeAdaptationLastExposureFromBuffer()
	{
		if (bUpdateLastExposure && bValidEyeAdaptationBuffer)
		{
			EyeAdaptationManager.UpdateLastExposureFromBuffer();
		}
	}

	void EnqueueEyeAdaptationExposureBufferReadback(FRDGBuilder& GraphBuilder)
	{
		if (bUpdateLastExposure && bValidEyeAdaptationBuffer)
		{
			EyeAdaptationManager.EnqueueExposureBufferReadback(GraphBuilder);
		}
	}

	float GetLastEyeAdaptationExposure() const
	{
		return EyeAdaptationManager.GetLastExposure();
	}

	float GetLastAverageLocalExposure() const
	{
		return EyeAdaptationManager.GetLastAverageLocalExposure();
	}

	float GetLastAverageSceneLuminance() const
	{
		return EyeAdaptationManager.GetLastAverageSceneLuminance();
	}

	FRHITexture* GetExposureCompensationCurveLUT() const
	{
		return ExposureCompensationCurveLUT.Texture;
	}

	virtual void SetExposureCompensationCurve(UCurveFloat* NewCurve) override;

	void UpdateExposureCompensationCurveLUT(FRHICommandList& RHICmdList, UCurveFloat* Curve);

	bool HasValidTonemappingLUT() const
	{
		return bValidTonemappingLUT;
	}

	void SetValidTonemappingLUT(bool bValid = true)
	{
		bValidTonemappingLUT = bValid;
	}

	static FPooledRenderTargetDesc CreateLUTRenderTarget(const int32 LUTSize, const bool bNeedFloatOutput, const TCHAR* const DebugName);

	// Returns a reference to the render target used for the LUT. Allocated on the first request.
	IPooledRenderTarget* GetTonemappingLUT(FRHICommandList& RHICmdList, const int32 LUTSize, const bool bNeedFloatOutput);

	// Returns a reference to the render target used for the inner LUT. Allocated on the first request.
	IPooledRenderTarget* GetInnerTonemappingLUT(FRHICommandList& RHICmdList, const int32 LUTSize, const bool bNeedFloatOutput);

	IPooledRenderTarget* GetTonemappingLUT() const
	{
		return CombinedLUTRenderTarget.GetReference();
	}

	void ClearInnerTonemappingLUT()
	{
		InnerLUTRenderTarget.SafeRelease();
	}

	FSubstrateViewDebugData& GetSubstrateViewDebugData()
	{
		return SubstrateViewDebugData;
	}

	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		HZBOcclusionTests.InitRHI(RHICmdList);
	}

	virtual void ReleaseRHI() override;

	// FSceneViewStateInterface
	RENDERER_API virtual void Destroy() override;

	virtual FSceneViewState* GetConcreteViewState() override
	{
		return this;
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	// needed for GetReusableMID()
	virtual void OnStartPostProcessing(FSceneView& CurrentView) override
	{
		check(IsInGameThread());

		// Needs to be done once for all viewstates.  If multiple FSceneViews are sharing the same ViewState, this will cause problems.
		// Sharing should be illegal right now though.
		MIDUsedCount = 0;
	}

	/** Returns the current PreExposure value. PreExposure is a custom scale applied to the scene color to prevent buffer overflow. */
	virtual float GetPreExposure() const override
	{
		return PreExposure;
	}

	// Note: OnStartPostProcessing() needs to be called each frame for each view
	virtual UMaterialInstanceDynamic* GetReusableMID(class UMaterialInterface* InSource) override;

	virtual void ClearMIDPool(FStringView MidParentRootPath = {}) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	virtual const FViewMatrices* GetFrozenViewMatrices() const override
	{
		if (bIsFrozen && bIsFrozenViewMatricesCached)
		{
			return &CachedViewMatrices;
		}
		return nullptr;
	}

	virtual void ActivateFrozenViewMatrices(FSceneView& SceneView) override;

	virtual void RestoreUnfrozenViewMatrices(FSceneView& SceneView) override;
#endif

	virtual FTemporalLODState& GetTemporalLODState() override
	{
		return TemporalLODState;
	}

	virtual const FTemporalLODState& GetTemporalLODState() const override
	{
		return TemporalLODState;
	}

	float GetTemporalLODTransition() const override
	{
		return TemporalLODState.GetTemporalLODTransition(LastRenderTime);
	}

	uint32 GetViewKey() const override
	{
		return UniqueID;
	}

	uint32 GetShareOriginViewKey() const
	{
		return ShareOriginUniqueID;
	}

	uint32 GetOcclusionFrameCounter() const
	{
		return OcclusionFrameCounter;
	}

	virtual SIZE_T GetSizeBytes() const override;
	uint64 GetGPUSizeBytes(bool bLogSizes = false) const;

	virtual void SetSequencerState(ESequencerState InSequencerState) override
	{
		SequencerState = InSequencerState;
	}

	virtual ESequencerState GetSequencerState() override
	{
		return SequencerState;
	}

	virtual void AddLumenSceneData(FSceneInterface* InScene, float SurfaceCacheResolution) override;
	virtual void RemoveLumenSceneData(FSceneInterface* InScene) override;
	virtual bool HasLumenSceneData() const override;

	struct FOcclusion
	{
		/** Information about visibility/occlusion states in past frames for individual primitives. */
		TSet<FPrimitiveOcclusionHistory, FPrimitiveOcclusionHistoryKeyFuncs> PrimitiveOcclusionHistorySet;

		/** The last occlusion query of last frame to test in the following frame to block the GPU. */
		TArray<FRHIRenderQuery*, TInlineAllocator<FOcclusionQueryHelpers::MaxBufferedOcclusionFrames> > LastOcclusionQueryArray;

		/** The number of queries requested last frame. */
		uint32 NumRequestedQueries = 0;
	
	} Occlusion;

	virtual void SystemMemoryMirrorBackup(FSceneViewStateSystemMemoryMirror* SystemMemoryMirror) override final;
	virtual void SystemMemoryMirrorRestore(FSceneViewStateSystemMemoryMirror* SystemMemoryMirror) override final;
};