// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "UniformBuffer.h"
#include "RHIFwd.h"
#include "RHIGPUReadback.h"
#include "RenderResource.h"
#include "ShaderBaseClasses.h"
#include "Math/MathFwd.h"
#include "Math/RandomStream.h"
#include "Templates/RefCounting.h"
#include "RenderGraphResources.h"
#include "PooledRenderTarget.h"
#include "GlobalDistanceField.h"
#include "GlobalDistanceFieldConstants.h"
#include <atomic>

class FDistanceFieldSceneData;

/**
 * Distance cull fading uniform buffer containing fully faded in.
 */
class FGlobalDistanceCullFadeUniformBuffer : public TUniformBuffer< FDistanceCullFadeUniformShaderParameters >
{
public:
	void InitContents()
	{
		FDistanceCullFadeUniformShaderParameters Parameters;
		Parameters.FadeTimeScaleBias.X = 0.0f;
		Parameters.FadeTimeScaleBias.Y = 1.0f;
		SetContents(FRenderResource::GetImmediateCommandList(), Parameters);
	}
};

/** Global primitive uniform buffer resource containing faded in */
extern TGlobalResource< FGlobalDistanceCullFadeUniformBuffer > GDistanceCullFadedInUniformBuffer;

/**
 * Dither uniform buffer containing fully faded in.
 */
class FGlobalDitherUniformBuffer : public TUniformBuffer< FDitherUniformShaderParameters >
{
public:
	void InitContents()
	{
		FDitherUniformShaderParameters Parameters;
		Parameters.LODFactor = 0.0f;
		SetContents(FRenderResource::GetImmediateCommandList(), Parameters);
	}
};

/** Global primitive uniform buffer resource containing faded in */
extern TGlobalResource< FGlobalDitherUniformBuffer > GDitherFadedInUniformBuffer;

/**
 * Stores fading state for a single primitive in a single view
 */
class FPrimitiveFadingState
{
public:
	FPrimitiveFadingState()
		: FadeTimeScaleBias(ForceInitToZero)
		, FrameNumber(0)
		, EndTime(0.0f)
		, bIsVisible(false)
		, bValid(false)
	{
	}

	/** Scale and bias to use on time to calculate fade opacity */
	FVector2D FadeTimeScaleBias;

	/** The uniform buffer for the fade parameters */
	FDistanceCullFadeUniformBufferRef UniformBuffer;

	/** Frame number when last updated */
	uint32 FrameNumber;

	/** Time when fade will be finished. */
	float EndTime;

	/** Currently visible? */
	bool bIsVisible;

	/** Valid? */
	bool bValid;
};

class FGlobalDistanceFieldCacheTypeState
{
public:
	TArray<FBox> PrimitiveModifiedBounds;
};

class FGlobalDistanceFieldClipmapState
{
public:

	FGlobalDistanceFieldClipmapState()
	{
		FullUpdateOriginInPages = FInt64Vector::ZeroValue;
		LastPartialUpdateOriginInPages = FInt64Vector::ZeroValue;
		CachedClipmapCenter = FVector3f(0.0f, 0.0f, 0.0f);
		CachedClipmapExtent = 0.0f;
		CacheClipmapInfluenceRadius = 0.0f;
		CacheMostlyStaticSeparately = 1;
		LastUsedSceneDataForFullUpdate = nullptr;
	}

	FInt64Vector FullUpdateOriginInPages;
	FInt64Vector LastPartialUpdateOriginInPages;
	uint32 CacheMostlyStaticSeparately;

	FVector3f CachedClipmapCenter;
	float CachedClipmapExtent;
	float CacheClipmapInfluenceRadius;

	FGlobalDistanceFieldCacheTypeState Cache[GDF_Num];

	// Used to perform a full update of the clip map when the scene data changes
	const class FDistanceFieldSceneData* LastUsedSceneDataForFullUpdate;
};

/** Maps a single primitive to it's per-view fading state data */
typedef TMap<FPrimitiveComponentId, FPrimitiveFadingState> FPrimitiveFadingStateMap;

class FOcclusionRandomStream
{
	enum {NumSamples = 3571};
public:

	/** Default constructor - should set seed prior to use. */
	FOcclusionRandomStream()
		: CurrentSample(0)
	{
		FRandomStream RandomStream(0x83246);
		for (int32 Index = 0; Index < NumSamples; Index++)
		{
			Samples[Index] = RandomStream.GetFraction();
		}
		Samples[0] = 0.0f; // we want to make sure we have at least a few zeros
		Samples[NumSamples/3] = 0.0f; // we want to make sure we have at least a few zeros
		Samples[(NumSamples*2)/3] = 0.0f; // we want to make sure we have at least a few zeros
	}

	/** @return A random number between 0 and 1. */
	inline float GetFraction()
	{
		uint32 Current = CurrentSample.fetch_add(1);

		if (Current >= NumSamples)
		{
			Current++;
			CurrentSample.compare_exchange_strong(Current, 0);
			Current = 0;
			// It is intended here to not check if exchange worked or failed. 
			// It might be overkill to call recursively GetFraction if exchange failed. 
			// Another thread might already have reset CurrentSample and it is acceptable 
			// to have two threads returning the same Fraction at index 0
		}
		float Fraction = Samples[Current];
		return Fraction;
	}
private:

	/** Index of the last sample we produced **/
	std::atomic<uint32> CurrentSample;
	/** A list of float random samples **/
	float Samples[NumSamples];
};

/** Random table for occlusion **/
extern FOcclusionRandomStream GOcclusionRandomStream;

/** HLOD tree persistent fading and visibility state */
class FHLODVisibilityState
{
public:
	FHLODVisibilityState()
		: TemporalLODSyncTime(0.0f)
		, FOVDistanceScaleSq(1.0f)
		, UpdateCount(0)
	{}

	bool IsNodeFading(const int32 PrimIndex) const
	{
		checkSlow(IsValidPrimitiveIndex(PrimIndex));
		return PrimitiveFadingLODMap[PrimIndex];
	}

	bool IsNodeFadingOut(const int32 PrimIndex) const
	{
		checkSlow(IsValidPrimitiveIndex(PrimIndex));
		return PrimitiveFadingOutLODMap[PrimIndex];
	}

	bool IsNodeForcedVisible(const int32 PrimIndex) const
	{
		checkSlow(IsValidPrimitiveIndex(PrimIndex));
		return  ForcedVisiblePrimitiveMap[PrimIndex];
	}

	bool IsNodeForcedHidden(const int32 PrimIndex) const
	{
		checkSlow(IsValidPrimitiveIndex(PrimIndex));
		return ForcedHiddenPrimitiveMap[PrimIndex];
	}

	bool IsValidPrimitiveIndex(const int32 PrimIndex) const
	{
		return ForcedHiddenPrimitiveMap.IsValidIndex(PrimIndex);
	}

	TBitArray<>	PrimitiveFadingLODMap;
	TBitArray<>	PrimitiveFadingOutLODMap;
	TBitArray<>	ForcedVisiblePrimitiveMap;
	TBitArray<>	ForcedHiddenPrimitiveMap;
	float		TemporalLODSyncTime;
	float		FOVDistanceScaleSq;
	uint16		UpdateCount;
};

/** HLOD scene node persistent fading and visibility state */
struct FHLODSceneNodeVisibilityState
{
	FHLODSceneNodeVisibilityState()
		: UpdateCount(0)
		, bWasVisible(0)
		, bIsVisible(0)
		, bIsFading(0)
	{}

	/** Last updated FrameCount */
	uint16 UpdateCount;

	/** Persistent visibility states */
	uint16 bWasVisible	: 1;
	uint16 bIsVisible	: 1;
	uint16 bIsFading	: 1;
};

struct FShaderPrintStateData
{
	TRefCountPtr<FRDGPooledBuffer> StateBuffer;
	TRefCountPtr<FRDGPooledBuffer> EntryBuffer;
	FVector PreViewTranslation = FVector::ZeroVector;
	bool bIsLocked = false;

	void Release()
	{
		bIsLocked = false;
		PreViewTranslation = FVector::ZeroVector;
		StateBuffer = nullptr;
		EntryBuffer = nullptr;
	}
};

// Some resources used across frames can prevent execution of PS, CS and VS work across overlapping frames work.
// This struct is used to transparently double buffer the sky aerial perspective volume on some platforms,
// in order to make sure two consecutive frames have no resource dependencies, resulting in no cross frame barrier/sync point.
struct FPersistentSkyAtmosphereData
{
	FPersistentSkyAtmosphereData();

	void InitialiseOrNextFrame(ERHIFeatureLevel::Type FeatureLevel, FPooledRenderTargetDesc& AerialPerspectiveDesc, FRHICommandListImmediate& RHICmdList, bool bSeparatedAtmosphereMieRayLeigh);

	TRefCountPtr<IPooledRenderTarget> GetCurrentCameraAerialPerspectiveVolume();
	TRefCountPtr<IPooledRenderTarget> GetCurrentCameraAerialPerspectiveVolumeMieOnly();
	TRefCountPtr<IPooledRenderTarget> GetCurrentCameraAerialPerspectiveVolumeRayOnly();

	uint64 GetGPUSizeBytes(bool bLogSizes) const;

private:
	bool bInitialised;
	int32 CurrentScreenResolution;
	int32 CurrentDepthResolution;
	EPixelFormat CurrentTextureAerialLUTFormat;
	TRefCountPtr<IPooledRenderTarget> CameraAerialPerspectiveVolumes[2];
	TRefCountPtr<IPooledRenderTarget> CameraAerialPerspectiveVolumesMieOnly[2];
	TRefCountPtr<IPooledRenderTarget> CameraAerialPerspectiveVolumesRayOnly[2];
	uint8 CameraAerialPerspectiveVolumeCount;
	uint8 CameraAerialPerspectiveVolumeIndex;
	bool bSeparatedAtmosphereMieRayLeigh;
};

struct FGlobalDistanceFieldStreamingReadback
{
	FGlobalDistanceFieldStreamingReadback()
	{
		PendingStreamingReadbackBuffers.SetNum(MaxPendingStreamingReadbackBuffers);
	}

	TArray<TUniquePtr<FRHIGPUBufferReadback>> PendingStreamingReadbackBuffers;
	uint32 MaxPendingStreamingReadbackBuffers = 4;
	uint32 ReadbackBuffersWriteIndex = 0;
	uint32 ReadbackBuffersNumPending = 0;
};

struct FPersistentGlobalDistanceFieldData : public FRefCountedObject
{
	// Array of ClipmapIndex
	TArray<int32> DeferredUpdates[GDF_Num];
	TArray<int32> DeferredUpdatesForMeshSDFStreaming[GDF_Num];

	int32	UpdateFrame = 0;
	bool	bFirstFrame = true;

	bool	bInitializedOrigins = false;
	bool	bPendingReset = false;
	FGlobalDistanceFieldClipmapState ClipmapState[GlobalDistanceField::MaxClipmaps];
	int32	UpdateIndex = 0;
	FVector	CameraVelocityOffset = FVector(0);
	bool	bUpdateViewOrigin = true;
	FVector	LastViewOrigin = FVector(0);
#if WITH_MGPU
	FRHIGPUMask LastGPUMask;
#endif

	FGlobalDistanceFieldStreamingReadback StreamingReadback[GDF_Num];

	TRefCountPtr<FRDGPooledBuffer> PageFreeListAllocatorBuffer;
	TRefCountPtr<FRDGPooledBuffer> PageFreeListBuffer;
	TRefCountPtr<IPooledRenderTarget> PageAtlasTexture;
	TRefCountPtr<IPooledRenderTarget> CoverageAtlasTexture;
	TRefCountPtr<FRDGPooledBuffer> PageObjectGridBuffer;
	TRefCountPtr<IPooledRenderTarget> PageTableCombinedTexture;
	TRefCountPtr<IPooledRenderTarget> PageTableLayerTextures[GDF_Num];
	TRefCountPtr<IPooledRenderTarget> MipTexture;

	uint64 GetGPUSizeBytes(bool bLogSizes) const;
};
