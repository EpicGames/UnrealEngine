// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneVisibility.h"
#include "SceneVisibilityPrivate.h"
#include "CoreMinimal.h"
#include "ComputeWorkerInterface.h"
#include "HAL/ThreadSafeCounter.h"
#include "Stats/Stats.h"
#include "Misc/MemStack.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "EngineDefines.h"
#include "EngineGlobals.h"
#include "EngineStats.h"
#include "RHIDefinitions.h"
#include "SceneTypes.h"
#include "SceneInterface.h"
#include "RendererInterface.h"
#include "PrimitiveViewRelevance.h"
#include "Materials/Material.h"
#include "MaterialShared.h"
#include "PrimitiveDrawingUtils.h"
#include "ScenePrivateBase.h"
#include "PostProcess/SceneRenderTargets.h"
#include "SceneCore.h"
#include "SceneOcclusion.h"
#include "LightSceneInfo.h"
#include "SceneRendering.h"
#include "DeferredShadingRenderer.h"
#include "DynamicPrimitiveDrawing.h"
#include "FXSystem.h"
#include "PostProcess/PostProcessing.h"
#include "SceneView.h"
#include "SkyAtmosphereRendering.h"
#include "Engine/LODActor.h"
#include "GPUScene.h"
#include "TranslucentRendering.h"
#include "TranslucentLighting.h"
#include "FrontLayerTranslucency.h"
#include "Async/ParallelFor.h"
#include "HairStrands/HairStrandsRendering.h"
#include "HairStrands/HairStrandsData.h"
#include "RectLightSceneProxy.h"
#include "Math/Halton.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Algo/Unique.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "InstanceCulling/InstanceCullingOcclusionQuery.h"
#include "PostProcess/TemporalAA.h"
#include "RayTracing/RayTracingInstanceCulling.h"
#include "RayTracing/RayTracing.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"

#include "SceneViewExtension.h"
#include "RenderCore.h"
#include "UnrealEngine.h"
#include "VT/VirtualTextureSystem.h"
#include "NaniteSceneProxy.h"
#include "ViewDebug.h"
#include "DecalRenderingCommon.h"
#include "CompositionLighting/PostProcessDeferredDecals.h"
#include "DecalRenderingShared.h"
#include "SceneCulling/SceneCullingRenderer.h"
#include "VelocityRendering.h"
#include "SceneViewState.h"

bool GDistanceCullToSphereEdge = true;
static FAutoConsoleVariableRef CVarDistanceCullToSphereEdge(
	TEXT("r.DistanceCullToSphereEdge"),
	GDistanceCullToSphereEdge,
	TEXT("If true frustum cull will check distance to bounding sphere edge rather than origin."),
	ECVF_RenderThreadSafe
);

static float GWireframeCullThreshold = 5.0f;
static FAutoConsoleVariableRef CVarWireframeCullThreshold(
	TEXT("r.WireframeCullThreshold"),
	GWireframeCullThreshold,
	TEXT("Threshold below which objects in ortho wireframe views will be culled."),
	ECVF_RenderThreadSafe
	);

float GMinScreenRadiusForLights = 0.03f;
static FAutoConsoleVariableRef CVarMinScreenRadiusForLights(
	TEXT("r.MinScreenRadiusForLights"),
	GMinScreenRadiusForLights,
	TEXT("Threshold below which lights will be culled."),
	ECVF_RenderThreadSafe
	);

float GMinScreenRadiusForDepthPrepass = 0.03f;
static FAutoConsoleVariableRef CVarMinScreenRadiusForDepthPrepass(
	TEXT("r.MinScreenRadiusForDepthPrepass"),
	GMinScreenRadiusForDepthPrepass,
	TEXT("Threshold below which meshes will be culled from depth only pass."),
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarTemporalAASamples(
	TEXT("r.TemporalAASamples"),
	8,
	TEXT("Number of jittered positions for temporal AA (4, 8=default, 16, 32, 64)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarTemporalAAScaleSamples(
	TEXT("r.TemporalAAScaleSamples"),
	1,
	TEXT("Whether or not to scale the number of jittered positions for temporal AA when upsampling to maintain a consistent density."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarInvertTemporalJitterX(
	TEXT("r.InvertTemporalJitterX"),
	0,
	TEXT("Whether or not to invert the X value of jittered positions for temporal AA."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarInvertTemporalJitterY(
	TEXT("r.InvertTemporalJitterY"),
	0,
	TEXT("Whether or not to invert the Y value of jittered positions for temporal AA."),
	ECVF_RenderThreadSafe);

static int32 GHZBOcclusion = 0;
static FAutoConsoleVariableRef CVarHZBOcclusion(
	TEXT("r.HZBOcclusion"),
	GHZBOcclusion,
	TEXT("Defines which occlusion system is used.\n")
	TEXT(" 0: Hardware occlusion queries\n")
	TEXT(" 1: Use HZB occlusion system (default, less GPU and CPU cost, more conservative results)")
	TEXT(" 2: Force HZB occlusion system (overrides rendering platform preferences)"),
	ECVF_RenderThreadSafe
	);

int32 GOcclusionFeedback_Enable = 0;
static FAutoConsoleVariableRef CVarOcclusionFeedback_Enable(
	TEXT("r.OcclusionFeedback.Enable"),
	GOcclusionFeedback_Enable,
	TEXT("Whether to enable occlusion system based on a rendering feedback. Currently works only with a mobile rendering\n"),
	ECVF_RenderThreadSafe
);

static int32 GVisualizeOccludedPrimitives = 0;
static FAutoConsoleVariableRef CVarVisualizeOccludedPrimitives(
	TEXT("r.VisualizeOccludedPrimitives"),
	GVisualizeOccludedPrimitives,
	TEXT("Draw boxes for all occluded primitives.\n")
	TEXT(" 0: Do not draw bounding boxes for occluded primitives (default).\n")
	TEXT(" 1: Draw the primitive bounding boxes for all occluded primitives.\n")
	TEXT(" 2: Draw the primitive bounding boxes for all occluded primitives after expansion. See r.Expand*BBoxes* for expansion.\n"),
	ECVF_RenderThreadSafe | ECVF_Cheat
);

static int32 GVisualizePrimitiveBBoxes = 0;
static FAutoConsoleVariableRef CVarVisualizePrimitiveBBoxes(
	TEXT("r.VisualizePrimitiveBBoxes"),
	GVisualizePrimitiveBBoxes,
	TEXT("Draw boxes for all primitives.\n")
	TEXT(" 0: Do not draw bounding boxes for all primitives (default).\n")
	TEXT(" 1: Draw bounding boxes only after expansion. See r.Expand*BBoxes for expansion.\n")
	TEXT(" 2: Draw bounding boxes for both before expansion and after expansion."),
	ECVF_RenderThreadSafe | ECVF_Cheat
);

static int32 GAllowSubPrimitiveQueries = 1;
static FAutoConsoleVariableRef CVarAllowSubPrimitiveQueries(
	TEXT("r.AllowSubPrimitiveQueries"),
	GAllowSubPrimitiveQueries,
	TEXT("Enables sub primitive queries, currently only used by hierarchical instanced static meshes. 1: Enable, 0 Disabled. When disabled, one query is used for the entire proxy."),
	ECVF_RenderThreadSafe
	);

RENDERER_API TAutoConsoleVariable<float> CVarStaticMeshLODDistanceScale(
	TEXT("r.StaticMeshLODDistanceScale"),
	1.0f,
	TEXT("Scale factor for the distance used in computing discrete LOD for static meshes. (defaults to 1)\n")
	TEXT("(higher values make LODs transition earlier, e.g., 2 is twice as fast / half the distance)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarAutomaticViewMipBiasMin(
	TEXT("r.ViewTextureMipBias.Min"),
	-2.0f,
	TEXT("Automatic view mip bias's minimum value (default to -2)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarAutomaticViewMipBiasOffset(
	TEXT("r.ViewTextureMipBias.Offset"),
	-0.3f,
	TEXT("Automatic view mip bias's constant offset (default to -0.3)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAutomaticViewMipBiasQuantization(
	TEXT("r.ViewTextureMipBias.Quantization"),
	1024,
	TEXT("Quantization step count to use for automatic view mip bias, measured between -4.0f and 4.0f.\n")
	TEXT("Lowering this can significantly reduce unique sampler states when Dynamic Resolution is active."),
	ECVF_RenderThreadSafe);

static int32 GILCUpdatePrimTaskEnabled = 1;

static FAutoConsoleVariableRef CVarILCUpdatePrimitivesTask(
	TEXT("r.Cache.UpdatePrimsTaskEnabled"),
	GILCUpdatePrimTaskEnabled,
	TEXT("Enable threading for ILC primitive update.  Will overlap with the rest the end of InitViews."),
	ECVF_RenderThreadSafe
	);

int32 GEarlyInitDynamicShadows = 1;
static FAutoConsoleVariableRef CVarEarlyInitDynamicShadows(
	TEXT("r.EarlyInitDynamicShadows"),
	GEarlyInitDynamicShadows,
	TEXT("Starts shadow culling tasks earlier in the frame."),
	ECVF_RenderThreadSafe
);

static int32 GFramesNotOcclusionTestedToExpandBBoxes = 5;
static FAutoConsoleVariableRef CVarFramesNotOcclusionTestedToExpandBBoxes(
	TEXT("r.GFramesNotOcclusionTestedToExpandBBoxes"),
	GFramesNotOcclusionTestedToExpandBBoxes,
	TEXT("If we don't occlusion test a primitive for this many frames, then we expand the BBox when we do occlusion test it for a few frames. See also r.ExpandNewlyOcclusionTestedBBoxesAmount, r.FramesToExpandNewlyOcclusionTestedBBoxes"),
	ECVF_RenderThreadSafe
);

static int32 GFramesToExpandNewlyOcclusionTestedBBoxes = 2;
static FAutoConsoleVariableRef CVarFramesToExpandNewlyOcclusionTestedBBoxes(
	TEXT("r.FramesToExpandNewlyOcclusionTestedBBoxes"),
	GFramesToExpandNewlyOcclusionTestedBBoxes,
	TEXT("If we don't occlusion test a primitive for r.GFramesNotOcclusionTestedToExpandBBoxes frames, then we expand the BBox when we do occlusion test it for this number of frames. See also r.GFramesNotOcclusionTestedToExpandBBoxes, r.ExpandNewlyOcclusionTestedBBoxesAmount"),
	ECVF_RenderThreadSafe
);

static float GExpandNewlyOcclusionTestedBBoxesAmount = 0.0f;
static FAutoConsoleVariableRef CVarExpandNewlyOcclusionTestedBBoxesAmount(
	TEXT("r.ExpandNewlyOcclusionTestedBBoxesAmount"),
	GExpandNewlyOcclusionTestedBBoxesAmount,
	TEXT("If we don't occlusion test a primitive for r.GFramesNotOcclusionTestedToExpandBBoxes frames, then we expand the BBox when we do occlusion test it for a few frames by this amount. See also r.FramesToExpandNewlyOcclusionTestedBBoxes, r.GFramesNotOcclusionTestedToExpandBBoxes."),
	ECVF_RenderThreadSafe
);

static float GExpandAllTestedBBoxesAmount = 0.0f;
static FAutoConsoleVariableRef CVarExpandAllTestedBBoxesAmount(
	TEXT("r.ExpandAllOcclusionTestedBBoxesAmount"),
	GExpandAllTestedBBoxesAmount,
	TEXT("Amount to expand all occlusion test bounds by.\n")
	TEXT("This variable can be applied in conjunction with r.ExpandNewlyOcclusionTestedBBoxesScreenSpace and r.ExpandAllOcclusionTestedBBoxesScreenSpace. This variable always applies before those two.\n"),
	ECVF_RenderThreadSafe
);

// TODO: UE-273143 This is the start of a significant refactor in the way that objects get culled.
// The previous implementation took cvars to determine how much all bounding boxes should expand in world space.
// The current implementation extends this to expand by a screen space amount (in pixels).
// The idealized ultimate solution would take into account camera positional and/or angular velocity, over the previous n frames,
// and extrapolate to determine the size at which bboxes should expand (in the event that they are likely to become unoccluded in the next m frames).

static float GExpandNewlyOcclusionTestedBBoxesScreenSpace = 0.0f;
static FAutoConsoleVariableRef CVarExpandNewlyOcclusionTestedBBoxesScreenSpace(
	TEXT("r.ExpandNewlyOcclusionTestedBBoxesScreenSpace"),
	GExpandNewlyOcclusionTestedBBoxesScreenSpace,
	TEXT("If we don't occlusion test a primitive for r.GFramesNotOcclusionTestedToExpandBBoxes frames, then we expand the BBox when we do occlusion test it for a few frames by this amount. See also r.FramesToExpandNewlyOcclusionTestedBBoxes, r.GFramesNotOcclusionTestedToExpandBBoxes."),
	ECVF_RenderThreadSafe
);

static float GExpandAllTestedBBoxesScreenSpace = 0.0f;
static FAutoConsoleVariableRef CVarExpandAllTestedBBoxesScreenSpace(
	TEXT("r.ExpandAllOcclusionTestedBBoxesScreenSpace"),
	GExpandAllTestedBBoxesScreenSpace,
	TEXT("Screen space amount in pixels to increase the occlusion test of BBoxes by.\n")
	TEXT("This variable can be applied in conjunction with r.ExpandNewlyOcclusionTestedBBoxesAmount and r.ExpandAllOcclusionTestedBBoxesAmount. This variable always applies after those two.\n"),
	ECVF_RenderThreadSafe
);

static float GNeverOcclusionTestDistance = 0.0f;
static FAutoConsoleVariableRef CVarNeverOcclusionTestDistance(
	TEXT("r.NeverOcclusionTestDistance"),
	GNeverOcclusionTestDistance,
	TEXT("When the distance between the viewpoint and the bounding sphere center is less than this, never occlusion cull."),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static int32 GForceSceneHasDecals = 0;
static FAutoConsoleVariableRef CVarForceSceneHasDecals(
	TEXT("r.ForceSceneHasDecals"),
	GForceSceneHasDecals,
	TEXT("Whether to always assume that scene has decals, so we don't switch depth state conditionally. This can significantly reduce total number of PSOs at a minor GPU cost."),
	ECVF_RenderThreadSafe
);

static float GCameraCutTranslationThreshold = 10000.0f;
static FAutoConsoleVariableRef CVarCameraCutTranslationThreshold(
	TEXT("r.CameraCutTranslationThreshold"),
	GCameraCutTranslationThreshold,
	TEXT("The maximum camera translation disatance in centimeters allowed between two frames before a camera cut is automatically inserted."),
	ECVF_RenderThreadSafe
);

float GOverlayMaterialMaxDrawDistanceOverride = 0.0f;
static FAutoConsoleVariableRef CVarGMaterialOverlayMaxDistanceOverride(
	TEXT("r.OverlayMaterialMaxDrawDistanceOverride"),
	GOverlayMaterialMaxDrawDistanceOverride,
	TEXT("Override the max draw distance for an Overlay Material. Any value greater than 0 will trigger the override."),
	ECVF_RenderThreadSafe
);

/** Distance fade cvars */
static int32 GDisableLODFade = false;
static FAutoConsoleVariableRef CVarDisableLODFade( TEXT("r.DisableLODFade"), GDisableLODFade, TEXT("Disable fading for distance culling"), ECVF_RenderThreadSafe );

static float GFadeTime = 0.25f;
static FAutoConsoleVariableRef CVarLODFadeTime( TEXT("r.LODFadeTime"), GFadeTime, TEXT("How long LOD takes to fade (in seconds)."), ECVF_RenderThreadSafe );

static float GDistanceFadeMaxTravel = 1000.0f;
static FAutoConsoleVariableRef CVarDistanceFadeMaxTravel( TEXT("r.DistanceFadeMaxTravel"), GDistanceFadeMaxTravel, TEXT("Max distance that the player can travel during the fade time."), ECVF_RenderThreadSafe );

extern int32 GVisibilitySkipAlwaysVisible;

static int32 GVisibilityTaskSchedule = 1;
static FAutoConsoleVariableRef CVarVisibilityTaskSchedule(
	TEXT("r.Visibility.TaskSchedule"),
	GVisibilityTaskSchedule,
	TEXT("Controls how the visibility task graph is scheduled.")
	TEXT("0: Work is primarily done on the render thread with the potential for parallel help;")
	TEXT("1: Work is done on an async task graph (if supported by platform);"),
	ECVF_RenderThreadSafe
);

static int32 GFrustumCullNumPrimitivesPerTask = 0;
static FAutoConsoleVariableRef CVarFrustumCullNumPrimitivesPerTask(
	TEXT("r.Visibility.FrustumCull.NumPrimitivesPerTask"),
	GFrustumCullNumPrimitivesPerTask,
	TEXT("Assigns a fixed number of primitives for each frustum cull task.")
	TEXT(" 0: Automatic;")
	TEXT(">0: Fixed number of primitives per task (clamped to fixed limits);"),
	ECVF_RenderThreadSafe
);

static bool GFrustumCullEnabled = true;
static FAutoConsoleVariableRef CVarFrustumCullEnable(
	TEXT("r.Visibility.FrustumCull.Enabled"),
	GFrustumCullEnabled,
	TEXT("Enables frustum culling."),
	ECVF_RenderThreadSafe
);

static bool GFrustumCullUseOctree = false;
static FAutoConsoleVariableRef CVarFrustumCullUseOctree(
	TEXT("r.Visibility.FrustumCull.UseOctree"),
	GFrustumCullUseOctree,
	TEXT("Use the octree for visibility calculations."),
	ECVF_RenderThreadSafe
);

static bool GFrustumCullUseSphereTestFirst = false;
static FAutoConsoleVariableRef CVarFrustumCullUseSphereTestFirst(
	TEXT("r.Visibility.FrustumCull.UseSphereTestFirst"),
	GFrustumCullUseSphereTestFirst,
	TEXT("Performance tweak. Uses a sphere cull before and in addition to a box for frustum culling."),
	ECVF_RenderThreadSafe
);

static bool GFrustumCullUseFastIntersect = true;
static FAutoConsoleVariableRef CVarFrustumCullUseFastIntersect(
	TEXT("r.Visibility.FrustumCull.UseFastIntersect"),
	GFrustumCullUseFastIntersect,
	TEXT("Use optimized 8 plane fast intersection code if we have 8 permuted planes."),
	ECVF_RenderThreadSafe
);

static int32 GOcclusionCullMaxQueriesPerTask = 0;
static FAutoConsoleVariableRef CVarOcclusionCullMaxQueriesPerTask(
	TEXT("r.Visibility.OcclusionCull.MaxQueriesPerTask"),
	GOcclusionCullMaxQueriesPerTask,
	TEXT("Assigns a fixed number of occlusion queries for each occlusion cull task.")
	TEXT(" 0: Automatic;")
	TEXT(">0: Fixed number of occlusion queries per task;"),
	ECVF_RenderThreadSafe
);

static int32 GNumDynamicMeshElementTasks = 4;
static FAutoConsoleVariableRef CVarNumDynamicMeshElementTasks(
	TEXT("r.Visibility.DynamicMeshElements.NumMainViewTasks"),
	GNumDynamicMeshElementTasks,
	TEXT("Controls the number of gather dynamic mesh elements tasks to run asynchronously during view visibility."),
	ECVF_RenderThreadSafe
);

inline uint32 GetNumDynamicMeshElementTasks()
{
	if (!IsParallelGatherDynamicMeshElementsEnabled())
	{
		return 0;
	}

	return FMath::Clamp<int32>(GNumDynamicMeshElementTasks, 0, LowLevelTasks::FScheduler::Get().GetNumWorkers());
}

static bool GOcclusionCullEnabled = true;
static FAutoConsoleVariableRef CVarOcclusionCullEnable(
	// TODO: Move to r.Visibility.OcclusionCull.Enable. Still several explicit references.
	TEXT("r.AllowOcclusionQueries"),
	GOcclusionCullEnabled,
	TEXT("Enables hardware occlusion culling."),
	ECVF_RenderThreadSafe
);

bool DoOcclusionQueries(const FSceneViewFamily& ViewFamily)
{
	return GOcclusionCullEnabled && !ViewFamily.EngineShowFlags.DisableOcclusionQueries;
}

bool FSceneRenderer::DoOcclusionQueries() const
{
	return ::DoOcclusionQueries(ViewFamily);
}

static int32 GRelevanceNumPrimitivesPerPacket = 0;
static FAutoConsoleVariableRef CVarRelevanceNumPrimitivesPerPacket(
	TEXT("r.Visibility.Relevance.NumPrimitivesPerPacket"),
	GRelevanceNumPrimitivesPerPacket,
	TEXT("Assigns a fixed number of primitives for each relevance packet.")
	TEXT(" 0: Automatic;")
	TEXT(">0: Fixed number of primitives per packet (clamped to fixed limits);"),
	ECVF_RenderThreadSafe
);

float GLightMaxDrawDistanceScale = 1.0f;
static FAutoConsoleVariableRef CVarLightMaxDrawDistanceScale(
	TEXT("r.LightMaxDrawDistanceScale"),
	GLightMaxDrawDistanceScale,
	TEXT("Scale applied to the MaxDrawDistance of lights.  Useful for fading out local lights more aggressively on some platforms."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

#if !UE_BUILD_SHIPPING

static TAutoConsoleVariable<int32> CVarTAADebugOverrideTemporalIndex(
	TEXT("r.TemporalAA.Debug.OverrideTemporalIndex"), -1,
	TEXT("Override the temporal index for debugging purposes."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarFreezeTemporalSequences(
	TEXT("r.Test.FreezeTemporalSequences"), 0,
	TEXT("Freezes all temporal sequences."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarFreezeTemporalHistories(
	TEXT("r.Test.FreezeTemporalHistories"), 0,
	TEXT("Freezes all temporal histories as well as the temporal sequence."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarFreezeTemporalHistoriesProgress(
	TEXT("r.Test.FreezeTemporalHistories.Progress"), 0,
	TEXT("Progress the temporal histories by one frame when modified."),
	ECVF_RenderThreadSafe);

#endif

TRACE_DECLARE_INT_COUNTER(Scene_Visibility_NumProcessedPrimitives, TEXT("Scene/Visibility/NumProcessedPrimitives"));
TRACE_DECLARE_INT_COUNTER(Scene_Visibility_FrustumCull_NumCulledPrimitives, TEXT("Scene/Visibility/FrustumCull/NumCulledPrimitives"));
TRACE_DECLARE_INT_COUNTER(Scene_Visibility_FrustumCull_NumPrimitivesPerTask, TEXT("Scene/Visibility/FrustumCull/NumPrimitivesPerTask"));
TRACE_DECLARE_INT_COUNTER(Scene_Visibility_OcclusionCull_NumTestedQueries, TEXT("Scene/Visibility/OcclusionCull/NumTestedQueries"));
TRACE_DECLARE_INT_COUNTER(Scene_Visibility_OcclusionCull_NumCulledPrimitives, TEXT("Scene/Visibility/OcclusionCull/NumCulledPrimitives"));
TRACE_DECLARE_INT_COUNTER(Scene_Visibility_Relevance_NumPrimitivesPerPacket, TEXT("Scene/Visibility/Relevance/NumPrimitivesPerPacket"));
TRACE_DECLARE_INT_COUNTER(Scene_Visibility_Relevance_NumPrimitivesProcessed, TEXT("Scene/Visibility/Relevance/NumPrimitivesProcessed"));

///////////////////////////////////////////////////////////////////////////////

IVisibilityTaskData* LaunchVisibilityTasks(FRHICommandListImmediate& RHICmdList, FSceneRenderer& SceneRenderer, const UE::Tasks::FTask& BeginInitVisibilityPrerequisites)
{
	FVisibilityTaskData* TaskData = SceneRenderer.Allocator.Create<FVisibilityTaskData>(RHICmdList, SceneRenderer);
	TaskData->LaunchVisibilityTasks(BeginInitVisibilityPrerequisites);
	return TaskData;
}

///////////////////////////////////////////////////////////////////////////////

bool FViewInfo::IsDistanceCulled( float DistanceSquared, float MinDrawDistance, float InMaxDrawDistance, const FPrimitiveSceneInfo* PrimitiveSceneInfo)
{
	bool bMayBeFading;
	bool bFadingIn;
	bool bDistanceCulled = IsDistanceCulled_AnyThread(DistanceSquared, MinDrawDistance, InMaxDrawDistance, PrimitiveSceneInfo, bMayBeFading, bFadingIn);

	if (bMayBeFading)
	{
		bDistanceCulled = UpdatePrimitiveFadingState(PrimitiveSceneInfo, bFadingIn);
	}

	return bDistanceCulled;
}

bool FViewInfo::IsDistanceCulled_AnyThread(float DistanceSquared, float InMinDrawDistance, float InMaxDrawDistance, const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool& bOutMayBeFading, bool& bOutFadingIn) const
{
	const float MaxDrawDistanceScale = GetCachedScalabilityCVars().ViewDistanceScale;
	const float FadeRadius = GDisableLODFade ? 0.0f : GDistanceFadeMaxTravel;
	const float MaxDrawDistance = InMaxDrawDistance * MaxDrawDistanceScale;
	const float MinDrawDistance = InMinDrawDistance * MaxDrawDistanceScale;

	bool bHasMaxDrawDistance = InMaxDrawDistance != FLT_MAX;
	bool bHasMinDrawDistance = InMinDrawDistance > 0;
	bOutMayBeFading = false;


	if (!bHasMaxDrawDistance && !bHasMinDrawDistance)
	{
		return false;
	}

	// If cull distance is disabled, always show (except foliage)
	if (Family->EngineShowFlags.DistanceCulledPrimitives && !PrimitiveSceneInfo->Proxy->IsDetailMesh())
	{
		return false;
	}

	// The primitive is always culled if it exceeds the max fade distance.
	if ((bHasMaxDrawDistance && DistanceSquared > FMath::Square(MaxDrawDistance + FadeRadius)) || (bHasMinDrawDistance && DistanceSquared < FMath::Square(MinDrawDistance)))
	{
		return true;
	}

	const bool bDistanceCulled = bHasMaxDrawDistance && (DistanceSquared > FMath::Square(MaxDrawDistance));
	const bool bMayBeFading = bHasMaxDrawDistance && (DistanceSquared > FMath::Square(MaxDrawDistance - FadeRadius));

	if (!GDisableLODFade && bMayBeFading && State != NULL && !bDisableDistanceBasedFadeTransitions && PrimitiveSceneInfo->Proxy->IsUsingDistanceCullFade())
	{
		// Don't update primitive fading state yet because current thread may be not render thread
		bOutMayBeFading = true;
		bOutFadingIn = !bDistanceCulled;
	}

	return bDistanceCulled && !bOutMayBeFading;
}

inline bool IntersectBox8Plane(const FVector& InOrigin, const FVector& InExtent, const FPlane*PermutedPlanePtr)
{
	// this removes a lot of the branches as we know there's 8 planes
	// copied directly out of ConvexVolume.cpp
	const VectorRegister Origin = VectorLoadFloat3(&InOrigin);
	const VectorRegister Extent = VectorLoadFloat3(&InExtent);

	const VectorRegister PlanesX_0 = VectorLoadAligned(&PermutedPlanePtr[0]);
	const VectorRegister PlanesY_0 = VectorLoadAligned(&PermutedPlanePtr[1]);
	const VectorRegister PlanesZ_0 = VectorLoadAligned(&PermutedPlanePtr[2]);
	const VectorRegister PlanesW_0 = VectorLoadAligned(&PermutedPlanePtr[3]);

	const VectorRegister PlanesX_1 = VectorLoadAligned(&PermutedPlanePtr[4]);
	const VectorRegister PlanesY_1 = VectorLoadAligned(&PermutedPlanePtr[5]);
	const VectorRegister PlanesZ_1 = VectorLoadAligned(&PermutedPlanePtr[6]);
	const VectorRegister PlanesW_1 = VectorLoadAligned(&PermutedPlanePtr[7]);

	// Splat origin into 3 vectors
	VectorRegister OrigX = VectorReplicate(Origin, 0);
	VectorRegister OrigY = VectorReplicate(Origin, 1);
	VectorRegister OrigZ = VectorReplicate(Origin, 2);
	// Splat the already abs Extent for the push out calculation
	VectorRegister AbsExtentX = VectorReplicate(Extent, 0);
	VectorRegister AbsExtentY = VectorReplicate(Extent, 1);
	VectorRegister AbsExtentZ = VectorReplicate(Extent, 2);

	// Calculate the distance (x * x) + (y * y) + (z * z) - w
	VectorRegister DistX_0 = VectorMultiply(OrigX, PlanesX_0);
	VectorRegister DistY_0 = VectorMultiplyAdd(OrigY, PlanesY_0, DistX_0);
	VectorRegister DistZ_0 = VectorMultiplyAdd(OrigZ, PlanesZ_0, DistY_0);
	VectorRegister Distance_0 = VectorSubtract(DistZ_0, PlanesW_0);
	// Now do the push out FMath::Abs(x * x) + FMath::Abs(y * y) + FMath::Abs(z * z)
	VectorRegister PushX_0 = VectorMultiply(AbsExtentX, VectorAbs(PlanesX_0));
	VectorRegister PushY_0 = VectorMultiplyAdd(AbsExtentY, VectorAbs(PlanesY_0), PushX_0);
	VectorRegister PushOut_0 = VectorMultiplyAdd(AbsExtentZ, VectorAbs(PlanesZ_0), PushY_0);

	// Check for completely outside
	if (VectorAnyGreaterThan(Distance_0, PushOut_0))
	{
		return false;
	}

	// Calculate the distance (x * x) + (y * y) + (z * z) - w
	VectorRegister DistX_1 = VectorMultiply(OrigX, PlanesX_1);
	VectorRegister DistY_1 = VectorMultiplyAdd(OrigY, PlanesY_1, DistX_1);
	VectorRegister DistZ_1 = VectorMultiplyAdd(OrigZ, PlanesZ_1, DistY_1);
	VectorRegister Distance_1 = VectorSubtract(DistZ_1, PlanesW_1);
	// Now do the push out FMath::Abs(x * x) + FMath::Abs(y * y) + FMath::Abs(z * z)
	VectorRegister PushX_1 = VectorMultiply(AbsExtentX, VectorAbs(PlanesX_1));
	VectorRegister PushY_1 = VectorMultiplyAdd(AbsExtentY, VectorAbs(PlanesY_1), PushX_1);
	VectorRegister PushOut_1 = VectorMultiplyAdd(AbsExtentZ, VectorAbs(PlanesZ_1), PushY_1);

	// Check for completely outside
	if (VectorAnyGreaterThan(Distance_1, PushOut_1))
	{
		return false;
	}
	return true;
}

struct FFrustumCullingFlags
{
	bool bShouldVisibilityCull;
	bool bUseCustomCulling;
	bool bUseSphereTestFirst;
	bool bUseFastIntersect;
	bool bUseVisibilityOctree;
	bool bHasHiddenPrimitives;
	bool bHasShowOnlyPrimitives;
};

// Returns true if the frustum and bounds intersect
inline bool IsPrimitiveVisible(FViewInfo& View, const FPlane* PermutedPlanePtr, const FConvexVolume& ViewCullingFrustum, const FPrimitiveBounds& Bounds, int32 VisibilityId, FFrustumCullingFlags Flags)
{
	// The custom culling and sphere culling are additional tests, meaning that if they pass, the
	// remaining culling tests will still be performed.  If any of the tests fail, then the primitive
	// is culled, and the remaining tests do not need be performed

	if (Flags.bUseCustomCulling && !View.CustomVisibilityQuery->IsVisible(VisibilityId, FBoxSphereBounds(Bounds.BoxSphereBounds.Origin, Bounds.BoxSphereBounds.BoxExtent, Bounds.BoxSphereBounds.SphereRadius)))
	{
		return false;
	}

	if (Flags.bUseSphereTestFirst && !ViewCullingFrustum.IntersectSphere(Bounds.BoxSphereBounds.Origin, Bounds.BoxSphereBounds.SphereRadius))
	{
		return false;
	}

	if (Flags.bUseFastIntersect)
	{
		return IntersectBox8Plane(Bounds.BoxSphereBounds.Origin, Bounds.BoxSphereBounds.BoxExtent, PermutedPlanePtr);
	}
	else
	{
		return ViewCullingFrustum.IntersectBox(Bounds.BoxSphereBounds.Origin, Bounds.BoxSphereBounds.BoxExtent);
	}
}

inline bool IsPrimitiveHidden(const FScene& Scene, FViewInfo& View, int32 PrimitiveIndex, FFrustumCullingFlags Flags)
{
	// If any primitives are explicitly hidden, remove them now.
	if (Flags.bHasHiddenPrimitives && View.HiddenPrimitives.Contains(Scene.PrimitiveComponentIds[PrimitiveIndex]))
	{
		return true;
	}

	// If the view has any show only primitives, hide everything else
	if (Flags.bHasShowOnlyPrimitives && !View.ShowOnlyPrimitives->Contains(Scene.PrimitiveComponentIds[PrimitiveIndex]))
	{
		return true;
	}

	return false;
}

#if RHI_RAYTRACING

inline bool ShouldCullForRayTracing(const FScene& Scene, FViewInfo& View, int32 PrimitiveIndex)
{
	const FRayTracingCullingParameters& RayTracingCullingParameters = View.RayTracingCullingParameters;

	if (RayTracing::CullPrimitiveByFlags(RayTracingCullingParameters, &Scene, PrimitiveIndex))
	{
		return true;
	}

	const bool bIsFarFieldPrimitive = EnumHasAnyFlags(Scene.PrimitiveRayTracingFlags[PrimitiveIndex], ERayTracingPrimitiveFlags::FarField);
	const Experimental::FHashElementId GroupId = Scene.PrimitiveRayTracingGroupIds[PrimitiveIndex];

	if (RayTracingCullingParameters.bCullUsingGroupIds && GroupId.IsValid())
	{
		const FBoxSphereBounds& GroupBounds = Scene.PrimitiveRayTracingGroups.GetByElementId(GroupId).Value.Bounds;
		const float GroupMinDrawDistance = Scene.PrimitiveRayTracingGroups.GetByElementId(GroupId).Value.MinDrawDistance;
		return RayTracing::ShouldCullBounds(RayTracingCullingParameters, GroupBounds, GroupMinDrawDistance, bIsFarFieldPrimitive);
	}
	else
	{
		const FPrimitiveBounds& RESTRICT Bounds = Scene.PrimitiveBounds[PrimitiveIndex];
		return RayTracing::ShouldCullBounds(RayTracingCullingParameters, Bounds.BoxSphereBounds, Bounds.MinDrawDistance, bIsFarFieldPrimitive);
	}
};
#endif //RHI_RAYTRACING

///////////////////////////////////////////////////////////////////////////////

// (brad.blankenship): This is an initial prototype of the work for having objects cull themselves by
// their projected screen size. It's very simple right now, only taking into account the r^2 vs. the cutoff. 
// as part of the future work for the prototype we are looking at enabling a fade between visible and not visible
// to reduce popin, as well as seeing what it would take to piggyback off of other features such as closeness to
// screen center, view relevance, etc.
float GMinScreenBoundsRadius = 0.0f;
static FAutoConsoleVariableRef CVarFrustumCullMinScreenBoundsRadius(
	TEXT("r.MinScreenBoundsRadius.Experimental"),
	GMinScreenBoundsRadius,
	TEXT("Specifies a minimum value for the screen space radius of the projected bounding sphere for visibility tests. ")
	TEXT("Value must be from 0.0 (disabled) to 1.0 (100% of the longest dimension of the screen). "),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

float GMinScreenBoundsSphereRadiusCutoff = 1.0f;
static FAutoConsoleVariableRef CVarFrustumCullMinScreenBoundsSphereRadiusCutoff(
	TEXT("r.MinScreenBoundsSphereRadiusCutoff.Experimental"),
	GMinScreenBoundsSphereRadiusCutoff,
	TEXT("Specifies a minimum value for the minimum size of a sphere radius needed for testing objects. ")
	TEXT("This helps to account for objects with near zero bounds that have not been properly updated or ")
	TEXT("are setup to not be dependent on their bounds for rendering checks. Defaults to 1.0 (in world units)."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

float GMinScreenBoundsIgnoreDistance = 0.0f;
static FAutoConsoleVariableRef CVarFrustumCullMinScreenBoundsIgnoreDistance(
	TEXT("r.MinScreenBoundsIgnoreDistance.Experimental"),
	GMinScreenBoundsIgnoreDistance,
	TEXT("Specifies a minimum distance at which screen bounds radius testing will begin and anything less will be ignored. ")
	TEXT("Value must be from 0.0 (disabled) and upwards (in world units). "),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

float GMinScreenBoundsRadiusRuntimeReflection = 0.02f;
static FAutoConsoleVariableRef CVarMinScreenBoundsRadiusRuntimeReflection(
	TEXT("r.MinScreenBoundsRadius.RuntimeReflection"),
	GMinScreenBoundsRadiusRuntimeReflection,
	TEXT("Specifies a minimum value for the screen space radius of the projected bounding sphere for visibility tests, for runtime reflection captures. ")
	TEXT("Value must be from 0.0 (disabled) to 1.0 (100% of the longest dimension of the screen). "),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

///////////////////////////////////////////////////////////////////////////////

inline void ComputeDistances(const FPrimitiveBounds& Bounds, const FVector& ViewOriginForDistanceCulling, float& OutClosestDistSq, float& OutFurthestDistSq)
{
	float Dist = (Bounds.BoxSphereBounds.Origin - ViewOriginForDistanceCulling).Length();
	float ClosestDist = Dist - Bounds.BoxSphereBounds.SphereRadius;
	float FurthestDist = Dist + Bounds.BoxSphereBounds.SphereRadius;

	OutClosestDistSq = Dist >= Bounds.BoxSphereBounds.SphereRadius ? FMath::Square(ClosestDist) : 0;
	OutFurthestDistSq = FMath::Square(FurthestDist);
}

static void CullOctree(const FScene& Scene, FViewInfo& View, const FFrustumCullingFlags& Flags, FSceneBitArray& OutVisibleNodes, const FConvexVolume& ViewCullingFrustum)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SceneVisibility_CullOctree);

	// Two bits per octree node, 1st bit is Inside Frustum, 2nd bit is Outside Frustum
	OutVisibleNodes.Init(false, Scene.PrimitiveOctree.GetNumNodes() * 2);

	Scene.PrimitiveOctree.FindNodesWithPredicate(
		[&View, &OutVisibleNodes, &Flags, &ViewCullingFrustum](FScenePrimitiveOctree::FNodeIndex ParentNodeIndex, FScenePrimitiveOctree::FNodeIndex NodeIndex, const FBoxCenterAndExtent& NodeBounds)
		{
			// If the parent node is completely contained there is no need to test containment
			if (ParentNodeIndex != INDEX_NONE && !OutVisibleNodes[(ParentNodeIndex * 2) + 1])
			{
				OutVisibleNodes[NodeIndex * 2] = true;
				OutVisibleNodes[NodeIndex * 2 + 1] = false;
				return true;
			}

			const FPlane* PermutedPlanePtr = ViewCullingFrustum.PermutedPlanes.GetData();
			bool bIntersects = false;

			if (Flags.bUseFastIntersect)
			{
				bIntersects = IntersectBox8Plane(NodeBounds.Center, NodeBounds.Extent, PermutedPlanePtr);
			}
			else
			{
				bIntersects = ViewCullingFrustum.IntersectBox(NodeBounds.Center, NodeBounds.Extent);
			}

			if (bIntersects)
			{
				OutVisibleNodes[NodeIndex * 2] = true;
				OutVisibleNodes[NodeIndex * 2 + 1] = ViewCullingFrustum.GetBoxIntersectionOutcode(NodeBounds.Center, NodeBounds.Extent).GetOutside();
			}

			return bIntersects;
		},
		[](FScenePrimitiveOctree::FNodeIndex /*ParentNodeIndex*/, FScenePrimitiveOctree::FNodeIndex /*NodeIndex*/, const FBoxCenterAndExtent& /*NodeBounds*/)
		{

		});
}

static void UpdateAlwaysVisible(const FScene& Scene, FViewInfo& View, FFrustumCullingFlags Flags, const FVisibilityTaskConfig& TaskConfig, int32 TaskIndex, float CurrentWorldTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AlwaysVisible_Loop);

	check(Scene.PrimitivesAlwaysVisibleOffset != ~0u);
	const int32 BitArrayNumInner = TaskConfig.NumVisiblePrimitives;
	const int32 StartWord = int32(Scene.PrimitivesAlwaysVisibleOffset) / NumBitsPerDWORD;
	const int32 TaskWordOffset = TaskIndex * TaskConfig.AlwaysVisible.NumWordsPerTask;

	uint32* RESTRICT VisWords = View.PrimitiveVisibilityMap.GetData();
#if RHI_RAYTRACING
	uint32* RESTRICT  RTWords = View.PrimitiveRayTracingVisibilityMap.GetData();

	const bool bRayTracingEnabled = IsRayTracingEnabled(View.GetShaderPlatform()) && View.IsRayTracingAllowedForView();
#endif

	for (int32 WordIndex = TaskWordOffset; WordIndex < TaskWordOffset + int32(TaskConfig.AlwaysVisible.NumWordsPerTask) && WordIndex * NumBitsPerDWORD < BitArrayNumInner; ++WordIndex)
	{
		uint32 Mask = 0x1;

		uint32 VisBits = 0;
	#if RHI_RAYTRACING
		uint32 RayTracingBits = 0;
	#endif

		for (int32 BitSubIndex = 0; BitSubIndex < NumBitsPerDWORD && WordIndex * NumBitsPerDWORD + BitSubIndex < BitArrayNumInner; ++BitSubIndex, Mask <<= 1)
		{
			const int32 Index = (StartWord + WordIndex) * NumBitsPerDWORD + BitSubIndex;
			VisBits |= Mask;

		#if RHI_RAYTRACING
			if (bRayTracingEnabled && !IsPrimitiveHidden(Scene, View, Index, Flags) && !ShouldCullForRayTracing(Scene, View, Index))
			{
				RayTracingBits |= Mask;
			}
		#endif
		}

		VisWords[StartWord + WordIndex] = VisBits;

	#if RHI_RAYTRACING
		if (RayTracingBits)
		{
			RTWords[StartWord + WordIndex] = RayTracingBits;
		}
	#endif
	}
}

float ScreenVisibilityGetMinScreenBoundsRadiusSquared(const FSceneView& View)
{
	// Values above 1 are considered invalid, and disable size culling
	const float MinRadiusToUse = View.IsRuntimeReflectionCapture() ? GMinScreenBoundsRadiusRuntimeReflection : GMinScreenBoundsRadius;
	float MinScreenBoundsRadiusSquared = FMath::Square(MinRadiusToUse);
	if (MinScreenBoundsRadiusSquared > 1.0f)
	{
		MinScreenBoundsRadiusSquared = 0.0f;
	}

	return MinScreenBoundsRadiusSquared;
}

bool IsScreenSizeCullingElligible(FViewInfo& View, const FBoxSphereBounds& Bounds, const float MinScreenBoundsRadiusSquared)
{
	// Never cull if we do not specify the value
	if (MinScreenBoundsRadiusSquared <= 0.0f)
	{
		return false;
	}

	// Never cull objects whose sphere radius is not properly set or too small
	if (Bounds.SphereRadius < GMinScreenBoundsSphereRadiusCutoff)
	{
		return false;
	}

	// Never cull if we are close enough to the object to be culled
	if (GMinScreenBoundsIgnoreDistance > 0.0f &&
		((Bounds.Origin - View.CullingOrigin).Length() - Bounds.SphereRadius) < GMinScreenBoundsIgnoreDistance)
	{
		return false;
	}

	return true;
}

static int32 FrustumCull(const FScene& Scene, FViewInfo& View, FFrustumCullingFlags Flags, float MaxDrawDistanceScale, const FHLODVisibilityState* const HLODState, const FSceneBitArray* VisibleNodes, const FVisibilityTaskConfig& TaskConfig, int32 TaskIndex)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FrustumCull_Loop);

	bool bDisableLODFade = GDisableLODFade || View.bDisableDistanceBasedFadeTransitions;
	const FPlane* PermutedPlanePtr = View.GetCullingFrustum().PermutedPlanes.GetData();
	const FConvexVolume& ViewCullingFrustum = View.GetCullingFrustum();
	FVector ViewOriginForDistanceCulling = View.CullingOrigin;
	float FadeRadius = bDisableLODFade ? 0.0f : GDistanceFadeMaxTravel;
	uint8 CustomVisibilityFlags = EOcclusionFlags::CanBeOccluded | EOcclusionFlags::HasPrecomputedVisibility;

	int32 BitArrayNumInner = TaskConfig.NumTestedPrimitives;

	uint32 NumPrimitivesCulledForTask = 0;

	// Primitives may be explicitly removed from stereo views when using mono
	const int32 TaskWordOffset = TaskIndex * TaskConfig.FrustumCull.NumWordsPerTask;

	uint32* RESTRICT VisWords = View.PrimitiveVisibilityMap.GetData();
	uint32* RESTRICT FadeWords = View.PotentiallyFadingPrimitiveMap.GetData();
#if RHI_RAYTRACING
	uint32* RESTRICT  RTWords = View.PrimitiveRayTracingVisibilityMap.GetData();

	const bool bRayTracingEnabled = IsRayTracingEnabled(View.GetShaderPlatform()) && View.IsRayTracingAllowedForView();
#endif

	const float MinScreenBoundsRadiusSquared = ScreenVisibilityGetMinScreenBoundsRadiusSquared(View);

	for (int32 WordIndex = TaskWordOffset; WordIndex < TaskWordOffset + int32(TaskConfig.FrustumCull.NumWordsPerTask) && WordIndex * NumBitsPerDWORD < BitArrayNumInner; WordIndex++)
	{
		uint32 Mask = 0x1; 
		uint32 VisBits = 0;
		uint32 FadingBits = 0;
	#if RHI_RAYTRACING
		uint32 RayTracingBits = 0;
	#endif

		// If visibility culling is disabled, make sure to use the existing visibility state
		if (!Flags.bShouldVisibilityCull)
		{
			VisBits = VisWords[WordIndex];
		}

		for (int32 BitSubIndex = 0; BitSubIndex < NumBitsPerDWORD && WordIndex * NumBitsPerDWORD + BitSubIndex < BitArrayNumInner; BitSubIndex++, Mask <<= 1)
		{
			int32 Index = WordIndex * NumBitsPerDWORD + BitSubIndex;
			bool bPrimitiveIsHidden = IsPrimitiveHidden(Scene, View, Index, Flags);
			bool bIsVisible = Flags.bShouldVisibilityCull ? true : (VisBits & Mask) == Mask;

			bIsVisible = bIsVisible && !bPrimitiveIsHidden;

		#if RHI_RAYTRACING
			bool bIsVisibleInRayTracing = true;

			if (bPrimitiveIsHidden || !bRayTracingEnabled || ShouldCullForRayTracing(Scene, View, Index))
			{
				bIsVisibleInRayTracing = false;
			}
		#endif

			const FPrimitiveBounds& RESTRICT Bounds = Scene.PrimitiveBounds[Index];

			// Zero sized bounds indicates that we are not visible.
			bIsVisible &= Bounds.BoxSphereBounds.SphereRadius > 0;

			if (Flags.bShouldVisibilityCull && bIsVisible)
			{
				bool bShouldDistanceCull = true;
				bool bPartiallyOutside = true;
				bool bShouldFrustumCull = true;

				// Fading HLODs and their children must be visible, objects hidden by HLODs can be culled
				if (HLODState)
				{
					if (HLODState->IsNodeForcedVisible(Index))
					{
						bShouldDistanceCull = false;
					}
					else if (HLODState->IsNodeForcedHidden(Index))
					{
						bIsVisible = false;
					}
				}

				// Frustum first
				bShouldFrustumCull = bShouldFrustumCull && bIsVisible;
				if (bShouldFrustumCull)
				{
					if (Flags.bUseVisibilityOctree)
					{
						// If the parent octree node was completely contained by the frustum, there is no need do an additional frustum test on the primitive bounds
						// If the parent octree node is partially in the frustum, perform an additional test on the primitive bounds
						uint32 OctreeNodeIndex = Scene.PrimitiveOctreeIndex[Index];

						bIsVisible = (*VisibleNodes)[OctreeNodeIndex * 2];
						bPartiallyOutside = (*VisibleNodes)[OctreeNodeIndex * 2 + 1];
					}

					if (bIsVisible)
					{
						int32 VisibilityId = INDEX_NONE;

						if (Flags.bUseCustomCulling &&
							((Scene.PrimitiveOcclusionFlags[Index] & CustomVisibilityFlags) == CustomVisibilityFlags))
						{
							VisibilityId = Scene.PrimitiveSceneProxies[Index]->GetVisibilityId();
						}

						bIsVisible = !bPartiallyOutside || IsPrimitiveVisible(View, PermutedPlanePtr, ViewCullingFrustum, Bounds, VisibilityId, Flags);
					}
				}

				// Screen size culling next
				if (bIsVisible && IsScreenSizeCullingElligible(View, Bounds.BoxSphereBounds, MinScreenBoundsRadiusSquared))
				{
					bIsVisible = !IsScreenSizeCulled(View, Bounds.BoxSphereBounds, MinScreenBoundsRadiusSquared);
				}

				// Distance cull if frustum cull passed
				bShouldDistanceCull = bShouldDistanceCull && bIsVisible;
				if (bShouldDistanceCull)
				{
					// If cull distance is disabled, always show the primitive (except foliage)
					if (View.Family->EngineShowFlags.DistanceCulledPrimitives
						&& !Scene.PrimitiveSceneProxies[Index]->IsDetailMesh()) // Proxy call is intentionally behind the DistancedCulledPrimitives check to prevent an expensive memory read
					{
						bShouldDistanceCull = false;
					}
				}

				if (bShouldDistanceCull)
				{
					// Preserve infinite draw distance
					bool bHasMaxDrawDistance = Bounds.MaxCullDistance < FLT_MAX;
					bool bHasMinDrawDistance = Bounds.MinDrawDistance > 0;

					if (bHasMaxDrawDistance || bHasMinDrawDistance)
					{
						float MaxDrawDistance = Bounds.MaxCullDistance * MaxDrawDistanceScale;
						float MinDrawDistanceSq = FMath::Square(Bounds.MinDrawDistance * MaxDrawDistanceScale);
						float ClosestDistSquared, FurthestDistSquared;

						if (GDistanceCullToSphereEdge)
						{
							ComputeDistances(Bounds, ViewOriginForDistanceCulling, ClosestDistSquared, FurthestDistSquared);
						}
						else
						{
							ClosestDistSquared = FurthestDistSquared = FVector::DistSquared(Bounds.BoxSphereBounds.Origin, ViewOriginForDistanceCulling);
						}

						// Always test the fade in distance.  If a primitive was set to always draw, it may need to be faded in.
						if (bHasMaxDrawDistance)
						{
							float MaxFadeDistanceSquared = FMath::Square(MaxDrawDistance + FadeRadius);
							float MinFadeDistanceSquared = FMath::Square(MaxDrawDistance - FadeRadius);
							if ((ClosestDistSquared < MaxFadeDistanceSquared && ClosestDistSquared > MinFadeDistanceSquared)
								&& Scene.PrimitiveSceneProxies[Index]->IsUsingDistanceCullFade())  // Proxy call is intentionally behind the fade check to prevent an expensive memory read
							{
								FadingBits |= Mask;
							}
						}

						// Check for distance culling first
						const bool bFarDistanceCulled = bHasMaxDrawDistance && (ClosestDistSquared > FMath::Square(MaxDrawDistance));
						const bool bNearDistanceCulled = bHasMinDrawDistance && (FurthestDistSquared < MinDrawDistanceSq);

						bool bIsDistanceCulled = bNearDistanceCulled || bFarDistanceCulled;

						if (bIsDistanceCulled)
						{
							bIsVisible = false;
						}

					#if RHI_RAYTRACING
						if (bFarDistanceCulled)
						{
							bIsVisibleInRayTracing = false;
						}
					#endif
					}
				}
			}

			if (bIsVisible)
			{
				// The primitive is visible!
				VisBits |= Mask;
			}
			else
			{
				++NumPrimitivesCulledForTask;
			}

		#if RHI_RAYTRACING
			if (bIsVisibleInRayTracing)
			{
				RayTracingBits |= Mask;
			}
		#endif
		}

		if (Flags.bShouldVisibilityCull && FadingBits)
		{
			FadeWords[WordIndex] = FadingBits;
		}

		if (Flags.bShouldVisibilityCull && VisBits)
		{
			VisWords[WordIndex] = VisBits;
		}

	#if RHI_RAYTRACING
		if (RayTracingBits)
		{
			RTWords[WordIndex] = RayTracingBits;
		}
	#endif
	}

	return NumPrimitivesCulledForTask;
}

///////////////////////////////////////////////////////////////////////////////

static void ClearStalePrimitiveFadingStates(FViewInfo& View, FSceneViewState* ViewState)
{
	if (!ViewState)
	{
		return;
	}

	const uint32 PrevFrameNumber = ViewState->PrevFrameNumber;
	const float CurrentRealTime = View.Family->Time.GetRealTimeSeconds();

	// First clear any stale fading states.
	for (FPrimitiveFadingStateMap::TIterator It(ViewState->PrimitiveFadingStates); It; ++It)
	{
		FPrimitiveFadingState& FadingState = It.Value();
		if (FadingState.FrameNumber != PrevFrameNumber ||
			(IsValidRef(FadingState.UniformBuffer) && CurrentRealTime >= FadingState.EndTime))
		{
			It.RemoveCurrent();
		}
	}
}

static void UpdatePrimitiveFadingStateHelper(FPrimitiveFadingState& FadingState, const FViewInfo& View, bool bVisible)
{
	if (FadingState.bValid)
	{
		if (FadingState.bIsVisible != bVisible)
		{
			float CurrentRealTime = View.Family->Time.GetRealTimeSeconds();

			// Need to kick off a fade, so make sure that we have fading state for that
			if (!IsValidRef(FadingState.UniformBuffer))
			{
				// Primitive is not currently fading.  Start a new fade!
				FadingState.EndTime = CurrentRealTime + GFadeTime;

				if (bVisible)
				{
					// Fading in
					// (Time - StartTime) / FadeTime
					FadingState.FadeTimeScaleBias.X = 1.0f / GFadeTime;
					FadingState.FadeTimeScaleBias.Y = -CurrentRealTime / GFadeTime;
				}
				else
				{
					// Fading out
					// 1 - (Time - StartTime) / FadeTime
					FadingState.FadeTimeScaleBias.X = -1.0f / GFadeTime;
					FadingState.FadeTimeScaleBias.Y = 1.0f + CurrentRealTime / GFadeTime;
				}

				FDistanceCullFadeUniformShaderParameters Uniforms;
				Uniforms.FadeTimeScaleBias = FVector2f(FadingState.FadeTimeScaleBias);	// LWC_TODO: Precision loss
				FadingState.UniformBuffer = FDistanceCullFadeUniformBufferRef::CreateUniformBufferImmediate(Uniforms, UniformBuffer_MultiFrame);
			}
			else
			{
				// Reverse fading direction but maintain current opacity
				// Solve for d: a*x+b = -a*x+d
				FadingState.FadeTimeScaleBias.Y = 2.0f * CurrentRealTime * FadingState.FadeTimeScaleBias.X + FadingState.FadeTimeScaleBias.Y;
				FadingState.FadeTimeScaleBias.X = -FadingState.FadeTimeScaleBias.X;

				if (bVisible)
				{
					// Fading in
					// Solve for x: a*x+b = 1
					FadingState.EndTime = (1.0f - FadingState.FadeTimeScaleBias.Y) / FadingState.FadeTimeScaleBias.X;
				}
				else
				{
					// Fading out
					// Solve for x: a*x+b = 0
					FadingState.EndTime = -FadingState.FadeTimeScaleBias.Y / FadingState.FadeTimeScaleBias.X;
				}

				FDistanceCullFadeUniformShaderParameters Uniforms;
				Uniforms.FadeTimeScaleBias = FVector2f(FadingState.FadeTimeScaleBias);	// LWC_TODO: Precision loss
				FadingState.UniformBuffer = FDistanceCullFadeUniformBufferRef::CreateUniformBufferImmediate(Uniforms, UniformBuffer_MultiFrame);
			}
		}
	}

	FadingState.FrameNumber = View.Family->FrameNumber;
	FadingState.bIsVisible = bVisible;
	FadingState.bValid = true;
}

static void UpdatePrimitiveFading(const FScene& Scene, FViewInfo& View, FSceneViewState* ViewState, FPrimitiveRange PrimitiveRange)
{
	if (!ViewState)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_UpdatePrimitiveFading);

#if RHI_RAYTRACING
	const bool bRayTracingEnabled = IsRayTracingEnabled(View.GetShaderPlatform()) && View.IsRayTracingAllowedForView();
#endif 

	// Should we allow fading transitions at all this frame?  For frames where the camera moved
	// a large distance or where we haven't rendered a view in awhile, it's best to disable
	// fading so users don't see unexpected object transitions.
	if (!GDisableLODFade && !View.bDisableDistanceBasedFadeTransitions)
	{
		// Do a pass over potentially fading primitives and update their states.
		for (FSceneSetBitIterator BitIt(View.PotentiallyFadingPrimitiveMap, PrimitiveRange.StartIndex); BitIt.GetIndex() < PrimitiveRange.EndIndex; ++BitIt)
		{
			bool bVisible = View.PrimitiveVisibilityMap.AccessCorrespondingBit(BitIt);
			FPrimitiveFadingState& FadingState = ViewState->PrimitiveFadingStates.FindOrAdd(Scene.PrimitiveComponentIds[BitIt.GetIndex()]);
			UpdatePrimitiveFadingStateHelper(FadingState, View, bVisible);
			FRHIUniformBuffer* UniformBuffer = FadingState.UniformBuffer;
			if (UniformBuffer && !bVisible)
			{
				// If the primitive is fading out make sure it remains visible.
				View.PrimitiveVisibilityMap.AccessCorrespondingBit(BitIt) = true;

			#if RHI_RAYTRACING
				// Cannot just assume the ray tracing visibility will be true, so a complete recalculation for its culling needs to happen
				// This should be a very rare occurrence, so the hit is not worrisome.
				// TODO:  Could this be moved into the actual culling phase?

				if (bRayTracingEnabled && !ShouldCullForRayTracing(Scene, View, BitIt.GetIndex()))
				{
					View.PrimitiveRayTracingVisibilityMap.AccessCorrespondingBit(BitIt) = true;
				}
			#endif
			}
			View.PrimitiveFadeUniformBuffers[BitIt.GetIndex()] = UniformBuffer;
			View.PrimitiveFadeUniformBufferMap[BitIt.GetIndex()] = UniformBuffer != nullptr;
		}
	}
}

bool FViewInfo::UpdatePrimitiveFadingState(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bFadingIn)
{
	// Update distance-based visibility and fading state if it has not already been updated.
	const int32 PrimitiveIndex = PrimitiveSceneInfo->GetIndex();
	const FRelativeBitReference PrimitiveBit(PrimitiveIndex);
	bool bStillFading = false;

	if (!PotentiallyFadingPrimitiveMap.AccessCorrespondingBit(PrimitiveBit))
	{
		FPrimitiveFadingState& FadingState = ((FSceneViewState*)State)->PrimitiveFadingStates.FindOrAdd(PrimitiveSceneInfo->PrimitiveComponentId);
		UpdatePrimitiveFadingStateHelper(FadingState, *this, bFadingIn);
		FRHIUniformBuffer* UniformBuffer = FadingState.UniformBuffer;
		bStillFading = UniformBuffer != nullptr;
		PrimitiveFadeUniformBuffers[PrimitiveIndex] = UniformBuffer;
		PrimitiveFadeUniformBufferMap[PrimitiveIndex] = UniformBuffer != nullptr;
		PotentiallyFadingPrimitiveMap.AccessCorrespondingBit(PrimitiveBit) = true;
	}

	// If we're still fading then make sure the object is still drawn, even if it's beyond the max draw distance
	return !bFadingIn && !bStillFading;
}

///////////////////////////////////////////////////////////////////////////////

FFilterStaticMeshesForViewData::FFilterStaticMeshesForViewData(FViewInfo& View)
{
	ViewOrigin = View.ViewMatrices.GetViewOrigin();

	// outside of the loop to be more efficient
	ForcedLODLevel = (View.Family->EngineShowFlags.LOD) ? GetCVarForceLOD() : 0;

	LODScale = CVarStaticMeshLODDistanceScale.GetValueOnRenderThread() * View.LODDistanceFactor;

	MinScreenRadiusForDepthPrepassSquared = GMinScreenRadiusForDepthPrepass * GMinScreenRadiusForDepthPrepass;

	bFullEarlyZPass = ShouldForceFullDepthPass(View.GetShaderPlatform());
}

///////////////////////////////////////////////////////////////////////////////

FDrawCommandRelevancePacket::FDrawCommandRelevancePacket()
{
	bUseCachedMeshDrawCommands = UseCachedMeshDrawCommands();

	for (int32 PassIndex = 0; PassIndex < EMeshPass::Num; ++PassIndex)
	{
		NumDynamicBuildRequestElements[PassIndex] = 0;
	}
}

void FDrawCommandRelevancePacket::AddCommandsForMesh(
	int32 PrimitiveIndex, 
	const FPrimitiveSceneInfo* InPrimitiveSceneInfo,
	const FStaticMeshBatchRelevance& RESTRICT StaticMeshRelevance, 
	const FStaticMeshBatch& RESTRICT StaticMesh, 
	EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags,
	const FScene& Scene,
	bool bCanCache, 
	EMeshPass::Type PassType)
{
	const bool bIsNaniteMesh = Scene.PrimitiveFlagsCompact[PrimitiveIndex].bIsNaniteMesh;
	if (bIsNaniteMesh && Scene.PrimitivesAlwaysVisibleOffset != ~0u)
	{
		return;
	}

	const EShadingPath ShadingPath = GetFeatureLevelShadingPath(Scene.GetFeatureLevel());
	const bool bUseCachedMeshCommand = bUseCachedMeshDrawCommands
		&& !!(FPassProcessorManager::GetPassFlags(ShadingPath, PassType) & EMeshPassFlags::CachedMeshCommands)
		&& StaticMeshRelevance.bSupportsCachingMeshDrawCommands
		&& bCanCache;

	if (bUseCachedMeshCommand)
	{
		const int32 StaticMeshCommandInfoIndex = StaticMeshRelevance.GetStaticMeshCommandInfoIndex(PassType);
		if (StaticMeshCommandInfoIndex >= 0)
		{
			const FCachedMeshDrawCommandInfo& CachedMeshDrawCommand = InPrimitiveSceneInfo->StaticMeshCommandInfos[StaticMeshCommandInfoIndex];
			const FCachedPassMeshDrawList& SceneDrawList = Scene.CachedDrawLists[PassType];

			// AddUninitialized_GetRef()
			VisibleCachedDrawCommands[(uint32)PassType].AddUninitialized();
			FVisibleMeshDrawCommand& NewVisibleMeshDrawCommand = VisibleCachedDrawCommands[(uint32)PassType].Last();

			const FMeshDrawCommand* MeshDrawCommand = CachedMeshDrawCommand.StateBucketId >= 0
				? &Scene.CachedMeshDrawCommandStateBuckets[PassType].GetByElementId(CachedMeshDrawCommand.StateBucketId).Key
				: &SceneDrawList.MeshDrawCommands[CachedMeshDrawCommand.CommandIndex];

			NewVisibleMeshDrawCommand.Setup(
				MeshDrawCommand,
				InPrimitiveSceneInfo->GetMDCIdInfo(),
				CachedMeshDrawCommand.StateBucketId,
				CachedMeshDrawCommand.MeshFillMode,
				CachedMeshDrawCommand.MeshCullMode,
				CachedMeshDrawCommand.Flags,
				CachedMeshDrawCommand.SortKey,
				CachedMeshDrawCommand.CullingPayload,
				CullingPayloadFlags);
		}
	}
	else
	{
		NumDynamicBuildRequestElements[PassType] += StaticMeshRelevance.NumElements;
		DynamicBuildRequests[PassType].Add(&StaticMesh);
		DynamicBuildFlags[PassType].Add(CullingPayloadFlags);
	}
}

///////////////////////////////////////////////////////////////////////////////

FRelevancePacket::FRelevancePacket(
	FVisibilityTaskData& InTaskData,
	const FViewInfo& InView,
	int32 InViewIndex,
	const FFilterStaticMeshesForViewData& InViewData,
	FSceneCullingRendererHierarchicalCullingQuery* InSceneCullingQuery,
	uint8* InMarkMasks,
	const UE::Tasks::FTask& InPrerequisitesTask)
	: CurrentWorldTime(InView.Family->Time.GetWorldTimeSeconds())
	, DeltaWorldTime(InView.Family->Time.GetDeltaWorldTimeSeconds())
	, TaskData(InTaskData)
	, TaskConfig(InTaskData.TaskConfig)
	, Scene(TaskData.Scene)
	, View(InView)
	, ViewCommands(TaskData.DynamicMeshElements.ViewCommandsPerView[InViewIndex])
	, ViewBit(1 << InViewIndex)
	, ViewData(InViewData)
	, DynamicPrimitiveViewMasks(TaskData.DynamicMeshElements.PrimitiveViewMasks)
	, SceneCullingQuery(InSceneCullingQuery)
	, MarkMasks(InMarkMasks)
	, PrerequisitesTask(InPrerequisitesTask)
	, Input(TaskConfig.Relevance.NumPrimitivesPerPacket)
	, NotDrawRelevant(TaskConfig.Relevance.NumPrimitivesPerPacket)
	, TranslucentSelfShadowPrimitives(TaskConfig.Relevance.NumPrimitivesPerPacket)
	, VisibleDynamicPrimitivesWithSimpleLights(TaskConfig.Relevance.NumPrimitivesPerPacket)
	, DirtyIndirectLightingCacheBufferPrimitives(TaskConfig.Relevance.NumPrimitivesPerPacket)
	, NaniteCustomDepthInstances(TaskConfig.Relevance.NumPrimitivesPerPacket)
	, PrimitivesLODMask(TaskConfig.Relevance.NumPrimitivesPerPacket)
	, bAddLightmapDensityCommands(TaskData.bAddLightmapDensityCommands)
{}

void FRelevancePacket::LaunchComputeRelevanceTask()
{
	check(!bComputeRelevanceTaskLaunched);

	if (!Input.IsEmpty())
	{
		bComputeRelevanceTaskLaunched = true;

		if (TaskData.DynamicMeshElements.CommandPipe)
		{
			TaskData.DynamicMeshElements.CommandPipe->AddNumCommands(1);
		}

		ComputeRelevanceTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]
		{
			FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
			FDynamicPrimitiveIndexList DynamicPrimitiveIndexList;
			ComputeRelevance(DynamicPrimitiveIndexList);

			if (TaskData.DynamicMeshElements.CommandPipe)
			{
				if (DynamicPrimitiveIndexList.IsEmpty())
				{
					TaskData.DynamicMeshElements.CommandPipe->ReleaseNumCommands(1);
				}
				else
				{
					TaskData.DynamicMeshElements.CommandPipe->EnqueueCommand(MoveTemp(DynamicPrimitiveIndexList));
				}
			}

		}, PrerequisitesTask, TaskConfig.Relevance.ComputeRelevanceTaskPriority);
	}
}

void FRelevancePacket::Finalize()
{
	FViewInfo& WriteView = const_cast<FViewInfo&>(View);
	FViewCommands& WriteViewCommands = const_cast<FViewCommands&>(ViewCommands);
	const EShadingPath ShadingPath = GetFeatureLevelShadingPath(Scene.GetFeatureLevel());

	for (int32 BitIndex : NotDrawRelevant.Prims)
	{
		WriteView.PrimitiveVisibilityMap[BitIndex] = false;
	}

	TaskConfig.Relevance.NumPrimitivesProcessed += Input.Prims.Num();

	WriteView.ShadingModelMaskInView |= CombinedShadingModelMask;
	WriteView.bUsesGlobalDistanceField |= bUsesGlobalDistanceField;
	WriteView.bUsesLightingChannels |= bUsesLightingChannels;
	WriteView.bTranslucentSurfaceLighting |= bTranslucentSurfaceLighting;
	WriteView.bSceneHasSkyMaterial |= bSceneHasSkyMaterial;
	WriteView.bHasSingleLayerWaterMaterial |= bHasSingleLayerWaterMaterial;
	WriteView.bUsesSecondStageDepthPass |= bUsesSecondStageDepthPass && ShadingPath != EShadingPath::Mobile;
	VisibleDynamicPrimitivesWithSimpleLights.AppendTo(WriteView.VisibleDynamicPrimitivesWithSimpleLights);
	WriteView.NumVisibleDynamicPrimitives += NumVisibleDynamicPrimitives;
	WriteView.NumVisibleDynamicEditorPrimitives += NumVisibleDynamicEditorPrimitives;
	WriteView.TranslucentPrimCount.Append(TranslucentPrimCount);
	WriteView.bHasDistortionPrimitives |= bHasDistortionPrimitives;
	WriteView.bHasCustomDepthPrimitives |= bHasCustomDepthPrimitives;
	WriteView.CustomDepthStencilValues.Append(CustomDepthStencilValues);
	NaniteCustomDepthInstances.AppendTo(WriteView.NaniteCustomDepthInstances);
	WriteView.bUsesCustomDepth |= bUsesCustomDepth;
	WriteView.bUsesMotionVectorWorldOffset |= bUsesMotionVectorWorldOffset;
	WriteView.bSLWUsesMotionVectorWorldOffset |= bSLWUsesMotionVectorWorldOffset;
	WriteView.bUsesCustomStencil |= bUsesCustomStencil;
	WriteView.SubstrateViewData.MaxClosurePerPixel = FMath::Max(WriteView.SubstrateViewData.MaxClosurePerPixel, 8u - FMath::CountLeadingZeros8(SubstrateClosureCountMask));
	WriteView.SubstrateViewData.MaxBytesPerPixel = FMath::Max(WriteView.SubstrateViewData.MaxBytesPerPixel, SubstrateUintPerPixel * 4u);
	WriteView.SubstrateViewData.UsesTileTypeMask |= SubstrateTileTypeMask;
	WriteView.SubstrateViewData.bUsesAnisotropy |= bUsesAnisotropy;
	DirtyIndirectLightingCacheBufferPrimitives.AppendTo(WriteView.DirtyIndirectLightingCacheBufferPrimitives);

	WriteView.VolumetricMeshBatches.Append(VolumetricMeshBatches);
	WriteView.HeterogeneousVolumesMeshBatches.Append(HeterogeneousVolumesMeshBatches);
	WriteView.SkyMeshBatches.Append(SkyMeshBatches);
	WriteView.SortedTrianglesMeshBatches.Append(SortedTrianglesMeshBatches);

	for (FPrimitiveLODMask PrimitiveLODMask : PrimitivesLODMask.Prims)
	{
		WriteView.PrimitivesLODMask[PrimitiveLODMask.PrimitiveIndex] = PrimitiveLODMask.LODMask;
	}

	for (int32 PassIndex = 0; PassIndex < EMeshPass::Num; PassIndex++)
	{
		if (PassIndex == EMeshPass::NaniteMeshPass && Scene.PrimitivesAlwaysVisibleOffset != ~0u)
		{
			continue;
		}

		FPassDrawCommandArray& SrcCommands = DrawCommandPacket.VisibleCachedDrawCommands[PassIndex];
		FMeshCommandOneFrameArray& DstCommands = WriteViewCommands.MeshCommands[PassIndex];
		if (SrcCommands.Num() > 0)
		{
			static_assert(sizeof(SrcCommands[0]) == sizeof(DstCommands[0]), "Memcpy sizes must match.");
			const int32 PrevNum = DstCommands.AddUninitialized(SrcCommands.Num());
			FMemory::Memcpy(&DstCommands[PrevNum], &SrcCommands[0], SrcCommands.Num() * sizeof(SrcCommands[0]));
		}

		FPassDrawCommandBuildRequestArray& SrcRequests = DrawCommandPacket.DynamicBuildRequests[PassIndex];
		TArray<const FStaticMeshBatch*, SceneRenderingAllocator>& DstRequests = WriteViewCommands.DynamicMeshCommandBuildRequests[PassIndex];
		if (SrcRequests.Num() > 0)
		{
			static_assert(sizeof(SrcRequests[0]) == sizeof(DstRequests[0]), "Memcpy sizes must match.");
			const int32 PrevNum = DstRequests.AddUninitialized(SrcRequests.Num());
			FMemory::Memcpy(&DstRequests[PrevNum], &SrcRequests[0], SrcRequests.Num() * sizeof(SrcRequests[0]));
		}

		FPassDrawCommandBuildFlagsArray& SrcFlags = DrawCommandPacket.DynamicBuildFlags[PassIndex];
		TArray<EMeshDrawCommandCullingPayloadFlags, SceneRenderingAllocator>& DstFlags = WriteViewCommands.DynamicMeshCommandBuildFlags[PassIndex];
		if (SrcFlags.Num() > 0)
		{
			static_assert(sizeof(SrcFlags[0]) == sizeof(DstFlags[0]), "Memcpy sizes must match.");
			const int32 PrevNum = DstFlags.AddUninitialized(SrcFlags.Num());
			FMemory::Memcpy(&DstFlags[PrevNum], &SrcFlags[0], SrcFlags.Num() * sizeof(SrcFlags[0]));
		}

		WriteViewCommands.NumDynamicMeshCommandBuildRequestElements[PassIndex] += DrawCommandPacket.NumDynamicBuildRequestElements[PassIndex];
	}

	// Prepare translucent self shadow uniform buffers.
	for (int32 PrimitiveIndex : TranslucentSelfShadowPrimitives.Prims)
	{
		FUniformBufferRHIRef& UniformBuffer = WriteView.TranslucentSelfShadowUniformBufferMap.FindOrAdd(PrimitiveIndex);

		if (!UniformBuffer)
		{
			FTranslucentSelfShadowUniformParameters Parameters;
			SetupTranslucentSelfShadowUniformParameters(nullptr, Parameters);
			UniformBuffer = FTranslucentSelfShadowUniformParameters::CreateUniformBuffer(Parameters, EUniformBufferUsage::UniformBuffer_SingleFrame);
		}
	}
}

FLODMask ComputePrimitiveMeshLODs(const FViewInfo& View, int32 ViewForcedLODLevel, float ViewLODScale, const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FPrimitiveSceneInfo* PrimitiveSceneInfo, int32 PrimitiveIndex, const FBoxSphereBounds& Bounds, FSceneCullingRendererHierarchicalCullingQuery* SceneCullingQuery)
{
	const FDesiredLODLevel DesiredLODLevel = PrimitiveSceneProxy->GetDesiredLODLevel_RenderThread(&View);
	// Always perform this first, since it will also calculate culling results that are needed for rendering (not just LOD selection).
	TOptional<FLODMask> InstanceLODMask;
	if (SceneCullingQuery)
	{
		InstanceLODMask = SceneCullingQuery->ComputeCulling(PrimitiveIndex, Bounds, View, ViewForcedLODLevel, DesiredLODLevel, ViewLODScale);
	}

	check(DesiredLODLevel.LOD >= 0);
	if (DesiredLODLevel.IsFixed())
	{
		FLODMask LODToRender;
		LODToRender.SetLOD(DesiredLODLevel.LOD);
		return LODToRender;
	}
	else if (InstanceLODMask.IsSet())
	{
		return InstanceLODMask.GetValue();
	}

	float MeshScreenSizeSquared = 0.0f;
	return ComputeLODForMeshes(PrimitiveSceneInfo->StaticMeshRelevances, View, Bounds.Origin, Bounds.SphereRadius, PrimitiveSceneInfo->GpuLodInstanceRadius, ViewForcedLODLevel, MeshScreenSizeSquared, DesiredLODLevel.LOD, ViewLODScale);
}

void FRelevancePacket::ComputeRelevance(FDynamicPrimitiveIndexList& DynamicPrimitiveIndexList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeViewRelevance);
	SCOPE_CYCLE_COUNTER(STAT_ComputeViewRelevance);

	CombinedShadingModelMask = 0;
	SubstrateUintPerPixel = 0;
	SubstrateTileTypeMask = 0;
	bUsesAnisotropy = false;
	SubstrateClosureCountMask = 0;
	bSceneHasSkyMaterial = 0;
	bHasSingleLayerWaterMaterial = 0;
	bUsesSecondStageDepthPass = 0;
	bUsesGlobalDistanceField = false;
	bUsesLightingChannels = false;
	bTranslucentSurfaceLighting = false;
	const EShadingPath ShadingPath = GetFeatureLevelShadingPath(Scene.GetFeatureLevel());
	const bool bHairStrandsEnabled = IsHairStrandsEnabled(EHairStrandsShaderType::All, Scene.GetShaderPlatform());
	const bool bIsLumenTranslucencyRadianceCacheEnabled = ShouldRenderLumenDiffuseGI(&Scene, View);
	const bool bIsTLVUsingVoxelMarking = IsTranslucencyLightingVolumeUsingVoxelMarking();
	const bool bIsFrontLayerTranslucencyEnabled = ShouldRenderFrontLayerTranslucency(View);

	int32 NumVisibleStaticMeshElements = 0;
	FViewInfo& WriteView = const_cast<FViewInfo&>(View);
	const FSceneViewState* ViewState = (FSceneViewState*)View.State;
	const bool bMobileMaskedInEarlyPass = (ShadingPath == EShadingPath::Mobile) && Scene.EarlyZPassMode == DDM_MaskedOnly;
	const bool bMobileBasePassAlwaysUsesCSM = (ShadingPath == EShadingPath::Mobile) && MobileBasePassAlwaysUsesCSM(Scene.GetShaderPlatform());
	const bool bVelocityPassWritesDepth = Scene.EarlyZPassMode == DDM_AllOpaqueNoVelocity;
	const bool bIsTranslucentHoldoutEnabled = IsPrimitiveAlphaHoldoutEnabled(View);
	const bool bIsTranslucentClippedDepthEnabled = NeedTSRTranslucentClippedDepthPass(View);
	const bool bHLODActive = Scene.SceneLODHierarchy.IsActive();
	const bool bUsingSingleLayerWaterFallback = IsUsingSingleLayerWaterFallback(View.GetShaderPlatform());
	const FHLODVisibilityState* const HLODState = bHLODActive && ViewState ? &ViewState->HLODVisibilityState : nullptr;
	float MaxDrawDistanceScale = GetCachedScalabilityCVars().ViewDistanceScale;
	MaxDrawDistanceScale *= GetCachedScalabilityCVars().CalculateFieldOfViewDistanceScale(View.DesiredFOV);
	
	const auto AddEditorDynamicPrimitive = [this, &DynamicPrimitiveIndexList](int32 PrimitiveIndex)
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			++NumVisibleDynamicEditorPrimitives;

			if (DynamicPrimitiveViewMasks)
			{
				FPlatformAtomics::InterlockedOr((volatile int8*)&DynamicPrimitiveViewMasks->EditorPrimitives[PrimitiveIndex], ViewBit);
			}
			else
			{
				DynamicPrimitiveIndexList.EditorPrimitives.Emplace(PrimitiveIndex, ViewBit);
			}
		}
#endif
	};

	const auto AddDynamicPrimitive = [this, &DynamicPrimitiveIndexList](int32 PrimitiveIndex)
	{
		++NumVisibleDynamicPrimitives;

		if (DynamicPrimitiveViewMasks)
		{
			FPlatformAtomics::InterlockedOr((volatile int8*)&DynamicPrimitiveViewMasks->Primitives[PrimitiveIndex], ViewBit);
		}
		else
		{
			DynamicPrimitiveIndexList.Primitives.Emplace(PrimitiveIndex, ViewBit);
		}
	};

	for (int32 InputPrimsIndex = 0; InputPrimsIndex < Input.Prims.Num(); ++InputPrimsIndex)
	{
		int32 BitIndex = Input.Prims[InputPrimsIndex];

		if (InputPrimsIndex + 1 < Input.Prims.Num())
		{
			int32 NextBitIndex = Input.Prims[InputPrimsIndex + 1];

			// Prefetch the next primitive / proxy pair for the next loop.
			FPlatformMisc::Prefetch(Scene.Primitives[NextBitIndex]);
			FPlatformMisc::Prefetch(Scene.PrimitiveSceneProxies[NextBitIndex]);
		}

		FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene.Primitives[BitIndex];
		FPrimitiveViewRelevance& ViewRelevance = const_cast<FPrimitiveViewRelevance&>(View.PrimitiveViewRelevanceMap[BitIndex]);

		const FPrimitiveSceneProxy* PrimitiveSceneProxy = PrimitiveSceneInfo->Proxy;

		// Prefetch the scene data and static mesh relevance array now while we call GetViewRelevance to reduce memory waits.
		FPlatformMisc::Prefetch(PrimitiveSceneInfo->StaticMeshRelevances.GetData());
		FPlatformMisc::Prefetch(PrimitiveSceneInfo->GetSceneData());

		ViewRelevance = PrimitiveSceneProxy->GetViewRelevance(&View);
		ViewRelevance.bInitializedThisFrame = true;

		const bool bStaticRelevance = ViewRelevance.bStaticRelevance;
		const bool bDrawRelevance = ViewRelevance.bDrawRelevance;
		const bool bDynamicRelevance = ViewRelevance.bDynamicRelevance;
		const bool bShadowRelevance = ViewRelevance.bShadowRelevance;
		const bool bEditorRelevance = ViewRelevance.bEditorPrimitiveRelevance;
		const bool bTranslucentRelevance = ViewRelevance.HasTranslucency();
		const bool bHairStrandsRelevance = bHairStrandsEnabled && ViewRelevance.bHairStrands;
		// Filter out hair strands geometry, as custom depth is handled in a custom manner. This is only for hair strands, hair cards/meshes use the regular path. 
		// Use !bRenderInMainPass as a discriminant for filtering only hair strands geometry.
		const bool bIsCompatibleWithCustomDepth = !(bHairStrandsRelevance && !ViewRelevance.bRenderInMainPass);

		if (View.bIsReflectionCapture && !PrimitiveSceneProxy->IsVisibleInReflectionCaptures())
		{
			NotDrawRelevant.AddPrim(BitIndex);
			continue;
		}

		if (bStaticRelevance && (bDrawRelevance || bShadowRelevance))
		{
			int32 PrimitiveIndex = BitIndex;
			const FPrimitiveBounds& Bounds = Scene.PrimitiveBounds[PrimitiveIndex];
			const bool bIsPrimitiveDistanceCullFading = View.PrimitiveFadeUniformBufferMap[PrimitiveIndex];

			FLODMask LODToRender = ComputePrimitiveMeshLODs(View, ViewData.ForcedLODLevel, ViewData.LODScale, PrimitiveSceneProxy, PrimitiveSceneInfo, PrimitiveIndex, Bounds.BoxSphereBounds, SceneCullingQuery);
			PrimitivesLODMask.AddPrim(FRelevancePacket::FPrimitiveLODMask(PrimitiveIndex, LODToRender));

			const bool bIsHLODFading = HLODState ? HLODState->IsNodeFading(PrimitiveIndex) : false;
			const bool bIsHLODFadingOut = HLODState ? HLODState->IsNodeFadingOut(PrimitiveIndex) : false;
			const bool bIsLODDithered = LODToRender.IsDithered();
			const bool bIsLODRange = LODToRender.IsLODRange();

			float DistanceSquared = (Bounds.BoxSphereBounds.Origin - ViewData.ViewOrigin).SizeSquared();
			const float LODFactorDistanceSquared = DistanceSquared * FMath::Square(ViewData.LODScale);
			const bool bDrawDepthOnly = ViewData.bFullEarlyZPass || ((ShadingPath != EShadingPath::Mobile) && (FMath::Square(Bounds.BoxSphereBounds.SphereRadius) > GMinScreenRadiusForDepthPrepass * GMinScreenRadiusForDepthPrepass * LODFactorDistanceSquared));

			const int32 NumStaticMeshes = PrimitiveSceneInfo->StaticMeshRelevances.Num();
			for (int32 MeshIndex = 0; MeshIndex < NumStaticMeshes; MeshIndex++)
			{
				const FStaticMeshBatchRelevance& StaticMeshRelevance = PrimitiveSceneInfo->StaticMeshRelevances[MeshIndex];
				const FStaticMeshBatch& StaticMesh = PrimitiveSceneInfo->StaticMeshes[MeshIndex];

				if (StaticMeshRelevance.bOverlayMaterial && !View.Family->EngineShowFlags.DistanceCulledPrimitives)
				{
					// Overlay mesh can have its own cull distance that is shorter than primitive cull distance
					float OverlayMaterialMaxDrawDistance = GOverlayMaterialMaxDrawDistanceOverride > 0.0f ? GOverlayMaterialMaxDrawDistanceOverride : StaticMeshRelevance.ScreenSize;
					if (OverlayMaterialMaxDrawDistance > 0.f && OverlayMaterialMaxDrawDistance != FLT_MAX)
					{
						if (DistanceSquared > FMath::Square(OverlayMaterialMaxDrawDistance * MaxDrawDistanceScale))
						{
							// distance culled
							continue;
						}
					}
				}

				int8 StaticMeshLODIndex = StaticMeshRelevance.GetLODIndex();
				if (LODToRender.ContainsLOD(StaticMeshLODIndex))
				{
					uint8 MarkMask = 0;
					bool bHiddenByHLODFade = false; // Hide mesh LOD levels that HLOD is substituting

					if (bIsHLODFading)
					{
						if (bIsHLODFadingOut)
						{
							if (bIsLODDithered && LODToRender.LODIndex1 == StaticMeshLODIndex)
							{
								bHiddenByHLODFade = true;
							}
							else
							{
								MarkMask |= EMarkMaskBits::StaticMeshFadeOutDitheredLODMapMask;
							}
						}
						else
						{
							if (bIsLODDithered && LODToRender.LODIndex0 == StaticMeshLODIndex)
							{
								bHiddenByHLODFade = true;
							}
							else
							{
								MarkMask |= EMarkMaskBits::StaticMeshFadeInDitheredLODMapMask;
							}
						}
					}
					else if (bIsLODDithered)
					{
						if (LODToRender.LODIndex0 == StaticMeshLODIndex)
						{
							MarkMask |= EMarkMaskBits::StaticMeshFadeOutDitheredLODMapMask;
						}
						else
						{
							MarkMask |= EMarkMaskBits::StaticMeshFadeInDitheredLODMapMask;
						}
					}

					// Don't cache if it requires per view per mesh state for LOD dithering or distance cull fade.
					const bool bIsMeshDitheringLOD = StaticMeshRelevance.bDitheredLODTransition && (MarkMask & (EMarkMaskBits::StaticMeshFadeOutDitheredLODMapMask | EMarkMaskBits::StaticMeshFadeInDitheredLODMapMask));
					const bool bCanCache = !bIsPrimitiveDistanceCullFading && !bIsMeshDitheringLOD;

					// When we apply LOD selection on GPU we submit a range of LODs and then cull each one according to screen size.
					// At both ends of a LOD range we only want to cull by screen size in one direction. This ensures that all possible screen sizes map to one LOD in the range.
					EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags = EMeshDrawCommandCullingPayloadFlags::Default;
					CullingPayloadFlags |= bIsLODRange && !LODToRender.IsMaxLODInRange(StaticMeshRelevance.GetLODIndex()) ? EMeshDrawCommandCullingPayloadFlags::MinScreenSizeCull : (EMeshDrawCommandCullingPayloadFlags)0;
					CullingPayloadFlags |= bIsLODRange && !LODToRender.IsMinLODInRange(StaticMeshRelevance.GetLODIndex()) ? EMeshDrawCommandCullingPayloadFlags::MaxScreenSizeCull : (EMeshDrawCommandCullingPayloadFlags)0;

					if (ViewRelevance.bDrawRelevance)
					{
						if ((StaticMeshRelevance.bUseForMaterial || StaticMeshRelevance.bUseAsOccluder)
							&& (ViewRelevance.bRenderInMainPass || ViewRelevance.bRenderCustomDepth || ViewRelevance.bRenderInDepthPass)
							&& !bHiddenByHLODFade)
						{
							// Add velocity commands first to track for case where velocity pass writes depth.
							bool bIsMeshInVelocityPass = false;
							if (StaticMeshRelevance.bUseForMaterial && ViewRelevance.bRenderInMainPass)
							{
								if (ViewRelevance.HasVelocity())
								{
									if (FVelocityMeshProcessor::PrimitiveHasVelocityForView(View, PrimitiveSceneProxy))
									{
										if (ViewRelevance.bVelocityRelevance &&
											FOpaqueVelocityMeshProcessor::PrimitiveCanHaveVelocity(View.GetShaderPlatform(), PrimitiveSceneProxy) &&
											FOpaqueVelocityMeshProcessor::PrimitiveHasVelocityForFrame(View.GetShaderPlatform(), PrimitiveSceneProxy))
										{
											DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::Velocity);
											bIsMeshInVelocityPass = true;

											if (PrimitiveSceneProxy->AnyMaterialUsesMotionVectorWorldOffset())
											{
												bUsesMotionVectorWorldOffset = true;
											}
										}

										if (ViewRelevance.bOutputsTranslucentVelocity &&
											FTranslucentVelocityMeshProcessor::PrimitiveCanHaveVelocity(View.GetShaderPlatform(), PrimitiveSceneProxy) &&
											FTranslucentVelocityMeshProcessor::PrimitiveHasVelocityForFrame(PrimitiveSceneProxy))
										{
											DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::TranslucentVelocity);
											
											// Only Before DoF Translucency needs this pass
											const bool bAnyMaterialUsesTemporalResponsiveness = PrimitiveSceneProxy->AnyMaterialUsesTemporalResponsiveness();
											if (ViewRelevance.bNormalTranslucency && bIsTranslucentClippedDepthEnabled && bAnyMaterialUsesTemporalResponsiveness)
											{
												DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::TranslucentVelocityClippedDepth);
											}
										}
									}
								}
							}

							// Add depth commands.
							if (StaticMeshRelevance.bUseForDepthPass && (bDrawDepthOnly || (bMobileMaskedInEarlyPass && ViewRelevance.bMasked)))
							{
								if (!(bIsMeshInVelocityPass && bVelocityPassWritesDepth))
								{
									if (ViewRelevance.bRenderInSecondStageDepthPass && ShadingPath != EShadingPath::Mobile)
									{
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::SecondStageDepthPass);
									}
									else
									{
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::DepthPass);
									}
								}
#if RHI_RAYTRACING
								// Only used by ray traced shadows
								if (IsRayTracingEnabled() && View.bHasRayTracingShadows && View.IsRayTracingAllowedForView())
								{
									if (MarkMask & EMarkMaskBits::StaticMeshFadeOutDitheredLODMapMask)
									{
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::DitheredLODFadingOutMaskPass);
									}
								}
#endif
							}

							// Mark static mesh as visible for rendering
							if (StaticMeshRelevance.bUseForMaterial && (ViewRelevance.bRenderInMainPass || ViewRelevance.bRenderCustomDepth))
							{
								// Specific logic for mobile packets
								if (ShadingPath == EShadingPath::Mobile)
								{
									if (StaticMeshRelevance.bUseSingleLayerWaterMaterial && !bUsingSingleLayerWaterFallback)
									{
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::SingleLayerWaterPass);
									}
									// Skydome must not be added to base pass bucket
									else if (!StaticMeshRelevance.bUseSkyMaterial)
									{
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::BasePass);
										if (!bMobileBasePassAlwaysUsesCSM)
										{
											DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::MobileBasePassCSM);
										}
									}
									else
									{
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::SkyPass);
									}
									// bUseSingleLayerWaterMaterial is added to BasePass on Mobile. No need to add it to SingleLayerWaterPass

									MarkMask |= EMarkMaskBits::StaticMeshVisibilityMapMask;
								}
								else // Regular shading path
								{
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::BasePass);
									MarkMask |= EMarkMaskBits::StaticMeshVisibilityMapMask;

									if (StaticMeshRelevance.bUseSkyMaterial)
									{
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::SkyPass);
									}
									if (StaticMeshRelevance.bUseSingleLayerWaterMaterial)
									{
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::SingleLayerWaterPass);
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::SingleLayerWaterDepthPrepass);

										if (PrimitiveSceneProxy->AnyMaterialUsesMotionVectorWorldOffset())
										{
											bSLWUsesMotionVectorWorldOffset = true;
										}
									}
								}

								if (StaticMeshRelevance.bUseAnisotropy)
								{
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::AnisotropyPass);
								}

								if (ViewRelevance.bRenderCustomDepth && bIsCompatibleWithCustomDepth)
								{
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::CustomDepth);
								}

								if (bAddLightmapDensityCommands)
								{
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::LightmapDensity);
								}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
								else if (View.Family->UseDebugViewPS())
								{
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::DebugViewMode);
								}
#endif

#if WITH_EDITOR
								if (StaticMeshRelevance.bSelectable)
								{
									if (View.bAllowTranslucentPrimitivesInHitProxy)
									{
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::HitProxy);
									}
									else
									{
										DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::HitProxyOpaqueOnly);
									}
								}
#endif
								++NumVisibleStaticMeshElements;

								INC_DWORD_STAT_BY(STAT_StaticMeshTriangles, StaticMesh.GetNumPrimitives());
							}
						}

						if (StaticMeshRelevance.bUseForMaterial
							&& ViewRelevance.HasTranslucency()
							&& !ViewRelevance.bEditorPrimitiveRelevance
							&& ViewRelevance.bRenderInMainPass)
						{
							if (View.Family->AllowTranslucencyAfterDOF())
							{
								if ((ViewRelevance.bNormalTranslucency || (View.AutoBeforeDOFTranslucencyBoundary > 0.0f && ViewRelevance.bSeparateTranslucency)))
								{
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::TranslucencyStandard);
								}

								if ((ViewRelevance.bNormalTranslucency || (View.AutoBeforeDOFTranslucencyBoundary > 0.0f && ViewRelevance.bSeparateTranslucency)) && ViewRelevance.bTranslucencyModulate && View.Family->AllowStandardTranslucencySeparated())
								{
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::TranslucencyStandardModulate);
								}

								if (ViewRelevance.bSeparateTranslucency)
								{
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::TranslucencyAfterDOF);
								}

								if (ViewRelevance.bSeparateTranslucency && ViewRelevance.bTranslucencyModulate)
								{
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::TranslucencyAfterDOFModulate);
								}

								if (ViewRelevance.bPostMotionBlurTranslucency)
								{
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::TranslucencyAfterMotionBlur);
								}
							}
							else
							{
								// Otherwise, everything is rendered in a single bucket. This is not related to whether DOF is currently enabled or not.
								// When using all translucency, Standard and AfterDOF are sorted together instead of being rendered like 2 buckets.
								DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::TranslucencyAll);
							}

							if (bIsTranslucentHoldoutEnabled)
							{
								DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::TranslucencyHoldout);
							}

							if ((ViewRelevance.bTranslucentSurfaceLighting && bIsLumenTranslucencyRadianceCacheEnabled) || bIsTLVUsingVoxelMarking)
							{
								DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::LumenTranslucencyRadianceCacheMark);
							}

							if (ViewRelevance.bTranslucentSurfaceLighting && bIsFrontLayerTranslucencyEnabled)
							{
								DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::FrontLayerTranslucencyGBuffer);
							}

							if (ViewRelevance.bDistortion)
							{
								DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::Distortion);
							}
						}

#if WITH_EDITOR
						if (ViewRelevance.bEditorVisualizeLevelInstanceRelevance)
						{
							DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::EditorLevelInstance);
						}

						if (ViewRelevance.bEditorStaticSelectionRelevance)
						{
							DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, EMeshPass::EditorSelection);
						}
#endif

						if (ViewRelevance.bHasVolumeMaterialDomain)
						{
							if (ShouldRenderMeshBatchWithHeterogeneousVolumes(&StaticMesh, PrimitiveSceneProxy, View.FeatureLevel))
							{
								HeterogeneousVolumesMeshBatches.AddUninitialized(1);
								FVolumetricMeshBatch& BatchAndProxy = HeterogeneousVolumesMeshBatches.Last();
								BatchAndProxy.Mesh = &StaticMesh;
								BatchAndProxy.Proxy = PrimitiveSceneProxy;
							}
							else
							{
								VolumetricMeshBatches.AddUninitialized(1);
								FVolumetricMeshBatch& BatchAndProxy = VolumetricMeshBatches.Last();
								BatchAndProxy.Mesh = &StaticMesh;
								BatchAndProxy.Proxy = PrimitiveSceneProxy;
							}
						}

						if (ViewRelevance.bUsesSkyMaterial)
						{
							SkyMeshBatches.AddUninitialized(1);
							FSkyMeshBatch& BatchAndProxy = SkyMeshBatches.Last();
							BatchAndProxy.Mesh = &StaticMesh;
							BatchAndProxy.Proxy = PrimitiveSceneProxy;
							BatchAndProxy.bVisibleInMainPass = ViewRelevance.bRenderInMainPass;
							BatchAndProxy.bVisibleInRealTimeSkyCapture = PrimitiveSceneInfo->bVisibleInRealTimeSkyCapture;
						}

						if (ViewRelevance.HasTranslucency() && PrimitiveSceneProxy->SupportsSortedTriangles()) // Need to check material as well
						{
							SortedTrianglesMeshBatches.AddUninitialized(1);
							FSortedTrianglesMeshBatch& BatchAndProxy = SortedTrianglesMeshBatches.Last();
							BatchAndProxy.Mesh = &StaticMesh;
							BatchAndProxy.Proxy = PrimitiveSceneProxy;
						}

						// FIXME: Now if a primitive has one batch with a decal material all primitive mesh batches will be added as decals
						// Because ViewRelevance is a sum of all material relevances in the primitive
						if (ViewRelevance.bRenderInMainPass && ViewRelevance.bDecal && StaticMeshRelevance.bUseForMaterial)
						{
							for (uint8 DecalRenderTargetMode = 0; DecalRenderTargetMode < (uint8)EDecalRenderTargetMode::Num; ++DecalRenderTargetMode)
							{
								if (DecalRendering::IsCompatibleWithRenderTargetMode(StaticMeshRelevance.DecalRenderTargetModeMask, (EDecalRenderTargetMode)DecalRenderTargetMode))
								{
									EMeshPass::Type DecalMeshPassType = DecalRendering::GetMeshPassType((EDecalRenderTargetMode)DecalRenderTargetMode);
									DrawCommandPacket.AddCommandsForMesh(PrimitiveIndex, PrimitiveSceneInfo, StaticMeshRelevance, StaticMesh, CullingPayloadFlags, Scene, bCanCache, DecalMeshPassType);
								}
							}
						}
					}

					if (MarkMask)
					{
						MarkMasks[StaticMeshRelevance.Id] = MarkMask;
					}
				}
			}
		}

		if (!bDrawRelevance)
		{
			NotDrawRelevant.AddPrim(BitIndex);
			continue;
		}

		// Note: Nanite editor draw lists (selection outline + visualize-level-instance) used to be populated
		// here per-packet into FViewInfo::NaniteEditorDraws. That data has moved to Nanite::FEditorSceneExtension
		// and is now built scene-wide by the extension's updater via an RDG setup task.

		if (bEditorRelevance)
		{
			AddEditorDynamicPrimitive(BitIndex);
		}
		else if(bDynamicRelevance)
		{
			AddDynamicPrimitive(BitIndex);

			if (ViewRelevance.bHasSimpleLights)
			{
				VisibleDynamicPrimitivesWithSimpleLights.AddPrim(PrimitiveSceneInfo);
			}
		}
		// Filter with !bStaticRelevance to avoid regular/non-strands geometry to be added two times.
		else if (!bStaticRelevance && bHairStrandsRelevance)
		{
			AddDynamicPrimitive(BitIndex);
		}

		if (bTranslucentRelevance && !bEditorRelevance && ViewRelevance.bRenderInMainPass)
		{
			if (View.Family->AllowTranslucencyAfterDOF())
			{
				if ((ViewRelevance.bNormalTranslucency || (View.AutoBeforeDOFTranslucencyBoundary > 0.0f && ViewRelevance.bSeparateTranslucency)))
				{
					TranslucentPrimCount.Add(ETranslucencyPass::TPT_TranslucencyStandard, ViewRelevance.bUsesSceneColorCopy);
				}

				if ((ViewRelevance.bNormalTranslucency || (View.AutoBeforeDOFTranslucencyBoundary > 0.0f && ViewRelevance.bSeparateTranslucency)) && ViewRelevance.bTranslucencyModulate && View.Family->AllowStandardTranslucencySeparated())
				{
					TranslucentPrimCount.Add(ETranslucencyPass::TPT_TranslucencyStandardModulate, ViewRelevance.bUsesSceneColorCopy);
				}

				if (ViewRelevance.bSeparateTranslucency)
				{
					TranslucentPrimCount.Add(ETranslucencyPass::TPT_TranslucencyAfterDOF, ViewRelevance.bUsesSceneColorCopy);
				}

				if (ViewRelevance.bSeparateTranslucency && ViewRelevance.bTranslucencyModulate)
				{
					TranslucentPrimCount.Add(ETranslucencyPass::TPT_TranslucencyAfterDOFModulate, ViewRelevance.bUsesSceneColorCopy);
				}

				if (ViewRelevance.bPostMotionBlurTranslucency)
				{
					TranslucentPrimCount.Add(ETranslucencyPass::TPT_TranslucencyAfterMotionBlur, ViewRelevance.bUsesSceneColorCopy);
				}
			}
			else // Otherwise, everything is rendered in a single bucket. This is not related to whether DOF is currently enabled or not.
			{
				// When using all translucency, Standard and AfterDOF are sorted together instead of being rendered like 2 buckets.
				TranslucentPrimCount.Add(ETranslucencyPass::TPT_AllTranslucency, ViewRelevance.bUsesSceneColorCopy);
			}
			
			if (bIsTranslucentHoldoutEnabled)
			{
				TranslucentPrimCount.Add(ETranslucencyPass::TPT_TranslucencyHoldout, true); // use scene copy
			}

			if (ViewRelevance.bDistortion)
			{
				bHasDistortionPrimitives = true;
			}
		}

		CombinedShadingModelMask |= ViewRelevance.ShadingModelMask;
		SubstrateUintPerPixel = FMath::Max(SubstrateUintPerPixel, uint8(ViewRelevance.SubstrateUintPerPixel));
		SubstrateTileTypeMask |= ViewRelevance.SubstrateTileTypeMask;
		bUsesAnisotropy |= ViewRelevance.bUsesAnisotropy;
		SubstrateClosureCountMask |= ViewRelevance.SubstrateClosureCountMask;
		bUsesGlobalDistanceField |= ViewRelevance.bUsesGlobalDistanceField;
		bUsesLightingChannels |= ViewRelevance.bUsesLightingChannels;
		bTranslucentSurfaceLighting |= ViewRelevance.bTranslucentSurfaceLighting;
		bUsesCustomDepth |= (ViewRelevance.CustomDepthStencilUsageMask & 1) > 0;
		bUsesCustomStencil |= (ViewRelevance.CustomDepthStencilUsageMask & (1 << 1)) > 0;
		bSceneHasSkyMaterial |= ViewRelevance.bUsesSkyMaterial;
		bHasSingleLayerWaterMaterial |= ViewRelevance.bUsesSingleLayerWaterMaterial;
		bUsesSecondStageDepthPass |= ViewRelevance.bRenderInSecondStageDepthPass && ShadingPath!=EShadingPath::Mobile;

		if (ViewRelevance.bRenderCustomDepth && bIsCompatibleWithCustomDepth)
		{
			bHasCustomDepthPrimitives = true;
			CustomDepthStencilValues.Add(PrimitiveSceneInfo->Proxy->GetCustomDepthStencilValue());

			if (PrimitiveSceneInfo->Proxy->IsNaniteMesh())
			{
				check(PrimitiveSceneInfo->IsIndexValid());
				NaniteCustomDepthInstances.AddPrim(
					FPrimitiveInstanceRange {
						PrimitiveSceneInfo->GetIndex(),
						PrimitiveSceneInfo->GetInstanceSceneDataOffset(),
						PrimitiveSceneInfo->GetNumInstanceSceneDataEntries()
					}
				);
			}
		}

		extern bool GUseTranslucencyShadowDepths;
		if (GUseTranslucencyShadowDepths && ViewRelevance.bTranslucentSelfShadow)
		{
			TranslucentSelfShadowPrimitives.AddPrim(BitIndex);
		}

		PrimitiveSceneInfo->LastRenderTime = CurrentWorldTime;

		// Update the last component render time only if we know for certain the primitive is un-occluded.
		if ((View.PrimitiveDefinitelyUnoccludedMap[BitIndex].AtomicGet() || View.Family->EngineShowFlags.Wireframe) && !PrimitiveSceneProxy->IsAlwaysVisible())
		{
			const bool bUpdateLastRenderTimeOnScreen = true;
			PrimitiveSceneInfo->UpdateComponentLastRenderTime(CurrentWorldTime, bUpdateLastRenderTimeOnScreen);
		}

		// Cache the nearest reflection proxy if needed
		if (PrimitiveSceneInfo->NeedsReflectionCaptureUpdate())
		{
			// mobile should not have any outstanding reflection capture update requests at this point, except for when lighting isn't rebuilt		
			PrimitiveSceneInfo->CacheReflectionCaptures();
		}

		if (PrimitiveSceneInfo->NeedsIndirectLightingCacheBufferUpdate())
		{
			DirtyIndirectLightingCacheBufferPrimitives.AddPrim(PrimitiveSceneInfo);
		}
	}

	static_assert(sizeof(WriteView.NumVisibleStaticMeshElements) == sizeof(int32), "Atomic is the wrong size");
	FPlatformAtomics::InterlockedAdd((volatile int32*)&WriteView.NumVisibleStaticMeshElements, NumVisibleStaticMeshElements);
}

///////////////////////////////////////////////////////////////////////////////

FComputeAndMarkRelevance::FComputeAndMarkRelevance(FVisibilityTaskData& InTaskData, FScene& InScene, FViewInfo& InView, uint8 InViewIndex, const UE::Tasks::FTask& InPrerequisitesTask)
	: TaskData(InTaskData)
	, Scene(InScene)
	, View(InView)
	, ViewCommands(TaskData.DynamicMeshElements.ViewCommandsPerView[InViewIndex])
	, ViewIndex(InViewIndex)
	, ViewData(View)
	, NumMeshes(Scene.StaticMeshes.GetMaxIndex())
	, NumPrimitivesPerPacket(InTaskData.TaskConfig.Relevance.NumPrimitivesPerPacket)
	, PrerequisitesTask(InPrerequisitesTask)
	, bLaunchOnAddPrimitive(TaskData.TaskConfig.Schedule == EVisibilityTaskSchedule::Parallel)
	, bFinished(!bLaunchOnAddPrimitive)
{
	if (FSceneCullingRenderer* SceneCullingRenderer = TaskData.SceneRenderer.GetSceneExtensionsRenderers().GetRendererPtr<FSceneCullingRenderer>())
	{
		SceneCullingQuery = SceneCullingRenderer->GetPrimaryViewHierarchicalCullingQuery(View.SceneRendererPrimaryViewId);
	}
	
	MarkMasks = (uint8*)TaskData.Allocator.Malloc(NumMeshes + 31, 8); // some padding to simplify the high speed transpose
	FMemory::Memzero(MarkMasks, NumMeshes + 31);
	Packets.Reserve(InTaskData.TaskConfig.Relevance.NumEstimatedPackets);
	CreateRelevancePacket();
}

void FComputeAndMarkRelevance::AddPrimitives(FPrimitiveIndexList&& PrimitiveIndexList)
{
	if (PrimitiveIndexList.Num() == NumPrimitivesPerPacket)
	{
		// Create a one-off packet that will take all the primitives.
		FRelevancePacket* Packet = CreateRelevancePacket();
		Packet->Input.Prims = MoveTemp(PrimitiveIndexList);

		if (bLaunchOnAddPrimitive)
		{
			Packet->LaunchComputeRelevanceTask();
		}

		// Put the previous last packet that was in progress back in the last slot.
		Swap(Packets[Packets.Num() - 1], Packets[Packets.Num() - 2]);
	}

	else
	{
		for (int32 Index : PrimitiveIndexList)
		{
			AddPrimitive(Index);
		}
	}
}

void FComputeAndMarkRelevance::AddPrimitive(int32 Index)
{
	FRelevancePacket* Packet = Packets.Last();

	if (Packet->Input.AddPrim(Index); Packet->Input.IsFull())
	{
		if (bLaunchOnAddPrimitive)
		{
			Packet->LaunchComputeRelevanceTask();
		}
		CreateRelevancePacket();
	}
}

void FComputeAndMarkRelevance::Finish(UE::Tasks::FTaskEvent& ComputeRelevanceTaskEvent)
{
	check(!bFinished);
	bFinished = true;

	SCOPED_NAMED_EVENT(FinishRelevance, FColor::Magenta);

	Packets.Last()->LaunchComputeRelevanceTask();

	for (FRelevancePacket* Packet : Packets)
	{
		if (Packet->bComputeRelevanceTaskLaunched)
		{
			ComputeRelevanceTaskEvent.AddPrerequisites(Packet->ComputeRelevanceTask);
		}
	}

	ComputeRelevanceTaskEvent.Trigger();
}

void FComputeAndMarkRelevance::Finalize()
{
	check(bFinished && !bFinalized);
	bFinalized = true;
	PrerequisitesTask.Wait();

	if (!bLaunchOnAddPrimitive)
	{
		ParallelFor(Packets.Num(), [this](int32 Index)
		{
			FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
			FDynamicPrimitiveIndexList DynamicPrimitiveIndexList;

			FRelevancePacket* Packet = Packets[Index];
			Packet->ComputeRelevance(DynamicPrimitiveIndexList);
		});
	}

	SCOPED_NAMED_EVENT(FinalizeRelevance, FColor::Magenta);

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ComputeAndMarkRelevanceForViewParallel_Finalize);

		for (int32 PassIndex = 0; PassIndex < EMeshPass::Num; PassIndex++)
		{
			int32 NumVisibleCachedMeshDrawCommands = 0;
			int32 NumDynamicBuildRequests = 0;
			int32 NumDynamicBuildFlags = 0;

			for (auto Packet : Packets)
			{
				NumVisibleCachedMeshDrawCommands += Packet->DrawCommandPacket.VisibleCachedDrawCommands[PassIndex].Num();
				NumDynamicBuildRequests += Packet->DrawCommandPacket.DynamicBuildRequests[PassIndex].Num();
				NumDynamicBuildFlags += Packet->DrawCommandPacket.DynamicBuildFlags[PassIndex].Num();
			}

			ViewCommands.MeshCommands[PassIndex].Reserve(NumVisibleCachedMeshDrawCommands);
			ViewCommands.DynamicMeshCommandBuildRequests[PassIndex].Reserve(NumDynamicBuildRequests);
			ViewCommands.DynamicMeshCommandBuildFlags[PassIndex].Reserve(NumDynamicBuildFlags);
			check(NumDynamicBuildRequests == NumDynamicBuildFlags);
		}

		View.DirtyIndirectLightingCacheBufferPrimitivesMutex.Lock();

		for (auto Packet : Packets)
		{
			Packet->Finalize();
			delete Packet;
		}

		View.DirtyIndirectLightingCacheBufferPrimitivesMutex.Unlock();

		Packets.Empty();

		// Finalize Nanite materials
		if (TaskData.bAddNaniteRelevance)
		{
			// This needs to complete before InitViews runs so that combined primitive/material relevance has been computed for Nanite
			Scene.WaitForCacheNaniteMaterialBinsTask();

			FViewInfo& WriteView = View;
			{
				const FNaniteShadingPipelines& ShadingPipelines = Scene.NaniteShadingPipelines[ENaniteMeshPass::BasePass];
				const FPrimitiveViewRelevance& CombinedRelevance = ShadingPipelines.CombinedRelevance;

				WriteView.ShadingModelMaskInView |= CombinedRelevance.ShadingModelMask;
				WriteView.bUsesGlobalDistanceField |= CombinedRelevance.bUsesGlobalDistanceField;
				WriteView.bUsesLightingChannels |= CombinedRelevance.bUsesLightingChannels;
				WriteView.bSceneHasSkyMaterial |= CombinedRelevance.bUsesSkyMaterial;
				WriteView.bHasDistortionPrimitives |= CombinedRelevance.bDistortion;
				WriteView.bHasCustomDepthPrimitives |= CombinedRelevance.bRenderCustomDepth;
				WriteView.bUsesCustomDepth |= (CombinedRelevance.CustomDepthStencilUsageMask & 1) > 0;
				WriteView.bUsesCustomStencil |= (CombinedRelevance.CustomDepthStencilUsageMask & (1 << 1)) > 0;
				WriteView.SubstrateViewData.MaxClosurePerPixel = FMath::Max(WriteView.SubstrateViewData.MaxClosurePerPixel, 8u - FMath::CountLeadingZeros8(CombinedRelevance.SubstrateClosureCountMask));
				WriteView.SubstrateViewData.MaxBytesPerPixel = FMath::Max(WriteView.SubstrateViewData.MaxBytesPerPixel, CombinedRelevance.SubstrateUintPerPixel * 4u);
				WriteView.SubstrateViewData.UsesTileTypeMask |= CombinedRelevance.SubstrateTileTypeMask;
				WriteView.SubstrateViewData.bUsesAnisotropy |= CombinedRelevance.bUsesAnisotropy;
			}
		}
	}

	TRACE_COUNTER_SET(Scene_Visibility_Relevance_NumPrimitivesProcessed, TaskData.TaskConfig.Relevance.NumPrimitivesProcessed);

	QUICK_SCOPE_CYCLE_COUNTER(STAT_ComputeAndMarkRelevanceForViewParallel_TransposeMeshBits);
	check(View.StaticMeshVisibilityMap.Num() == NumMeshes &&
		View.StaticMeshFadeOutDitheredLODMap.Num() == NumMeshes &&
		View.StaticMeshFadeInDitheredLODMap.Num() == NumMeshes
	);
	uint32* RESTRICT StaticMeshVisibilityMap_Words = View.StaticMeshVisibilityMap.GetData();
	uint32* RESTRICT StaticMeshFadeOutDitheredLODMap_Words = View.StaticMeshFadeOutDitheredLODMap.GetData();
	uint32* RESTRICT StaticMeshFadeInDitheredLODMap_Words = View.StaticMeshFadeInDitheredLODMap.GetData();
	const uint64* RESTRICT MarkMasks64 = (const uint64 * RESTRICT)MarkMasks;
	const uint8* RESTRICT MarkMasks8 = MarkMasks;
	for (uint32 BaseIndex = 0; BaseIndex < NumMeshes; BaseIndex += 32)
	{
		uint32 StaticMeshVisibilityMap_Word = 0;
		uint32 StaticMeshFadeOutDitheredLODMap_Word = 0;
		uint32 StaticMeshFadeInDitheredLODMap_Word = 0;
		uint32 Mask = 1;
		bool bAny = false;
		for (int32 QWordIndex = 0; QWordIndex < 4; QWordIndex++)
		{
			if (*MarkMasks64++)
			{
				for (int32 ByteIndex = 0; ByteIndex < 8; ByteIndex++, Mask <<= 1, MarkMasks8++)
				{
					uint8 MaskMask = *MarkMasks8;
					StaticMeshVisibilityMap_Word |= (MaskMask & EMarkMaskBits::StaticMeshVisibilityMapMask) ? Mask : 0;
					StaticMeshFadeOutDitheredLODMap_Word |= (MaskMask & EMarkMaskBits::StaticMeshFadeOutDitheredLODMapMask) ? Mask : 0;
					StaticMeshFadeInDitheredLODMap_Word |= (MaskMask & EMarkMaskBits::StaticMeshFadeInDitheredLODMapMask) ? Mask : 0;
				}
				bAny = true;
			}
			else
			{
				MarkMasks8 += 8;
				Mask <<= 8;
			}
		}
		if (bAny)
		{
			checkSlow(!*StaticMeshVisibilityMap_Words && !*StaticMeshFadeOutDitheredLODMap_Words && !*StaticMeshFadeInDitheredLODMap_Words);
			*StaticMeshVisibilityMap_Words = StaticMeshVisibilityMap_Word;
			*StaticMeshFadeOutDitheredLODMap_Words = StaticMeshFadeOutDitheredLODMap_Word;
			*StaticMeshFadeInDitheredLODMap_Words = StaticMeshFadeInDitheredLODMap_Word;
		}
		StaticMeshVisibilityMap_Words++;
		StaticMeshFadeOutDitheredLODMap_Words++;
		StaticMeshFadeInDitheredLODMap_Words++;
	}

	if (View.bIsSinglePassStereo)
	{
		check(View.StereoPass != EStereoscopicPass::eSSP_SECONDARY); // Secondary views should not calculate relevance
		if (const FViewInfo* SecondaryViewConst = View.GetInstancedView())
		{
			FViewInfo* SecondaryView = const_cast<FViewInfo*>(SecondaryViewConst);
			SecondaryView->PrimitiveVisibilityMap = View.PrimitiveVisibilityMap;
			SecondaryView->PrimitiveRayTracingVisibilityMap = View.PrimitiveRayTracingVisibilityMap;
			SecondaryView->PrimitiveDefinitelyUnoccludedMap = View.PrimitiveDefinitelyUnoccludedMap;
			SecondaryView->PrimitiveViewRelevanceMap = View.PrimitiveViewRelevanceMap;
			SecondaryView->StaticMeshVisibilityMap = View.StaticMeshVisibilityMap;
			SecondaryView->PrimitivesLODMask = View.PrimitivesLODMask;
			SecondaryView->NumVisibleDynamicPrimitives = View.NumVisibleDynamicPrimitives;
			SecondaryView->NumVisibleDynamicEditorPrimitives = View.NumVisibleDynamicEditorPrimitives;

			SecondaryView->ShadingModelMaskInView = View.ShadingModelMaskInView;
			SecondaryView->bUsesGlobalDistanceField = View.bUsesGlobalDistanceField;
			SecondaryView->bUsesLightingChannels = View.bUsesLightingChannels;
			SecondaryView->bTranslucentSurfaceLighting = View.bTranslucentSurfaceLighting;
			SecondaryView->bSceneHasSkyMaterial = View.bSceneHasSkyMaterial;
			SecondaryView->bHasSingleLayerWaterMaterial = View.bHasSingleLayerWaterMaterial;
			SecondaryView->bUsesSecondStageDepthPass = View.bUsesSecondStageDepthPass;
			SecondaryView->TranslucentPrimCount = View.TranslucentPrimCount;
			SecondaryView->bHasDistortionPrimitives = View.bHasDistortionPrimitives;
			SecondaryView->bHasCustomDepthPrimitives = View.bHasCustomDepthPrimitives;
			SecondaryView->CustomDepthStencilValues = View.CustomDepthStencilValues;
			SecondaryView->bUsesCustomDepth = View.bUsesCustomDepth;
			SecondaryView->bUsesCustomStencil = View.bUsesCustomStencil;
			SecondaryView->SubstrateViewData.MaxClosurePerPixel = View.SubstrateViewData.MaxClosurePerPixel;
			SecondaryView->SubstrateViewData.MaxBytesPerPixel = View.SubstrateViewData.MaxClosurePerPixel;
			SecondaryView->SubstrateViewData.UsesTileTypeMask = View.SubstrateViewData.UsesTileTypeMask;
			SecondaryView->SubstrateViewData.bUsesAnisotropy = View.SubstrateViewData.bUsesAnisotropy;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////

static void ComputeDynamicMeshRelevance(
	EShadingPath ShadingPath,
	bool bAddLightmapDensityCommands,
	bool bIsLumenTranslucencyRadianceCacheEnabled,
	bool bIsTLVUsingVoxelMarking,
	bool bIsFrontLayerTranslucencyEnabled,
	const FPrimitiveViewRelevance& ViewRelevance,
	const FMeshBatchAndRelevance& MeshBatch,
	FViewInfo& View,
	FMeshPassMask& PassMask,
	FPrimitiveSceneInfo* PrimitiveSceneInfo,
	const FPrimitiveBounds& Bounds)
{
	const int32 NumElements = MeshBatch.Mesh->Elements.Num();
	const bool bIsTranslucentClippedDepthEnabled = NeedTSRTranslucentClippedDepthPass(View);
	// Filter out hair strands geometry, as custom depth is handled in a custom manner. This is only for hair strands, hair cards/meshes use the regular path. 
	// Use !bRenderInMainPass as a discriminant for filtering only hair strands geometry.
	const bool bIsCompatibleWithCustomDepth = !(ViewRelevance.bHairStrands && !ViewRelevance.bRenderInMainPass);

	if (ViewRelevance.bDrawRelevance && (ViewRelevance.bRenderInMainPass || ViewRelevance.bRenderCustomDepth || ViewRelevance.bRenderInDepthPass))
	{
		if (ViewRelevance.bRenderInSecondStageDepthPass && ShadingPath != EShadingPath::Mobile)
		{
			PassMask.Set(EMeshPass::SecondStageDepthPass);
			View.NumVisibleDynamicMeshElements[EMeshPass::SecondStageDepthPass] += NumElements;
		}
		else
		{
			PassMask.Set(EMeshPass::DepthPass);
			View.NumVisibleDynamicMeshElements[EMeshPass::DepthPass] += NumElements;
		}

		if (ViewRelevance.bRenderInMainPass || ViewRelevance.bRenderCustomDepth)
		{
			PassMask.Set(EMeshPass::BasePass);
			View.NumVisibleDynamicMeshElements[EMeshPass::BasePass] += NumElements;

			if (ViewRelevance.bUsesSkyMaterial)
			{
				PassMask.Set(EMeshPass::SkyPass);
				View.NumVisibleDynamicMeshElements[EMeshPass::SkyPass] += NumElements;
			}

			if (ViewRelevance.bUsesAnisotropy)
			{
				PassMask.Set(EMeshPass::AnisotropyPass);
				View.NumVisibleDynamicMeshElements[EMeshPass::AnisotropyPass] += NumElements;
			}

			if (ShadingPath == EShadingPath::Mobile)
			{
				PassMask.Set(EMeshPass::MobileBasePassCSM);
				View.NumVisibleDynamicMeshElements[EMeshPass::MobileBasePassCSM] += NumElements;
			}

			if (ViewRelevance.bRenderCustomDepth && bIsCompatibleWithCustomDepth)
			{
				PassMask.Set(EMeshPass::CustomDepth);
				View.NumVisibleDynamicMeshElements[EMeshPass::CustomDepth] += NumElements;
			}

			if (bAddLightmapDensityCommands)
			{
				PassMask.Set(EMeshPass::LightmapDensity);
				View.NumVisibleDynamicMeshElements[EMeshPass::LightmapDensity] += NumElements;
			}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			else if (View.Family->UseDebugViewPS())
			{
				PassMask.Set(EMeshPass::DebugViewMode);
				View.NumVisibleDynamicMeshElements[EMeshPass::DebugViewMode] += NumElements;
			}
#endif

#if WITH_EDITOR
			if (View.bAllowTranslucentPrimitivesInHitProxy)
			{
				PassMask.Set(EMeshPass::HitProxy);
				View.NumVisibleDynamicMeshElements[EMeshPass::HitProxy] += NumElements;
			}
			else
			{
				PassMask.Set(EMeshPass::HitProxyOpaqueOnly);
				View.NumVisibleDynamicMeshElements[EMeshPass::HitProxyOpaqueOnly] += NumElements;
			}
#endif

			if (ViewRelevance.bVelocityRelevance)
			{
				PassMask.Set(EMeshPass::Velocity);
				View.NumVisibleDynamicMeshElements[EMeshPass::Velocity] += NumElements;

				if (MeshBatch.PrimitiveSceneProxy->AnyMaterialUsesMotionVectorWorldOffset())
				{
					View.bUsesMotionVectorWorldOffset = true;
				}
			}

			if (ViewRelevance.bOutputsTranslucentVelocity)
			{
				PassMask.Set(EMeshPass::TranslucentVelocity);
				View.NumVisibleDynamicMeshElements[EMeshPass::TranslucentVelocity] += NumElements;
				const bool bAnyMaterialUsesTemporalResponsiveness = MeshBatch.PrimitiveSceneProxy->AnyMaterialUsesTemporalResponsiveness();
				if (ViewRelevance.bNormalTranslucency && bIsTranslucentClippedDepthEnabled && bAnyMaterialUsesTemporalResponsiveness)
				{
					PassMask.Set(EMeshPass::TranslucentVelocityClippedDepth);
					View.NumVisibleDynamicMeshElements[EMeshPass::TranslucentVelocityClippedDepth] += NumElements;
				}
			}

			if (ViewRelevance.bUsesSingleLayerWaterMaterial)
			{
				PassMask.Set(EMeshPass::SingleLayerWaterPass);
				View.NumVisibleDynamicMeshElements[EMeshPass::SingleLayerWaterPass] += NumElements;
				PassMask.Set(EMeshPass::SingleLayerWaterDepthPrepass);
				View.NumVisibleDynamicMeshElements[EMeshPass::SingleLayerWaterDepthPrepass] += NumElements;

				if (MeshBatch.PrimitiveSceneProxy->AnyMaterialUsesMotionVectorWorldOffset())
				{
					View.bSLWUsesMotionVectorWorldOffset = true;
				}
			}
		}
	}

	if (ViewRelevance.HasTranslucency()
		&& !ViewRelevance.bEditorPrimitiveRelevance
		&& ViewRelevance.bRenderInMainPass)
	{
		if (View.Family->AllowTranslucencyAfterDOF())
		{
			if ((ViewRelevance.bNormalTranslucency || (View.AutoBeforeDOFTranslucencyBoundary > 0.0f && ViewRelevance.bSeparateTranslucency)))
			{
				PassMask.Set(EMeshPass::TranslucencyStandard);
				View.NumVisibleDynamicMeshElements[EMeshPass::TranslucencyStandard] += NumElements;
			}

			if ((ViewRelevance.bNormalTranslucency || (View.AutoBeforeDOFTranslucencyBoundary > 0.0f && ViewRelevance.bSeparateTranslucency)) && ViewRelevance.bTranslucencyModulate && View.Family->AllowStandardTranslucencySeparated())
			{
				PassMask.Set(EMeshPass::TranslucencyStandardModulate);
				View.NumVisibleDynamicMeshElements[EMeshPass::TranslucencyStandardModulate] += NumElements;
			}

			if (ViewRelevance.bSeparateTranslucency)
			{
				PassMask.Set(EMeshPass::TranslucencyAfterDOF);
				View.NumVisibleDynamicMeshElements[EMeshPass::TranslucencyAfterDOF] += NumElements;
			}

			if (ViewRelevance.bSeparateTranslucency && ViewRelevance.bTranslucencyModulate)
			{
				PassMask.Set(EMeshPass::TranslucencyAfterDOFModulate);
				View.NumVisibleDynamicMeshElements[EMeshPass::TranslucencyAfterDOFModulate] += NumElements;
			}

			if (ViewRelevance.bPostMotionBlurTranslucency)
			{
				PassMask.Set(EMeshPass::TranslucencyAfterMotionBlur);
				View.NumVisibleDynamicMeshElements[EMeshPass::TranslucencyAfterMotionBlur] += NumElements;
			}
		}
		else
		{
			PassMask.Set(EMeshPass::TranslucencyAll);
			View.NumVisibleDynamicMeshElements[EMeshPass::TranslucencyAll] += NumElements;
		}

		{
			PassMask.Set(EMeshPass::TranslucencyHoldout);
			View.NumVisibleDynamicMeshElements[EMeshPass::TranslucencyHoldout] += NumElements;
		}

		if ((ViewRelevance.bTranslucentSurfaceLighting && bIsLumenTranslucencyRadianceCacheEnabled) || bIsTLVUsingVoxelMarking)
		{
			PassMask.Set(EMeshPass::LumenTranslucencyRadianceCacheMark);
			View.NumVisibleDynamicMeshElements[EMeshPass::LumenTranslucencyRadianceCacheMark] += NumElements;
		}

		if (ViewRelevance.bTranslucentSurfaceLighting && bIsFrontLayerTranslucencyEnabled)
		{
			PassMask.Set(EMeshPass::FrontLayerTranslucencyGBuffer);
			View.NumVisibleDynamicMeshElements[EMeshPass::FrontLayerTranslucencyGBuffer] += NumElements;
		}

		if (ViewRelevance.bDistortion)
		{
			PassMask.Set(EMeshPass::Distortion);
			View.NumVisibleDynamicMeshElements[EMeshPass::Distortion] += NumElements;
		}
	}

#if WITH_EDITOR
	if (ViewRelevance.bDrawRelevance)
	{
		PassMask.Set(EMeshPass::EditorSelection);
		View.NumVisibleDynamicMeshElements[EMeshPass::EditorSelection] += NumElements;

		PassMask.Set(EMeshPass::EditorLevelInstance);
		View.NumVisibleDynamicMeshElements[EMeshPass::EditorLevelInstance] += NumElements;
	}

	// Hair strands are not rendered into the base pass (bRenderInMainPass=0) and so this 
	// adds a special pass for allowing hair strands to be selectable.
	if (ViewRelevance.bHairStrands)
	{
		const EMeshPass::Type MeshPassType = View.bAllowTranslucentPrimitivesInHitProxy ? EMeshPass::HitProxy : EMeshPass::HitProxyOpaqueOnly;
		PassMask.Set(MeshPassType);
		View.NumVisibleDynamicMeshElements[MeshPassType] += NumElements;
	}
#endif

	if (ViewRelevance.bHasVolumeMaterialDomain)
	{
		if (ShouldRenderMeshBatchWithHeterogeneousVolumes(MeshBatch.Mesh, MeshBatch.PrimitiveSceneProxy, View.FeatureLevel))
		{
			View.HeterogeneousVolumesMeshBatches.AddUninitialized(1);
			FVolumetricMeshBatch& BatchAndProxy = View.HeterogeneousVolumesMeshBatches.Last();
			BatchAndProxy.Mesh = MeshBatch.Mesh;
			BatchAndProxy.Proxy = MeshBatch.PrimitiveSceneProxy;
		}
		else
		{
			View.VolumetricMeshBatches.AddUninitialized(1);
			FVolumetricMeshBatch& BatchAndProxy = View.VolumetricMeshBatches.Last();
			BatchAndProxy.Mesh = MeshBatch.Mesh;
			BatchAndProxy.Proxy = MeshBatch.PrimitiveSceneProxy;
		}
	}

	if (ViewRelevance.bUsesSkyMaterial)
	{
		View.SkyMeshBatches.AddUninitialized(1);
		FSkyMeshBatch& BatchAndProxy = View.SkyMeshBatches.Last();
		BatchAndProxy.Mesh = MeshBatch.Mesh;
		BatchAndProxy.Proxy = MeshBatch.PrimitiveSceneProxy;
		BatchAndProxy.bVisibleInMainPass = ViewRelevance.bRenderInMainPass;
		BatchAndProxy.bVisibleInRealTimeSkyCapture = PrimitiveSceneInfo->bVisibleInRealTimeSkyCapture;
	}

	if (ViewRelevance.HasTranslucency() && PrimitiveSceneInfo->Proxy->SupportsSortedTriangles())
	{
		View.SortedTrianglesMeshBatches.AddUninitialized(1);
		FSortedTrianglesMeshBatch& BatchAndProxy = View.SortedTrianglesMeshBatches.Last();
		BatchAndProxy.Mesh = MeshBatch.Mesh;
		BatchAndProxy.Proxy = MeshBatch.PrimitiveSceneProxy;
	}

	if (ViewRelevance.bRenderInMainPass && ViewRelevance.bDecal)
	{
		// DecalRenderTargetModeMask could be cached in FMeshBatchAndRelevance as well
		const FMaterial& Material = MeshBatch.Mesh->MaterialRenderProxy->GetIncompleteMaterialWithFallback(View.FeatureLevel);
		uint8 DecalRenderTargetModeMask = DecalRendering::GetDecalRenderTargetModeMask(Material, View.FeatureLevel);
		for (uint8 DecalRenderTargetMode = 0; DecalRenderTargetMode < (uint8)EDecalRenderTargetMode::Num; ++DecalRenderTargetMode)
		{
			if (DecalRendering::IsCompatibleWithRenderTargetMode(DecalRenderTargetModeMask, (EDecalRenderTargetMode)DecalRenderTargetMode))
			{
				EMeshPass::Type DecalMeshPassType = DecalRendering::GetMeshPassType((EDecalRenderTargetMode)DecalRenderTargetMode);
				PassMask.Set(DecalMeshPassType);
				View.NumVisibleDynamicMeshElements[DecalMeshPassType] += NumElements;
			}
		}
	}

	const bool bIsHairStrandsCompatible = ViewRelevance.bHairStrands && IsHairStrandsEnabled(EHairStrandsShaderType::All, View.GetShaderPlatform());
	if (bIsHairStrandsCompatible)
	{
		// Disable bCheckLengthScaleInitialize when running hit proxy as LengthScale is not initialized
		const bool bCheckLengthScale = !View.Family->EngineShowFlags.HitProxies;
		if (HairStrands::IsHairStrandsVF(MeshBatch.Mesh) && HairStrands::IsHairVisible(MeshBatch, bCheckLengthScale))
		{
			View.HairStrandsMeshElements.AddUninitialized(1);
			FMeshBatchAndRelevance& BatchAndProxy = View.HairStrandsMeshElements.Last();
			BatchAndProxy = MeshBatch;
		}

		if (HairStrands::IsHairCardsVF(MeshBatch.Mesh) && ViewRelevance.bRenderInMainPass)
		{
			View.HairCardsMeshElements.AddUninitialized(1);
			FMeshBatchAndRelevance& BatchAndProxy = View.HairCardsMeshElements.Last();
			BatchAndProxy = MeshBatch;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////

FGPUOcclusionPacket::FGPUOcclusionPacket(FVisibilityViewPacket& InViewPacket, const FGPUOcclusionState& InOcclusionState)
	: ViewPacket(InViewPacket)
	, View(ViewPacket.View)
	, ViewState(*ViewPacket.ViewState)
	, ViewElementPDI(ViewPacket.ViewElementPDI)
	, HZBOcclusionTests(ViewState.HZBOcclusionTests)
	, OcclusionFeedback(ViewState.OcclusionFeedback)
	, PrimitiveOcclusionHistorySet(ViewState.Occlusion.PrimitiveOcclusionHistorySet)
	, Scene(ViewPacket.Scene)
	, OcclusionState(InOcclusionState)
	, ViewOrigin(View.ViewMatrices.GetViewOrigin())
	, OcclusionFrameCounter(ViewState.OcclusionFrameCounter)
	, PrimitiveProbablyVisibleTime(GEngine->PrimitiveProbablyVisibleTime)
	, CurrentRealTime(View.Family->Time.GetRealTimeSeconds())
	, NeverOcclusionTestDistanceSquared(GNeverOcclusionTestDistance* GNeverOcclusionTestDistance)
	, bUseOcclusionFeedback(View.FeatureLevel == ERHIFeatureLevel::ES3_1 && OcclusionFeedback.IsInitialized())
	, bNewlyConsideredBBoxExpandActive(
		(GExpandNewlyOcclusionTestedBBoxesAmount > 0.0f || GExpandNewlyOcclusionTestedBBoxesScreenSpace > 0.0f) &&
		GFramesToExpandNewlyOcclusionTestedBBoxes > 0 && GFramesNotOcclusionTestedToExpandBBoxes > 0)
{}

template <bool bIsParallel, typename VisitorType>
bool FGPUOcclusionPacket::OcclusionCullPrimitive(VisitorType& Visitor, FOcclusionCullResult& Result, int32 Index)
{
	const uint8 OcclusionFlags = Scene.PrimitiveOcclusionFlags[Index];

	int32 NumSubQueries = 1;
	bool bSubQueries = false;
	const TArray<FBoxSphereBounds>* SubBounds = nullptr;
	int32 SubIsOccludedStart = 0;

	const auto SetAtomicBit = [] (FSceneBitArray& Array, int32 Index, bool Value)
	{
		if constexpr (bIsParallel)
		{
			Array[Index].AtomicSet(Value);
		}
		else
		{
			Array[Index] = Value;
		}
	};

	if (OcclusionFlags & EOcclusionFlags::IsForceHidden)
	{
		// Primitive is not visible and does not need to be occlusion tested
		return false;
	}

	if ((OcclusionFlags & EOcclusionFlags::HasSubprimitiveQueries) && OcclusionState.bAllowSubQueries)
	{
		FPrimitiveSceneProxy* Proxy = Scene.PrimitiveSceneProxies[Index];
		SubBounds = Proxy->GetOcclusionQueries(&View);
		NumSubQueries = SubBounds->Num();
		bSubQueries = true;
		if (!NumSubQueries)
		{
			SetAtomicBit(View.PrimitiveVisibilityMap, Index, false);
			return false;
		}

		if (!SubIsOccluded || SubIsOccluded->Num() + NumSubQueries > SubIsOccluded->Max())
		{
			SubIsOccluded = View.Allocator.Create<TArray<bool>>();
			SubIsOccluded->Reserve(FMath::Max<uint32>(NumSubQueries, SubIsOccludedPageSize));
		}

		SubIsOccludedStart = SubIsOccluded->Num();
	}

	bool bIsVisible = true;
	bool bAllSubOcclusionStateIsDefinite = true;
	bool bAllSubOccluded = true;
	FPrimitiveComponentId PrimitiveId = Scene.PrimitiveComponentIds[Index];

	// Presolve for screen space bbox expansion
	FVector4 ExpansionWorldSpaceNewlyConsidered;
	FVector4 ExpansionWorldSpaceNotNewlyConsidered;
	FVector4 ViewProjectionHomogeneousVector;
	if (GExpandAllTestedBBoxesScreenSpace > 0.0f || GExpandNewlyOcclusionTestedBBoxesScreenSpace > 0.0f)
	{
		const FMatrix& InvViewProjectionMatrix = View.ViewMatrices.GetClipToWorld();

		float ExtentNewly = GExpandAllTestedBBoxesScreenSpace + GExpandNewlyOcclusionTestedBBoxesScreenSpace;
		FVector4 ExpansionScreenSpaceNewlyConsidered = FVector4(ExtentNewly / View.ViewRect.Width(), ExtentNewly / View.ViewRect.Height(), 0.0f, 0.0f);
		ExpansionWorldSpaceNewlyConsidered = InvViewProjectionMatrix.TransformFVector4(ExpansionScreenSpaceNewlyConsidered);

		float ExtentNotNewly = GExpandAllTestedBBoxesScreenSpace;
		FVector4 ExpansionScreenSpaceNotNewlyConsidered = FVector4(ExtentNotNewly / View.ViewRect.Width(), ExtentNotNewly / View.ViewRect.Height(), 0.0f, 0.0f);
		ExpansionWorldSpaceNotNewlyConsidered = InvViewProjectionMatrix.TransformFVector4(ExpansionScreenSpaceNotNewlyConsidered);

		const FMatrix& ViewProjectionMatrix = View.ViewMatrices.GetWorldToClip();
		ViewProjectionHomogeneousVector = FVector4(ViewProjectionMatrix.M[0][3], ViewProjectionMatrix.M[1][3], ViewProjectionMatrix.M[2][3], ViewProjectionMatrix.M[3][3]);
	}

	for (int32 SubQuery = 0; SubQuery < NumSubQueries; SubQuery++)
	{
		FPrimitiveOcclusionHistory* PrimitiveOcclusionHistory = PrimitiveOcclusionHistorySet.Find(FPrimitiveOcclusionHistoryKey(PrimitiveId, SubQuery));

		bool bIsOccluded = false;
		bool bOcclusionStateIsDefinite = false;

		if (!PrimitiveOcclusionHistory)
		{
			PrimitiveOcclusionHistory = Visitor.AddOcclusionHistory(FPrimitiveOcclusionHistory(PrimitiveId, SubQuery));
		}
		else
		{
			if (View.bIgnoreExistingQueries)
			{
				// If the view is ignoring occlusion queries, the primitive is definitely unoccluded.
				bOcclusionStateIsDefinite = View.bDisableQuerySubmissions;
			}
			else
			{
				if (bUseOcclusionFeedback)
				{
					bIsOccluded = OcclusionFeedback.IsOccluded(FPrimitiveOcclusionHistoryKey(PrimitiveId, SubQuery));
					bOcclusionStateIsDefinite = true;
				}
				else if (OcclusionState.bHZBOcclusion)
				{
					if (HZBOcclusionTests.IsValidFrame(PrimitiveOcclusionHistory->LastTestFrameNumber))
					{
						bIsOccluded = !HZBOcclusionTests.IsVisible(PrimitiveOcclusionHistory->HZBTestIndex);
						bOcclusionStateIsDefinite = true;
					}
				}
				else
				{
					// Read the occlusion query results.
					uint64 NumSamples = 0;
					bool bGrouped = false;
					FRHIRenderQuery* PastQuery = PrimitiveOcclusionHistory->GetQueryForReading(OcclusionFrameCounter, OcclusionState.NumBufferedFrames, OcclusionState.ReadBackLagTolerance, bGrouped);
					if (PastQuery)
					{
						if (RHIGetRenderQueryResult(PastQuery, NumSamples, true))
						{
							// we render occlusion without MSAA
							uint32 NumPixels = (uint32)NumSamples;

							// The primitive is occluded if none of its bounding box's pixels were visible in the previous frame's occlusion query.
							bIsOccluded = (NumPixels == 0);

							if (!bIsOccluded)
							{
								checkSlow(View.OneOverNumPossiblePixels > 0.0f);
								PrimitiveOcclusionHistory->LastPixelsPercentage = NumPixels * View.OneOverNumPossiblePixels;
							}
							else
							{
								PrimitiveOcclusionHistory->LastPixelsPercentage = 0.0f;
							}

							// Flag the primitive's occlusion state as definite if it wasn't grouped.
							bOcclusionStateIsDefinite = !bGrouped;
						}
						else
						{
							// If the occlusion query failed, treat the primitive as visible.  
							// already set bIsOccluded = false;
						}

						Result.NumTestedQueries++;
					}
					else
					{
						if (OcclusionState.NumBufferedFrames > 1 || GRHIMaximumInFlightQueries < MAX_int32)
						{
							// If there's no occlusion query for the primitive, assume it is whatever it was last frame
							bIsOccluded = PrimitiveOcclusionHistory->WasOccludedLastFrame;
							bOcclusionStateIsDefinite = PrimitiveOcclusionHistory->OcclusionStateWasDefiniteLastFrame;
						}
						else
						{
							// If there's no occlusion query for the primitive, set it's visibility state to whether it has been unoccluded recently.
							bIsOccluded = (PrimitiveOcclusionHistory->LastProvenVisibleTime + GEngine->PrimitiveProbablyVisibleTime < CurrentRealTime);
							// the state was definite last frame, otherwise we would have ran a query
							bOcclusionStateIsDefinite = true;
						}
						if (bIsOccluded)
						{
							PrimitiveOcclusionHistory->LastPixelsPercentage = 0.0f;
						}
						else
						{
							PrimitiveOcclusionHistory->LastPixelsPercentage = GEngine->MaxOcclusionPixelsFraction;
						}
					}
				}

				if (GVisualizeOccludedPrimitives == 1 && bIsOccluded)
				{
					const FBoxSphereBounds& Bounds = bSubQueries ? (*SubBounds)[SubQuery] : Scene.PrimitiveOcclusionBounds[Index];
					Visitor.AddVisualizeQuery(Bounds.GetBox(), FColor(50, 255, 50));
				}
			}
		}

		if (OcclusionState.bSubmitQueries)
		{
			bool bSkipNewlyConsidered = false;

			if (bNewlyConsideredBBoxExpandActive)
			{
				if (!PrimitiveOcclusionHistory->BecameEligibleForQueryCooldown && OcclusionFrameCounter - PrimitiveOcclusionHistory->LastConsideredFrameNumber > uint32(GFramesNotOcclusionTestedToExpandBBoxes))
				{
					PrimitiveOcclusionHistory->BecameEligibleForQueryCooldown = GFramesToExpandNewlyOcclusionTestedBBoxes;
				}

				bSkipNewlyConsidered = !!PrimitiveOcclusionHistory->BecameEligibleForQueryCooldown;

				if (bSkipNewlyConsidered)
				{
					PrimitiveOcclusionHistory->BecameEligibleForQueryCooldown--;
				}
			}

			const FBoxSphereBounds PrimitiveOcclusionBounds = (bSubQueries ? (*SubBounds)[SubQuery] : Scene.PrimitiveOcclusionBounds[Index]);

			// Print unexpanded bounding boxes in green if not already visualized by GVisualizeOccludedPrimitives
			if (GVisualizePrimitiveBBoxes == 2 && !(GVisualizeOccludedPrimitives == 1 && bIsOccluded))
			{
				Visitor.AddVisualizeQuery(PrimitiveOcclusionBounds.GetBox(), FColor(0, 155, 0));
			}

			// scale occlusion bounds by GExpand*TestedBBoxesAmount in world space
			FBoxSphereBounds WorldSpaceExpandedOcclusionBounds = PrimitiveOcclusionBounds;
			if (GExpandAllTestedBBoxesAmount > 0.0f || GExpandNewlyOcclusionTestedBBoxesAmount > 0.0f)
			{
				WorldSpaceExpandedOcclusionBounds = WorldSpaceExpandedOcclusionBounds.ExpandBy(GExpandAllTestedBBoxesAmount + (bSkipNewlyConsidered ? GExpandNewlyOcclusionTestedBBoxesAmount : 0.0));
			}
			
			// scale occlusion bounds by GExpand*TestedBBoxesScreenSpace in screen space
			FBoxSphereBounds FinalOcclusionBounds = WorldSpaceExpandedOcclusionBounds;
			if (GExpandAllTestedBBoxesScreenSpace > 0.0f || GExpandNewlyOcclusionTestedBBoxesScreenSpace > 0.0f)
			{
				FVector ExpansionWorldSpace = (bSkipNewlyConsidered) ? ExpansionWorldSpaceNewlyConsidered : ExpansionWorldSpaceNotNewlyConsidered;

				const float BoundsScreenSpace_W = FVector::DotProduct(FinalOcclusionBounds.Origin, ViewProjectionHomogeneousVector) + ViewProjectionHomogeneousVector.W;

				ExpansionWorldSpace *= BoundsScreenSpace_W;
				ExpansionWorldSpace = FVector(abs(ExpansionWorldSpace.X), abs(ExpansionWorldSpace.Y), abs(ExpansionWorldSpace.Z));

				FinalOcclusionBounds.BoxExtent = FinalOcclusionBounds.BoxExtent + ExpansionWorldSpace;
				FinalOcclusionBounds.SphereRadius = FinalOcclusionBounds.BoxExtent.Length();
			}

			const FBoxSphereBounds OcclusionBounds = FinalOcclusionBounds;

			if (GVisualizeOccludedPrimitives == 2 && bIsOccluded)
			{
				Visitor.AddVisualizeQuery(OcclusionBounds.GetBox(), FColor(50, 255, 50));
			}
			else if (GVisualizePrimitiveBBoxes > 0)
			{
				Visitor.AddVisualizeQuery(OcclusionBounds.GetBox(), FColor(155, 0, 0));
			}

			bool bAllowBoundsTest;
			if (FVector::DistSquared(ViewOrigin, OcclusionBounds.Origin) < NeverOcclusionTestDistanceSquared)
			{
				bAllowBoundsTest = false;
			}
			else if (View.bHasNearClippingPlane)
			{
				bAllowBoundsTest = View.NearClippingPlane.PlaneDot(OcclusionBounds.Origin) <
					-(FVector::BoxPushOut(View.NearClippingPlane, OcclusionBounds.BoxExtent));

			}
			else if (!View.IsPerspectiveProjection())
			{
				// Transform parallel near plane
				bAllowBoundsTest = View.WorldToScreen(OcclusionBounds.Origin).Z - View.ViewMatrices.GetViewToClip().M[2][2] * OcclusionBounds.SphereRadius < 1;
			}
			else
			{
				bAllowBoundsTest = OcclusionBounds.SphereRadius < HALF_WORLD_MAX;
			}

			if (bAllowBoundsTest)
			{
				PrimitiveOcclusionHistory->LastTestFrameNumber = OcclusionFrameCounter;

				if (bUseOcclusionFeedback)
				{
					const FVector BoundOrigin = OcclusionBounds.Origin + View.ViewMatrices.GetPreViewTranslation();
					const FVector BoundExtent = OcclusionBounds.BoxExtent;

					Visitor.AddOcclusionFeedback(FOcclusionFeedbackEntry(FPrimitiveOcclusionHistoryKey(PrimitiveId, SubQuery), BoundOrigin, BoundExtent));
				}
				else if (OcclusionState.bHZBOcclusion)
				{
					Visitor.AddHZBBounds(FHZBBound(PrimitiveOcclusionHistory, OcclusionBounds.Origin, OcclusionBounds.BoxExtent));
				}
				else
				{
					// decide if a query should be run this frame
					bool bRunQuery, bGroupedQuery;

					if (!bSubQueries && // sub queries are never grouped, we assume the custom code knows what it is doing and will group internally if it wants
						(OcclusionFlags & EOcclusionFlags::AllowApproximateOcclusion))
					{
						if (bIsOccluded)
						{
							// Primitives that were occluded the previous frame use grouped queries.
							bGroupedQuery = true;
							bRunQuery = true;
						}
						else if (bOcclusionStateIsDefinite)
						{
							bGroupedQuery = false;
							float Rnd = GOcclusionRandomStream.GetFraction();
							if (GRHISupportsExactOcclusionQueries)
							{
								float FractionMultiplier = FMath::Max(PrimitiveOcclusionHistory->LastPixelsPercentage / GEngine->MaxOcclusionPixelsFraction, 1.0f);
								bRunQuery = (FractionMultiplier * Rnd) < GEngine->MaxOcclusionPixelsFraction;
							}
							else
							{
								bRunQuery = CurrentRealTime - PrimitiveOcclusionHistory->LastProvenVisibleTime > PrimitiveProbablyVisibleTime * (0.5f * 0.25f * Rnd);
							}
						}
						else
						{
							bGroupedQuery = false;
							bRunQuery = true;
						}
					}
					else
					{
						// Primitives that need precise occlusion results use individual queries.
						bGroupedQuery = false;
						bRunQuery = true;
					}

					if (bRunQuery)
					{
						const FVector BoundOrigin = OcclusionBounds.Origin + View.ViewMatrices.GetPreViewTranslation();
						const FVector BoundExtent = OcclusionBounds.BoxExtent;

						if (GRHIMaximumInFlightQueries < MAX_int32 && !bGroupedQuery)
						{
							Visitor.AddThrottledOcclusionQuery(FThrottledOcclusionQuery(FPrimitiveOcclusionHistoryKey(PrimitiveId, SubQuery), BoundOrigin, BoundExtent, PrimitiveOcclusionHistory->LastQuerySubmitFrame()));
						}
						else
						{
							Visitor.AddOcclusionQuery(FOcclusionQuery(PrimitiveOcclusionHistory, BoundOrigin, BoundExtent, bGroupedQuery));
						}
					}
				}
			}
			else
			{
				// If the primitive's bounding box intersects the near clipping plane, treat it as definitely unoccluded.
				bIsOccluded = false;
				bOcclusionStateIsDefinite = true;
			}
		}

		// Set the primitive's considered time to keep its occlusion history from being trimmed.
		PrimitiveOcclusionHistory->LastConsideredTime = CurrentRealTime;
		if (!bIsOccluded && bOcclusionStateIsDefinite)
		{
			PrimitiveOcclusionHistory->LastProvenVisibleTime = CurrentRealTime;
		}

		PrimitiveOcclusionHistory->LastConsideredFrameNumber = OcclusionFrameCounter;
		PrimitiveOcclusionHistory->WasOccludedLastFrame = bIsOccluded;
		PrimitiveOcclusionHistory->OcclusionStateWasDefiniteLastFrame = bOcclusionStateIsDefinite;

		if (bSubQueries)
		{
			SubIsOccluded->Add(bIsOccluded);
			if (!bIsOccluded)
			{
				bAllSubOccluded = false;
			}
			if (bIsOccluded || !bOcclusionStateIsDefinite)
			{
				bAllSubOcclusionStateIsDefinite = false;
			}
		}
		else
		{
			if (bIsOccluded)
			{
				SetAtomicBit(View.PrimitiveVisibilityMap, Index, false);
				bIsVisible = false;
				Result.NumCulledPrimitives++;
			}
			else if (bOcclusionStateIsDefinite)
			{
				SetAtomicBit(View.PrimitiveDefinitelyUnoccludedMap, Index, true);
			}
		}
	}

	if (bSubQueries)
	{
		FPrimitiveSceneProxy* Proxy = Scene.PrimitiveSceneProxies[Index];
		Proxy->AcceptOcclusionResults(&View, SubIsOccluded, SubIsOccludedStart, SubIsOccluded->Num() - SubIsOccludedStart);

		if (bAllSubOccluded)
		{
			SetAtomicBit(View.PrimitiveVisibilityMap, Index, false);
			bIsVisible = false;
			Result.NumCulledPrimitives++;
		}
		else if (bAllSubOcclusionStateIsDefinite)
		{
			SetAtomicBit(View.PrimitiveDefinitelyUnoccludedMap, Index, true);
		}
	}

	return bIsVisible;
}

void FGPUOcclusionPacket::FProcessVisitor::AddOcclusionQuery(const FOcclusionQuery& Query)
{
	FRHIRenderQuery* RenderQuery = nullptr;

	if (Query.bGroupedQuery)
	{
		RenderQuery = Packet.View.GroupedOcclusionQueries.BatchPrimitive(Query.Bounds.Origin, Query.Bounds.Extent, DynamicVertexBuffer);
	}
	else
	{
		RenderQuery = Packet.View.IndividualOcclusionQueries.BatchPrimitive(Query.Bounds.Origin, Query.Bounds.Extent, DynamicVertexBuffer);
	}

	const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryIssueIndex(Packet.ViewState.OcclusionFrameCounter, Packet.OcclusionState.NumBufferedFrames);
	Packet.ViewState.Occlusion.LastOcclusionQueryArray[QueryIndex] = RenderQuery;
	Packet.ViewState.Occlusion.NumRequestedQueries++;

	Query.PrimitiveOcclusionHistory->SetCurrentQuery(
		Packet.ViewState.OcclusionFrameCounter,
		RenderQuery,
		Packet.OcclusionState.NumBufferedFrames,
		Query.bGroupedQuery,
		Packet.OcclusionState.bUseRoundRobinOcclusion
	);
}

void FGPUOcclusionPacket::FProcessVisitor::SubmitThrottledOcclusionQueries()
{
	if (ThrottledOcclusionQueries.IsEmpty())
	{
		return;
	}

	TArray<FThrottledOcclusionQuery*, SceneRenderingAllocator> SortedQueries;
	SortedQueries.Reserve(ThrottledOcclusionQueries.Num());

	for (FThrottledOcclusionQuery& ThrottledOcclusionQuery : ThrottledOcclusionQueries)
	{
		SortedQueries.Emplace(&ThrottledOcclusionQuery);
	}

	const int32 NumRequestedThrottledQueries = SortedQueries.Num();
	const int32 NumUsedQueries = Packet.View.GroupedOcclusionQueries.GetNumBatchOcclusionQueries();
	const int32 ThrottleThreshold = GRHIMaximumInFlightQueries / FMath::Min(Packet.OcclusionState.NumBufferedFrames, 2); // extra RHIT frame does not count

	int32 NumThrottledQueries = NumRequestedThrottledQueries;

	if (NumUsedQueries + NumRequestedThrottledQueries > ThrottleThreshold)
	{
		// We need to make progress, even if it means stalling and waiting for the GPU. At a minimum, we will do 10%.
		NumThrottledQueries = (NumRequestedThrottledQueries + 9) / 10;

		if (NumUsedQueries + NumThrottledQueries < ThrottleThreshold)
		{
			// We can do more than the minimum.
			NumThrottledQueries = ThrottleThreshold - NumUsedQueries;
		}
	}

	if (NumThrottledQueries < NumRequestedThrottledQueries)
	{
		SortedQueries.Sort([](const FThrottledOcclusionQuery& A, const FThrottledOcclusionQuery& B)
		{
			return A.LastQuerySubmitFrame < B.LastQuerySubmitFrame;
		});
	}

	for (int32 Index = 0; Index < NumThrottledQueries; ++Index)
	{
		FThrottledOcclusionQuery* Query = SortedQueries[Index];
		FPrimitiveOcclusionHistory* PrimitiveOcclusionHistory = PrimitiveOcclusionHistorySet.Find(Query->PrimitiveOcclusionHistoryKey);

		FRHIRenderQuery* RenderQuery = Packet.View.IndividualOcclusionQueries.BatchPrimitive(Query->Bounds.Origin, Query->Bounds.Extent, DynamicVertexBuffer);

		const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryIssueIndex(Packet.ViewState.OcclusionFrameCounter, Packet.OcclusionState.NumBufferedFrames);
		Packet.ViewState.Occlusion.LastOcclusionQueryArray[QueryIndex] = RenderQuery;
		Packet.ViewState.Occlusion.NumRequestedQueries++;

		PrimitiveOcclusionHistory->SetCurrentQuery(
			Packet.ViewState.OcclusionFrameCounter,
			RenderQuery,
			Packet.OcclusionState.NumBufferedFrames,
			false,
			Packet.OcclusionState.bUseRoundRobinOcclusion
		);
	}
}

///////////////////////////////////////////////////////////////////////////////

FGPUOcclusion::FGPUOcclusion(FVisibilityViewPacket& InViewPacket)
	: ViewPacket(InViewPacket)
	, Scene(ViewPacket.Scene)
	, View(ViewPacket.View)
	, ViewState(*ViewPacket.ViewState)
{
	// In single pass stereo we only submit queries from the primary view.
	State.bSubmitQueries = !View.bDisableQuerySubmissions && !(View.bIsSinglePassStereo && View.StereoPass == EStereoscopicPass::eSSP_SECONDARY);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	State.bSubmitQueries &= !ViewState.bIsFrozen;
#endif

	// Perform round-robin occlusion queries
	const bool bUseRoundRobinOcclusion = ViewState.IsRoundRobinEnabled() &&
		!View.bIsSinglePassStereo && // No round robin on single-pass stereo
		!View.bIsSceneCapture && // We only round-robin on the main renderer (not scene captures)
		IStereoRendering::IsStereoEyeView(View); // Only relevant to stereo views
	if (bUseRoundRobinOcclusion && !View.bIgnoreExistingQueries) // We do not alternate occlusion queries when we want to refresh the occlusion history
	{
		// For even frames, prevent left eye from occlusion querying
		// For odd frames, prevent right eye from occlusion querying
		const bool FrameParity = ((ViewState.PrevFrameNumber & 0x01) == 1);
		State.bSubmitQueries &= (FrameParity && IStereoRendering::IsAPrimaryView(View)) || (!FrameParity && IStereoRendering::IsASecondaryView(View));
	}

	// Disable HZB on OpenGL platforms to avoid rendering artifacts
	// It can be forced on by setting HZBOcclusion to 2
	State.bHZBOcclusion = !IsOpenGLPlatform(View.GetShaderPlatform());
	State.bHZBOcclusion &= FDataDrivenShaderPlatformInfo::GetSupportsHZBOcclusion(View.GetShaderPlatform());
	State.bHZBOcclusion &= GHZBOcclusion != 0;
	State.bHZBOcclusion |= GHZBOcclusion == 2;

	State.bUseRoundRobinOcclusion = bUseRoundRobinOcclusion;

	State.NumBufferedFrames = FOcclusionQueryHelpers::GetNumBufferedFrames(Scene.GetFeatureLevel());
	State.ReadBackLagTolerance = State.NumBufferedFrames;

	State.bAllowSubQueries = GAllowSubPrimitiveQueries && !View.bDisableQuerySubmissions;

	if (State.bUseRoundRobinOcclusion)
	{
		// Round-robin occlusion culling involves reading frames that could be twice as stale as without round-robin
		State.ReadBackLagTolerance = State.NumBufferedFrames * 2;
	}
}

void FGPUOcclusion::Map(FRHICommandListImmediate& RHICmdListImmediate)
{
	SCOPED_NAMED_EVENT(MapOcclusionResults, FColor::Magenta);

	if (View.FeatureLevel == ERHIFeatureLevel::ES3_1 &&
		ViewState.OcclusionFeedback.IsInitialized())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_OcclusionFeedback_ReadbackResults);
		ViewState.OcclusionFeedback.ReadbackResults(RHICmdListImmediate);
		ViewState.OcclusionFeedback.AdvanceFrame(ViewState.OcclusionFrameCounter);
	}
	else if (State.bHZBOcclusion)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_MapHZBResults);
		check(!ViewState.HZBOcclusionTests.IsValidFrame(ViewState.OcclusionFrameCounter));
		ViewState.HZBOcclusionTests.MapResults(RHICmdListImmediate);
	}

	ViewState.PrimitiveOcclusionQueryPool.AdvanceFrame(
		ViewState.OcclusionFrameCounter,
		FOcclusionQueryHelpers::GetNumBufferedFrames(Scene.GetFeatureLevel()),
		State.bUseRoundRobinOcclusion);

	ViewState.Occlusion.NumRequestedQueries = 0;
}

void FGPUOcclusion::Unmap(FRHICommandListImmediate& RHICmdListImmediate)
{
	if (State.bHZBOcclusion)
	{
		ViewState.HZBOcclusionTests.UnmapResults(RHICmdListImmediate);
	}

	if (View.FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		// Initialize/release OcclusionFeedback system on demand
		if (GOcclusionFeedback_Enable == 0 && ViewState.OcclusionFeedback.IsInitialized())
		{
			ViewState.OcclusionFeedback.ReleaseResource();
		}
		else if (GOcclusionFeedback_Enable != 0 && !ViewState.OcclusionFeedback.IsInitialized())
		{
			ViewState.OcclusionFeedback.InitResource(RHICmdListImmediate);
		}
	}

	if (State.bHZBOcclusion && State.bSubmitQueries)
	{
		ViewState.HZBOcclusionTests.SetValidFrameNumber(ViewState.OcclusionFrameCounter);
	}
}

void FGPUOcclusion::WaitForLastOcclusionQuery()
{
	const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryLookupIndex(ViewState.OcclusionFrameCounter, State.NumBufferedFrames);
	if (ViewState.Occlusion.LastOcclusionQueryArray[QueryIndex])
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GPUBound_WaitingForGPUForOcclusionQueries_SeeGPUTrack);

		uint64 Result;
		const bool bWait = true;
		RHIGetRenderQueryResult(ViewState.Occlusion.LastOcclusionQueryArray[QueryIndex], Result, bWait);
		ViewState.Occlusion.LastOcclusionQueryArray[QueryIndex] = nullptr;
	}
}

///////////////////////////////////////////////////////////////////////////////

bool FGPUOcclusionParallelPacket::AddPrimitive(int32 PrimitiveIndex)
{
	EOcclusionFlags::Type OcclusionFlags;

	if (CanBeOccluded(PrimitiveIndex, OcclusionFlags))
	{
		Input.AddElement(PrimitiveIndex);
		int32 NumSubQueries = 1;

		if (EnumHasAnyFlags(OcclusionFlags, EOcclusionFlags::HasSubprimitiveQueries) && OcclusionState.bAllowSubQueries)
		{
			NumSubQueries = Scene.PrimitiveSceneProxies[PrimitiveIndex]->GetOcclusionQueries(&View)->Num();
		}

		NumInputSubQueries += NumSubQueries;
		return true;
	}

	View.PrimitiveDefinitelyUnoccludedMap[PrimitiveIndex].AtomicSet(true);
	return false;
}

void FGPUOcclusionParallelPacket::LaunchOcclusionCullTask()
{
	check(!bTaskLaunched);

	if (Input.IsEmpty())
	{
		return;
	}

	bTaskLaunched = true;

	ViewPacket.Relevance.CommandPipe.AddNumCommands(1);

	FVisibilityTaskConfig& TaskConfig = ViewPacket.TaskConfig;

	Task = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &TaskConfig]
	{
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

		FPrimitiveIndexList PrimitiveIndexList;
		const FOcclusionCullResult Result = OcclusionCullTask(PrimitiveIndexList);

		if (PrimitiveIndexList.IsEmpty())
		{
			ViewPacket.Relevance.CommandPipe.ReleaseNumCommands(1);
		}
		else
		{
			ViewPacket.Relevance.CommandPipe.EnqueueCommand(MoveTemp(PrimitiveIndexList));
		}

		RecordOcclusionCullResult(Result);

	}, TaskConfig.OcclusionCull.TaskPriority);
}

FOcclusionCullResult FGPUOcclusionParallelPacket::OcclusionCullTask(FPrimitiveIndexList& PrimitiveIndexList)
{
	SCOPED_NAMED_EVENT(OcclusionCull, FColor::Magenta);

	FOcclusionCullResult Result;

	PrimitiveIndexList.Reserve(Input.Num());

	for (int32 Index : Input)
	{
		if (OcclusionCullPrimitive<true>(RecordVisitor, Result, Index))
		{
			PrimitiveIndexList.Emplace(Index);
		}
	}

	return Result;
}

///////////////////////////////////////////////////////////////////////////////

void FGPUOcclusionParallel::AddPrimitives(FPrimitiveRange PrimitiveRange)
{
	WaitForLastOcclusionQuery();

	for (FSceneSetBitIterator BitIt(View.PrimitiveVisibilityMap, PrimitiveRange.StartIndex); BitIt.GetIndex() < PrimitiveRange.EndIndex; ++BitIt)
	{
		FGPUOcclusionParallelPacket* Packet = Packets.Last();

		if (Packet->AddPrimitive(BitIt.GetIndex()))
		{
			if (Packet->IsFull())
			{
				Packet->LaunchOcclusionCullTask();
				CreateOcclusionPacket();
			}
		}
		else
		{
			// The primitive will not be occluded, so accumulate a packet of primitives to send directly to the relevance pipe to reduce latency.
			NonOccludedPrimitives.Emplace(BitIt.GetIndex());

			if (NonOccludedPrimitives.Num() == MaxNonOccludedPrimitives)
			{
				ViewPacket.Relevance.CommandPipe.AddNumCommands(1);
				ViewPacket.Relevance.CommandPipe.EnqueueCommand(MoveTemp(NonOccludedPrimitives));
				NonOccludedPrimitives.Reset();
				NonOccludedPrimitives.Reserve(MaxNonOccludedPrimitives);
			}
		}
	}
}

void FGPUOcclusionParallel::Finish(UE::Tasks::FTaskEvent& OcclusionCullTasks)
{
	check(!bFinished);
	bFinished = true;

	Packets.Last()->LaunchOcclusionCullTask();

	if (!NonOccludedPrimitives.IsEmpty())
	{
		ViewPacket.Relevance.CommandPipe.AddNumCommands(1);
		ViewPacket.Relevance.CommandPipe.EnqueueCommand(MoveTemp(NonOccludedPrimitives));
		NonOccludedPrimitives.Reset();
	}

	for (FGPUOcclusionParallelPacket* Packet : Packets)
	{
		if (Packet->bTaskLaunched)
		{
			OcclusionCullTasks.AddPrerequisites(Packet->Task);
		}
	}

	OcclusionCullTasks.Trigger();
}

void FGPUOcclusionParallel::Finalize()
{
	check(bFinished);
	bFinalized = true;

	SCOPED_NAMED_EVENT(FinalizeOcclusionCull, FColor::Magenta);

	for (FGPUOcclusionParallelPacket* Packet : Packets)
	{
		FGPUOcclusionPacket::FProcessVisitor ProcessVisitor(*Packet, DynamicVertexBuffer);
		ProcessVisitor.Replay(Packet->RecordVisitor);
	}

	for (FGPUOcclusionParallelPacket* Packet : Packets)
	{
		// Wait to add occlusion histories until after replaying commands since it will invalidate pointers.
		for (FPrimitiveOcclusionHistory& OcclusionHistory : Packet->RecordVisitor.OcclusionHistories)
		{
			ViewState.Occlusion.PrimitiveOcclusionHistorySet.Add(OcclusionHistory);
		}

		delete Packet;
	}

	Packets.Empty();
	DynamicVertexBuffer.Commit();
	RHICmdList->FinishRecording();
	FinalizeTask.Trigger();
}

void FGPUOcclusionParallel::Map(FRHICommandListImmediate& RHICmdListImmediate)
{
	FGPUOcclusion::Map(RHICmdListImmediate);
	RHICmdList = new FRHICommandList(RHICmdListImmediate.GetGPUMask());
	RHICmdList->SwitchPipeline(ERHIPipeline::Graphics);
	DynamicVertexBuffer.Init(*RHICmdList);
}

void FGPUOcclusionParallel::Unmap(FRHICommandListImmediate& RHICmdListImmediate)
{
	FinalizeTask.Wait();
	check(bFinalized);
	RHICmdListImmediate.QueueAsyncCommandListSubmit(RHICmdList);
	FGPUOcclusion::Unmap(RHICmdListImmediate);
}

///////////////////////////////////////////////////////////////////////////////

void FGPUOcclusionSerial::AddPrimitives(FPrimitiveRange PrimitiveRange)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(FetchVisibilityForPrimitives);

	for (FSceneSetBitIterator BitIt(View.PrimitiveVisibilityMap, PrimitiveRange.StartIndex); BitIt.GetIndex() < PrimitiveRange.EndIndex; ++BitIt)
	{
		EOcclusionFlags::Type OcclusionFlags;
		if (Packet.CanBeOccluded(BitIt.GetIndex(), OcclusionFlags))
		{
			Packet.OcclusionCullPrimitive<false>(ProcessVisitor, OcclusionCullResult, BitIt.GetIndex());
		}
		else
		{
			View.PrimitiveDefinitelyUnoccludedMap.AccessCorrespondingBit(BitIt) = true;
		}
	}
}

void FGPUOcclusionSerial::Map(FRHICommandListImmediate& RHICmdListImmediate)
{
	FGPUOcclusion::Map(RHICmdListImmediate);
	DynamicVertexBuffer.Init(RHICmdListImmediate);
}

void FGPUOcclusionSerial::Unmap(FRHICommandListImmediate& RHICmdListImmediate)
{
	DynamicVertexBuffer.Commit();
	ProcessVisitor.SubmitThrottledOcclusionQueries();
	Packet.RecordOcclusionCullResult(OcclusionCullResult);
	FGPUOcclusion::Unmap(RHICmdListImmediate);
}

///////////////////////////////////////////////////////////////////////////////

static int32 PrecomputedOcclusionCull(FVisibilityViewPacket& ViewPacket, FPrimitiveRange PrimitiveRange)
{
	int32 NumOccludedPrimitives = 0;

	const FScene& Scene = ViewPacket.Scene;
	FViewInfo& View = ViewPacket.View;

	if (View.PrecomputedVisibilityData)
	{
		SCOPED_NAMED_EVENT(PrecomputedOcclusionCull, FColor::Magenta);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_PrecomputedOcclusionCull);

		uint8 PrecomputedVisibilityFlags = EOcclusionFlags::CanBeOccluded | EOcclusionFlags::HasPrecomputedVisibility;
		for (FSceneSetBitIterator BitIt(View.PrimitiveVisibilityMap, PrimitiveRange.StartIndex); BitIt.GetIndex() < PrimitiveRange.EndIndex; ++BitIt)
		{
			if ((Scene.PrimitiveOcclusionFlags[BitIt.GetIndex()] & PrecomputedVisibilityFlags) == PrecomputedVisibilityFlags)
			{
				FPrimitiveVisibilityId VisibilityId = Scene.PrimitiveVisibilityIds[BitIt.GetIndex()];
				if ((View.PrecomputedVisibilityData[VisibilityId.ByteIndex] & VisibilityId.BitMask) == 0)
				{
					if (GVisualizeOccludedPrimitives)
					{
						const FBoxSphereBounds& Bounds = Scene.PrimitiveOcclusionBounds[BitIt.GetIndex()];
						DrawWireBox(&ViewPacket.ViewElementPDI, Bounds.GetBox(), FColor(100, 50, 50), SDPG_Foreground);
					}

					View.PrimitiveVisibilityMap.AccessCorrespondingBit(BitIt) = false;
					INC_DWORD_STAT_BY(STAT_StaticallyOccludedPrimitives, 1);
					NumOccludedPrimitives++;
				}
			}
		}
	}

	return NumOccludedPrimitives;
}

///////////////////////////////////////////////////////////////////////////////

FVisibilityTaskConfig::FVisibilityTaskConfig(const FScene& Scene, TConstArrayView<FViewInfo*> Views)
{
	Schedule = GVisibilityTaskSchedule != 0 ? EVisibilityTaskSchedule::Parallel : EVisibilityTaskSchedule::RenderThread;

	if (Schedule == EVisibilityTaskSchedule::Parallel)
	{
		if (!FApp::ShouldUseThreadingForPerformance() || !GIsThreadedRendering || !GSupportsParallelOcclusionQueries || GVisualizeOccludedPrimitives > 0 || GVisualizePrimitiveBBoxes > 0)
		{
			Schedule = EVisibilityTaskSchedule::RenderThread;
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		for (FViewInfo* View : Views)
		{
			auto* ViewState = static_cast<FSceneViewState*>(View->State);

			// Frozen matrices require modifying the view matrices which does not support the async path.
			if (ViewState && ViewState->bIsFrozen)
			{
				Schedule = EVisibilityTaskSchedule::RenderThread;
				break;
			}
		}
#endif
	}

	NumTestedPrimitives = uint32(Scene.Primitives.Num());
	NumVisiblePrimitives = 0u;

	if (Scene.PrimitivesAlwaysVisibleOffset != ~0u)
	{
		NumTestedPrimitives  = Scene.PrimitivesAlwaysVisibleOffset;
		NumVisiblePrimitives = uint32(Scene.Primitives.Num()) - NumTestedPrimitives;

		// Ensure that the dword alignment code is correct and we never have partial dword offsets
		check(uint32(NumTestedPrimitives % NumBitsPerDWORD) == 0u);
	}

	// Clear out the list of primitives if we are rendering timesliced shadows only.  This makes visibility and mesh command generation a NOP,
	// more or less.  Completely disabling the visibility task might be another option, but it requires still running light visibility (handled
	// by the visibility task), and several other downstream changes.  This approach of just not processing any primitives for visibility is
	// presumed to be simpler to maintain, and we could always do something more aggressive later.
	if (IsMobilePlatform(Scene.GetShaderPlatform()) && Views[0]->IsRuntimeReflectionCapture() && Scene.ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_RT == 0)
	{
		NumTestedPrimitives = 0;
	}

	const uint32 NumWorkerThreads = FMath::Min(LowLevelTasks::FScheduler::Get().GetNumWorkers(), 16u);

	// These values tune the task granularity based on number of primitives in the scene and the number of worker tasks available.
	const uint32 NumAlwaysVisibleTasksPerThread = 2;
	const uint32 NumFrustumCullTasksPerThread   = 2;
	const uint32 NumOcclusionCullTasksPerThread = 2;
	const uint32 NumRelevanceTasksPerThread     = 32;

	const uint32 NumWordsPerTaskIfRenderThread = 128;

	// Always Visible
	if (NumVisiblePrimitives > 0u)
	{
		const uint32 NumPrimitiveWords = FMath::DivideAndRoundUp<uint32>(NumVisiblePrimitives, NumBitsPerDWORD);

		if (Schedule == EVisibilityTaskSchedule::RenderThread)
		{
			AlwaysVisible.NumWordsPerTask = NumWordsPerTaskIfRenderThread;
		}
		else
		{
			AlwaysVisible.NumWordsPerTask = FMath::DivideAndRoundUp(NumPrimitiveWords, NumWorkerThreads * NumAlwaysVisibleTasksPerThread);
		}

		AlwaysVisible.NumWordsPerTask = FMath::Clamp(AlwaysVisible.NumWordsPerTask, AlwaysVisible.MinWordsPerTask, NumPrimitiveWords);
		AlwaysVisible.NumPrimitivesPerTask = AlwaysVisible.NumWordsPerTask * NumBitsPerDWORD;
		AlwaysVisible.NumTasks = FMath::DivideAndRoundUp(NumVisiblePrimitives, AlwaysVisible.NumPrimitivesPerTask);
	}
	else
	{
		AlwaysVisible.NumWordsPerTask = 0;
		AlwaysVisible.NumPrimitivesPerTask = 0;
		AlwaysVisible.NumTasks = 0;
	}

	// Frustum Cull
	{
		const uint32 NumPrimitiveWords = FMath::DivideAndRoundUp<uint32>(NumTestedPrimitives, NumBitsPerDWORD);

		if (GFrustumCullNumPrimitivesPerTask > 0)
		{
			FrustumCull.NumWordsPerTask = FMath::DivideAndRoundUp<uint32>(GFrustumCullNumPrimitivesPerTask, NumBitsPerDWORD);
		}
		else if (Schedule == EVisibilityTaskSchedule::RenderThread)
		{
			FrustumCull.NumWordsPerTask = NumWordsPerTaskIfRenderThread;
		}
		else
		{
			FrustumCull.NumWordsPerTask = FMath::DivideAndRoundUp(NumPrimitiveWords, NumWorkerThreads * NumFrustumCullTasksPerThread);
		}

		FrustumCull.NumWordsPerTask = FMath::Clamp(FrustumCull.NumWordsPerTask, FrustumCull.MinWordsPerTask, NumPrimitiveWords);
		FrustumCull.NumPrimitivesPerTask = FrustumCull.NumWordsPerTask * NumBitsPerDWORD;
		FrustumCull.NumTasks = FMath::DivideAndRoundUp(NumTestedPrimitives, FrustumCull.NumPrimitivesPerTask);
	}

	// Occlusion Cull
	{
		OcclusionCull.Views.SetNum(Views.Num());

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			uint32& MaxQueriesPerTask = OcclusionCull.Views[ViewIndex].MaxQueriesPerTask;

			if (GOcclusionCullMaxQueriesPerTask > 0)
			{
				MaxQueriesPerTask = GOcclusionCullMaxQueriesPerTask;
			}
			else if (FSceneViewState* ViewState = static_cast<FSceneViewState*>(Views[ViewIndex]->State))
			{
				MaxQueriesPerTask = FMath::DivideAndRoundUp(ViewState->Occlusion.NumRequestedQueries, FMath::Max(1u, NumWorkerThreads) * NumOcclusionCullTasksPerThread);
			}

			MaxQueriesPerTask = FMath::Max(MaxQueriesPerTask, OcclusionCull.MinQueriesPerTask);
		}
	}

	// Relevance
	{
		const uint32 NumPrimitivesPerPacketIfRenderThread = 128;

		if (GRelevanceNumPrimitivesPerPacket > 0)
		{
			Relevance.NumPrimitivesPerPacket = GRelevanceNumPrimitivesPerPacket;
		}
		else if (Schedule == EVisibilityTaskSchedule::RenderThread)
		{
			Relevance.NumPrimitivesPerPacket = NumPrimitivesPerPacketIfRenderThread;
		}
		else
		{
			Relevance.NumPrimitivesPerPacket = FMath::DivideAndRoundUp(NumTestedPrimitives, NumWorkerThreads * NumRelevanceTasksPerThread);
		}

		Relevance.NumPrimitivesPerPacket = FMath::Clamp(Relevance.NumPrimitivesPerPacket, Relevance.MinPrimitivesPerTask, Relevance.MaxPrimitivesPerTask);
		Relevance.NumEstimatedPackets = FMath::DivideAndRoundUp(NumTestedPrimitives, Relevance.NumPrimitivesPerPacket);
	}
}

///////////////////////////////////////////////////////////////////////////////

FVisibilityViewPacket::FVisibilityViewPacket(FVisibilityTaskData& InTaskData, FScene& InScene, FViewInfo& InView, int32 InViewIndex)
	: TaskData(InTaskData)
	, TaskConfig(TaskData.TaskConfig)
	, Scene(InScene)
	, View(InView)
	, ViewState(static_cast<FSceneViewState*>(View.ViewState))
	, ViewIndex(InViewIndex)
	, ViewElementPDI(&View, nullptr, nullptr)
{
	if (ViewState && !View.Family->EngineShowFlags.Wireframe && DoOcclusionQueries(*View.Family))
	{
		if (TaskConfig.Schedule == EVisibilityTaskSchedule::Parallel)
		{
			OcclusionCull.ContextIfParallel = TaskData.Allocator.Create<FGPUOcclusionParallel>(*this);
		}
		else
		{
			OcclusionCull.ContextIfSerial = TaskData.Allocator.Create<FGPUOcclusionSerial>(*this);
		}
	}

	if (TaskConfig.Schedule == EVisibilityTaskSchedule::Parallel)
	{
		// Chain the frustum cull task to the relevance task since we only wait on relevance.
		Tasks.ComputeRelevance.AddPrerequisites(Tasks.FrustumCull);

		// Callback for when an occlusion command is queued from frustum culling.
		OcclusionCull.CommandPipe.SetCommandFunction([this](FPrimitiveRange PrimitiveRange)
		{
			UpdatePrimitiveFading(Scene, View, ViewState, PrimitiveRange);

			const int32 NumCulledPrimitives = PrecomputedOcclusionCull(*this, PrimitiveRange);

			if (OcclusionCull.ContextIfParallel)
			{
				OcclusionCull.ContextIfParallel->AddPrimitives(PrimitiveRange);
			}
			else
			{
				// When occlusion is disabled primitives are queued directly to the relevance context rather than as a command on the relevance pipe.
				for (FSceneSetBitIterator BitIt(View.PrimitiveVisibilityMap, PrimitiveRange.StartIndex); BitIt.GetIndex() < PrimitiveRange.EndIndex; ++BitIt)
				{
					Relevance.Context->AddPrimitive(BitIt.GetIndex());
				}
			}

			TaskConfig.OcclusionCull.NumCulledPrimitives.fetch_add(NumCulledPrimitives, std::memory_order_relaxed);
		});

		// Callback for when the occlusion pipe is done accepting commands.
		OcclusionCull.CommandPipe.SetEmptyFunction([this]
		{
			if (OcclusionCull.ContextIfParallel)
			{
				OcclusionCull.ContextIfParallel->Finish(Tasks.OcclusionCull);

				// Launch a follow-up task to finalize occlusion results after all occlusion packets have completed.
				UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]
				{
					FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
					Relevance.CommandPipe.ReleaseNumCommands(1);
					OcclusionCull.ContextIfParallel->Finalize();

				}, Tasks.OcclusionCull, TaskConfig.OcclusionCull.FinalizeTaskPriority);
			}
			else
			{
				Relevance.CommandPipe.ReleaseNumCommands(1);
				Tasks.OcclusionCull.Trigger();
			}
		});

		// Take a reference on the occlusion command pipe that is released when all frustum cull commands are enqueued.
		OcclusionCull.CommandPipe.AddNumCommands(1);

		// Callback for when a relevance command is queued from occlusion.
		Relevance.CommandPipe.SetCommandFunction([this](FPrimitiveIndexList&& PrimitiveIndexList)
		{
			Relevance.Context->AddPrimitives(MoveTemp(PrimitiveIndexList));
		});

		// Callback for when the relevance pipe is done accepting commands.
		Relevance.CommandPipe.SetEmptyFunction([this]
		{
			Relevance.Context->Finish(Tasks.ComputeRelevance);

			// Only used in the ViewPackets.Num() == 1 case
			if (TaskData.DynamicMeshElements.CommandPipe)
			{
				TaskData.DynamicMeshElements.CommandPipe->ReleaseNumCommands(1);
			}
		});

		// Take a reference on the relevance command pipe that is released by the occlusion pipe when all occlusion commands are complete.
		Relevance.CommandPipe.AddNumCommands(1);
	}
}

void FVisibilityViewPacket::BeginInitVisibility()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("BeginInitVisibility %d"), ViewIndex));

	// Mark all primitives as visible when not visibility culling
	bool bShouldVisibilityCull = GFrustumCullEnabled;

	UE::Tasks::FTaskEvent RelevancePrereqs{ UE_SOURCE_LOCATION };
	RelevancePrereqs.AddPrerequisites(Scene.GetCacheMeshDrawCommandsTask());

	// Offload initialization of maps that are not touched by frustum culling / occlusion until the relevance phase. These can be quite expensive to zero out.
	RelevancePrereqs.AddPrerequisites(UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SceneVisibility_InitMaps);
		View.DynamicMeshElementRanges.SetNumZeroed(Scene.Primitives.Num());
		View.PrimitivesLODMask.Init(FLODMask(), Scene.Primitives.Num());
		View.StaticMeshVisibilityMap.Init(false, Scene.StaticMeshes.GetMaxIndex());
		View.StaticMeshFadeOutDitheredLODMap.Init(false, Scene.StaticMeshes.GetMaxIndex());
		View.StaticMeshFadeInDitheredLODMap.Init(false, Scene.StaticMeshes.GetMaxIndex());

	}, TaskConfig.TaskPriority));

	// Allocate the view's visibility maps.
	View.PrimitiveVisibilityMap.Init(!bShouldVisibilityCull, Scene.Primitives.Num());
	View.PrimitiveRayTracingVisibilityMap.Init(false, Scene.Primitives.Num());
	View.PrimitiveDefinitelyUnoccludedMap.Init(!DoOcclusionQueries(*View.Family), Scene.Primitives.Num());
	View.PotentiallyFadingPrimitiveMap.Init(false, Scene.Primitives.Num());
	View.PrimitiveFadeUniformBuffers.AddZeroed(Scene.Primitives.Num());
	View.PrimitiveFadeUniformBufferMap.Init(false, Scene.Primitives.Num());
	View.PrimitiveViewRelevanceMap.Reset(Scene.Primitives.Num());
	View.PrimitiveViewRelevanceMap.AddZeroed(Scene.Primitives.Num());

	RelevancePrereqs.Trigger();

	if (View.ShowOnlyPrimitives.IsSet())
	{
		View.bHasNoVisiblePrimitive = View.ShowOnlyPrimitives->Num() == 0;
	}

	Relevance.Context = TaskData.Allocator.Create<FComputeAndMarkRelevance>(TaskData, Scene, View, ViewIndex, RelevancePrereqs);

	ClearStalePrimitiveFadingStates(View, ViewState);

	// Development builds sometimes override frustum culling, e.g. dependent views in the editor.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (ViewState)
	{
		// For views with frozen visibility, check if the primitive is in the frozen visibility set.
		if (ViewState->bIsFrozen)
		{
			bShouldVisibilityCull = false;
			for (int32 Index = 0; Index < int32(TaskConfig.NumTestedPrimitives); ++Index)
			{
				if (ViewState->FrozenPrimitives.Contains(Scene.PrimitiveComponentIds[Index]))
				{
					View.PrimitiveVisibilityMap[Index] = true;
				}
			}
		}
	}
#endif

	const float MaxDrawDistanceScale = GetCachedScalabilityCVars().ViewDistanceScale * GetCachedScalabilityCVars().CalculateFieldOfViewDistanceScale(View.DesiredFOV);
	const FHLODVisibilityState* HLODState = nullptr;

	// Most views use standard frustum culling.
	if (bShouldVisibilityCull)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HLODInit);

		// Update HLOD transition/visibility states to allow use during distance culling
		FLODSceneTree& HLODTree = Scene.SceneLODHierarchy;

		if (HLODTree.IsActive())
		{
			HLODTree.UpdateVisibilityStates(View, Tasks.LightVisibility);

			if (ViewState)
			{
				HLODState = &ViewState->HLODVisibilityState;
			}
		}
		else
		{
			HLODTree.ClearVisibilityState(View);
		}
	}

	Tasks.LightVisibility.Trigger();

	FFrustumCullingFlags Flags;
	Flags.bShouldVisibilityCull  = bShouldVisibilityCull;
	Flags.bUseCustomCulling      = View.CustomVisibilityQuery && View.CustomVisibilityQuery->Prepare();
	Flags.bUseSphereTestFirst    = GFrustumCullUseSphereTestFirst;
	Flags.bUseFastIntersect      = (View.GetCullingFrustum().PermutedPlanes.Num() == 8) && GFrustumCullUseFastIntersect;
	Flags.bUseVisibilityOctree   = GFrustumCullUseOctree;
	Flags.bHasHiddenPrimitives   = View.HiddenPrimitives.Num() > 0;
	Flags.bHasShowOnlyPrimitives = View.ShowOnlyPrimitives.IsSet();

	UE::Tasks::FTask PrerequisiteTask;

#if RHI_RAYTRACING
	View.RayTracingCullingParameters.Init(View);

	// Sync cached raytracing tasks ShouldCullForRayTracing.
	PrerequisiteTask = Scene.GetCacheRayTracingPrimitivesTask();
#endif

	FSceneBitArray* VisibleNodes = nullptr;

	if (bShouldVisibilityCull && Flags.bUseVisibilityOctree)
	{
		VisibleNodes = TaskData.Allocator.Create<FSceneBitArray>();
		const FConvexVolume& ViewCullingFrustum = View.GetCullingFrustum();
		CullOctree(Scene, View, Flags, *VisibleNodes, ViewCullingFrustum);
	}

	const bool bCullingIsThreadsafe = (!Flags.bUseCustomCulling || View.CustomVisibilityQuery->IsThreadsafe());

	if (TaskConfig.Schedule == EVisibilityTaskSchedule::Parallel)
	{
		// Always Visible
		const bool bHasAlwaysVisible = TaskConfig.NumVisiblePrimitives > 0;
		if (bHasAlwaysVisible)
		{
			const float CurrentWorldTime = View.Family->Time.GetWorldTimeSeconds();
			for (uint32 TaskIndex = 0; TaskIndex < TaskConfig.AlwaysVisible.NumTasks; ++TaskIndex)
			{
				Tasks.AlwaysVisible.AddPrerequisites(
					UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Flags, TaskIndex, CurrentWorldTime]() mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(SceneVisibility_AlwaysVisible);
					SCOPE_CYCLE_COUNTER(STAT_UpdateAlwaysVisible);

					FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);
					UpdateAlwaysVisible(Scene, View, Flags, TaskConfig, TaskIndex, CurrentWorldTime);

				}, PrerequisiteTask, TaskConfig.TaskPriority, UE::Tasks::EExtendedTaskPriority::None));
			}
		}

		// Frustum Cull
		{
			// Frustum culling tasks have to run serially if custom culling is not thread-safe.
			const UE::Tasks::EExtendedTaskPriority ExtendedTaskPriority = GetExtendedTaskPriority(bCullingIsThreadsafe);

			// Assign the number of expected commands first so the pipe can determine when the last task has completed.
			OcclusionCull.CommandPipe.AddNumCommands(TaskConfig.FrustumCull.NumTasks);

			for (uint32 TaskIndex = 0; TaskIndex < TaskConfig.FrustumCull.NumTasks; ++TaskIndex)
			{
				Tasks.FrustumCull.AddPrerequisites(
					UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Flags, MaxDrawDistanceScale, HLODState, VisibleNodes, TaskIndex]() mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(SceneVisibility_FrustumCull);
					FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);
					int32 NumCulledPrimitives = FrustumCull(Scene, View, Flags, MaxDrawDistanceScale, HLODState, VisibleNodes, TaskConfig, TaskIndex);

					FPrimitiveRange PrimitiveRange;
					PrimitiveRange.StartIndex = TaskConfig.FrustumCull.NumPrimitivesPerTask * (TaskIndex);
					PrimitiveRange.EndIndex   = TaskConfig.FrustumCull.NumPrimitivesPerTask + PrimitiveRange.StartIndex;
					PrimitiveRange.EndIndex   = FMath::Min(PrimitiveRange.EndIndex, int32(TaskConfig.NumTestedPrimitives));

					// Skip rendering of dynamic objects without static lighting for static reflection captures.
					if (View.bStaticSceneOnly)
					{
						for (FSceneSetBitIterator BitIt(View.PrimitiveVisibilityMap, PrimitiveRange.StartIndex); BitIt.GetIndex() < PrimitiveRange.EndIndex; ++BitIt)
						{
							if (!Scene.PrimitiveSceneProxies[BitIt.GetIndex()]->HasStaticLighting())
							{
								View.PrimitiveVisibilityMap.AccessCorrespondingBit(BitIt) = false;
								NumCulledPrimitives++;
							}
						}
					}

					// Skip rendering of small objects when in wireframe mode for performance since wireframe doesn't enable occlusion culling.
					if (View.Family->EngineShowFlags.Wireframe)
					{
						const float ScreenSizeScale = FMath::Max(View.ViewMatrices.GetViewToClip().M[0][0] * View.ViewRect.Width(), View.ViewMatrices.GetViewToClip().M[1][1] * View.ViewRect.Height());

						for (FSceneSetBitIterator BitIt(View.PrimitiveVisibilityMap, PrimitiveRange.StartIndex); BitIt.GetIndex() < PrimitiveRange.EndIndex; ++BitIt)
						{
							if (ScreenSizeScale * Scene.PrimitiveBounds[BitIt.GetIndex()].BoxSphereBounds.SphereRadius <= GWireframeCullThreshold)
							{
								View.PrimitiveVisibilityMap.AccessCorrespondingBit(BitIt) = false;
								NumCulledPrimitives++;
							}
						}
					}

					const uint32 NumVisiblePrimitives = PrimitiveRange.EndIndex - PrimitiveRange.StartIndex - NumCulledPrimitives;

					if (NumVisiblePrimitives == 0)
					{
						OcclusionCull.CommandPipe.ReleaseNumCommands(1);
					}
					else
					{
						OcclusionCull.CommandPipe.EnqueueCommand(PrimitiveRange);
					}

					TaskConfig.FrustumCull.NumCulledPrimitives.fetch_add(NumCulledPrimitives, std::memory_order_relaxed);

				}, bHasAlwaysVisible ? Tasks.AlwaysVisible : PrerequisiteTask, TaskConfig.TaskPriority, ExtendedTaskPriority));
			}

			OcclusionCull.CommandPipe.ReleaseNumCommands(1);
		}
	}
	else
	{
		const bool bSingleThreaded = !FApp::ShouldUseThreadingForPerformance() || !bCullingIsThreadsafe;

		PrerequisiteTask.Wait();

		const float CurrentWorldTime = View.Family->Time.GetWorldTimeSeconds();
		ParallelFor(TaskConfig.AlwaysVisible.NumTasks, [this, Flags, CurrentWorldTime](int32 TaskIndex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SceneVisibility_AlwaysVisible);
			FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);
			UpdateAlwaysVisible(Scene, View, Flags, TaskConfig, TaskIndex, CurrentWorldTime);

		}, bSingleThreaded);

		ParallelFor(TaskConfig.FrustumCull.NumTasks, [this, Flags, MaxDrawDistanceScale, HLODState, VisibleNodes](int32 TaskIndex)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SceneVisibility_FrustumCull);
			FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);
			int32 NumCulledPrimitives = FrustumCull(Scene, View, Flags, MaxDrawDistanceScale, HLODState, VisibleNodes, TaskConfig, TaskIndex);
			TaskConfig.FrustumCull.NumCulledPrimitives.fetch_add(NumCulledPrimitives, std::memory_order_relaxed);

		}, bSingleThreaded);
	}

	Tasks.AlwaysVisible.Trigger();
	Tasks.FrustumCull.Trigger();
}

///////////////////////////////////////////////////////////////////////////////

FDynamicMeshElementContext::FDynamicMeshElementContext(FSceneRenderer& SceneRenderer)
	: FirstViewFamily(SceneRenderer.ViewFamily)
	, Views(SceneRenderer.AllViews)
	, Primitives(SceneRenderer.Scene->Primitives)
	// Defer committing GPU scene and Material updates until the mesh collectors are finished. Deferring materials allows VT updates to
	// overlap with GDME (uniform expression updates can't happen at the same time as the async VT system update), and since we are going
	// wide across a single view we would otherwise have to lock for GPU scene updates.
	, MeshCollector(SceneRenderer.FeatureLevel, SceneRenderer.Allocator, FMeshElementCollector::ECommitFlags::DeferAll)
#if WITH_EDITOR
	, EditorMeshCollector(SceneRenderer.FeatureLevel, SceneRenderer.Allocator, FMeshElementCollector::ECommitFlags::DeferAll)
#endif
	, RHICmdList(new FRHICommandList(FRHIGPUMask::All()))
	, DynamicVertexBuffer(*RHICmdList)
	, DynamicIndexBuffer(*RHICmdList)
{
	// With Custom Render Passes, it's possible to have Views that point to a different Family.  Divide Views into groups with the same family.
	uint8 ViewSubsetMask = 0;
	int32 LastViewIndex = Views.Num() - 1;
	for (int32 ViewIndex = 0; ViewIndex <= LastViewIndex; ViewIndex++)
	{
		ViewSubsetMask |= 1u << ViewIndex;

		// Check if the next element has a different ViewFamily, or this is the last view.  If so, emit the GetDynamicMeshElements call
		// with visibility mask bits set for the subset of views with the same ViewFamily, and reset the mask for the next family.
		if (ViewIndex == LastViewIndex || Views[ViewIndex]->Family != Views[ViewIndex + 1]->Family)
		{
			// Emit a batch with the given mask, and reset the mask
			ViewFamilyGroups.Add({ Views[ViewIndex]->Family, ViewSubsetMask });
			ViewSubsetMask = 0;
		}
	}

	RHICmdList->SwitchPipeline(ERHIPipeline::Graphics);

	ViewMeshArraysPerView.SetNum(Views.Num());

	MeshCollector.Start(
		*RHICmdList,
		DynamicVertexBuffer,
		DynamicIndexBuffer,
		SceneRenderer.DynamicReadBufferForInitViews
	);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = *Views[ViewIndex];
		FViewMeshArrays& ViewMeshArrays = ViewMeshArraysPerView[ViewIndex];

		MeshCollector.AddViewMeshArrays(
			&View,
			&ViewMeshArrays.DynamicMeshElements,
			&ViewMeshArrays.SimpleElementCollector,
			&View.DynamicPrimitiveCollector
#if UE_ENABLE_DEBUG_DRAWING
			, &ViewMeshArrays.DebugSimpleElementCollector
#endif
		);
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		EditorMeshCollector.Start(
			*RHICmdList,
			DynamicVertexBuffer,
			DynamicIndexBuffer,
			SceneRenderer.DynamicReadBufferForInitViews
		);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			FViewInfo& View = *Views[ViewIndex];
			FViewMeshArrays& ViewMeshArrays = ViewMeshArraysPerView[ViewIndex];

			EditorMeshCollector.AddViewMeshArrays(
				&View,
				&ViewMeshArrays.DynamicEditorMeshElements,
				&ViewMeshArrays.EditorSimpleElementCollector,
				&View.DynamicPrimitiveCollector
#if UE_ENABLE_DEBUG_DRAWING
				, &ViewMeshArrays.DebugSimpleElementCollector
#endif
			);
		}
	}
#endif
}

FGraphEventRef FDynamicMeshElementContext::LaunchRenderThreadTask(FDynamicPrimitiveIndexList&& PrimitiveIndexList)
{
	return FFunctionGraphTask::CreateAndDispatchWhenReady([this, PrimitiveIndexList = MoveTemp(PrimitiveIndexList)]
	{
		SCOPED_NAMED_EVENT(GatherDynamicMeshElements, FColor::Magenta);

		for (FDynamicPrimitiveIndex PrimitiveIndex : PrimitiveIndexList.Primitives)
		{
			GatherDynamicMeshElementsForPrimitive(Primitives[PrimitiveIndex.Index], PrimitiveIndex.ViewMask);
		}

#if WITH_EDITOR
		for (FDynamicPrimitiveIndex PrimitiveIndex : PrimitiveIndexList.EditorPrimitives)
		{
			GatherDynamicMeshElementsForEditorPrimitive(Primitives[PrimitiveIndex.Index], PrimitiveIndex.ViewMask);
		}
#endif
	}, TStatId{}, nullptr, ENamedThreads::GetRenderThread_Local());
}

UE::Tasks::FTask FDynamicMeshElementContext::LaunchAsyncTask(FDynamicPrimitiveIndexQueue* PrimitiveIndexQueue, UE::Tasks::ETaskPriority TaskPriority)
{
	return Pipe.Launch(UE_SOURCE_LOCATION, [this, PrimitiveIndexQueue]
	{
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

		SCOPED_NAMED_EVENT(GatherDynamicMeshElements, FColor::Magenta);

		FDynamicPrimitiveIndex PrimitiveIndex;

		while (PrimitiveIndexQueue->Pop(PrimitiveIndex))
		{
			GatherDynamicMeshElementsForPrimitive(Primitives[PrimitiveIndex.Index], PrimitiveIndex.ViewMask);
		}

#if WITH_EDITOR
		while (PrimitiveIndexQueue->PopEditor(PrimitiveIndex))
		{
			GatherDynamicMeshElementsForEditorPrimitive(Primitives[PrimitiveIndex.Index], PrimitiveIndex.ViewMask);
		}
#endif
	}, TaskPriority);
}

void FDynamicMeshElementContext::GatherDynamicMeshElementsForPrimitive(FPrimitiveSceneInfo* Primitive, uint8 ViewMask)
{
	SCOPED_NAMED_EVENT(DynamicPrimitive, FColor::Magenta);

	TArray<int32, TInlineAllocator<4>> MeshBatchCountBefore;
	MeshBatchCountBefore.SetNumUninitialized(Views.Num());
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		MeshBatchCountBefore[ViewIndex] = MeshCollector.GetMeshBatchCount(ViewIndex);
	}

	MeshCollector.SetPrimitive(Primitive->Proxy, Primitive->DefaultDynamicHitProxyId);

	// If Custom Render Passes aren't in use, there will be only one group, which is the common case.
	if (ViewFamilyGroups.Num() == 1 || Primitive->Proxy->SinglePassGDME())
	{
		Primitive->Proxy->GetDynamicMeshElements(FirstViewFamily.AllViews, FirstViewFamily, ViewMask, MeshCollector);
	}
	else
	{
		for (FViewFamilyGroup& Group : ViewFamilyGroups)
		{
			if (uint8 MaskedViewMask = ViewMask & Group.ViewSubsetMask)
			{
				Primitive->Proxy->GetDynamicMeshElements(FirstViewFamily.AllViews, *Group.Family, MaskedViewMask, MeshCollector);
			}
		}
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = *Views[ViewIndex];

		if (ViewMask & (1 << ViewIndex))
		{
			FDynamicPrimitive& DynamicPrimitive = DynamicPrimitives.Emplace_GetRef();
			DynamicPrimitive.PrimitiveIndex = Primitive->GetIndex();
			DynamicPrimitive.ViewIndex = ViewIndex;
			DynamicPrimitive.StartElementIndex = MeshBatchCountBefore[ViewIndex];
			DynamicPrimitive.EndElementIndex = MeshCollector.GetMeshBatchCount(ViewIndex);
		}
	}
}

void FDynamicMeshElementContext::GatherDynamicMeshElementsForEditorPrimitive(FPrimitiveSceneInfo* Primitive, uint8 ViewMask)
{
#if WITH_EDITOR
	EditorMeshCollector.SetPrimitive(Primitive->Proxy, Primitive->DefaultDynamicHitProxyId);

	// If Custom Render Passes aren't in use, there will be only one group, which is the common case.
	if (ViewFamilyGroups.Num() == 1 || Primitive->Proxy->SinglePassGDME())
	{
		Primitive->Proxy->GetDynamicMeshElements(FirstViewFamily.AllViews, FirstViewFamily, ViewMask, EditorMeshCollector);
	}
	else
	{
		for (FViewFamilyGroup& Group : ViewFamilyGroups)
		{
			if (uint8 MaskedViewMask = ViewMask & Group.ViewSubsetMask)
			{
				Primitive->Proxy->GetDynamicMeshElements(FirstViewFamily.AllViews, *Group.Family, MaskedViewMask, EditorMeshCollector);
			}
		}
	}
#endif
}

void FDynamicMeshElementContext::Finish()
{
	MeshCollector.Finish();

#if WITH_EDITOR
	if (GIsEditor)
	{
		EditorMeshCollector.Finish();
	}
#endif

	DynamicVertexBuffer.Commit();
	DynamicIndexBuffer.Commit();
	RHICmdList->FinishRecording();

	// Even though task dependencies are setup so all work is done by this point, we still have to wait on the
	// pipe to clear out its internal state. Otherwise it can assert that it still has work at shutdown.
	Pipe.WaitUntilEmpty();
}

FDynamicMeshElementContextContainer::~FDynamicMeshElementContextContainer()
{
	check(CommandLists.IsEmpty());
}

UE::Tasks::FTask FDynamicMeshElementContextContainer::LaunchAsyncTask(FDynamicPrimitiveIndexQueue* PrimitiveIndexQueue, int32 Index, UE::Tasks::ETaskPriority TaskPriority)
{
	return Contexts[Index]->LaunchAsyncTask(PrimitiveIndexQueue, TaskPriority);
}

FGraphEventRef FDynamicMeshElementContextContainer::LaunchRenderThreadTask(FDynamicPrimitiveIndexList&& PrimitiveIndexList)
{
	return Contexts.Last()->LaunchRenderThreadTask(MoveTemp(PrimitiveIndexList));
}

void FDynamicMeshElementContextContainer::Init(FSceneRenderer& SceneRenderer, int32 NumAsyncContexts)
{
	const int32 NumRenderThreadContexts = 1;
	const int32 NumContexts = NumAsyncContexts + NumRenderThreadContexts;
	Views = SceneRenderer.AllViews;
	Contexts.Reserve(NumContexts);
	CommandLists.Reserve(Contexts.Num());

	for (int32 Index = 0; Index < NumContexts; ++Index)
	{
		FDynamicMeshElementContext* Context = SceneRenderer.Allocator.Create<FDynamicMeshElementContext>(SceneRenderer);
		Contexts.Emplace(Context);
		CommandLists.Emplace(Context->RHICmdList);
	}
}

void FDynamicMeshElementContextContainer::MergeContexts(TArray<FDynamicPrimitive, SceneRenderingAllocator>& OutDynamicPrimitives)
{
	SCOPED_NAMED_EVENT(MergeGatherDynamicMeshElementContexts, FColor::Magenta);

	check(!Views.IsEmpty());

	// Fast path for one context; just move the memory instead of copying.
	if (Contexts.Num() == 1)
	{
		FDynamicMeshElementContext* Context = Contexts[0];
		OutDynamicPrimitives = MoveTemp(Context->DynamicPrimitives);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			FViewInfo& View = *Views[ViewIndex];
			FDynamicMeshElementContext::FViewMeshArrays& ViewMeshArrays = Context->ViewMeshArraysPerView[ViewIndex];

			View.SimpleElementCollector = MoveTemp(ViewMeshArrays.SimpleElementCollector);
			View.DynamicMeshElements    = MoveTemp(ViewMeshArrays.DynamicMeshElements);

#if WITH_EDITOR
			View.EditorSimpleElementCollector = MoveTemp(ViewMeshArrays.EditorSimpleElementCollector);
			View.DynamicEditorMeshElements    = MoveTemp(ViewMeshArrays.DynamicEditorMeshElements);
#endif

#if UE_ENABLE_DEBUG_DRAWING
			View.DebugSimpleElementCollector = MoveTemp(ViewMeshArrays.DebugSimpleElementCollector);
#endif
		}
	}
	// >1 context means we have to merge the containers.
	else
	{
		struct FViewAllocationInfo
		{
			FSimpleElementCollector::FAllocationInfo SimpleElementCollector;
			uint32 NumDynamicMeshElements = 0;
#if WITH_EDITOR
			FSimpleElementCollector::FAllocationInfo EditorSimpleElementCollector;
			uint32 NumDynamicEditorMeshElements = 0;
#endif
#if UE_ENABLE_DEBUG_DRAWING
			FSimpleElementCollector::FAllocationInfo DebugSimpleElementCollector;
#endif
		};

		TArray<FViewAllocationInfo, TInlineAllocator<2>> AllocationInfosPerView;
		AllocationInfosPerView.AddDefaulted(Views.Num());
		uint32 NumDynamicPrimitives = 0;

		for (FDynamicMeshElementContext* Context : Contexts)
		{
			NumDynamicPrimitives += Context->DynamicPrimitives.Num();
			check(AllocationInfosPerView.Num() == Context->ViewMeshArraysPerView.Num());

			// Accumulate allocation info for each context in order to reserve container memory once.
			for (int32 ViewIndex = 0; ViewIndex < AllocationInfosPerView.Num(); ++ViewIndex)
			{
				const FDynamicMeshElementContext::FViewMeshArrays& ViewMeshArrays = Context->ViewMeshArraysPerView[ViewIndex];
				FViewAllocationInfo& AllocationInfo = AllocationInfosPerView[ViewIndex];

				ViewMeshArrays.SimpleElementCollector.AddAllocationInfo(AllocationInfo.SimpleElementCollector);
				AllocationInfo.NumDynamicMeshElements += ViewMeshArrays.DynamicMeshElements.Num();
#if WITH_EDITOR
				ViewMeshArrays.EditorSimpleElementCollector.AddAllocationInfo(AllocationInfo.EditorSimpleElementCollector);
				AllocationInfo.NumDynamicEditorMeshElements += ViewMeshArrays.DynamicEditorMeshElements.Num();
#endif
#if UE_ENABLE_DEBUG_DRAWING
				ViewMeshArrays.DebugSimpleElementCollector.AddAllocationInfo(AllocationInfo.DebugSimpleElementCollector);
#endif
			}
		}

		OutDynamicPrimitives.Reserve(NumDynamicPrimitives);

		// Reserve memory for merged containers.
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			FViewAllocationInfo& AllocationInfo = AllocationInfosPerView[ViewIndex];
			FViewInfo& View = *Views[ViewIndex];

			View.SimpleElementCollector.Reserve(AllocationInfo.SimpleElementCollector);
			View.DynamicMeshElements.Reserve(AllocationInfo.NumDynamicMeshElements);
#if WITH_EDITOR
			View.EditorSimpleElementCollector.Reserve(AllocationInfo.EditorSimpleElementCollector);
			View.DynamicEditorMeshElements.Reserve(AllocationInfo.NumDynamicEditorMeshElements);
#endif
#if UE_ENABLE_DEBUG_DRAWING
			View.DebugSimpleElementCollector.Reserve(AllocationInfo.DebugSimpleElementCollector);
#endif

			// Reset dynamic element count to use as offset for copying ranges in the next loop.
			AllocationInfo.NumDynamicMeshElements = 0;
		}

		for (FDynamicMeshElementContext* Context : Contexts)
		{
			for (FDynamicPrimitive DynamicPrimitive : Context->DynamicPrimitives)
			{
				const uint32 NumDynamicMeshElements = AllocationInfosPerView[DynamicPrimitive.ViewIndex].NumDynamicMeshElements;

				// Offset the dynamic element range by the current number of meshes in the final container.
				DynamicPrimitive.StartElementIndex += NumDynamicMeshElements;
				DynamicPrimitive.EndElementIndex   += NumDynamicMeshElements;

				OutDynamicPrimitives.Emplace(DynamicPrimitive);

				Views[DynamicPrimitive.ViewIndex]->DynamicMeshElementRanges[DynamicPrimitive.PrimitiveIndex] = FInt32Vector2(DynamicPrimitive.StartElementIndex, DynamicPrimitive.EndElementIndex);
			}

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
			{
				FViewInfo& View = *Views[ViewIndex];
				FDynamicMeshElementContext::FViewMeshArrays& ViewMeshArrays = Context->ViewMeshArraysPerView[ViewIndex];

				AllocationInfosPerView[ViewIndex].NumDynamicMeshElements += ViewMeshArrays.DynamicMeshElements.Num();

				View.SimpleElementCollector.Append(ViewMeshArrays.SimpleElementCollector);
				View.DynamicMeshElements.Append(ViewMeshArrays.DynamicMeshElements);
#if WITH_EDITOR
				View.EditorSimpleElementCollector.Append(ViewMeshArrays.EditorSimpleElementCollector);
				View.DynamicEditorMeshElements.Append(ViewMeshArrays.DynamicEditorMeshElements);
#endif
#if UE_ENABLE_DEBUG_DRAWING
				View.DebugSimpleElementCollector.Append(ViewMeshArrays.DebugSimpleElementCollector);
#endif
			}

			Context->DynamicPrimitives.Empty();
			Context->ViewMeshArraysPerView.Empty();
		}
	}
}

void FDynamicMeshElementContextContainer::Submit(FRHICommandListImmediate& RHICmdList)
{
	for (FDynamicMeshElementContext* Context : Contexts)
	{
		Context->Finish();
	}

	RHICmdList.QueueAsyncCommandListSubmit(CommandLists);
	CommandLists.Empty();
}

///////////////////////////////////////////////////////////////////////////////

FVisibilityTaskData::FVisibilityTaskData(FRHICommandListImmediate& InRHICmdList, FSceneRenderer& InSceneRenderer)
	: RHICmdList(InRHICmdList)
	, SceneRenderer(InSceneRenderer)
	, Scene(*SceneRenderer.Scene)
	, Views(SceneRenderer.AllViews)
	, ViewFamily(SceneRenderer.ViewFamily)
	, ShadingPath(GetFeatureLevelShadingPath(Scene.GetFeatureLevel()))
	, TaskConfig(Scene, Views)
	, bAddNaniteRelevance(InSceneRenderer.ShouldRenderNanite())
	, bAddLightmapDensityCommands(ViewFamily.EngineShowFlags.LightMapDensity&& AllowDebugViewmodes())
{
	Tasks.bWaitingAllowed = TaskConfig.Schedule == EVisibilityTaskSchedule::Parallel;
}

void FVisibilityTaskData::LaunchVisibilityTasks(const UE::Tasks::FTask& BeginInitVisibilityPrerequisites)
{
	SCOPED_NAMED_EVENT(LaunchVisibilityTasks, FColor::Magenta);
	ViewPackets.Reserve(Views.Num());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		if (Views[ViewIndex]->bIsSinglePassStereo && Views[ViewIndex]->StereoPass == EStereoscopicPass::eSSP_SECONDARY)
		{
			continue;
		}

		// Each non-vestigial view gets its own visibility task packet which contains all the state to manage the task graph for a view.
		FVisibilityViewPacket& ViewPacket = ViewPackets.Emplace_GetRef(*this, Scene, *Views[ViewIndex], ViewIndex);

		// Each respective stage of visibility for each view is connected to its corresponding task event in order to track when all views complete each stage.
		Tasks.LightVisibility.AddPrerequisites(ViewPacket.Tasks.LightVisibility);
		Tasks.FrustumCull.AddPrerequisites(ViewPacket.Tasks.FrustumCull);
		Tasks.OcclusionCull.AddPrerequisites(ViewPacket.Tasks.OcclusionCull);
		Tasks.ComputeRelevance.AddPrerequisites(ViewPacket.Tasks.ComputeRelevance);

		if (ViewPacket.ViewState)
		{
			SCOPE_CYCLE_COUNTER(STAT_DecompressPrecomputedOcclusion);
			ViewPacket.View.PrecomputedVisibilityData = ViewPacket.ViewState->ResolvePrecomputedVisibilityData(ViewPacket.View, &Scene);

			if (ViewPacket.View.PrecomputedVisibilityData)
			{
				SceneRenderer.bUsedPrecomputedVisibility = true;
			}
		}
	}

	// Each relevance task should have this as a prerequisite, but in case there aren't any tasks we make it explicit.
	Tasks.ComputeRelevance.AddPrerequisites(Scene.GetCacheMeshDrawCommandsTask());

	// Wait on the GPU skin update task prior to GDME.
	Tasks.DynamicMeshElementsPrerequisites.AddPrerequisites(Scene.GetGPUSkinUpdateTask());

	bool bAllocatePrimitiveViewMasks = true;

	if (TaskConfig.Schedule == EVisibilityTaskSchedule::Parallel)
	{
		if (ViewPackets.Num() == 1)
		{
			// When using a single view, dynamic mesh elements are pushed into a pipe that is executed on the render thread which allows for some overlap with compute relevance work.
			DynamicMeshElements.CommandPipe = Allocator.Create<TCommandPipe<FDynamicPrimitiveIndexList>>(TEXT("GatherDynamicMeshElementsPipe"));

			DynamicMeshElements.CommandPipe->SetCommandFunction([this](FDynamicPrimitiveIndexList&& DynamicPrimitiveIndexList)
			{
				GatherDynamicMeshElements(MoveTemp(DynamicPrimitiveIndexList));
			});

			DynamicMeshElements.CommandPipe->SetPrerequisiteTask(Tasks.DynamicMeshElementsPrerequisites);

			Tasks.DynamicMeshElementsPipe = FGraphEvent::CreateGraphEvent();

			DynamicMeshElements.CommandPipe->SetEmptyFunction([this]
			{
				Tasks.DynamicMeshElementsPipe->DispatchSubsequents();
				Tasks.DynamicMeshElements.Trigger();
			});

			// Take a reference that is released when the relevance pipe has completed. We only need to take one since there can only be one view.
			DynamicMeshElements.CommandPipe->AddNumCommands(1);

			// We don't need the primitive view masks when in parallel mode with a single view.
			bAllocatePrimitiveViewMasks = false;
		}
	}

	if (bAllocatePrimitiveViewMasks)
	{
		DynamicMeshElements.PrimitiveViewMasks = Allocator.Create<FDynamicPrimitiveViewMasks>();
		DynamicMeshElements.PrimitiveViewMasks->Primitives.AddZeroed(Scene.Primitives.Num());

	#if WITH_EDITOR
		if (GIsEditor)
		{
			DynamicMeshElements.PrimitiveViewMasks->EditorPrimitives.AddZeroed(Scene.Primitives.Num());
		}
	#endif
	}

	DynamicMeshElements.ContextContainer.Init(SceneRenderer, GetNumDynamicMeshElementTasks());
	DynamicMeshElements.ViewCommandsPerView.SetNum(Views.Num());

	Tasks.LightVisibility.AddPrerequisites(UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]
	{
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
		SceneRenderer.ComputeLightVisibility();

	}, TaskConfig.TaskPriority));

	if (TaskConfig.Schedule == EVisibilityTaskSchedule::Parallel)
	{
		SceneRenderer.WaitOcclusionTests(RHICmdList);

		// Parallel occlusion culling is not supported on mobile
		check(!Views.IsEmpty())
		checkf(!Views[0]->UsesMobileMultiViewShaders(), TEXT("This culling path was not tested with MMV"));

		for (FVisibilityViewPacket& ViewPacket : ViewPackets)
		{
			if (ViewPacket.OcclusionCull.ContextIfParallel)
			{
				ViewPacket.OcclusionCull.ContextIfParallel->Map(RHICmdList);
			}

			Tasks.BeginInitVisibility.AddPrerequisites(UE::Tasks::Launch(UE_SOURCE_LOCATION, [&ViewPacket]
			{
				FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
				ViewPacket.BeginInitVisibility();

			}, BeginInitVisibilityPrerequisites, TaskConfig.TaskPriority));
		}

		// Static relevance is finalized for ALL views after each view completes static mesh filtering tasks.
		Tasks.FinalizeRelevance = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FSceneRenderer_FinalizeStaticRelevance);

			for (FVisibilityViewPacket& ViewPacket : ViewPackets)
			{
				ViewPacket.Relevance.Context->Finalize();
			}

		}, Tasks.ComputeRelevance, TaskConfig.TaskPriority);
	}

	// All task events are connected to prerequisites now and can be safely triggered.
	Tasks.BeginInitVisibility.Trigger();
	Tasks.LightVisibility.Trigger();
	Tasks.FrustumCull.Trigger();
	Tasks.OcclusionCull.Trigger();
	Tasks.ComputeRelevance.Trigger();
}

void FVisibilityTaskData::GatherDynamicMeshElements(FDynamicPrimitiveIndexList&& DynamicPrimitiveIndexList)
{
	FDynamicPrimitiveIndexList RenderThreadDynamicPrimitiveIndexList;

	const int32 NumAsyncContexts = DynamicMeshElements.ContextContainer.GetNumAsyncContexts();

	if (NumAsyncContexts > 0)
	{
		const auto FilterDynamicPrimitives = [] (TArrayView<FPrimitiveSceneProxy*> PrimitiveSceneProxies, FDynamicPrimitiveIndexList::FList& Primitives, FDynamicPrimitiveIndexList::FList& RenderThreadPrimitives)
		{
			for (int32 Index = 0; Index < Primitives.Num(); )
			{
				const FDynamicPrimitiveIndex PrimitiveIndex = Primitives[Index];

				if (!PrimitiveSceneProxies[PrimitiveIndex.Index]->SupportsParallelGDME())
				{
					RenderThreadPrimitives.Emplace(PrimitiveIndex);
					Primitives.RemoveAtSwap(Index, EAllowShrinking::No);
				}
				else
				{
					Index++;
				}
			}
		};

		FilterDynamicPrimitives(Scene.PrimitiveSceneProxies, DynamicPrimitiveIndexList.Primitives, RenderThreadDynamicPrimitiveIndexList.Primitives);
#if WITH_EDITOR
		FilterDynamicPrimitives(Scene.PrimitiveSceneProxies, DynamicPrimitiveIndexList.EditorPrimitives, RenderThreadDynamicPrimitiveIndexList.EditorPrimitives);
#endif
	}
	else
	{
		RenderThreadDynamicPrimitiveIndexList = MoveTemp(DynamicPrimitiveIndexList);
		DynamicPrimitiveIndexList = {};
	}

	if (!RenderThreadDynamicPrimitiveIndexList.IsEmpty())
	{
		DynamicMeshElements.ContextContainer.LaunchRenderThreadTask(MoveTemp(RenderThreadDynamicPrimitiveIndexList));
	}

	if (!DynamicPrimitiveIndexList.IsEmpty())
	{
		FDynamicPrimitiveIndexQueue* Queue = Allocator.Create<FDynamicPrimitiveIndexQueue>(MoveTemp(DynamicPrimitiveIndexList));

		for (int32 Index = 0; Index < DynamicMeshElements.ContextContainer.GetNumAsyncContexts(); ++Index)
		{
			Tasks.DynamicMeshElements.AddPrerequisites(DynamicMeshElements.ContextContainer.LaunchAsyncTask(Queue, Index, TaskConfig.TaskPriority));
		}
	}
}

void FVisibilityTaskData::GatherDynamicMeshElements(const FDynamicPrimitiveViewMasks& DynamicPrimitiveViewMasks)
{
	SCOPED_NAMED_EVENT(GatherDynamicMeshElements, FColor::Magenta);

	Tasks.DynamicMeshElementsPrerequisites.Wait();

	const auto GetPrimaryViewMask = [this] (uint8 ViewMask) -> uint8
	{
		// If a mesh is visible in a secondary view, mark it as visible in the primary view
		uint8 ViewMaskFinal = ViewMask;

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = *Views[ViewIndex];

			if (ViewMask & (1 << ViewIndex) && IStereoRendering::IsASecondaryView(View))
			{
				ViewMaskFinal |= 1 << Views[ViewIndex]->PrimaryViewIndex;
			}
		}

		return ViewMaskFinal;
	};

	FDynamicPrimitiveIndexList DynamicPrimitiveIndexList;
	DynamicPrimitiveIndexList.Primitives.Reserve(128);

	const int32 NumPrimitives = DynamicPrimitiveViewMasks.Primitives.Num();

	for (int32 PrimitiveIndex = 0; PrimitiveIndex < NumPrimitives; ++PrimitiveIndex)
	{
		const uint8 ViewMask = DynamicPrimitiveViewMasks.Primitives[PrimitiveIndex];

		if (ViewMask != 0)
		{
			DynamicPrimitiveIndexList.Primitives.Emplace(PrimitiveIndex, GetPrimaryViewMask(ViewMask));
		}
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		DynamicPrimitiveIndexList.EditorPrimitives.Reserve(128);

		for (int32 PrimitiveIndex = 0; PrimitiveIndex < NumPrimitives; ++PrimitiveIndex)
		{
			const uint8 ViewMask = DynamicPrimitiveViewMasks.EditorPrimitives[PrimitiveIndex];

			if (ViewMask != 0)
			{
				DynamicPrimitiveIndexList.EditorPrimitives.Emplace(PrimitiveIndex, GetPrimaryViewMask(ViewMask));
			}
		}
	}
#endif

	GatherDynamicMeshElements(MoveTemp(DynamicPrimitiveIndexList));
	Tasks.DynamicMeshElements.Trigger();
}

void FVisibilityTaskData::SetupMeshPasses(FExclusiveDepthStencil::Type BasePassDepthStencilAccess, FInstanceCullingManager& InstanceCullingManager)
{
	DynamicMeshElements.ContextContainer.MergeContexts(DynamicMeshElements.DynamicPrimitives);

	{
		SCOPED_NAMED_EVENT(DynamicRelevance, FColor::Magenta);

		TArray<bool, TInlineAllocator<LUMEN_MAX_VIEWS>> IsLumenTranslucencyRadianceCacheEnabledForView;
		IsLumenTranslucencyRadianceCacheEnabledForView.Reserve(Views.Num());

		TArray<bool, TInlineAllocator<LUMEN_MAX_VIEWS>> IsFrontLayerTranslucencyEnabledForView;
		IsFrontLayerTranslucencyEnabledForView.Reserve(Views.Num());

		for (FViewInfo* View : Views)
		{
			View->DynamicMeshElementsPassRelevance.SetNum(View->DynamicMeshElements.Num());

			IsLumenTranslucencyRadianceCacheEnabledForView.Add(ShouldRenderLumenDiffuseGI(&Scene, *View));
			IsFrontLayerTranslucencyEnabledForView.Add(ShouldRenderFrontLayerTranslucency(*View));
		}

		const bool bIsTLVUsingVoxelMarking = IsTranslucencyLightingVolumeUsingVoxelMarking();

		for (FDynamicPrimitive DynamicPrimitive : DynamicMeshElements.DynamicPrimitives)
		{
			FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene.Primitives[DynamicPrimitive.PrimitiveIndex];
			const FPrimitiveBounds& Bounds = Scene.PrimitiveBounds[DynamicPrimitive.PrimitiveIndex];
			FViewInfo& View = *Views[DynamicPrimitive.ViewIndex];
			const FPrimitiveViewRelevance& ViewRelevance = View.PrimitiveViewRelevanceMap[DynamicPrimitive.PrimitiveIndex];

			const bool bIsLumenTranslucencyRadianceCacheEnabled = IsLumenTranslucencyRadianceCacheEnabledForView[DynamicPrimitive.ViewIndex];
			const bool bIsFrontLayerTranslucencyEnabled = IsFrontLayerTranslucencyEnabledForView[DynamicPrimitive.ViewIndex];

			for (int32 ElementIndex = DynamicPrimitive.StartElementIndex; ElementIndex < DynamicPrimitive.EndElementIndex; ++ElementIndex)
			{
				const FMeshBatchAndRelevance& MeshBatch = View.DynamicMeshElements[ElementIndex];
				FMeshPassMask& PassRelevance = View.DynamicMeshElementsPassRelevance[ElementIndex];

				ComputeDynamicMeshRelevance(
					ShadingPath,
					bAddLightmapDensityCommands,
					bIsLumenTranslucencyRadianceCacheEnabled,
					bIsTLVUsingVoxelMarking,
					bIsFrontLayerTranslucencyEnabled,
					ViewRelevance,
					MeshBatch,
					View,
					PassRelevance,
					PrimitiveSceneInfo,
					Bounds);
			}
		}
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = *Views[ViewIndex];

		// Note: The per-view Nanite editor hit-proxy sort + power-of-two pad has moved to
		// Nanite::FEditorSceneExtension::FUpdater's setup task (scene-wide, runs once per frame).

	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FSceneViewState* ViewState = static_cast<FSceneViewState*>(View.State);

		// if we are freezing the scene, then remember the primitives that are rendered.
		if (ViewState && ViewState->bIsFreezing)
		{
			for (FSceneSetBitIterator BitIt(View.PrimitiveVisibilityMap); BitIt; ++BitIt)
			{
				ViewState->FrozenPrimitives.Add(Scene.PrimitiveComponentIds[BitIt.GetIndex()]);
			}
		}
	#endif

		if (!View.ShouldRenderView())
		{
			continue;
		}

		FViewCommands& ViewCommands = DynamicMeshElements.ViewCommandsPerView[ViewIndex];

	#if !UE_BUILD_SHIPPING
		FViewDebugInfo::Get().ProcessPrimitives(&Scene, View, ViewCommands);
	#endif

		SceneRenderer.SetupMeshPass(View, BasePassDepthStencilAccess, ViewCommands, InstanceCullingManager);
	}
}

void FVisibilityTaskData::ProcessRenderThreadTasks()
{
	SCOPE_CYCLE_COUNTER(STAT_ViewVisibilityTime);
	SCOPED_NAMED_EVENT(ProcessVisibilityTasks, FColor::Magenta);

	UE::Tasks::FTask VirtualTextureTask;

	StartGatherDynamicMeshElements();

	FPrimitiveRange PrimitiveRange;
	PrimitiveRange.StartIndex = 0u;
	PrimitiveRange.EndIndex = TaskConfig.NumTestedPrimitives;

	if (TaskConfig.Schedule == EVisibilityTaskSchedule::RenderThread)
	{
		for (FVisibilityViewPacket& ViewPacket : ViewPackets)
		{
		#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (ViewPacket.ViewState)
			{
				ViewPacket.ViewState->ActivateFrozenViewMatrices(ViewPacket.View);
			}
		#endif

			ViewPacket.BeginInitVisibility();
			UpdatePrimitiveFading(Scene, ViewPacket.View, ViewPacket.ViewState, PrimitiveRange);
		}

		SceneRenderer.WaitOcclusionTests(RHICmdList);

		for (FVisibilityViewPacket& ViewPacket : ViewPackets)
		{
			SCOPED_NAMED_EVENT(OcclusionCull, FColor::Magenta);
			const int32 NumCulledPrimitives = PrecomputedOcclusionCull(ViewPacket, PrimitiveRange);

			if (ViewPacket.OcclusionCull.ContextIfSerial)
			{
				ViewPacket.OcclusionCull.ContextIfSerial->Map(RHICmdList);
				ViewPacket.OcclusionCull.ContextIfSerial->AddPrimitives(PrimitiveRange);
				ViewPacket.OcclusionCull.ContextIfSerial->Unmap(RHICmdList);
			}

			if (NumCulledPrimitives > 0)
			{
				TaskConfig.OcclusionCull.NumCulledPrimitives.fetch_add(NumCulledPrimitives, std::memory_order_relaxed);
			}

			ViewPacket.Tasks.OcclusionCull.Trigger();
		}

		// Relevance requires that cached mesh commands be available first.
		Scene.WaitForCacheMeshDrawCommandsTask();

		for (FVisibilityViewPacket& ViewPacket : ViewPackets)
		{
			for (FSceneSetBitIterator BitIt(ViewPacket.View.PrimitiveVisibilityMap, PrimitiveRange.StartIndex); BitIt.GetIndex() < PrimitiveRange.EndIndex; ++BitIt)
			{
				ViewPacket.Relevance.Context->AddPrimitive(BitIt.GetIndex());
			}
			ViewPacket.Relevance.Context->Finalize();
			ViewPacket.Tasks.ComputeRelevance.Trigger();
		}

		Tasks.bWaitingAllowed = true;

		check(DynamicMeshElements.PrimitiveViewMasks);
		GatherDynamicMeshElements(*DynamicMeshElements.PrimitiveViewMasks);

	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		for (FVisibilityViewPacket& ViewPacket : ViewPackets)
		{
			if (ViewPacket.ViewState)
			{
				ViewPacket.ViewState->RestoreUnfrozenViewMatrices(ViewPacket.View);
			}
		}
	#endif
	}
	else
	{
		if (DynamicMeshElements.CommandPipe)
		{
			SCOPED_NAMED_EVENT(WaitForVisibilityTasks, FColor::Magenta);
			CSV_SCOPED_SET_WAIT_STAT(Visibility);

			// Wait on the command pipe first as it will be continually updating the render thread event (and process tasks while we wait).
			Tasks.DynamicMeshElementsPipe->Wait(ENamedThreads::GetRenderThread_Local());
		}
		else
		{
			Tasks.ComputeRelevance.Wait();
			check(DynamicMeshElements.PrimitiveViewMasks);
			GatherDynamicMeshElements(*DynamicMeshElements.PrimitiveViewMasks);
		}

		for (FVisibilityViewPacket& ViewPacket : ViewPackets)
		{
			if (ViewPacket.OcclusionCull.ContextIfParallel)
			{
				ViewPacket.OcclusionCull.ContextIfParallel->Unmap(RHICmdList);
			}
		}
	}

	// Now process all gather dynamic mesh element tasks that were queued up to run on the render thread.
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GetRenderThread_Local());

	Tasks.LightVisibility.Wait();
	Tasks.FinalizeRelevance.Wait();

	INC_DWORD_STAT_BY(STAT_ProcessedPrimitives, PrimitiveRange.EndIndex * Views.Num());
	INC_DWORD_STAT_BY(STAT_CulledPrimitives, TaskConfig.FrustumCull.NumCulledPrimitives);
	INC_DWORD_STAT_BY(STAT_OccludedPrimitives, TaskConfig.OcclusionCull.NumCulledPrimitives);

	TRACE_COUNTER_SET(Scene_Visibility_NumProcessedPrimitives, PrimitiveRange.EndIndex * Views.Num());
	TRACE_COUNTER_SET(Scene_Visibility_FrustumCull_NumPrimitivesPerTask, TaskConfig.FrustumCull.NumPrimitivesPerTask);
	TRACE_COUNTER_SET(Scene_Visibility_FrustumCull_NumCulledPrimitives, TaskConfig.FrustumCull.NumCulledPrimitives);
	TRACE_COUNTER_SET(Scene_Visibility_OcclusionCull_NumCulledPrimitives, TaskConfig.OcclusionCull.NumCulledPrimitives);
	TRACE_COUNTER_SET(Scene_Visibility_OcclusionCull_NumTestedQueries, TaskConfig.OcclusionCull.NumTestedQueries);
	TRACE_COUNTER_SET(Scene_Visibility_Relevance_NumPrimitivesPerPacket, TaskConfig.Relevance.NumPrimitivesPerPacket);
}

void FVisibilityTaskData::FinishGatherDynamicMeshElements(FExclusiveDepthStencil::Type BasePassDepthStencilAccess, FInstanceCullingManager& InstanceCullingManager, FVirtualTextureUpdater* VirtualTextureUpdater)
{
	check(IsInRenderingThread());
	SCOPED_NAMED_EVENT(FinishDynamicMeshElements, FColor::Magenta);

	FVirtualTextureSystem::Get().WaitForTasks(VirtualTextureUpdater);
	Tasks.DynamicMeshElements.Wait();
	DynamicMeshElements.ContextContainer.Submit(RHICmdList);

	Tasks.MeshPassSetup = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, BasePassDepthStencilAccess, &InstanceCullingManager]
	{
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
		SetupMeshPasses(BasePassDepthStencilAccess, InstanceCullingManager);

	}, TaskConfig.TaskPriority);

	FSceneRenderer::DynamicReadBufferForInitViews.Commit(RHICmdList);
}

void FVisibilityTaskData::Finish()
{
	SCOPED_NAMED_EVENT(FinishVisibility, FColor::Magenta);

	Tasks.ComputeRelevance.Wait();
	Tasks.FinalizeRelevance.Wait();
	Tasks.DynamicMeshElements.Wait();
	Tasks.MeshPassSetup.Wait();

	ViewPackets.Empty();
	DynamicMeshElements.DynamicPrimitives.Empty();
	Allocator.BulkDelete();
	bFinished = true;
}

///////////////////////////////////////////////////////////////////////////////

/**
 * Helper for InitViews to detect large camera movement, in both angle and position.
 */
static bool IsLargeCameraMovement(FSceneView& View, const FMatrix& PrevViewMatrix, const FVector& PrevViewOrigin, float CameraRotationThreshold, float CameraTranslationThreshold)
{
	float RotationThreshold = FMath::Cos(FMath::DegreesToRadians(CameraRotationThreshold));
	float ViewRightAngle = View.ViewMatrices.GetWorldToView().GetColumn(0) | PrevViewMatrix.GetColumn(0);
	float ViewUpAngle = View.ViewMatrices.GetWorldToView().GetColumn(1) | PrevViewMatrix.GetColumn(1);
	float ViewDirectionAngle = View.ViewMatrices.GetWorldToView().GetColumn(2) | PrevViewMatrix.GetColumn(2);

	FVector Distance = FVector(View.ViewMatrices.GetViewOrigin()) - PrevViewOrigin;
	return 
		ViewRightAngle < RotationThreshold ||
		ViewUpAngle < RotationThreshold ||
		ViewDirectionAngle < RotationThreshold ||
		Distance.SizeSquared() > CameraTranslationThreshold * CameraTranslationThreshold;
}

void FSceneRenderer::PreVisibilityFrameSetup(FRDGBuilder& GraphBuilder)
{
	// Possible stencil dither optimization approach
	for (FViewInfo& View : Views)
	{
		View.bAllowStencilDither = DepthPass.bDitheredLODTransitionsUseStencil;
	}

	// Initialize InstanceFactor. AllViews includes CustomRenderPass views, which might possibly also require InstanceFactor to be correctly assigned.
	const int32 InstanceFactor = FMath::Max(1, GEngine->StereoRenderingDevice.IsValid() ? GEngine->StereoRenderingDevice->GetDesiredNumberOfViews(true) : 1);
	if (InstanceFactor > 1)
	{
		for (FViewInfo* View : AllViews)
		{
			if (View->bIsInstancedStereoEnabled && IStereoRendering::IsStereoEyeView(*View))
			{
				View->InstanceFactor = InstanceFactor;
			}
		}
	}
	
	FRHICommandListImmediate& RHICmdList = GraphBuilder.RHICmdList;
	
	if (GetRendererOutput() == ERendererOutput::FinalSceneColor)
	{
		FHairStrandsBookmarkParameters Parameters;
        bool bHasHairStrandsBookmarks = false;
		
		if (Views.Num() > 0 && !ViewFamily.EngineShowFlags.HitProxies)
		{
			CreateHairStrandsBookmarkParameters(Scene, Views, AllViews, Parameters, false /*bComputeVisibleInstances*/);
			Parameters.TransientResources = AllocateHairTransientResources(GraphBuilder, Scene, Views);
			if (Parameters.HasInstances())
			{
				if (Scene && IsHairStrandsEnabled(EHairStrandsShaderType::All, Scene->GetShaderPlatform()))
				{
					// Need GPUSkinCache results to be available in order to get correct skel. mesh data
					Scene->GetGPUSkinCacheTask().Wait();

					// 1.Update binding surfaces
					// Prepare surface data (MeshLODData)
					RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessBindingSurfaceUpdate, Parameters);

					// 2. Prepare surface data for guides
					if (Views[0].AllowGPUParticleUpdate())
					{
						RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessGuideInterpolation, Parameters);
						bHasHairStrandsBookmarks = true;
					}
				}
			}
		}

		if (Scene)
		{
			for (IComputeTaskWorker* ComputeTaskWorker : Scene->ComputeTaskWorkers)
			{
				if (ComputeTaskWorker->HasWork(ComputeTaskExecutionGroup::BeginInitViews))
				{
					FComputeContext Context
					{
						.GraphBuilder = GraphBuilder,
						.ExecutionGroupName = ComputeTaskExecutionGroup::BeginInitViews,
						.FeatureLevel = FeatureLevel,
						.Scene = Scene,
					};

					ComputeTaskWorker->SubmitWork(Context);
				}
			}
		}

		// Notify the FX system that the scene is about to perform visibility checks.

		if (FXSystem && Views.IsValidIndex(0))
		{
			TArray<const FSceneViewFamily*, TInlineAllocator<2>> AllLinkedFamilies;
			EnumerateLinkedViewFamilies([&] (const FSceneViewFamily& Family)
			{
				AllLinkedFamilies.Emplace(&Family);
				return true;
			});
			FXSystem->PreInitViews(GraphBuilder, Views[0].AllowGPUParticleUpdate() && !ViewFamily.EngineShowFlags.HitProxies, AllLinkedFamilies, &ViewFamily);
		}

		if(bHasHairStrandsBookmarks)
		{
			RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessGuideDeformation, Parameters);
		}

#if WITH_EDITOR
		// Draw lines to lights affecting this mesh if its selected.
		if (ViewFamily.EngineShowFlags.LightInfluences && Scene)
		{
			Scene->WaitForCreateLightPrimitiveInteractionsTask();

			for (TConstSetBitIterator<> It(Scene->PrimitivesSelected); It; ++It)
			{
				const FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene->Primitives[It.GetIndex()];

				TArray<const FLightSceneProxy*> RelevantLights;
				Scene->GetRelevantLights_RenderThread(PrimitiveSceneInfo->Proxy, RelevantLights);

				for (const FLightSceneProxy* LightSceneProxy : RelevantLights)
				{
					bool bDynamic = true;
					bool bRelevant = false;
					bool bLightMapped = true;
					bool bShadowMapped = false;
					PrimitiveSceneInfo->Proxy->GetLightRelevance(LightSceneProxy, bDynamic, bRelevant, bLightMapped, bShadowMapped);

					if (bRelevant)
					{
						// Draw blue for light-mapped lights and orange for dynamic lights
						const FColor LineColor = bLightMapped ? FColor(0, 140, 255) : FColor(255, 140, 0);
						for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
						{
							FViewInfo& View = Views[ViewIndex];
							FViewElementPDI LightInfluencesPDI(&View, nullptr, &View.DynamicPrimitiveCollector);
							LightInfluencesPDI.DrawLine(PrimitiveSceneInfo->Proxy->GetBounds().Origin, LightSceneProxy->GetLightToWorld().GetOrigin(), LineColor, SDPG_World);
						}
					}
				}
			}
		}
#endif

#if RHI_RAYTRACING
		if (Scene)
		{
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				FViewInfo& View = Views[ViewIndex];
				View.bRayTracingFeedbackEnabled = Scene->RayTracingScene.IsRayTracingFeedbackEnabled();
			}
		}
#endif
	}

	for (const auto& ViewExtension : ViewFamily.ViewExtensions)
	{
		ViewExtension->PreInitViews_RenderThread(GraphBuilder);
	}
}

void FSceneRenderer::PrepareViewStateForVisibility(const FSceneTexturesConfig& SceneTexturesConfig)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PrepareViewStateForVisibility);

#if UE_BUILD_SHIPPING
	const bool bFreezeTemporalHistories = false;
	const bool bFreezeTemporalSequences = false;
#else
	bool bFreezeTemporalHistories = CVarFreezeTemporalHistories.GetValueOnRenderThread() != 0;

	static int32 CurrentFreezeTemporalHistoriesProgress = 0;
	if (CurrentFreezeTemporalHistoriesProgress != CVarFreezeTemporalHistoriesProgress.GetValueOnRenderThread())
	{
		bFreezeTemporalHistories = false;
		CurrentFreezeTemporalHistoriesProgress = CVarFreezeTemporalHistoriesProgress.GetValueOnRenderThread();
	}

	bool bFreezeTemporalSequences = bFreezeTemporalHistories || CVarFreezeTemporalSequences.GetValueOnRenderThread() != 0;
#endif

	// Setup motion blur parameters (also check for camera movement thresholds)
	for(int32 ViewIndex = 0;ViewIndex < AllViews.Num();ViewIndex++)
	{
		FViewInfo& View = *AllViews[ViewIndex];
		FSceneViewState* ViewState = View.ViewState;

#if DO_CHECK || USING_CODE_ANALYSIS
		check(View.VerifyMembersChecks());
#endif

		// Setup global dither fade in and fade out uniform buffers.
		{
			FDitherUniformShaderParameters DitherUniformShaderParameters;
			DitherUniformShaderParameters.LODFactor = View.GetTemporalLODTransition();
			View.DitherFadeOutUniformBuffer = FDitherUniformBufferRef::CreateUniformBufferImmediate(DitherUniformShaderParameters, UniformBuffer_SingleFrame);

			DitherUniformShaderParameters.LODFactor = View.GetTemporalLODTransition() - 1.0f;
			View.DitherFadeInUniformBuffer = FDitherUniformBufferRef::CreateUniformBufferImmediate(DitherUniformShaderParameters, UniformBuffer_SingleFrame);
		}

		// Once per render increment the occlusion frame counter.
		if (ViewState)
		{
			ViewState->OcclusionFrameCounter++;
		}

		// HighResScreenshot should get best results so we don't do the occlusion optimization based on the former frame
		extern bool GIsHighResScreenshot;
		const bool bIsHitTesting = ViewFamily.EngineShowFlags.HitProxies;
		// Don't test occlusion queries in collision viewmode as they can be bigger then the rendering bounds.
		const bool bCollisionView = ViewFamily.EngineShowFlags.CollisionVisibility || ViewFamily.EngineShowFlags.CollisionPawn;
		if (GIsHighResScreenshot || !DoOcclusionQueries() || bIsHitTesting || bCollisionView)
		{
			View.bDisableQuerySubmissions = true;
			View.bIgnoreExistingQueries = true;
		}

		// set up the screen area for occlusion
		{
			float OcclusionPixelMultiplier = 1.0f;
			if (UseDownsampledOcclusionQueries())
			{
				OcclusionPixelMultiplier = 1.0f / static_cast<float>(FMath::Square(SceneTexturesConfig.SmallDepthDownsampleFactor));
			}
			float NumPossiblePixels = static_cast<float>(View.ViewRect.Width() * View.ViewRect.Height()) * OcclusionPixelMultiplier;
			View.OneOverNumPossiblePixels = NumPossiblePixels > 0.0 ? 1.0f / NumPossiblePixels : 0.0f;
		}

		// Still need no jitter to be set for temporal feedback on SSR (it is enabled even when temporal AA is off).
		check(View.TemporalJitterPixels.X == 0.0f);
		check(View.TemporalJitterPixels.Y == 0.0f);
		
		// Cache the projection matrix b		
		// Cache the projection matrix before AA is applied
		View.ViewMatrices.SaveProjectionNoAAMatrix();

		if (ViewState)
		{
			check(View.bStatePrevViewInfoIsReadOnly);
			View.bStatePrevViewInfoIsReadOnly = ViewFamily.bWorldIsPaused || ViewFamily.EngineShowFlags.HitProxies || bFreezeTemporalHistories;

			ViewState->SetupDistanceFieldTemporalOffset(ViewFamily);

			if (!View.bStatePrevViewInfoIsReadOnly && !bFreezeTemporalSequences)
			{
				ViewState->FrameIndex++;
			}

			if (View.OverrideFrameIndexValue.IsSet())
			{
				ViewState->FrameIndex = View.OverrideFrameIndexValue.GetValue();
			}

			if (View.OverrideOutputFrameIndexValue.IsSet())
			{
				ViewState->OutputFrameIndex = View.OverrideOutputFrameIndexValue.GetValue();
			}
			else
			{
				// If the output frame index isn't being overwritten then we keep it in sync with
				// FrameIndex so that downstream systems are unchanged.
				ViewState->OutputFrameIndex = ViewState->FrameIndex;
			}
		}
		
		// Subpixel jitter for temporal AA
		int32 CVarTemporalAASamplesValue = CVarTemporalAASamples.GetValueOnRenderThread();
		const bool bShouldScaleTemporalAASampleCount = (CVarTemporalAAScaleSamples.GetValueOnRenderThread() != 0);

		// Custom render passes support sharing temporal state with the main view
		FViewInfo& TemporalSourceView = View.TemporalSourceView ? *View.TemporalSourceView : View;
		FSceneViewState* TemporalViewState = TemporalSourceView.ViewState;

		EMainTAAPassConfig TAAConfig = GetMainTAAPassConfig(TemporalSourceView);

		bool bTemporalUpsampling = TemporalSourceView.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale;
		
		// Apply a sub pixel offset to the view.
		if (IsTemporalAccumulationBasedMethod(TemporalSourceView.AntiAliasingMethod) && TemporalViewState && (CVarTemporalAASamplesValue > 0 || bTemporalUpsampling) && TemporalSourceView.bAllowTemporalJitter)
		{
			float EffectivePrimaryResolutionFraction = float(View.ViewRect.Width()) / float(TemporalSourceView.GetSecondaryViewRectSize().X);

			// Compute number of TAA samples.
			int32 TemporalAASamples;
			{
				if (TAAConfig == EMainTAAPassConfig::TSR)
				{
					// Force the number of AA sample to make sure the quality doesn't get
					// compromised by previously set settings for Gen4 TAA
					TemporalAASamples = 8;
				}
				else
				{
					TemporalAASamples = FMath::Clamp(CVarTemporalAASamplesValue, 1, 255);
				}

				if (bTemporalUpsampling && bShouldScaleTemporalAASampleCount)
				{
					// When doing TAA upsample with screen percentage < 100%, we need extra temporal samples to have a
					// constant temporal sample density for final output pixels to avoid output pixel aligned converging issues.
					TemporalAASamples = FMath::RoundToInt(float(TemporalAASamples) * FMath::Max(1.f, 1.f / (EffectivePrimaryResolutionFraction * EffectivePrimaryResolutionFraction)));
				}
				else if (CVarTemporalAASamplesValue == 5)
				{
					TemporalAASamples = 4;
				}

				// Use immediately higher prime number to break up coherence between the TAA jitter sequence and any
				// other random signal that are power of two of View.StateFrameIndex
				if (TAAConfig == EMainTAAPassConfig::TSR)
				{
					static const uint8 kFirstPrimeNumbers[] = {
						2, 3, 5, 7, 11, 13, 17, 19, 23, 29, 31, 37, 41, 43, 47, 53, 59, 61, 67, 71, 73, 79, 83, 89, 97,
						101, 103, 107, 109, 113, 127, 131, 137, 139, 149, 151, 157, 163, 167, 173, 179, 181, 191, 193, 197, 199,
						211, 223, 227, 229, 233, 239, 241, 251,
					};

					for (int32 PrimeNumberId = FMath::Max(4, (TemporalAASamples - 1) / 5); PrimeNumberId < UE_ARRAY_COUNT(kFirstPrimeNumbers); PrimeNumberId++)
					{
						if (int32(kFirstPrimeNumbers[PrimeNumberId]) >= TemporalAASamples)
						{
							TemporalAASamples = int32(kFirstPrimeNumbers[PrimeNumberId]);
							break;
						}
					}
				}
			}

			// Compute the new sample index in the temporal sequence.
			int32 TemporalSampleIndex;
			if (View.TemporalSourceView)
			{
				// Where a TemporalSourceView is specified, the temporal sample index is pulled directly from the source view's state, and not incremented,
				// assuming the	source view already will have updated that.
				TemporalSampleIndex = TemporalViewState->TemporalAASampleIndex;
			}
			else
			{
				TemporalSampleIndex = TemporalViewState->TemporalAASampleIndex + 1;
				if (TemporalSampleIndex >= TemporalAASamples || View.bCameraCut)
				{
					TemporalSampleIndex = 0;
				}

				#if !UE_BUILD_SHIPPING
				if (CVarTAADebugOverrideTemporalIndex.GetValueOnRenderThread() >= 0)
				{
					TemporalSampleIndex = CVarTAADebugOverrideTemporalIndex.GetValueOnRenderThread();
				}
				#endif

				// Updates view state.
				if (!View.bStatePrevViewInfoIsReadOnly && !bFreezeTemporalSequences)
				{
					TemporalViewState->TemporalAASampleIndex = TemporalSampleIndex;
				}
			}

			// Choose sub pixel sample coordinate in the temporal sequence.
			float SampleX = 0.0f, SampleY = 0.0f;
			if (TemporalSourceView.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale)
			{
				// Uniformly distribute temporal jittering in [-.5; .5], because there is no longer any alignement of input and output pixels.
				SampleX = Halton(TemporalSampleIndex + 1, 2) - 0.5f;
				SampleY = Halton(TemporalSampleIndex + 1, 3) - 0.5f;

				// Adjust Automatic View Mip Bias
				float MipBias = -(FMath::Max(-FMath::Log2(EffectivePrimaryResolutionFraction), 0.0f)) + CVarAutomaticViewMipBiasOffset.GetValueOnRenderThread();
				MipBias = FMath::Clamp(FMath::Max(MipBias, CVarAutomaticViewMipBiasMin.GetValueOnRenderThread()), -4.0f, 4.0f);

				// Quantize to discrete values between -4.0f and 4.0f (range of 8.0f), since each value creates a unique sampler state
				const int32 MipBiasQuantization = CVarAutomaticViewMipBiasQuantization.GetValueOnRenderThread();

				if (MipBiasQuantization > 0)
				{
					// Divide the 8.0f range by the number of desired steps to get the step size
					const float QuantizationStepSize = 8.0f / ((float)MipBiasQuantization);
					MipBias = FMath::CeilToFloat(MipBias / QuantizationStepSize) * QuantizationStepSize;
				}
				
				View.MaterialTextureMipBias = MipBias;
			}
			else if( CVarTemporalAASamplesValue == 2 )
			{
				// 2xMSAA
				// Pattern docs: http://msdn.microsoft.com/en-us/library/windows/desktop/ff476218(v=vs.85).aspx
				//   N.
				//   .S
				float SamplesX[] = { -4.0f/16.0f, 4.0/16.0f };
				float SamplesY[] = { -4.0f/16.0f, 4.0/16.0f };
				check(TemporalAASamples == UE_ARRAY_COUNT(SamplesX));
				SampleX = SamplesX[ TemporalSampleIndex ];
				SampleY = SamplesY[ TemporalSampleIndex ];
			}
			else if( CVarTemporalAASamplesValue == 3 )
			{
				// 3xMSAA
				//   A..
				//   ..B
				//   .C.
				// Rolling circle pattern (A,B,C).
				float SamplesX[] = { -2.0f/3.0f,  2.0f/3.0f,  0.0f/3.0f };
				float SamplesY[] = { -2.0f/3.0f,  0.0f/3.0f,  2.0f/3.0f };
				check(TemporalAASamples == UE_ARRAY_COUNT(SamplesX));
				SampleX = SamplesX[ TemporalSampleIndex ];
				SampleY = SamplesY[ TemporalSampleIndex ];
			}
			else if( CVarTemporalAASamplesValue == 4 )
			{
				// 4xMSAA
				// Pattern docs: http://msdn.microsoft.com/en-us/library/windows/desktop/ff476218(v=vs.85).aspx
				//   .N..
				//   ...E
				//   W...
				//   ..S.
				// Rolling circle pattern (N,E,S,W).
				float SamplesX[] = { -2.0f/16.0f,  6.0/16.0f, 2.0/16.0f, -6.0/16.0f };
				float SamplesY[] = { -6.0f/16.0f, -2.0/16.0f, 6.0/16.0f,  2.0/16.0f };
				check(TemporalAASamples == UE_ARRAY_COUNT(SamplesX));
				SampleX = SamplesX[ TemporalSampleIndex ];
				SampleY = SamplesY[ TemporalSampleIndex ];
			}
			else if( CVarTemporalAASamplesValue == 5 )
			{
				// Compressed 4 sample pattern on same vertical and horizontal line (less temporal flicker).
				// Compressed 1/2 works better than correct 2/3 (reduced temporal flicker).
				//   . N .
				//   W . E
				//   . S .
				// Rolling circle pattern (N,E,S,W).
				float SamplesX[] = {  0.0f/2.0f,  1.0/2.0f,  0.0/2.0f, -1.0/2.0f };
				float SamplesY[] = { -1.0f/2.0f,  0.0/2.0f,  1.0/2.0f,  0.0/2.0f };
				check(TemporalAASamples == UE_ARRAY_COUNT(SamplesX));
				SampleX = SamplesX[ TemporalSampleIndex ];
				SampleY = SamplesY[ TemporalSampleIndex ];
			}
			else if(View.IsPerspectiveProjection())
			{
				float u1 = Halton( TemporalSampleIndex + 1, 2 );
				float u2 = Halton( TemporalSampleIndex + 1, 3 );

				// Generates samples in normal distribution
				// exp( x^2 / Sigma^2 )
					
				static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.TemporalAAFilterSize"));
				float FilterSize = CVar->GetFloat();

				// Scale distribution to set non-unit variance
				// Variance = Sigma^2
				float Sigma = 0.47f * FilterSize;

				// Window to [-0.5, 0.5] output
				// Without windowing we could generate samples far away on the infinite tails.
				float OutWindow = 0.5f;
				float InWindow = FMath::Exp( -0.5 * FMath::Square( OutWindow / Sigma ) );
					
				// Box-Muller transform
				float Theta = 2.0f * PI * u2;
				float r = Sigma * FMath::Sqrt( -2.0f * FMath::Loge( (1.0f - u1) * InWindow + u1 ) );
					
				SampleX = r * FMath::Cos( Theta );
				SampleY = r * FMath::Sin( Theta );
			}

			if (CVarInvertTemporalJitterX.GetValueOnRenderThread() != 0)
			{
				SampleX = -SampleX;
			}

			if (CVarInvertTemporalJitterY.GetValueOnRenderThread() != 0)
			{
				SampleY = -SampleY;
			}

			View.TemporalJitterSequenceLength = TemporalAASamples;
			View.TemporalJitterIndex = TemporalSampleIndex;
			View.TemporalJitterPixels.X = SampleX;
			View.TemporalJitterPixels.Y = SampleY;

			View.ViewMatrices.HackAddTemporalAAProjectionJitter(FVector2D(SampleX * 2.0f / View.ViewRect.Width(), SampleY * -2.0f / View.ViewRect.Height()));
		}
	}
}

void FSceneRenderer::PrepareViewStatePrevFrameViewInfo()
{
	// Load this field once so it has a consistent value for all views (and to avoid the atomic load in the loop).
	// While the value may not be perfectly in sync when we render other view families, this is ok as this
	// invalidation mechanism is only used for interactive rendering where we expect to be constantly drawing the scene.
	// Therefore it is acceptable for some view families to be a frame or so behind others.
	uint32 CurrentPathTracingInvalidationCounter = Scene->PathTracingInvalidationCounter.Load();

	for (int32 ViewIndex = 0; ViewIndex < AllViews.Num(); ViewIndex++)
	{
		FViewInfo& View = *AllViews[ViewIndex];
		FSceneViewState* ViewState = View.ViewState;

#if DO_CHECK || USING_CODE_ANALYSIS
		check(View.VerifyMembersChecks());
#endif

		// Setup a new FPreviousViewInfo from current frame infos.
		FPreviousViewInfo NewPrevViewInfo;
		{
			NewPrevViewInfo.ViewRect = View.ViewRect;
			NewPrevViewInfo.ViewMatrices = View.ViewMatrices;
		}

		if ( ViewState )
		{
			// update previous frame matrices in case world origin was rebased on this frame
			if (!View.OriginOffsetThisFrame.IsZero())
			{
				ViewState->PrevFrameViewInfo.ViewMatrices.ApplyWorldOffset(View.OriginOffsetThisFrame);
			}

			// determine if we are initializing or we should reset the persistent state
			const float DeltaTime = View.Family->Time.GetRealTimeSeconds() - ViewState->LastRenderTime;
			const bool bFirstFrameOrTimeWasReset = DeltaTime < -0.0001f || ViewState->LastRenderTime < 0.0001f;
			const bool bIsLargeCameraMovement = IsLargeCameraMovement(
				View,
				ViewState->PrevFrameViewInfo.ViewMatrices.GetWorldToView(),
				ViewState->PrevFrameViewInfo.ViewMatrices.GetViewOrigin(),
				75.0f, GCameraCutTranslationThreshold);
			const bool bResetCamera = (bFirstFrameOrTimeWasReset || View.bCameraCut || bIsLargeCameraMovement || View.bForceCameraVisibilityReset);
			
#if RHI_RAYTRACING
			static const auto CVarTemporalDenoiser = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PathTracing.TemporalDenoiser.mode"));
			const int TemporalDenoiserMode = CVarTemporalDenoiser ? CVarTemporalDenoiser->GetValueOnAnyThread() : 0;

			if (View.bIsOfflineRender)
			{
				// In the offline context, we want precise control over when to restart the path tracer's accumulation to allow for motion blur
				// So we use the camera cut signal only. In particular - we should not use bForceCameraVisibilityReset since this has
				// interactions with the motion blur post process effect in tiled rendering (see comment below).
				if (View.bCameraCut || View.bForcePathTracerReset)
				{
					const bool bClearTemporalDenoisingHistory = (TemporalDenoiserMode == 1) ? View.bCameraCut : true;
					ViewState->PathTracingInvalidate(bClearTemporalDenoisingHistory);
				}
			}
			else
			{
				// for interactive usage - any movement or scene change should restart the path tracer

				// Note: 0.18 deg is the minimum angle for avoiding numerical precision issue (which would cause constant invalidation)
				// The translation tolerance is because camera movement in the engine has a bit of falloff animation which takes a few seconds to fully settle.
				// Allowing the path tracer to ignore this imperceptible drift helps the path tracer start accumulating frames sooner.
				const bool bIsCameraMove = IsLargeCameraMovement(
					View,
					ViewState->PrevFrameViewInfo.ViewMatrices.GetWorldToView(),
					ViewState->PrevFrameViewInfo.ViewMatrices.GetViewOrigin(),
					0.18f /*degree*/, 0.001f /*cm*/);
				const bool bIsProjMatrixDifferent = View.ViewMatrices.GetViewToClipNoAA() != View.ViewState->PrevFrameViewInfo.ViewMatrices.GetViewToClipNoAA();

				// For each view, we remember what the invalidation counter was set to last time we were here so we can catch all changes
				const bool bNeedsInvalidation = ViewState->PathTracingInvalidationCounter != CurrentPathTracingInvalidationCounter;
				ViewState->PathTracingInvalidationCounter = CurrentPathTracingInvalidationCounter;
				if (bNeedsInvalidation ||
					bIsProjMatrixDifferent ||
					bIsCameraMove ||
					View.bCameraCut ||
					View.bForceCameraVisibilityReset ||
					View.bForcePathTracerReset)
				{
					const bool bClearTemporalDenoisingHistory = (TemporalDenoiserMode == 2) ? View.bCameraCut : true;
					ViewState->PathTracingInvalidate(bClearTemporalDenoisingHistory);
				}
			}

#endif // RHI_RAYTRACING

			if (bResetCamera)
			{
				View.PrevViewInfo = NewPrevViewInfo;

				// PT: If the motion blur shader is the last shader in the post-processing chain then it is the one that is
				//     adjusting for the viewport offset.  So it is always required and we can't just disable the work the
				//     shader does.  The correct fix would be to disable the effect when we don't need it and to properly mark
				//     the uber-postprocessing effect as the last effect in the chain.

				View.bPrevTransformsReset = true;
			}
			else
			{
				View.PrevViewInfo = ViewState->PrevFrameViewInfo;
			}

			// Replace previous view info of the view state with this frame, clearing out references over render target.
			if (!View.bStatePrevViewInfoIsReadOnly)
			{
				ViewState->PrevFrameViewInfo = NewPrevViewInfo;
			}

			// If the view has a previous view transform, then overwrite the previous view info for the _current_ frame.
			if (View.PreviousViewTransform.IsSet())
			{
				// Note that we must ensure this transform ends up in ViewState->PrevFrameViewInfo else it will be used to calculate the next frame's motion vectors as well
				View.PrevViewInfo.ViewMatrices.UpdateViewMatrix(View.PreviousViewTransform->GetTranslation(), View.PreviousViewTransform->GetRotation().Rotator());
			}

			// detect conditions where we should reset occlusion queries
			if (bFirstFrameOrTimeWasReset || 
				ViewState->LastRenderTime + GEngine->PrimitiveProbablyVisibleTime < View.Family->Time.GetRealTimeSeconds() ||
				View.bCameraCut ||
				View.bForceCameraVisibilityReset ||
				IsLargeCameraMovement(
					View, 
				    FMatrix(ViewState->PrevViewMatrixForOcclusionQuery), 
				    ViewState->PrevViewOriginForOcclusionQuery, 
				    GEngine->CameraRotationThreshold, GEngine->CameraTranslationThreshold))
			{
				View.bIgnoreExistingQueries = true;
				View.bDisableDistanceBasedFadeTransitions = true;
			}

			// Turn on/off round-robin occlusion querying in the ViewState
			static const auto CVarRROCC = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("vr.RoundRobinOcclusion"));
			const bool bEnableRoundRobin = CVarRROCC ? (CVarRROCC->GetValueOnAnyThread() != false) : false;
			if (bEnableRoundRobin != ViewState->IsRoundRobinEnabled())
			{
				ViewState->UpdateRoundRobin(bEnableRoundRobin);
				View.bIgnoreExistingQueries = true;
			}

			ViewState->PrevViewMatrixForOcclusionQuery = FMatrix44f(View.ViewMatrices.GetWorldToView());	// LWC_TODO: Precision loss
			ViewState->PrevViewOriginForOcclusionQuery = View.ViewMatrices.GetViewOrigin();

			// we don't use DeltaTime as it can be 0 (in editor) and is computed by subtracting floats (loses precision over time)
			// Clamp DeltaWorldTime to reasonable values for the purposes of motion blur, things like TimeDilation can make it very small
			// Offline renders always control the timestep for the view and always need the timescales calculated.
			if (!ViewFamily.bWorldIsPaused || View.bIsOfflineRender)
			{
				ViewState->UpdateMotionBlurTimeScale(View);
			}
			

			ViewState->PrevFrameNumber = ViewState->PendingPrevFrameNumber;
			ViewState->PendingPrevFrameNumber = View.Family->FrameNumber;

			// This finishes the update of view state
			ViewState->UpdateLastRenderTime(*View.Family);

			ViewState->UpdateTemporalLODTransition(View);
		}
		else
		{
			// Without a viewstate, we just assume that camera has not moved.
			View.PrevViewInfo = NewPrevViewInfo;
		}
	}
}

void FSceneViewState::UpdateMotionBlurTimeScale(const FViewInfo& View)
{
	const int32 MotionBlurTargetFPS = View.FinalPostProcessSettings.MotionBlurTargetFPS;

	// Ensure we can divide by the Delta Time later without a divide by zero.
	float DeltaRealTime = FMath::Max(View.Family->Time.GetDeltaRealTimeSeconds(), SMALL_NUMBER);

	// Track the current FPS by using an exponential moving average of the current delta time.
	if (MotionBlurTargetFPS <= 0)
	{
		// Keep motion vector lengths stable for paused sequencer frames.
		if (GetSequencerState() == ESS_Paused)
		{
			// Reset the moving average to the current delta time.
			MotionBlurTargetDeltaTime = DeltaRealTime;
		}
		else
		{
			// Smooth the target delta time using a moving average.
			MotionBlurTargetDeltaTime = FMath::Lerp(MotionBlurTargetDeltaTime, DeltaRealTime, 0.1f);
		}
	}
	else // Track a fixed target FPS.
	{
		// Keep motion vector lengths stable for paused sequencer frames. Assumes a 60 FPS tick.
		// Tuned for content compatibility with existing content when target is the default 30 FPS.
		if (GetSequencerState() == ESS_Paused)
		{
			DeltaRealTime = 1.0f / 60.0f;
		}


		MotionBlurTargetDeltaTime = 1.0f / static_cast<float>(MotionBlurTargetFPS);
	}

	MotionBlurTimeScale = MotionBlurTargetDeltaTime / DeltaRealTime;
}

void FDeferredShadingSceneRenderer::ComputeLightVisibility()
{
	FSceneRenderer::ComputeLightVisibility();

	CreateIndirectCapsuleShadows();

	SetupVolumetricFog();
}

void FSceneRenderer::ComputeLightVisibility()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PostVisibilityFrameSetup_Light_Visibility);

	VisibleLightInfos.AddDefaulted(Scene->Lights.GetMaxIndex());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];
		View.VisibleLightInfos.Empty(Scene->Lights.GetMaxIndex());

		for (int32 LightIndex = 0; LightIndex < Scene->Lights.GetMaxIndex(); LightIndex++)
		{
			new (View.VisibleLightInfos) FVisibleLightViewInfo();
		}
	}

	const bool bSetupMobileLightShafts = FeatureLevel <= ERHIFeatureLevel::ES3_1 && ShouldRenderLightShafts(ViewFamily);

	// determine visibility of each light
	for(auto LightIt = Scene->Lights.CreateConstIterator();LightIt;++LightIt)
	{
		const FLightSceneInfoCompact& LightSceneInfoCompact = *LightIt;
		const FLightSceneInfo* LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;

		// view frustum cull lights in each view
		for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
		{		
			const FLightSceneProxy* Proxy = LightSceneInfo->Proxy;
			FViewInfo& View = Views[ViewIndex];
			FVisibleLightViewInfo& VisibleLightViewInfo = View.VisibleLightInfos[LightIt.GetIndex()];
			// dir lights are always visible, and point/spot only if in the frustum
			if( Proxy->GetLightType() == LightType_Point ||
				Proxy->GetLightType() == LightType_Spot ||
				Proxy->GetLightType() == LightType_Rect )
			{
				const FSphere& BoundingSphere = Proxy->GetBoundingSphere();
				const bool bInViewFrustum = View.GetCullingFrustum().IntersectSphere(BoundingSphere.Center, BoundingSphere.W);

				if (View.IsPerspectiveProjection())
				{
					const float DistanceSquared = (BoundingSphere.Center - View.CullingOrigin).SizeSquared();
					const float ProxyMaxDistance = Proxy->GetMaxDrawDistance();
					const float ScaledMaxDistance = ProxyMaxDistance * GLightMaxDrawDistanceScale;
					const bool bDrawLight = (FMath::Square(FMath::Min(0.0002f, GMinScreenRadiusForLights / BoundingSphere.W) * View.LODDistanceFactor) * DistanceSquared < 1.0f)
												&& (ProxyMaxDistance <= 0.0 || DistanceSquared < FMath::Square(ScaledMaxDistance));
							
					VisibleLightViewInfo.bInViewFrustum = bDrawLight && bInViewFrustum;
					VisibleLightViewInfo.bInDrawRange = bDrawLight;
				}
				else
				{
					VisibleLightViewInfo.bInViewFrustum = bInViewFrustum;
					VisibleLightViewInfo.bInDrawRange = true;
				}
			}
			else
			{
				VisibleLightViewInfo.bInViewFrustum = true;
				VisibleLightViewInfo.bInDrawRange = true;
				// Setup single sun-shaft from direction lights for mobile.
				if (bSetupMobileLightShafts && LightSceneInfo->bEnableLightShaftBloom && ShouldRenderLightShaftsForLight(View, *LightSceneInfo->Proxy))
				{
					View.MobileLightShaft = GetMobileLightShaftInfo(View, *LightSceneInfo);
				}
			}

			// Draw shapes for reflection captures
			if( View.bIsReflectionCapture 
				&& VisibleLightViewInfo.bInViewFrustum
				&& Proxy->HasStaticLighting() 
				&& Proxy->GetLightType() != LightType_Directional )
			{
				FVector Origin = Proxy->GetOrigin();
				FVector ToLight = Origin - View.ViewMatrices.GetViewOrigin();
				float DistanceSqr = ToLight | ToLight;
				float Radius = Proxy->GetRadius();

				if( DistanceSqr < Radius * Radius )
				{
					View.VisibleReflectionCaptureLights.Emplace(Proxy);
				}
			}
		}
	}

	InitFogConstants();
}

void FSceneRenderer::GatherReflectionCaptureLightMeshElements()
{
	// view frustum cull lights in each view
	for (FViewInfo& View : Views)
	{
		for (const FLightSceneProxy* Proxy : View.VisibleReflectionCaptureLights)
		{
			FVector Origin = Proxy->GetOrigin();
			FVector ToLight = Origin - View.ViewMatrices.GetViewOrigin();
			float DistanceSqr = ToLight | ToLight;
			float Radius = Proxy->GetRadius();

			FLightRenderParameters LightParameters;
			Proxy->GetLightShaderParameters(LightParameters);

			// Force to be at least 0.75 pixels
			float CubemapSize = (float)IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ReflectionCaptureResolution"))->GetValueOnAnyThread();
			float Distance = FMath::Sqrt(DistanceSqr);
			float MinRadius = Distance * 0.75f / CubemapSize;
			LightParameters.SourceRadius = FMath::Max(MinRadius, LightParameters.SourceRadius);

			// Snap to cubemap pixel center to reduce aliasing
			FVector Scale = ToLight.GetAbs();
			int32 MaxComponent = Scale.X > Scale.Y ? (Scale.X > Scale.Z ? 0 : 2) : (Scale.Y > Scale.Z ? 1 : 2);
			for (int32 k = 1; k < 3; k++)
			{
				float Projected = ToLight[(MaxComponent + k) % 3] / Scale[MaxComponent];
				float Quantized = (FMath::RoundToFloat(Projected * (0.5f * CubemapSize) - 0.5f) + 0.5f) / (0.5f * CubemapSize);
				ToLight[(MaxComponent + k) % 3] = Quantized * Scale[MaxComponent];
			}
			Origin = ToLight + View.ViewMatrices.GetViewOrigin();

			FLinearColor Color(LightParameters.Color.R, LightParameters.Color.G, LightParameters.Color.B, LightParameters.FalloffExponent);
			const bool bIsRectLight = Proxy->IsRectLight();
			if (!bIsRectLight)
			{
				const float SphereArea = (4.0f * PI) * FMath::Square(LightParameters.SourceRadius);
				const float CylinderArea = (2.0f * PI) * LightParameters.SourceRadius * LightParameters.SourceLength;
				const float SurfaceArea = SphereArea + CylinderArea;
				Color *= 4.0f / SurfaceArea;
			}

			if (Proxy->IsInverseSquared())
			{
				float LightRadiusMask = FMath::Square(1.0f - FMath::Square(DistanceSqr * FMath::Square(LightParameters.InvRadius)));
				Color.A = LightRadiusMask;
			}
			else
			{
				// Remove inverse square falloff
				Color *= DistanceSqr + 1.0f;

				// Apply falloff
				Color.A = FMath::Pow(1.0f - DistanceSqr * FMath::Square(LightParameters.InvRadius), LightParameters.FalloffExponent);
			}

			// Spot falloff
			FVector L = ToLight.GetSafeNormal();
			Color.A *= FMath::Square(FMath::Clamp(((L | (FVector)LightParameters.Direction) - LightParameters.SpotAngles.X) * LightParameters.SpotAngles.Y, 0.0f, 1.0f));

			Color.A *= LightParameters.SpecularScale;

			// Rect is one sided
			if (bIsRectLight && (L | (FVector)LightParameters.Direction) < 0.0f)
				continue;

			UTexture* SurfaceTexture = nullptr;
			if (bIsRectLight)
			{
				const FRectLightSceneProxy* RectLightProxy = (const FRectLightSceneProxy*)Proxy;
				SurfaceTexture = RectLightProxy->SourceTexture;
			}

			FMaterialRenderProxy* ColoredMeshInstance = nullptr;
			if (SurfaceTexture)
			{
				ColoredMeshInstance = Allocator.Create<FColoredTexturedMaterialRenderProxy>(GEngine->EmissiveMeshMaterial->GetRenderProxy(), Color, NAME_Color, SurfaceTexture, NAME_LinearColor);
			}
			else
			{
				ColoredMeshInstance = Allocator.Create<FColoredMaterialRenderProxy>(GEngine->EmissiveMeshMaterial->GetRenderProxy(), Color, NAME_Color);
			}

			FMatrix LightToWorld = Proxy->GetLightToWorld();
			LightToWorld.RemoveScaling();

			FViewElementPDI LightPDI(&View, NULL, &View.DynamicPrimitiveCollector);

			if (bIsRectLight)
			{
				DrawBox(&LightPDI, LightToWorld, FVector(0.0f, LightParameters.SourceRadius, LightParameters.SourceLength), ColoredMeshInstance, SDPG_World);
			}
			else if (LightParameters.SourceLength > 0.0f)
			{
				DrawSphere(&LightPDI, Origin + 0.5f * LightParameters.SourceLength * LightToWorld.GetUnitAxis(EAxis::Z), FRotator::ZeroRotator, LightParameters.SourceRadius * FVector::OneVector, 36, 24, ColoredMeshInstance, SDPG_World);
				DrawSphere(&LightPDI, Origin - 0.5f * LightParameters.SourceLength * LightToWorld.GetUnitAxis(EAxis::Z), FRotator::ZeroRotator, LightParameters.SourceRadius * FVector::OneVector, 36, 24, ColoredMeshInstance, SDPG_World);
				DrawCylinder(&LightPDI, Origin, LightToWorld.GetUnitAxis(EAxis::X), LightToWorld.GetUnitAxis(EAxis::Y), LightToWorld.GetUnitAxis(EAxis::Z), LightParameters.SourceRadius, 0.5f * LightParameters.SourceLength, 36, ColoredMeshInstance, SDPG_World);
			}
			else
			{
				DrawSphere(&LightPDI, Origin, FRotator::ZeroRotator, LightParameters.SourceRadius * FVector::OneVector, 36, 24, ColoredMeshInstance, SDPG_World);
			}
		}

		View.VisibleReflectionCaptureLights.Empty();
	}
}

void FSceneRenderer::PostVisibilityFrameSetup(FILCUpdatePrimTaskData*& OutILCTaskData)
{
	if (GetRendererOutput() != ERendererOutput::FinalSceneColor)
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_PostVisibilityFrameSetup);

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_PostVisibilityFrameSetup_Sort);
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{		
			FViewInfo& View = Views[ViewIndex];

			if (View.State)
			{
				((FSceneViewState*)View.State)->TrimHistoryRenderTargets(Scene);
			}
		}
	}

	GatherReflectionCaptureLightMeshElements();

	if (ViewFamily.EngineShowFlags.HitProxies == 0 && Scene->PrecomputedLightVolumes.Num() > 0
		&& GILCUpdatePrimTaskEnabled && FPlatformProcess::SupportsMultithreading())
	{
		OutILCTaskData = Allocator.Create<FILCUpdatePrimTaskData>();
		Scene->IndirectLightingCache.StartUpdateCachePrimitivesTask(Scene, *this, true, *OutILCTaskData);
		check(OutILCTaskData->TaskRef.IsValid());
	}
}

uint32 GetShadowQuality();
void UpdateHairResources(FRDGBuilder& GraphBuilder, const FViewInfo& View);

/** 
* Performs once per frame setup prior to visibility determination.
*/
void FDeferredShadingSceneRenderer::PreVisibilityFrameSetup(FRDGBuilder& GraphBuilder)
{
	

	FSceneRenderer::PreVisibilityFrameSetup(GraphBuilder);
}

/**
 * Initialize scene's views.
 * Check visibility, build visible mesh commands, etc.
 */
void FDeferredShadingSceneRenderer::BeginInitViews(
	FRDGBuilder& GraphBuilder,
	const FSceneTexturesConfig& SceneTexturesConfig,
	FInstanceCullingManager& InstanceCullingManager,
	FRDGExternalAccessQueue& ExternalAccessQueue,
	FInitViewTaskDatas& TaskDatas)
{
	SCOPED_NAMED_EVENT(FDeferredShadingSceneRenderer_InitViews, FColor::Emerald);
	SCOPE_CYCLE_COUNTER(STAT_InitViewsTime);
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, InitViews_Scene);

	const bool bRendererOutputFinalSceneColor = (GetRendererOutput() == ERendererOutput::FinalSceneColor);

	PreVisibilityFrameSetup(GraphBuilder);

	{
		// This is to init the ViewUniformBuffer before rendering for the Niagara compute shader.
		// This needs to run before ComputeViewVisibility() is called, but the views normally initialize the ViewUniformBuffer after that (at the end of this method).
		// Note: This MUST happen before we start to execute GDME tasks of any kind otherwise we will race with PostInitViews renderers!
		if (FXSystem && FXSystem->RequiresEarlyViewUniformBuffer() && Views.IsValidIndex(0) && bRendererOutputFinalSceneColor)
		{
			// during ISR, instanced view RHI resources need to be initialized first.
			if (FViewInfo* InstancedView = const_cast<FViewInfo*>(Views[0].GetInstancedView()))
			{
				InstancedView->InitRHIResources();
			}
			Views[0].InitRHIResources();
			FXSystem->PostInitViews(GraphBuilder, GetSceneViews(), !ViewFamily.EngineShowFlags.HitProxies);
		}
	}

	// Start processing dynamic mesh elements tasks early enough to overlap with shadows and GPU scene update.
	TaskDatas.VisibilityTaskData->StartGatherDynamicMeshElements();

#if RHI_RAYTRACING
	// Start processing dynamic ray tracing instances early enough to overlap with shadows and GPU scene update.
	if (TaskDatas.RayTracingGatherInstances != nullptr)
	{
		RayTracing::BeginGatherDynamicRayTracingInstances(*TaskDatas.RayTracingGatherInstances);
	}
#endif

	TaskDatas.Decals = FDecalVisibilityTaskData::Launch(GraphBuilder, *Scene, AllViews);

	// Attempt to launch dynamic shadow tasks early before finalizing visibility.
	if (bRendererOutputFinalSceneColor)
	{
		BeginInitDynamicShadows(GraphBuilder, TaskDatas, InstanceCullingManager);
	}

	FRHICommandListImmediate& RHICmdList = GraphBuilder.RHICmdList;

	if (InstanceCullingManager.IsEnabled()
		&& Scene->InstanceCullingOcclusionQueryRenderer
		&& Scene->InstanceCullingOcclusionQueryRenderer->InstanceOcclusionQueryBuffer)
	{
		InstanceCullingManager.InstanceOcclusionQueryBuffer = GraphBuilder.RegisterExternalBuffer(Scene->InstanceCullingOcclusionQueryRenderer->InstanceOcclusionQueryBuffer);
		InstanceCullingManager.InstanceOcclusionQueryBufferFormat = Scene->InstanceCullingOcclusionQueryRenderer->InstanceOcclusionQueryBufferFormat;
	}

	LumenScenePDIVisualization(TaskDatas.LumenFrameTemporaries);

	// This must happen before we start initialising and using views.
	UpdateSkyIrradianceGpuBuffer(GraphBuilder, ViewFamily.EngineShowFlags, Scene->SkyLight, Scene->SkyIrradianceEnvironmentMap);

	// Initialise Sky/View resources before the view global uniform buffer is built.
	if (ShouldRenderSkyAtmosphere(Scene, ViewFamily.EngineShowFlags))
	{
		InitSkyAtmosphereForViews(RHICmdList, GraphBuilder);
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_InitViews_InitRHIResources);
		// initialize per-view uniform buffer. Do it from back to front because secondary stereo view follows its primary one, but primary needs to know the instanced's params
		for (int32 ViewIndex = Views.Num() - 1; ViewIndex >= 0; --ViewIndex)
		{
			FViewInfo& View = Views[ViewIndex];
			// Set the pre-exposure before initializing the constant buffers.
			View.UpdatePreExposure();

			// Initialize the view's RHI resources.
			UpdateHairResources(GraphBuilder, View);
			View.InitRHIResources();
		}

		for (FCustomRenderPassInfo& PassInfo : CustomRenderPassInfos)
		{
			for (FViewInfo& View : PassInfo.Views)
			{
				View.InitRHIResources();
			}
		}
	}

	TaskDatas.VisibilityTaskData->ProcessRenderThreadTasks();

	// Make a second attempt to launch shadow tasks it wasn't able to the first time due to visibility being deferred.
	if (bRendererOutputFinalSceneColor)
	{
		BeginInitDynamicShadows(GraphBuilder, TaskDatas, InstanceCullingManager);
	}

	PostVisibilityFrameSetup(TaskDatas.ILCUpdatePrim);
}

void FSceneRenderer::SetupSceneReflectionCaptureBuffer(FRHICommandListImmediate& RHICmdList)
{
	const TArray<FReflectionCaptureSortData>& SortedCaptures = Scene->ReflectionSceneData.SortedCaptures;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];

		if (IsMobilePlatform(ShaderPlatform))
		{
			View.MobileReflectionCaptureUniformBuffer = Scene->ReflectionSceneData.MobileReflectionCaptureUniformBuffer;
		}
		else
		{
			View.ReflectionCaptureUniformBuffer = Scene->ReflectionSceneData.ReflectionCaptureUniformBuffer;
		}
		
		View.NumBoxReflectionCaptures = 0;
		View.NumSphereReflectionCaptures = 0;
		View.FurthestReflectionCaptureDistance = 0.0f;

		if (View.Family->EngineShowFlags.ReflectionEnvironment 
			// Avoid feedback
			&& !View.bIsReflectionCapture)
		{
			View.NumBoxReflectionCaptures = Scene->ReflectionSceneData.NumBoxCaptures;
			View.NumSphereReflectionCaptures = Scene->ReflectionSceneData.NumSphereCaptures;

			for (int32 CaptureIndex = 0; CaptureIndex < SortedCaptures.Num(); CaptureIndex++)
			{
				const FSphere BoundingSphere(SortedCaptures[CaptureIndex].Position.GetVector3d(), SortedCaptures[CaptureIndex].Radius);

				const float Distance = View.ViewMatrices.GetWorldToView().TransformPosition(BoundingSphere.Center).Z + BoundingSphere.W;

				View.FurthestReflectionCaptureDistance = FMath::Max(View.FurthestReflectionCaptureDistance, Distance);
			}
		}
	}
}

void FDeferredShadingSceneRenderer::EndInitViews(
	FRDGBuilder& GraphBuilder,
	FLumenSceneFrameTemporaries& FrameTemporaries,
	FInstanceCullingManager& InstanceCullingManager,
	FInitViewTaskDatas& TaskDatas)
{
	SCOPED_NAMED_EVENT(FDeferredShadingSceneRenderer_InitViewsAfterPrepass, FColor::Emerald);
	SCOPE_CYCLE_COUNTER(STAT_InitViewsPossiblyAfterPrepass);

	TaskDatas.VisibilityTaskData->Finish();

	// Trigger shadow GDME tasks after the main visibility tasks are synced. Projection stencil shadows reference the main view dynamic elements.
	BeginShadowGatherDynamicMeshElements(TaskDatas.DynamicShadows);

	FRHICommandListImmediate& RHICmdList = GraphBuilder.RHICmdList;

	// If parallel ILC update is disabled, then process it in place.
	if (ViewFamily.EngineShowFlags.HitProxies == 0
		&& Scene->PrecomputedLightVolumes.Num() > 0
		&& !(GILCUpdatePrimTaskEnabled && FPlatformProcess::SupportsMultithreading()))
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_PostVisibilityFrameSetup_IndirectLightingCache_Update);
		check(!TaskDatas.ILCUpdatePrim);
		Scene->IndirectLightingCache.UpdateCache(RHICmdList, Scene, *this, true);
	}

	// If we kicked off ILC update via task, wait and finalize.
	if (TaskDatas.ILCUpdatePrim)
	{
		Scene->IndirectLightingCache.FinalizeCacheUpdates(RHICmdList, Scene, *this, *TaskDatas.ILCUpdatePrim);
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_InitViews_UpdatePrimitiveIndirectLightingCacheBuffers);
		// Now that the indirect lighting cache is updated, we can update the primitive precomputed lighting buffers.
		UpdatePrimitiveIndirectLightingCacheBuffers(GraphBuilder.RHICmdList);
	}

	SeparateTranslucencyDimensions = UpdateSeparateTranslucencyDimensions(*this);

	SetupSceneReflectionCaptureBuffer(RHICmdList);

	if (IsForwardShadingEnabled(ShaderPlatform))
	{
		// Dynamic shadows are synced earlier when forward shading is enabled.
		FinishInitDynamicShadows(GraphBuilder, TaskDatas.DynamicShadows, InstanceCullingManager);
	}
}

/*------------------------------------------------------------------------------
	FLODSceneTree Implementation
------------------------------------------------------------------------------*/
void FLODSceneTree::AddChildNode(const FPrimitiveComponentId ParentId, FPrimitiveSceneInfo* ChildSceneInfo)
{
	if (ParentId.IsValid() && ChildSceneInfo)
	{
		FLODSceneNode* Parent = SceneNodes.Find(ParentId);

		// If parent SceneNode hasn't been created yet (possible, depending on the order actors are added to the scene)
		if (!Parent)
		{
			// Create parent SceneNode, assign correct SceneInfo
			Parent = &SceneNodes.Add(ParentId, FLODSceneNode());

			FPrimitiveSceneInfo* ParentSceneInfo = Scene->GetPrimitiveSceneInfo(ParentId);
			if (ParentSceneInfo)
			{
				Parent->SceneInfo = ParentSceneInfo;
				check(Parent->SceneInfo->PrimitiveComponentId == ParentId);
			}
		}

		Parent->AddChild(ChildSceneInfo);
	}
}

void FLODSceneTree::RemoveChildNode(const FPrimitiveComponentId ParentId, FPrimitiveSceneInfo* ChildSceneInfo)
{
	if (ParentId.IsValid() && ChildSceneInfo)
	{
		if (FLODSceneNode* Parent = SceneNodes.Find(ParentId))
		{
			Parent->RemoveChild(ChildSceneInfo);

			// Delete from scene if no children remain
			if (Parent->ChildrenSceneInfos.Num() == 0)
			{
				SceneNodes.Remove(ParentId);
			}
		}
	}
}

void FLODSceneTree::UpdateNodeSceneInfo(FPrimitiveComponentId NodeId, FPrimitiveSceneInfo* SceneInfo)
{
	if (FLODSceneNode* Node = SceneNodes.Find(NodeId))
	{
		Node->SceneInfo = SceneInfo;
	}
}

void FLODSceneTree::ClearVisibilityState(FViewInfo& View)
{
	if (FSceneViewState* ViewState = (FSceneViewState*)View.State)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Skip update logic when frozen
		if (ViewState->bIsFrozen)
		{
			return;
		}
#endif
		FHLODVisibilityState& HLODState = ViewState->HLODVisibilityState;

		if(HLODState.IsValidPrimitiveIndex(0))
		{
			HLODState.PrimitiveFadingLODMap.Empty(0);
			HLODState.PrimitiveFadingOutLODMap.Empty(0);
			HLODState.ForcedVisiblePrimitiveMap.Empty(0);
			HLODState.ForcedHiddenPrimitiveMap.Empty(0);
		}

		TMap<FPrimitiveComponentId, FHLODSceneNodeVisibilityState>& VisibilityStates = ViewState->HLODSceneNodeVisibilityStates;

		if(VisibilityStates.Num() > 0)
		{
			VisibilityStates.Empty(0);
		}
	}
}

void FLODSceneTree::UpdateVisibilityStates(FViewInfo& View, UE::Tasks::FTaskEvent& FlushCachedShadowsTaskEvent)
{
	if (FSceneViewState* ViewState = (FSceneViewState*)View.State)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Skip update logic when frozen
		if (ViewState->bIsFrozen)
		{
			return;
		}
#endif
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ViewVisibilityTime_HLODUpdate);

		// Per-frame initialization
		FHLODVisibilityState& HLODState = ViewState->HLODVisibilityState;
		TMap<FPrimitiveComponentId, FHLODSceneNodeVisibilityState>& VisibilityStates = ViewState->HLODSceneNodeVisibilityStates;

		HLODState.PrimitiveFadingLODMap.Init(false, Scene->Primitives.Num());
		HLODState.PrimitiveFadingOutLODMap.Init(false, Scene->Primitives.Num());
		HLODState.ForcedVisiblePrimitiveMap.Init(false, Scene->Primitives.Num());
		HLODState.ForcedHiddenPrimitiveMap.Init(false, Scene->Primitives.Num());
		TArray<FPrimitiveViewRelevance, SceneRenderingAllocator>& RelevanceMap = View.PrimitiveViewRelevanceMap;

		if (HLODState.PrimitiveFadingLODMap.Num() != Scene->Primitives.Num())
		{
			checkf(0, TEXT("HLOD update incorrectly allocated primitive maps"));
			return;
		}

		TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> FlushCachedShadowPrimitives;

		int32 UpdateCount = ++HLODState.UpdateCount;

		// Update persistent state on temporal dither sync frames
		const FTemporalLODState& LODState = ViewState->GetTemporalLODState();
		bool bSyncFrame = false;
		
		if (HLODState.TemporalLODSyncTime != LODState.TemporalLODTime[0])
		{
			HLODState.TemporalLODSyncTime = LODState.TemporalLODTime[0];
			bSyncFrame = true;

			// Only update our scaling on sync frames else we might end up changing transition direction mid-fade	
			const FCachedSystemScalabilityCVars& ScalabilityCVars = GetCachedScalabilityCVars();
			if (ScalabilityCVars.FieldOfViewAffectsHLOD)
			{
				HLODState.FOVDistanceScaleSq = ScalabilityCVars.CalculateFieldOfViewDistanceScale(View.DesiredFOV);
				HLODState.FOVDistanceScaleSq *= HLODState.FOVDistanceScaleSq;
			}
			else
			{
				HLODState.FOVDistanceScaleSq = 1.f;
			}
		}

		for (auto Iter = SceneNodes.CreateIterator(); Iter; ++Iter)
		{
			FLODSceneNode& Node = Iter.Value();
			FPrimitiveSceneInfo* SceneInfo = Node.SceneInfo;

			if (!SceneInfo || !SceneInfo->PrimitiveComponentId.IsValid() || !SceneInfo->IsIndexValid())
			{
				continue;
			}

			FHLODSceneNodeVisibilityState& NodeVisibility = VisibilityStates.FindOrAdd(SceneInfo->PrimitiveComponentId);
			const TArray<FStaticMeshBatchRelevance>& NodeMeshRelevances = SceneInfo->StaticMeshRelevances;

			// Ignore already updated nodes, or those that we can't work with
			if (NodeVisibility.UpdateCount == UpdateCount || !NodeMeshRelevances.IsValidIndex(0))
			{
				continue;
			}

			const int32 NodeIndex = SceneInfo->GetIndex();

			if (!Scene->PrimitiveBounds.IsValidIndex(NodeIndex))
			{
				checkf(0, TEXT("A HLOD Node's PrimitiveSceneInfo PackedIndex was out of Scene.Primitive bounds!"));
				continue;
			}

			FPrimitiveBounds& Bounds = Scene->PrimitiveBounds[NodeIndex];
			const bool bForcedIntoView = FMath::IsNearlyZero(Bounds.MinDrawDistance);

			// Update visibility states of this node and owned children
			float ClosestDistSquared, FurthestDistSquared;

			if (GDistanceCullToSphereEdge)
			{
				ComputeDistances(Bounds, View.ViewMatrices.GetViewOrigin(), ClosestDistSquared, FurthestDistSquared);
			}
			else
			{
				ClosestDistSquared = FurthestDistSquared = Bounds.BoxSphereBounds.ComputeSquaredDistanceFromBoxToPoint(View.ViewMatrices.GetViewOrigin());
			}

			const bool bNearCulled = FurthestDistSquared  < FMath::Square(Bounds.MinDrawDistance) * HLODState.FOVDistanceScaleSq;
			const bool bFarCulled = ClosestDistSquared > Bounds.MaxDrawDistance * Bounds.MaxDrawDistance * HLODState.FOVDistanceScaleSq;

			const bool bIsInDrawRange = !bNearCulled && !bFarCulled;

			const bool bWasFadingPreUpdate = !!NodeVisibility.bIsFading;
			const bool bIsDitheredTransition = NodeMeshRelevances[0].bDitheredLODTransition;

			if (bIsDitheredTransition && !bForcedIntoView)
			{		
				// Update fading state with syncs
				if (bSyncFrame)
				{
					// Fade when HLODs change threshold
					const bool bChangedRange = bIsInDrawRange != !!NodeVisibility.bWasVisible;

					if (NodeVisibility.bIsFading)
					{
						NodeVisibility.bIsFading = false;
					}
					else if (bChangedRange)
					{
						NodeVisibility.bIsFading = true;
					}

					NodeVisibility.bWasVisible = NodeVisibility.bIsVisible;
					NodeVisibility.bIsVisible = bIsInDrawRange;
				}
			}
			else
			{
				// Instant transitions without dithering
				NodeVisibility.bWasVisible = NodeVisibility.bIsVisible;
				NodeVisibility.bIsVisible = bIsInDrawRange || bForcedIntoView;
				NodeVisibility.bIsFading = false;
			}

			// Flush cached lighting data when changing visible contents
			if (NodeVisibility.bIsVisible != NodeVisibility.bWasVisible || bWasFadingPreUpdate || NodeVisibility.bIsFading)
			{
				FlushCachedShadowPrimitives.Emplace(SceneInfo);
			}

			// Force fully disabled view relevance so shadows don't attempt to recompute
			if (!NodeVisibility.bIsVisible)
			{
				if (RelevanceMap.IsValidIndex(NodeIndex))
				{
					FPrimitiveViewRelevance& ViewRelevance = RelevanceMap[NodeIndex];
					FMemory::Memzero(&ViewRelevance, sizeof(FPrimitiveViewRelevance));
					ViewRelevance.bInitializedThisFrame = true;
				}
				else
				{
					checkf(0, TEXT("A HLOD Node's PrimitiveSceneInfo PackedIndex was out of View.Relevancy bounds!"));
				}
			}

			// NOTE: We update our children last as HideNodeChildren can add new visibility
			// states, potentially invalidating our cached reference above, NodeVisibility
			if (NodeVisibility.bIsFading)
			{
				// Fade until state back in sync
				HLODState.PrimitiveFadingLODMap[NodeIndex] = true;
				HLODState.PrimitiveFadingOutLODMap[NodeIndex] = !NodeVisibility.bIsVisible;
				HLODState.ForcedVisiblePrimitiveMap[NodeIndex] = true;
				ApplyNodeFadingToChildren(ViewState, Node, NodeVisibility, true, !!NodeVisibility.bIsVisible);
			}
			else if (NodeVisibility.bIsVisible)
			{
				// If stable and visible, override hierarchy visibility
				HLODState.ForcedVisiblePrimitiveMap[NodeIndex] = true;
				HideNodeChildren(ViewState, Node);
			}
			else
			{
				// Not visible and waiting for a transition to fade, keep HLOD hidden
				HLODState.ForcedHiddenPrimitiveMap[NodeIndex] = true;

				// Also hide children when performing far culling
				if (bFarCulled)
				{
					HideNodeChildren(ViewState, Node);
				}
			}
		}

		if (!FlushCachedShadowPrimitives.IsEmpty())
		{
			// We don't want to block on the LPI creation task as it overlaps with this work. Spawn a new task that will be waited on further down the pipe.
			FlushCachedShadowsTaskEvent.AddPrerequisites(UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Primitives = MoveTemp(FlushCachedShadowPrimitives)]
			{
				for (FPrimitiveSceneInfo* SceneInfo : Primitives)
				{
					FLightPrimitiveInteraction* NodeLightList = SceneInfo->LightList;
					while (NodeLightList)
					{
						NodeLightList->FlushCachedShadowMapData();
						NodeLightList = NodeLightList->GetNextLight();
					}
				}
			}, Scene->GetCreateLightPrimitiveInteractionsTask()));
		}
	}
}

void FLODSceneTree::ApplyNodeFadingToChildren(FSceneViewState* ViewState, FLODSceneNode& Node, FHLODSceneNodeVisibilityState& NodeVisibility, const bool bIsFading, const bool bIsFadingOut)
{
	checkSlow(ViewState);
	if (Node.SceneInfo)
	{
		FHLODVisibilityState& HLODState = ViewState->HLODVisibilityState;
		NodeVisibility.UpdateCount = HLODState.UpdateCount;

		// Force visibility during fades
		for (const auto Child : Node.ChildrenSceneInfos)
		{
			if (!Child || !Child->PrimitiveComponentId.IsValid() || !Child->IsIndexValid())
			{
				continue;
			}

			const int32 ChildIndex = Child->GetIndex();

			if (!HLODState.PrimitiveFadingLODMap.IsValidIndex(ChildIndex))
			{
				checkf(0, TEXT("A HLOD Child's PrimitiveSceneInfo PackedIndex was out of FadingMap's bounds!"));
				continue;
			}
		
			HLODState.PrimitiveFadingLODMap[ChildIndex] = bIsFading;
			HLODState.PrimitiveFadingOutLODMap[ChildIndex] = bIsFadingOut;
			HLODState.ForcedHiddenPrimitiveMap[ChildIndex] = false;

			if (bIsFading)
			{
				HLODState.ForcedVisiblePrimitiveMap[ChildIndex] = true;
			}

			// Fading only occurs at the adjacent hierarchy level, below should be hidden
			if (FLODSceneNode* ChildNode = SceneNodes.Find(Child->PrimitiveComponentId))
			{
				HideNodeChildren(ViewState, *ChildNode);
			}
		}
	}
}

void FLODSceneTree::HideNodeChildren(FSceneViewState* ViewState, FLODSceneNode& Node)
{
	checkSlow(ViewState);
	if (Node.SceneInfo)
	{
		FHLODVisibilityState& HLODState = ViewState->HLODVisibilityState;
		TMap<FPrimitiveComponentId, FHLODSceneNodeVisibilityState>& VisibilityStates = ViewState->HLODSceneNodeVisibilityStates;
		FHLODSceneNodeVisibilityState& NodeVisibility = VisibilityStates.FindOrAdd(Node.SceneInfo->PrimitiveComponentId);

		if (NodeVisibility.UpdateCount != HLODState.UpdateCount)
		{
			NodeVisibility.UpdateCount = HLODState.UpdateCount;

			for (const auto Child : Node.ChildrenSceneInfos)
			{
				if (!Child || !Child->PrimitiveComponentId.IsValid() || !Child->IsIndexValid())
				{
					continue;
				}

				const int32 ChildIndex = Child->GetIndex();

				if (!HLODState.ForcedHiddenPrimitiveMap.IsValidIndex(ChildIndex))
				{
					checkf(0, TEXT("A HLOD Child's PrimitiveSceneInfo PackedIndex was out of ForcedHidden's bounds!"));
					continue;
				}

				HLODState.ForcedHiddenPrimitiveMap[ChildIndex] = true;
				
				// Clear the force visible flag in case the child was processed before it's parent
				HLODState.ForcedVisiblePrimitiveMap[ChildIndex] = false;

				if (FLODSceneNode* ChildNode = SceneNodes.Find(Child->PrimitiveComponentId))
				{
					HideNodeChildren(ViewState, *ChildNode);
				}
			}
		}
	}
}

float CalcStaticMeshLODScreenMultipleForRadius(const FMatrix& ProjMatrix, float ViewLODDistanceFactor)
{
	const float LODDistanceScale = GetCachedScalabilityCVars().StaticMeshLODDistanceScale * ViewLODDistanceFactor;
	// Note: to calculate the factor to use with the primitive/instance radius, we omit the 0.5 factor needed here that is used elswhere when used with the "size" AKA diameter of the object.
	const float ScreenMultiple = FMath::Max(ProjMatrix.M[0][0], ProjMatrix.M[1][1]) / LODDistanceScale;
	return ScreenMultiple;
}
