// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneManagement.h"
#include "SceneRendererInterface.h"
#include "StaticMeshBatch.h"
#include "MeshPassProcessor.h"
#include "Tasks/Task.h"

class FVisibilityTaskData;
class FSceneRenderer;
class FInstanceCullingManager;
class FVirtualTextureUpdater;
class FSceneCullingRendererHierarchicalCullingQuery;

class FViewCommands
{
public:
	FViewCommands()
	{
		for (int32 PassIndex = 0; PassIndex < EMeshPass::Num; ++PassIndex)
		{
			NumDynamicMeshCommandBuildRequestElements[PassIndex] = 0;
		}
	}

	TStaticArray<FMeshCommandOneFrameArray, EMeshPass::Num> MeshCommands;
	TStaticArray<int32, EMeshPass::Num> NumDynamicMeshCommandBuildRequestElements;
	TStaticArray<TArray<const FStaticMeshBatch*, SceneRenderingAllocator>, EMeshPass::Num> DynamicMeshCommandBuildRequests;
	TStaticArray<TArray<EMeshDrawCommandCullingPayloadFlags, SceneRenderingAllocator>, EMeshPass::Num> DynamicMeshCommandBuildFlags;
};

class IVisibilityTaskData
{
public:
	virtual ~IVisibilityTaskData() = default;

	/** [Optional] Call to allow early processing of async GDME tasks when it is safe to do so. Otherwise, this is automatically called from ProcessRenderThreadTasks. */
	virtual void StartGatherDynamicMeshElements() = 0;

	/** Processes all visibility tasks that must be performed on the render thread. */
	virtual void ProcessRenderThreadTasks() = 0;

	/** Called to finish processing of the GDME tasks. */
	virtual void FinishGatherDynamicMeshElements(FExclusiveDepthStencil::Type BasePassDepthStencilAccess, FInstanceCullingManager& InstanceCullingManager, FVirtualTextureUpdater* VirtualTextureUpdater) = 0;

	/** Waits for the task graph and cleans up. */
	virtual void Finish() = 0;

	/** Returns the array of view commands associated with the view visibility tasks. */
	virtual TArrayView<FViewCommands> GetViewCommandsPerView() = 0;

	//////////////////////////////////////////////////////////////////////////////
	// Use these tasks as dependencies to trigger tasks at specific stages of the visibility pipeline while initializing.

	/** Returns the task event representing all visibility frustum cull jobs for all views. */
	virtual UE::Tasks::FTask GetFrustumCullTask() const = 0;

	/** Returns the task event representing all compute relevance tasks for all views. */
	virtual UE::Tasks::FTask GetComputeRelevanceTask() const = 0;

	/** Returns the task event representing the light visibility computation. */
	virtual UE::Tasks::FTask GetLightVisibilityTask() const = 0;

	/** Returns true if tasks can be waited on from the render thread prior to FinishInit (this is always true after FinishInit). */
	virtual bool IsTaskWaitingAllowed() const = 0;

	//////////////////////////////////////////////////////////////////////////////
};

extern IVisibilityTaskData* LaunchVisibilityTasks(FRHICommandListImmediate& RHICmdList, FSceneRenderer& SceneRenderer, const UE::Tasks::FTask& BeginInitVisibilityTaskPrerequisites);

/**
 * Helper function to calculate the correct screen multiple for static mesh LOD selection based on the _radius_ of the rendered object (instance/primitive). 
 */
float CalcStaticMeshLODScreenMultipleForRadius(const FMatrix& ProjMatrix, float ViewLODDistanceFactor);

inline float CalcStaticMeshLODScreenMultipleForRadiusSquared(const FMatrix& ProjMatrix, float ViewLODDistanceFactor)
{
	return FMath::Square(CalcStaticMeshLODScreenMultipleForRadius(ProjMatrix, ViewLODDistanceFactor));
}

extern float ScreenVisibilityGetMinScreenBoundsRadiusSquared(const FSceneView& View);

inline bool IsScreenSizeCulled(const FSceneView& View, const FBoxSphereBounds& Bounds, float MinScreenBoundsRadiusSquared)
{
	if (!MinScreenBoundsRadiusSquared)
	{
		return false;
	}

	const float ComputedScreenRadiusSquared = ComputeBoundsScreenRadiusSquared(Bounds.Origin, Bounds.SphereRadius, View);
	return ComputedScreenRadiusSquared < MinScreenBoundsRadiusSquared;
}

/**
 * Compute the static mesh LODs for a primitive.
 */
FLODMask ComputePrimitiveMeshLODs(const FViewInfo& View, int32 ViewForcedLODLevel, float ViewLODScale, const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FPrimitiveSceneInfo* PrimitiveSceneInfo, int32 PrimitiveIndex, const FBoxSphereBounds& Bounds, FSceneCullingRendererHierarchicalCullingQuery* SceneCullingQuery);
