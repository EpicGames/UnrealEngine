// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneVisibility.h"
#include "ScenePrivate.h"
#include "SceneViewState.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "DynamicPrimitiveDrawing.h"
#include "Async/TaskGraphInterfaces.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"

class FInstanceCullingManager;
class FComputeAndMarkRelevance;
class FGPUOcclusion;
class FGPUOcclusionSerial;
class FGPUOcclusionParallel;
class FGPUOcclusionParallelPacket;
class FRelevancePacket;
class FVisibilityTaskData;
class FVirtualTextureUpdater;

/** An async MPSC queue that can schedule serialized tasks onto the render thread or a task thread. When a new command is enqueued into an empty queue, a
 *  task is launched to process all pending elements in the queue as soon as possible. Each command must be reserved with AddNumCommands,
 *  which acts as a reference count on the number of pending commands in the pipe. This is necessary to determine when the pipe is 'done' and
 *  can signal a completion event or callback.
 */
template <typename CommandType>
class TCommandPipe
{
public:
	using CommandFunctionType = TFunction<void(CommandType&&)>;
	using EmptyFunctionType = TFunction<void()>;

	TCommandPipe(const TCHAR* InName)
		: Pipe(InName)
	{}

	~TCommandPipe()
	{
		check(IsEmpty());
		Wait();
	}

	void SetCommandFunction(CommandFunctionType&& InCommandFunction)
	{
		CommandFunction = Forward<CommandFunctionType&&>(InCommandFunction);
	}

	void SetEmptyFunction(TFunction<void()>&& InEmptyFunction)
	{
		EmptyFunction = Forward<EmptyFunctionType&&>(InEmptyFunction);
	}

	void SetPrerequisiteTask(const UE::Tasks::FTask& InPrerequisiteTask)
	{
		PrerequisiteTask = InPrerequisiteTask;
	}

	void AddNumCommands(int32 InNumCommands)
	{
		NumCommands.fetch_add(InNumCommands, std::memory_order_relaxed);
	}

	void ReleaseNumCommands(int32 InNumCommands)
	{
		int32 FinalNumCommands = NumCommands.fetch_sub(InNumCommands, std::memory_order_acq_rel) - InNumCommands;
		check(FinalNumCommands >= 0);

		if (FinalNumCommands == 0)
		{
			if (EmptyFunction)
			{
				EmptyFunction();
			}
		}
	}

	template <typename... ArgTypes>
	void EnqueueCommand(ArgTypes&&... Args)
	{
		check(CommandFunction);

		QueueMutex.Lock();
		const bool bWasEmpty = Queue.IsEmpty();
		Queue.Emplace(Forward<ArgTypes>(Args)...);
		QueueMutex.Unlock();

		if (bWasEmpty)
		{
			Pipe.Launch(Pipe.GetDebugName(), [this]
			{
				FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
				SCOPED_NAMED_EVENT_TCHAR(Pipe.GetDebugName(), FColor::Magenta);

				TArray<CommandType, SceneRenderingAllocator> Commands;

				QueueMutex.Lock();
				Commands = MoveTemp(Queue);
				QueueMutex.Unlock();

				int32 NumProcessedCommands = 0;

				for (CommandType& Command : Commands)
				{
					CommandFunction(MoveTemp(Command));
					NumProcessedCommands++;
				}

				if (NumProcessedCommands)
				{
					ReleaseNumCommands(NumProcessedCommands);
				}

			}, PrerequisiteTask, UE::Tasks::ETaskPriority::High);
		}
	}

	bool IsEmpty() const
	{
		return Queue.IsEmpty();
	}

	void Wait()
	{
		Pipe.WaitUntilEmpty();
	}

private:
	UE::Tasks::FTask PrerequisiteTask;
	CommandFunctionType CommandFunction;
	EmptyFunctionType EmptyFunction;
	UE::FMutex QueueMutex;
	TArray<CommandType, SceneRenderingAllocator> Queue;
	UE::Tasks::FPipe Pipe;
	std::atomic_int32_t NumCommands{ 0 };
};

///////////////////////////////////////////////////////////////////////////////

inline UE::Tasks::EExtendedTaskPriority GetExtendedTaskPriority(bool bExecuteInParallel)
{
	return bExecuteInParallel ? UE::Tasks::EExtendedTaskPriority::None : UE::Tasks::EExtendedTaskPriority::Inline;
}

using FPrimitiveIndexList = TArray<int32, SceneRenderingAllocator>;

struct FPrimitiveRange
{
	int32 StartIndex;
	int32 EndIndex;
};

struct FDynamicPrimitive
{
	int32 PrimitiveIndex;
	int32 ViewIndex;
	int32 StartElementIndex;
	int32 EndElementIndex;
};

struct FDynamicPrimitiveIndex
{
	FDynamicPrimitiveIndex() = default;

	FDynamicPrimitiveIndex(int32 InIndex, uint8 InViewMask)
		: Index(InIndex)
		, ViewMask(InViewMask)
	{}

	uint32 Index    : 24;
	uint32 ViewMask : 8;
};

struct FDynamicPrimitiveIndexList
{
	using FList = TArray<FDynamicPrimitiveIndex, SceneRenderingAllocator>;

	bool IsEmpty() const
	{
		return Primitives.IsEmpty()
#if WITH_EDITOR
			&& EditorPrimitives.IsEmpty()
#endif
			;
	}

	FList Primitives;

#if WITH_EDITOR
	FList EditorPrimitives;
#endif
};

class FDynamicPrimitiveIndexQueue
{
public:
	FDynamicPrimitiveIndexQueue(FDynamicPrimitiveIndexList&& InList)
		: List(InList)
	{}

	bool Pop(FDynamicPrimitiveIndex& PrimitiveIndex)
	{
		const int32 Index = NextIndex.fetch_add(1, std::memory_order_relaxed);
		if (Index < List.Primitives.Num())
		{
			PrimitiveIndex = List.Primitives[Index];
			return true;
		}
		return false;
	}

#if WITH_EDITOR
	bool PopEditor(FDynamicPrimitiveIndex& PrimitiveIndex)
	{
		const int32 Index = NextEditorIndex.fetch_add(1, std::memory_order_relaxed);
		if (Index < List.EditorPrimitives.Num())
		{
			PrimitiveIndex = List.EditorPrimitives[Index];
			return true;
		}
		return false;
	}
#endif

private:
	FDynamicPrimitiveIndexList List;
	std::atomic_int32_t NextIndex = { 0 };
#if WITH_EDITOR
	std::atomic_int32_t NextEditorIndex = { 0 };
#endif
};

struct FDynamicPrimitiveViewMasks
{
	FPrimitiveViewMasks Primitives;

#if WITH_EDITOR
	FPrimitiveViewMasks EditorPrimitives;
#endif
};

///////////////////////////////////////////////////////////////////////////////

class FDynamicMeshElementContext
{
public:
	FDynamicMeshElementContext(FSceneRenderer& SceneRenderer);

	FGraphEventRef LaunchRenderThreadTask(FDynamicPrimitiveIndexList&& PrimitiveIndexList);

	UE::Tasks::FTask LaunchAsyncTask(FDynamicPrimitiveIndexQueue* PrimitiveIndexQueue, UE::Tasks::ETaskPriority TaskPriority);

	void GatherDynamicMeshElementsForPrimitive(FPrimitiveSceneInfo* Primitive, uint8 ViewMask);

	void GatherDynamicMeshElementsForEditorPrimitive(FPrimitiveSceneInfo* Primitive, uint8 ViewMask);

private:
	void Finish();

	struct FViewFamilyGroup
	{
		const FSceneViewFamily* Family;
		uint8 ViewSubsetMask;				// Subset of views with the given view family
	};

	struct FViewMeshArrays
	{
		TArray<FMeshBatchAndRelevance, SceneRenderingAllocator> DynamicMeshElements;
		FSimpleElementCollector SimpleElementCollector;

#if WITH_EDITOR
		TArray<FMeshBatchAndRelevance, SceneRenderingAllocator> DynamicEditorMeshElements;
		FSimpleElementCollector EditorSimpleElementCollector;
#endif

#if UE_ENABLE_DEBUG_DRAWING
		FSimpleElementCollector DebugSimpleElementCollector;
#endif
	};

	const FSceneViewFamily& FirstViewFamily;
	TArrayView<FViewInfo*> Views;
	TArrayView<FPrimitiveSceneInfo*> Primitives;
	TArray<FViewFamilyGroup, TInlineAllocator<1>> ViewFamilyGroups;
	TArray<FViewMeshArrays, TInlineAllocator<2>> ViewMeshArraysPerView;
	TArray<FDynamicPrimitive, SceneRenderingAllocator> DynamicPrimitives;
	FMeshElementCollector MeshCollector;
#if WITH_EDITOR
	FMeshElementCollector EditorMeshCollector;
#endif
	FRHICommandList* RHICmdList;
	FGlobalDynamicVertexBuffer DynamicVertexBuffer;
	FGlobalDynamicIndexBuffer DynamicIndexBuffer;
	UE::Tasks::FPipe Pipe{UE_SOURCE_LOCATION};

	friend class FDynamicMeshElementContextContainer;
};

class FDynamicMeshElementContextContainer
{
public:
	~FDynamicMeshElementContextContainer();

	int32 GetNumAsyncContexts() const
	{
		return Contexts.Num() - 1;
	}

	FDynamicMeshElementContext* GetRenderThreadContext() const
	{
		check(!bFinished);
		return Contexts.Last();
	}

	FGraphEventRef LaunchRenderThreadTask(FDynamicPrimitiveIndexList&& PrimitiveIndexList);

	UE::Tasks::FTask LaunchAsyncTask(FDynamicPrimitiveIndexQueue* PrimitiveIndexQueue, int32 Index, UE::Tasks::ETaskPriority TaskPriority);

	void Init(FSceneRenderer& InSceneRenderer, int32 NumAsyncContexts);

	void MergeContexts(TArray<FDynamicPrimitive, SceneRenderingAllocator>& OutDynamicPrimitives);

	void Submit(FRHICommandListImmediate& RHICmdList);

private:
	using FDynamicMeshElementContextArray = TArray<FDynamicMeshElementContext*>;

	TArrayView<FViewInfo*> Views;
	FDynamicMeshElementContextArray Contexts;
	TArray<FRHICommandListImmediate::FQueuedCommandList, FConcurrentLinearArrayAllocator> CommandLists;
	bool bFinished = false;
};

///////////////////////////////////////////////////////////////////////////////

enum class EVisibilityTaskSchedule
{
	// Visibility is processed on the render thread with assistance from other threads via parallel for.
	RenderThread,

	// Visibility is processed as an async task graph with only the dynamic mesh element gather on the render thread.
	Parallel,
};

///////////////////////////////////////////////////////////////////////////////

// Configuration state for visibility task granularity and priority.
class FVisibilityTaskConfig
{
public:
	FVisibilityTaskConfig(const FScene& Scene, TConstArrayView<FViewInfo*> Views);

	EVisibilityTaskSchedule Schedule;

	// Task priorities for non-specific tasks related to visibility.
	UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::High;

	uint32 NumVisiblePrimitives = 0;
	uint32 NumTestedPrimitives  = 0;

	struct FAlwaysVisible
	{
		static constexpr uint32 MinWordsPerTask = 32;

		const UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::High;

		// Always visible tasks are fixed size and process the same number of primitives.
		uint32 NumTasks = 0;
		uint32 NumWordsPerTask = 0;
		uint32 NumPrimitivesPerTask = 0;

	} AlwaysVisible;

	struct FFrustumCull
	{
		static constexpr uint32 MinWordsPerTask = 32;

		const UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::High;

		// Frustum culling tasks are fixed size and process the same number of primitives.
		uint32 NumTasks = 0;
		uint32 NumWordsPerTask = 0;
		uint32 NumPrimitivesPerTask = 0;
		std::atomic_uint32_t NumCulledPrimitives{ 0 };

	} FrustumCull;

	struct FOcclusionCull
	{
		static constexpr uint32 MinQueriesPerTask = 64;

		const UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::High;
		const UE::Tasks::ETaskPriority FinalizeTaskPriority = UE::Tasks::ETaskPriority::High;

		struct FView
		{
			uint32 MaxQueriesPerTask = 0;
		};

		TArray<FView, TInlineAllocator<2, SceneRenderingAllocator>> Views;
		std::atomic_uint32_t NumCulledPrimitives{ 0 };
		std::atomic_uint32_t NumTestedQueries{ 0 };

	} OcclusionCull;

	struct FRelevance
	{
		static constexpr uint32 MinPrimitivesPerTask = 32;
		static constexpr uint32 MaxPrimitivesPerTask = 2048;

		const UE::Tasks::ETaskPriority ComputeRelevanceTaskPriority = UE::Tasks::ETaskPriority::High;

		uint32 NumEstimatedPackets = 0;
		uint32 NumPrimitivesPerPacket = 0;
		uint32 NumPrimitivesProcessed = 0;

	} Relevance;
};

///////////////////////////////////////////////////////////////////////////////

class FVisibilityViewPacket
{
	friend FVisibilityTaskData;
	friend FGPUOcclusionParallel;
	friend FGPUOcclusionParallelPacket;
public:
	FVisibilityViewPacket(FVisibilityTaskData& TaskData, FScene& InScene, FViewInfo& InView, int32 ViewIndex);

	FVisibilityTaskData& TaskData;
	FVisibilityTaskConfig& TaskConfig;
	FScene& Scene;
	FViewInfo& View;
	FSceneViewState* ViewState;
	int32 ViewIndex;
	FViewElementPDI ViewElementPDI;

private:
	void BeginInitVisibility();

	struct FOcclusionCull
	{
		FGPUOcclusionSerial*   ContextIfSerial   = nullptr;
		FGPUOcclusionParallel* ContextIfParallel = nullptr;
		TCommandPipe<FPrimitiveRange> CommandPipe{ TEXT("OcclusionCullPipe") };

	} OcclusionCull;

	struct FRelevance
	{
		FComputeAndMarkRelevance* Context = nullptr;
		TCommandPipe<FPrimitiveIndexList> CommandPipe{ TEXT("RelevancePipe") };

	} Relevance;

	struct FTasks
	{
		UE::Tasks::FTaskEvent AlwaysVisible{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent FrustumCull{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent OcclusionCull{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent ComputeRelevance{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent LightVisibility{ UE_SOURCE_LOCATION };

	} Tasks;
};

///////////////////////////////////////////////////////////////////////////////

/*
	This class manages all state related to visibility computation for all views associated with a specific scene renderer. When in parallel mode, a complex task graph
	processes each visibility stage and pipelines results from one stage to the next. This avoids major join / fork sync points except for the dynamic mesh elements
	gather which is currently confined to the render thread. For platforms that don't benefit from parallelism or don't support it, a render-thread centric mode is
	also supported which processes visibility on the render thread with some parallel for support from task threads.

	Visibility is processed for all views independently where each view performs multiple stages of pipelined task work. The view stages are as follows:

		Frustum Cull          - Primitives are frustum / distance culled and visible primitives are emitted.
		Occlusion Cull        - Primitives are culled against occluders in the scene and visible primitives are emitted.
		Compute Relevance     - Primitives are queried for view relevance information dynamic primitives are identified and emitted, and static meshes are filtered
								into various mesh passes

	The following stages are then performed for all views:

		Gather Dynamic Mesh Elements (GDME) - Primitives identified to have dynamic relevance are queried with a view mask to supply dynamic meshes.
		Setup Mesh Passes                   - Tasks are launched to generate mesh draw commands for static and dynamic meshes.

	In order to facilitate processing of emitted data from each stage, and ultimately spawning load balanced tasks for the next stage, the visibility pipeline utilizes
	'command pipes' which are serial queues that run between stages to launch work for the next stage. Each view has two pipes: OcclusionCull, and Relevance. The OcclusionCull
	pipe can either process occlusion tasks, or act as the relevance pipe if occlusion is disabled. The relevance pipe only launches relevance tasks.

	The render thread performs the GDME stage which syncs the compute relevance stage for all views. Setup Mesh Passes then runs once all the GDME tasks
	have completed. When the renderer has only one view, GDME utilizes a command pipe to process requests from relevance as quickly as possible and achieves some overlap
	with relevance, reducing the critical path. With multiple views, it's necessary to sync beforehand as the gather requires a view mask for each dynamic primitive. To
	that end, GDME supports two paths for processing dynamic primitives: an index list or view bit mask. The former just supplies a list of dynamic primitives and the view
	bits are assumed to be 0x1. The latter requires deriving dynamic primitive indices by scanning the view mask array for non-zero elements.

	IMPORTANT:
		Accessing any visibility data prior to calling Finish must be done with extreme caution and synchronize with the various stages manually using the provided tasks.
		Unprotected access of visibility state is ONLY safe after calling Finish.
*/
class FVisibilityTaskData : public IVisibilityTaskData
{
	friend FVisibilityViewPacket;
	friend FRelevancePacket;
	friend FComputeAndMarkRelevance;
public:
	FVisibilityTaskData(FRHICommandListImmediate& RHICmdList, FSceneRenderer& SceneRenderer);

	~FVisibilityTaskData() override
	{
		check(bFinished);
	}

	void LaunchVisibilityTasks(const UE::Tasks::FTask& BeginInitVisibilityPrerequisites);

	void ProcessRenderThreadTasks() override;

	void StartGatherDynamicMeshElements() override
	{
		if (!Tasks.bDynamicMeshElementsPrerequisitesTriggered)
		{
			Tasks.DynamicMeshElementsPrerequisites.Trigger();
			Tasks.bDynamicMeshElementsPrerequisitesTriggered = true;
		}
	}

	void FinishGatherDynamicMeshElements(FExclusiveDepthStencil::Type BasePassDepthStencilAccess, FInstanceCullingManager& InstanceCullingManager, FVirtualTextureUpdater* VirtualTextureUpdater) override;

	void Finish() override;

	TArrayView<FViewCommands> GetViewCommandsPerView() override
	{
		return DynamicMeshElements.ViewCommandsPerView;
	}

	UE::Tasks::FTask GetFrustumCullTask() const override
	{
		return Tasks.FrustumCull;
	}

	UE::Tasks::FTask GetComputeRelevanceTask() const override
	{
		return Tasks.FinalizeRelevance;
	}

	UE::Tasks::FTask GetLightVisibilityTask() const override
	{
		return Tasks.LightVisibility;
	}

	bool IsTaskWaitingAllowed() const override
	{
		return Tasks.bWaitingAllowed;
	}

private:
	void GatherDynamicMeshElements(FDynamicPrimitiveIndexList&& Primitives);
	void GatherDynamicMeshElements(const FDynamicPrimitiveViewMasks& Primitives);

	void SetupMeshPasses(FExclusiveDepthStencil::Type BasePassDepthStencilAccess, FInstanceCullingManager& InstanceCullingManager);

	FRHICommandListImmediate& RHICmdList;
	FSceneRenderer& SceneRenderer;
	FScene& Scene;
	TArrayView<FViewInfo*> Views;
	FViewFamilyInfo& ViewFamily;
	EShadingPath ShadingPath;
	FSceneRenderingBulkObjectAllocator Allocator;
	TArray<FVisibilityViewPacket, SceneRenderingAllocator> ViewPackets;

	struct FDynamicMeshElements
	{
		// The command pipe and event are only non-null when in single-view mode.
		TCommandPipe<FDynamicPrimitiveIndexList>* CommandPipe = nullptr;

		// Primitive view masks are only non-null when in multi-view mode.
		FDynamicPrimitiveViewMasks* PrimitiveViewMasks = nullptr;

		FDynamicMeshElementContextContainer ContextContainer;
		TArray<FViewCommands, TInlineAllocator<4>> ViewCommandsPerView;
		TArray<FDynamicPrimitive, SceneRenderingAllocator> DynamicPrimitives;
		TArray<UE::Tasks::FPipe, SceneRenderingAllocator> DynamicPrimitiveTaskPipes;

	} DynamicMeshElements;

	struct FTasks
	{
		// These legacy tasks are used to interface with the jobs launched prior to gather dynamic mesh elements.
		FGraphEventRef FrustumCullLegacyTask;
		FGraphEventRef ComputeRelevanceLegacyTask;

		FGraphEventRef DynamicMeshElementsPipe;

		UE::Tasks::FTaskEvent LightVisibility{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent BeginInitVisibility{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent FrustumCull{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent OcclusionCull{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent ComputeRelevance{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent DynamicMeshElementsPrerequisites{ UE_SOURCE_LOCATION };
		UE::Tasks::FTaskEvent DynamicMeshElements{ UE_SOURCE_LOCATION };
		UE::Tasks::FTask FinalizeRelevance;
		UE::Tasks::FTask MeshPassSetup;

		bool bDynamicMeshElementsPrerequisitesTriggered = false;
		bool bWaitingAllowed = false;

	} Tasks;

	FVisibilityTaskConfig TaskConfig;

	const bool bAddNaniteRelevance;
	const bool bAddLightmapDensityCommands;
	bool bFinished = false;
};

///////////////////////////////////////////////////////////////////////////////

template<class T, int TAmplifyFactor = 1>
struct FRelevancePrimSet
{
	TArray<T, SceneRenderingAllocator> Prims;
	const int32 MaxOutputPrims;

	FORCEINLINE FRelevancePrimSet(int32 InputPrimCount)
		: MaxOutputPrims(InputPrimCount* TAmplifyFactor)
	{}

	FORCEINLINE void AddPrim(T Prim)
	{
		if (IsEmpty())
		{
			Prims.Reserve(MaxOutputPrims);
		}

		Prims.Add(Prim);
	}
	FORCEINLINE bool IsFull() const
	{
		return Prims.Num() >= MaxOutputPrims;
	}
	FORCEINLINE bool IsEmpty() const
	{
		return Prims.IsEmpty();
	}
	template<class TARRAY>
	FORCEINLINE void AppendTo(TARRAY& DestArray)
	{
		DestArray.Append(Prims);
	}
};

struct FFilterStaticMeshesForViewData
{
	FVector ViewOrigin;
	int32 ForcedLODLevel;
	float LODScale;
	float MinScreenRadiusForDepthPrepassSquared;
	bool bFullEarlyZPass;

	FFilterStaticMeshesForViewData(FViewInfo& View);
};

namespace EMarkMaskBits
{
	enum Type
	{
		StaticMeshVisibilityMapMask = 0x2,
		StaticMeshFadeOutDitheredLODMapMask = 0x10,
		StaticMeshFadeInDitheredLODMapMask = 0x20,
	};
}

using FPassDrawCommandArray = TArray<FVisibleMeshDrawCommand>;
using FPassDrawCommandBuildRequestArray = TArray<const FStaticMeshBatch*>;
using FPassDrawCommandBuildFlagsArray = TArray<EMeshDrawCommandCullingPayloadFlags>;

struct FDrawCommandRelevancePacket
{
	FDrawCommandRelevancePacket();

	FPassDrawCommandArray VisibleCachedDrawCommands[EMeshPass::Num];
	FPassDrawCommandBuildRequestArray DynamicBuildRequests[EMeshPass::Num];
	FPassDrawCommandBuildFlagsArray DynamicBuildFlags[EMeshPass::Num];
	int32 NumDynamicBuildRequestElements[EMeshPass::Num];
	bool bUseCachedMeshDrawCommands;

	void AddCommandsForMesh(
		int32 PrimitiveIndex,
		const FPrimitiveSceneInfo* InPrimitiveSceneInfo,
		const FStaticMeshBatchRelevance& RESTRICT StaticMeshRelevance,
		const FStaticMeshBatch& RESTRICT StaticMesh,
		EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags,
		const FScene& Scene,
		bool bCanCache,
		EMeshPass::Type PassType);
};

class FRelevancePacket : public FSceneRenderingAllocatorObject<FRelevancePacket>
{
public:
	FRelevancePacket(
		FVisibilityTaskData& InTaskData,
		const FViewInfo& InView,
		int32 InViewIndex,
		const FFilterStaticMeshesForViewData& InViewData,
		FSceneCullingRendererHierarchicalCullingQuery* InSceneCullingQuery,
		uint8* InMarkMasks,
		const UE::Tasks::FTask& PrerequisitesTask);

	void LaunchComputeRelevanceTask();

	void Finalize();

private:
	void ComputeRelevance(FDynamicPrimitiveIndexList& DynamicPrimitiveIndexList);

	friend FComputeAndMarkRelevance;

	const float CurrentWorldTime;
	const float DeltaWorldTime;

	FVisibilityTaskData& TaskData;
	FVisibilityTaskConfig& TaskConfig;
	const FScene& Scene;
	const FViewInfo& View;
	const FViewCommands& ViewCommands;
	const uint8 ViewBit;
	const FFilterStaticMeshesForViewData& ViewData;
	FDynamicPrimitiveViewMasks* DynamicPrimitiveViewMasks;
	FSceneCullingRendererHierarchicalCullingQuery* SceneCullingQuery = nullptr;
	uint8* RESTRICT MarkMasks;
	UE::Tasks::FTask PrerequisitesTask;

	FRelevancePrimSet<int32> Input;
	FRelevancePrimSet<int32> NotDrawRelevant;
	FRelevancePrimSet<int32> TranslucentSelfShadowPrimitives;
	FRelevancePrimSet<FPrimitiveSceneInfo*> VisibleDynamicPrimitivesWithSimpleLights;
	int32 NumVisibleDynamicPrimitives = 0;
	int32 NumVisibleDynamicEditorPrimitives = 0;
	FMeshPassMask VisibleDynamicMeshesPassMask;
	FTranslucenyPrimCount TranslucentPrimCount;
	FRelevancePrimSet<FPrimitiveSceneInfo*> DirtyIndirectLightingCacheBufferPrimitives;

	TArray<FVolumetricMeshBatch, SceneRenderingAllocator> VolumetricMeshBatches;
	TArray<FVolumetricMeshBatch, SceneRenderingAllocator> HeterogeneousVolumesMeshBatches;
	TArray<FSkyMeshBatch, SceneRenderingAllocator> SkyMeshBatches;
	TArray<FSortedTrianglesMeshBatch, SceneRenderingAllocator> SortedTrianglesMeshBatches;
	FDrawCommandRelevancePacket DrawCommandPacket;
	TSet<uint32, DefaultKeyFuncs<uint32>, SceneRenderingSetAllocator> CustomDepthStencilValues;
	FRelevancePrimSet<FPrimitiveInstanceRange> NaniteCustomDepthInstances;

	struct FPrimitiveLODMask
	{
		FPrimitiveLODMask()
			: PrimitiveIndex(INDEX_NONE)
		{}

		FPrimitiveLODMask(const int32 InPrimitiveIndex, const FLODMask& InLODMask)
			: PrimitiveIndex(InPrimitiveIndex)
			, LODMask(InLODMask)
		{}

		int32 PrimitiveIndex;
		FLODMask LODMask;
	};

	FRelevancePrimSet<FPrimitiveLODMask> PrimitivesLODMask; // group both lod mask with primitive index to be able to properly merge them in the view

	UE::Tasks::FTask ComputeRelevanceTask;
	uint16 CombinedShadingModelMask = 0;
	uint8 SubstrateUintPerPixel = 0;
	uint8 SubstrateClosureCountMask = 0;
	uint8 SubstrateTileTypeMask = 0;
	bool bUsesAnisotropy = false;
	bool bHasDistortionPrimitives = false;
	bool bHasCustomDepthPrimitives = false;
	bool bUsesGlobalDistanceField = false;
	bool bUsesLightingChannels = false;
	bool bTranslucentSurfaceLighting = false;
	bool bUsesCustomDepth = false;
	bool bUsesCustomStencil = false;
	bool bUsesMotionVectorWorldOffset = false;
	bool bSLWUsesMotionVectorWorldOffset = false;
	bool bSceneHasSkyMaterial = false;
	bool bHasSingleLayerWaterMaterial = false;
	bool bUsesSecondStageDepthPass = false;
	bool bAddLightmapDensityCommands = false;
	bool bComputeRelevanceTaskLaunched = false;
};

class FComputeAndMarkRelevance
{
public:
	FComputeAndMarkRelevance(FVisibilityTaskData& InTaskData, FScene& InScene, FViewInfo& InView, uint8 InViewIndex, const UE::Tasks::FTask& PreprequisitesTask);

	~FComputeAndMarkRelevance()
	{
		check(Packets.IsEmpty());
		check(bFinished && bFinalized);
	}

	void AddPrimitives(FPrimitiveIndexList&& PrimitiveIndexList);
	void AddPrimitive(int32 Index);

	// Call when all primitives have been added.
	void Finish(UE::Tasks::FTaskEvent& ComputeRelevanceTaskEvent);

	// Call in a dependent task when both compute relevance and filter static mesh task events have completed.
	void Finalize();

private:
	FRelevancePacket* CreateRelevancePacket()
	{
		check(!bLaunchOnAddPrimitive || !bFinished);
		return Packets.Emplace_GetRef(new FRelevancePacket(TaskData, View, ViewIndex, ViewData, SceneCullingQuery, MarkMasks, PrerequisitesTask));
	}

	FVisibilityTaskData& TaskData;
	FScene& Scene;
	FViewInfo& View;
	FViewCommands& ViewCommands;
	int32 ViewIndex;
	const FFilterStaticMeshesForViewData ViewData;
	FSceneCullingRendererHierarchicalCullingQuery* SceneCullingQuery = nullptr;
	const uint32 NumMeshes;
	const uint32 NumPrimitivesPerPacket;
	uint8* MarkMasks;
	TArray<FRelevancePacket*, SceneRenderingAllocator> Packets;
	UE::Tasks::FTask PrerequisitesTask;
	const bool bLaunchOnAddPrimitive;
	bool bFinished = false;
	bool bFinalized = false;
};

///////////////////////////////////////////////////////////////////////////////

struct FOcclusionBounds
{
	FOcclusionBounds(FVector InOrigin, FVector& InExtent)
		: Origin(InOrigin)
		, Extent(InExtent)
	{}

	const FVector Origin;
	const FVector Extent;
};

struct FThrottledOcclusionQuery
{
	FThrottledOcclusionQuery(FPrimitiveOcclusionHistoryKey InPrimitiveOcclusionHistoryKey, FVector InBoundsOrigin, FVector InBoundsExtent, uint32 InLastQuerySubmitFrame)
		: PrimitiveOcclusionHistoryKey(InPrimitiveOcclusionHistoryKey)
		, Bounds(InBoundsOrigin, InBoundsExtent)
		, LastQuerySubmitFrame(InLastQuerySubmitFrame)
	{}

	const FPrimitiveOcclusionHistoryKey PrimitiveOcclusionHistoryKey;
	const FOcclusionBounds Bounds;
	const uint32 LastQuerySubmitFrame;
};

struct FOcclusionQuery
{
	FOcclusionQuery(FPrimitiveOcclusionHistory* InPrimitiveOcclusionHistory, FVector InBoundsOrigin, FVector InBoundsExtent, bool bInGroupedQuery)
		: PrimitiveOcclusionHistory(InPrimitiveOcclusionHistory)
		, Bounds(InBoundsOrigin, InBoundsExtent)
		, bGroupedQuery(bInGroupedQuery)
	{}

	FPrimitiveOcclusionHistory* PrimitiveOcclusionHistory;
	const FOcclusionBounds Bounds;
	const bool bGroupedQuery;
};

struct FOcclusionFeedbackEntry
{
	FOcclusionFeedbackEntry(const FPrimitiveOcclusionHistoryKey& PrimitiveKey, FVector InBoundsOrigin, FVector InBoundsExtent)
		: PrimitiveKey(PrimitiveKey)
		, Bounds(InBoundsOrigin, InBoundsExtent)
	{}

	const FPrimitiveOcclusionHistoryKey PrimitiveKey;
	const FOcclusionBounds Bounds;
};

struct FHZBBound
{
	FHZBBound(FPrimitiveOcclusionHistory* InTargetHistory, const FVector& InBoundsOrigin, const FVector& InBoundsExtent)
		: TargetHistory(InTargetHistory)
		, BoundsOrigin(InBoundsOrigin)
		, BoundsExtent(InBoundsExtent)
	{}

	FPrimitiveOcclusionHistory* const TargetHistory;
	const FVector BoundsOrigin;
	const FVector BoundsExtent;
};

struct FGPUOcclusionState
{
	int32 ReadBackLagTolerance = 0;
	int32 NumBufferedFrames = 0;
	int32 NumPrimitiveRangeTasks = 0;

	bool bSubmitQueries = false;
	bool bHZBOcclusion = false;
	bool bUseRoundRobinOcclusion = false;
	bool bAllowSubQueries = false;
};

struct FOcclusionCullResult
{
	uint32 NumCulledPrimitives = 0;
	uint32 NumTestedQueries = 0;
};

class FGPUOcclusionPacket
{
public:
	FGPUOcclusionPacket(FVisibilityViewPacket& InViewPacket, const FGPUOcclusionState& InOcclusionState);

	inline bool CanBeOccluded(int32 PrimitiveIndex, EOcclusionFlags::Type& OutOcclusionFlags) const
	{
		OutOcclusionFlags = static_cast<EOcclusionFlags::Type>(Scene.PrimitiveOcclusionFlags[PrimitiveIndex]);

		bool bCanBeOccluded = EnumHasAnyFlags(EOcclusionFlags::CanBeOccluded, OutOcclusionFlags);
#if WITH_EDITOR
		if (GIsEditor)
		{
			if (Scene.PrimitivesSelected[PrimitiveIndex])
			{
				// to render occluded outline for selected objects
				bCanBeOccluded = false;
			}
		}
#endif
		return bCanBeOccluded;
	}

	void RecordOcclusionCullResult(FOcclusionCullResult Result)
	{
		FVisibilityTaskConfig& TaskConfig = ViewPacket.TaskConfig;
		TaskConfig.OcclusionCull.NumCulledPrimitives.fetch_add(Result.NumCulledPrimitives, std::memory_order_relaxed);
		TaskConfig.OcclusionCull.NumTestedQueries.fetch_add(Result.NumTestedQueries, std::memory_order_relaxed);
	}

	template <bool bIsParallel, typename VisitorType>
	bool OcclusionCullPrimitive(VisitorType& Visitor, FOcclusionCullResult& Result, int32 Index);

	//////////////////////////////////////////////////////////////////////////////

	// Visitor variant for tasks that records commands to replay later.
	struct FRecordVisitor
	{
		void AddThrottledOcclusionQuery(const FThrottledOcclusionQuery& Query)
		{
			// Throttled occlusion queries are not supported in the recorded path.
			checkNoEntry();
		}

		FPrimitiveOcclusionHistory* AddOcclusionHistory(const FPrimitiveOcclusionHistory& History)
		{
			return &OcclusionHistories[OcclusionHistories.AddElement(History)];
		}

		void AddHZBBounds(const FHZBBound& Bounds)
		{
			HZBBounds.Emplace(Bounds);
		}

		void AddOcclusionFeedback(const FOcclusionFeedbackEntry& Entry)
		{
			OcclusionFeedbacks.Emplace(Entry);
		}

		void AddVisualizeQuery(const FBox& Box)
		{
			VisualizeQueries.Emplace(TPair<FBox, FColor>(Box, FColor(50, 255, 50)));
		}

		void AddVisualizeQuery(const FBox& Box, const FColor& Color)
		{
			VisualizeQueries.Emplace(TPair<FBox, FColor>(Box, Color));
		}

		void AddOcclusionQuery(const FOcclusionQuery& Query)
		{
			OcclusionQueries.Emplace(Query);
		}

		static constexpr uint32 ArrayChunkSize = 1024;

		TChunkedArray<FPrimitiveOcclusionHistory, ArrayChunkSize, SceneRenderingAllocator> OcclusionHistories;
		TArray<FOcclusionQuery, SceneRenderingAllocator> OcclusionQueries;
		TArray<FOcclusionFeedbackEntry, SceneRenderingAllocator> OcclusionFeedbacks;
		TArray<FHZBBound, SceneRenderingAllocator> HZBBounds;
		TArray<TPair<FBox, FColor>, SceneRenderingAllocator> VisualizeQueries;
	};

	// Visitor variant that will process the requests immediately or replay recorded commands.
	struct FProcessVisitor
	{
		FProcessVisitor(FGPUOcclusionPacket& InPacket, FGlobalDynamicVertexBuffer& InDynamicVertexBuffer)
			: Packet(InPacket)
			, DynamicVertexBuffer(InDynamicVertexBuffer)
			, PrimitiveOcclusionHistorySet(Packet.ViewState.Occlusion.PrimitiveOcclusionHistorySet)
		{}

		void AddThrottledOcclusionQuery(const FThrottledOcclusionQuery& Query)
		{
			ThrottledOcclusionQueries.Emplace(Query);
		}

		FPrimitiveOcclusionHistory* AddOcclusionHistory(const FPrimitiveOcclusionHistory& History)
		{
			return &PrimitiveOcclusionHistorySet[PrimitiveOcclusionHistorySet.Add(History)];
		}

		void AddOcclusionFeedback(const FOcclusionFeedbackEntry& Entry)
		{
			Packet.OcclusionFeedback.AddPrimitive(Entry.PrimitiveKey, Entry.Bounds.Origin, Entry.Bounds.Extent, DynamicVertexBuffer);
		}

		void AddVisualizeQuery(const FBox& Box)
		{
			DrawWireBox(&Packet.ViewElementPDI, Box, FColor(50, 255, 50), SDPG_Foreground);
		}

		void AddVisualizeQuery(const FBox& Box, const FColor& Color)
		{
			DrawWireBox(&Packet.ViewElementPDI, Box, Color, SDPG_Foreground);
		}

		void AddHZBBounds(const FHZBBound& HZBBounds)
		{
			HZBBounds.TargetHistory->HZBTestIndex = Packet.HZBOcclusionTests.AddBounds(HZBBounds.BoundsOrigin, HZBBounds.BoundsExtent);
		}

		void AddOcclusionQuery(const FOcclusionQuery& Query);

		void Replay(const FRecordVisitor& RecordVisitor)
		{
			for (const FOcclusionFeedbackEntry& Entry : RecordVisitor.OcclusionFeedbacks)
			{
				AddOcclusionFeedback(Entry);
			}

			for (const FOcclusionQuery& Query : RecordVisitor.OcclusionQueries)
			{
				AddOcclusionQuery(Query);
			}

			for (const TPair<FBox, FColor>& Query : RecordVisitor.VisualizeQueries)
			{
				AddVisualizeQuery(Query.Key, Query.Value);
			}

			for (const FHZBBound& HZBBounds : RecordVisitor.HZBBounds)
			{
				AddHZBBounds(HZBBounds);
			}
		}

		void SubmitThrottledOcclusionQueries();

		FGPUOcclusionPacket& Packet;
		FGlobalDynamicVertexBuffer& DynamicVertexBuffer;
		TArray<FThrottledOcclusionQuery, SceneRenderingAllocator> ThrottledOcclusionQueries;
		TSet<FPrimitiveOcclusionHistory, FPrimitiveOcclusionHistoryKeyFuncs>& PrimitiveOcclusionHistorySet;
	};

	//////////////////////////////////////////////////////////////////////////////

protected:
	static constexpr uint32 SubIsOccludedPageSize = 1024;

	FVisibilityViewPacket& ViewPacket;
	FViewInfo& View;
	FSceneViewState& ViewState;
	FViewElementPDI& ViewElementPDI;
	FHZBOcclusionTester& HZBOcclusionTests;
	FOcclusionFeedback& OcclusionFeedback;
	TSet<FPrimitiveOcclusionHistory, FPrimitiveOcclusionHistoryKeyFuncs>& PrimitiveOcclusionHistorySet;

	TArray<bool>* SubIsOccluded = nullptr;

	const FScene& Scene;
	const FGPUOcclusionState& OcclusionState;
	const FVector ViewOrigin;
	const uint32 OcclusionFrameCounter;
	const float PrimitiveProbablyVisibleTime;
	const float CurrentRealTime;
	const float NeverOcclusionTestDistanceSquared;
	const bool bUseOcclusionFeedback;
	const bool bNewlyConsideredBBoxExpandActive;
};

class FGPUOcclusion
{
public:
	virtual ~FGPUOcclusion() = default;

	FGPUOcclusion(FVisibilityViewPacket& InViewPacket);

	virtual void AddPrimitives(FPrimitiveRange PrimitiveRange) = 0;

	virtual void Map(FRHICommandListImmediate& RHICmdList);
	virtual void Unmap(FRHICommandListImmediate& RHICmdList);

protected:
	void WaitForLastOcclusionQuery();

	FGlobalDynamicVertexBuffer DynamicVertexBuffer;
	FVisibilityViewPacket& ViewPacket;
	const FScene& Scene;
	FViewInfo& View;
	FSceneViewState& ViewState;
	FGPUOcclusionState State;
};

class FGPUOcclusionParallelPacket final
	: public FGPUOcclusionPacket
	, public FSceneRenderingAllocatorObject<FGPUOcclusionParallelPacket>
{
public:
	FGPUOcclusionParallelPacket(FVisibilityViewPacket& InViewPacket, const FGPUOcclusionState& InOcclusionState)
		: FGPUOcclusionPacket(InViewPacket, InOcclusionState)
		, MaxInputSubQueries(InViewPacket.TaskConfig.OcclusionCull.Views[InViewPacket.ViewIndex].MaxQueriesPerTask)
	{}

	bool AddPrimitive(int32 PrimitiveIndex);

	bool IsFull() const
	{
		return NumInputSubQueries >= MaxInputSubQueries;
	}

	void LaunchOcclusionCullTask();

private:
	friend FGPUOcclusionParallel;

	FOcclusionCullResult OcclusionCullTask(FPrimitiveIndexList& PrimitiveIndexList);

	static constexpr uint32 ArrayChunkSize = 1024;

	const uint32 MaxInputSubQueries;
	uint32 NumInputSubQueries = 0;
	TChunkedArray<int32, ArrayChunkSize, SceneRenderingAllocator> Input;
	FRecordVisitor RecordVisitor;
	UE::Tasks::FTask Task;
	bool bTaskLaunched = false;
};

class FGPUOcclusionParallel final : public FGPUOcclusion
{
public:
	FGPUOcclusionParallel(FVisibilityViewPacket& InViewPacket)
		: FGPUOcclusion(InViewPacket)
		, MaxNonOccludedPrimitives(ViewPacket.TaskConfig.Relevance.NumPrimitivesPerPacket)
	{
		NonOccludedPrimitives.Reserve(MaxNonOccludedPrimitives);
		CreateOcclusionPacket();
	}

	~FGPUOcclusionParallel()
	{
		check(bFinished);
	}

	void Finish(UE::Tasks::FTaskEvent& OcclusionCullTasks);
	void Finalize();

	void AddPrimitives(FPrimitiveRange PrimitiveRange)  override;
	void Map(FRHICommandListImmediate& RHICmdListImmediate) override;
	void Unmap(FRHICommandListImmediate& RHICmdListImmediate) override;

private:
	friend FGPUOcclusionParallelPacket;

	void CreateOcclusionPacket()
	{
		Packets.Emplace(new FGPUOcclusionParallelPacket(ViewPacket, State));
	}

	const uint32 MaxNonOccludedPrimitives;
	TArray<FGPUOcclusionParallelPacket*, SceneRenderingAllocator> Packets;
	FPrimitiveIndexList NonOccludedPrimitives;
	FRHICommandList* RHICmdList = nullptr;
	UE::Tasks::FTaskEvent FinalizeTask{ UE_SOURCE_LOCATION };
	bool bFinished = false;
	bool bFinalized = false;
};

class FGPUOcclusionSerial final : public FGPUOcclusion
{
public:
	FGPUOcclusionSerial(FVisibilityViewPacket& InViewPacket)
		: FGPUOcclusion(InViewPacket)
		, Packet(InViewPacket, State)
		, ProcessVisitor(Packet, DynamicVertexBuffer)
	{}

	void AddPrimitives(FPrimitiveRange PrimitiveRange) override;
	void Map(FRHICommandListImmediate& RHICmdListImmediate) override;
	void Unmap(FRHICommandListImmediate& RHICmdListImmediate) override;

private:
	FGPUOcclusionPacket Packet;
	FGPUOcclusionPacket::FProcessVisitor ProcessVisitor;
	FOcclusionCullResult OcclusionCullResult;
};