// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScreenPass.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "EngineGlobals.h"
#include "PostProcess/SceneFilterRendering.h"

#include "RenderGraphUtils.h"
#include "SystemTextures.h"
#include "UnrealClient.h"
#include "SceneRendering.h"

IMPLEMENT_GLOBAL_SHADER(FScreenPassVS, "/Engine/Private/ScreenPass.usf", "ScreenPassVS", SF_Vertex);


RENDERER_API const FScreenTransform FScreenTransform::Identity(FVector2f(1.0f, 1.0f), FVector2f(0.0f, 0.0f));
RENDERER_API const FScreenTransform FScreenTransform::ScreenPosToViewportUV(FVector2f(0.5f, -0.5f), FVector2f(0.5f, 0.5f));
RENDERER_API const FScreenTransform FScreenTransform::ViewportUVToScreenPos(FVector2f(2.0f, -2.0f), FVector2f(-1.0f, 1.0f));

FRHITexture* GetMiniFontTexture()
{
	if (GSystemTextures.AsciiTexture)
	{
		return GSystemTextures.AsciiTexture->GetRHI();
	}
	else
	{
		return GSystemTextures.WhiteDummy->GetRHI();
	}
}

FRDGTextureRef TryCreateViewFamilyTexture(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily)
{
	FRHITexture* TextureRHI = ViewFamily.RenderTarget->GetRenderTargetTexture();
	FRDGTextureRef Texture = nullptr;
	if (TextureRHI)
	{
		Texture = RegisterExternalTexture(GraphBuilder, TextureRHI, TEXT("ViewFamilyTexture"));
		GraphBuilder.SetTextureAccessFinal(Texture, ERHIAccess::RTV);
	}
	return Texture;
}

FRDGTextureRef TryCreateViewFamilyDepthTexture(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily)
{
	if (!ViewFamily.RenderTargetDepth)
	{
		return nullptr;
	}
	FRHITexture* TextureRHI = ViewFamily.RenderTargetDepth->GetRenderTargetTexture();
	FRDGTextureRef Texture = nullptr;
	if (TextureRHI)
	{
		Texture = RegisterExternalTexture(GraphBuilder, TextureRHI, TEXT("ViewFamilyDepthTexture"));
		GraphBuilder.SetTextureAccessFinal(Texture, ERHIAccess::DSVWrite);
	}
	return Texture;
}

// static
FScreenPassTexture FScreenPassTexture::CopyFromSlice(FRDGBuilder& GraphBuilder, const FScreenPassTextureSlice& ScreenTextureSlice, FScreenPassTexture OverrideOutput, uint32 MultiViewCount /*= 0*/)
{
	FRDGTextureSRV* InputTextureSRV = ScreenTextureSlice.TextureSRV;

	if (!InputTextureSRV)
	{
		return OverrideOutput;
	}

	FRDGTexture* InputTexture = InputTextureSRV->Desc.Texture;

	// We can avoid the copy if it's a 2D texture and there's no override output.
	if (InputTexture->Desc.IsTexture2D() && !InputTexture->Desc.IsTextureArray() && !OverrideOutput.IsValid())
	{
		return FScreenPassTexture(InputTexture, ScreenTextureSlice.ViewRect);
	}

	check(InputTexture->Desc.IsTexture2D() || InputTexture->Desc.IsTextureArray());

	FRDGTextureRef OutputTexture = OverrideOutput.Texture;

	if (!OutputTexture)
	{
		FRDGTextureDesc Desc = InputTexture->Desc;

		if (MultiViewCount > 0)
		{
			check(Desc.Dimension == ETextureDimension::Texture2DArray);
			check(Desc.ArraySize >= MultiViewCount);

			Desc.ArraySize = MultiViewCount;
		}
		else
		{
			Desc.Dimension = ETextureDimension::Texture2D;
			Desc.ArraySize = 1;
		}
		
		// If a pass uses blending to write to this post process texture, it needs to support being a render target, so make sure this flag is included.
		// Most post processing uses SceneColor or its FRDGTextureDesc, and SceneColor already has the RenderTargetable flag set, but TSR (the input to
		// the "Before Bloom" stage) writes to texture slices using compute, and its output doesn't have this flag.
		Desc.Flags |= ETextureCreateFlags::RenderTargetable;

		OutputTexture = GraphBuilder.CreateTexture(Desc, TEXT("CopyToScreenPassTexture2D"));
	}

	const FIntPoint ViewSize = ScreenTextureSlice.ViewRect.Size();

	FRHICopyTextureInfo CopyInfo;
	CopyInfo.SourceSliceIndex = InputTextureSRV->Desc.FirstArraySlice;
	CopyInfo.NumMips = InputTexture->Desc.NumMips;
	CopyInfo.SourcePosition = CopyInfo.DestPosition = FIntVector(ScreenTextureSlice.ViewRect.Min.X, ScreenTextureSlice.ViewRect.Min.Y, 0);
	CopyInfo.Size = FIntVector(ViewSize.X, ViewSize.Y, 1);

	if (MultiViewCount > 0)
	{
		CopyInfo.NumSlices = MultiViewCount;
	}
	
	AddCopyTexturePass(
		GraphBuilder,
		InputTexture,
		OutputTexture,
		CopyInfo);

	return FScreenPassTexture(OutputTexture, ScreenTextureSlice.ViewRect);
}

// static
FScreenPassTextureSlice FScreenPassTextureSlice::CreateFromScreenPassTexture(FRDGBuilder& GraphBuilder, const FScreenPassTexture& ScreenTexture)
{
	if (!ScreenTexture.Texture || !EnumHasAnyFlags(ScreenTexture.Texture->Desc.Flags, ETextureCreateFlags::ShaderResource))
	{
		return FScreenPassTextureSlice(nullptr, ScreenTexture.ViewRect);
	}

	return FScreenPassTextureSlice(GraphBuilder.CreateSRV(FRDGTextureSRVDesc(ScreenTexture.Texture)), ScreenTexture.ViewRect);
}

FScreenPassRenderTarget FScreenPassRenderTarget::CreateFromInput(
	FRDGBuilder& GraphBuilder,
	FScreenPassTexture Input,
	ERenderTargetLoadAction OutputLoadAction,
	const TCHAR* OutputName)
{
	check(Input.IsValid());

	FRDGTextureDesc OutputDesc = Input.Texture->Desc;
	OutputDesc.Reset();

	return FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, OutputName), Input.ViewRect, OutputLoadAction);
}

FScreenPassRenderTarget FScreenPassRenderTarget::CreateFromInput(
	FRDGBuilder& GraphBuilder,
	FRDGTexture* InputTexture,
	FIntPoint Extent,
	ERenderTargetLoadAction OutputLoadAction,
	const TCHAR* OutputName)
{
	check(InputTexture);

	FRDGTextureDesc OutputDesc = InputTexture->Desc;
	OutputDesc.Reset();
	EnumRemoveFlags(OutputDesc.Flags, ETextureCreateFlags::Presentable);
	OutputDesc.Extent = Extent;

	return FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, OutputName), OutputLoadAction);
}


FScreenPassRenderTarget FScreenPassRenderTarget::CreateViewFamilyOutput(FRDGTextureRef ViewFamilyTexture, const FViewInfo& View)
{
	if (!ViewFamilyTexture)
	{
		return FScreenPassRenderTarget{};
	}

	const FIntRect ViewRect = View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::RawOutput ? View.ViewRect : View.UnscaledViewRect;

	ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ENoAction;

	if (!View.IsFirstInFamily() || View.Family->bAdditionalViewFamily || View.IsRuntimeReflectionCapture())
	{
		LoadAction = ERenderTargetLoadAction::ELoad;
	}
	else if (ViewRect.Min != FIntPoint::ZeroValue || ViewRect.Size() != ViewFamilyTexture->Desc.Extent)
	{
		LoadAction = ERenderTargetLoadAction::EClear;
	}

	return FScreenPassRenderTarget(
		ViewFamilyTexture,
		// Raw output mode uses the original view rect. Otherwise the final unscaled rect is used.
		ViewRect,
		// First view clears the view family texture; all remaining views load.
		LoadAction);
}

FScreenPassTextureViewportParameters GetScreenPassTextureViewportParameters(const FScreenPassTextureViewport& InViewport)
{
	const FVector2f Extent(InViewport.Extent);
	const FVector2f ViewportMin(InViewport.Rect.Min.X, InViewport.Rect.Min.Y);
	const FVector2f ViewportMax(InViewport.Rect.Max.X, InViewport.Rect.Max.Y);
	const FVector2f ViewportSize = ViewportMax - ViewportMin;

	FScreenPassTextureViewportParameters Parameters;

	if (!InViewport.IsEmpty())
	{
		Parameters.Extent = Extent;
		Parameters.ExtentInverse = FVector2f(1.0f / Extent.X, 1.0f / Extent.Y);

		Parameters.ScreenPosToViewportScale = FVector2f(0.5f, -0.5f) * ViewportSize;
		Parameters.ScreenPosToViewportBias = (0.5f * ViewportSize) + ViewportMin;

		Parameters.ViewportMin = InViewport.Rect.Min;
		Parameters.ViewportMax = InViewport.Rect.Max;

		Parameters.ViewportSize = ViewportSize;
		Parameters.ViewportSizeInverse = FVector2f(1.0f / Parameters.ViewportSize.X, 1.0f / Parameters.ViewportSize.Y);

		Parameters.UVViewportMin = ViewportMin * Parameters.ExtentInverse;
		Parameters.UVViewportMax = ViewportMax * Parameters.ExtentInverse;

		Parameters.UVViewportSize = Parameters.UVViewportMax - Parameters.UVViewportMin;
		Parameters.UVViewportSizeInverse = FVector2f(1.0f / Parameters.UVViewportSize.X, 1.0f / Parameters.UVViewportSize.Y);

		Parameters.UVViewportBilinearMin = Parameters.UVViewportMin + 0.5f * Parameters.ExtentInverse;
		Parameters.UVViewportBilinearMax = Parameters.UVViewportMax - 0.5f * Parameters.ExtentInverse;
	}

	return Parameters;
}

// static
FScreenTransform FScreenTransform::ChangeTextureUVCoordinateFromTo(
	const FScreenPassTextureViewport& SrcViewport,
	const FScreenPassTextureViewport& DestViewport)
{
	return (
		ChangeTextureBasisFromTo(SrcViewport, ETextureBasis::TextureUV, ETextureBasis::ViewportUV) *
		ChangeTextureBasisFromTo(DestViewport, ETextureBasis::ViewportUV, ETextureBasis::TextureUV));
}

// static
FScreenTransform FScreenTransform::SvPositionToViewportUV(const FIntRect& SrcViewport)
{
	return (FScreenTransform::Identity - SrcViewport.Min) / SrcViewport.Size();
}

// static
FScreenTransform FScreenTransform::DispatchThreadIdToViewportUV(const FIntRect& SrcViewport)
{
	return (FScreenTransform::Identity + 0.5f) / SrcViewport.Size();
}

void SetScreenPassPipelineState(FRHICommandList& RHICmdList, const FScreenPassPipelineState& ScreenPassDraw)
{
	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.BlendState = ScreenPassDraw.BlendState;
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.DepthStencilState = ScreenPassDraw.DepthStencilState;
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = ScreenPassDraw.VertexDeclaration;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = ScreenPassDraw.VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ScreenPassDraw.PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, ScreenPassDraw.StencilRef);
}

FScreenPassViewInfo::FScreenPassViewInfo(const FSceneView& View)
	: FeatureLevel(View.FeatureLevel)
	, StereoViewIndex(View.StereoViewIndex)
	, InstanceCount(View.GetStereoPassInstanceFactor())
	, bHMDHiddenAreaMaskActive(View.bHMDHiddenAreaMaskActive)
{
}

FScreenPassViewInfo::FScreenPassViewInfo(const FSceneView& View, const uint32 OverrideInstanceCount)
	: FeatureLevel(View.FeatureLevel)
	, StereoViewIndex(View.StereoViewIndex)
	, InstanceCount(OverrideInstanceCount)
	, bHMDHiddenAreaMaskActive(View.bHMDHiddenAreaMaskActive)
{
}

void DrawScreenPass_PostSetup(
	FRHICommandList& RHICmdList,
	const FScreenPassViewInfo& ViewInfo,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& InputViewport,
	const FScreenPassPipelineState& PipelineState,
	EScreenPassDrawFlags Flags)
{
	const FIntRect InputRect = InputViewport.Rect;
	const FIntPoint InputSize = InputViewport.Extent;
	const FIntRect OutputRect = OutputViewport.Rect;
	const FIntPoint OutputSize = OutputViewport.Rect.Size();

	FIntPoint LocalOutputPos(FIntPoint::ZeroValue);
	FIntPoint LocalOutputSize(OutputSize);
	EDrawRectangleFlags DrawRectangleFlags = EDRF_UseTriangleOptimization;

	const bool bUseHMDHiddenAreaMask = EnumHasAllFlags(Flags, EScreenPassDrawFlags::AllowHMDHiddenAreaMask) && ViewInfo.bHMDHiddenAreaMaskActive;

	DrawPostProcessPass(
		RHICmdList,
		LocalOutputPos.X, LocalOutputPos.Y, LocalOutputSize.X, LocalOutputSize.Y,
		InputRect.Min.X, InputRect.Min.Y, InputRect.Width(), InputRect.Height(),
		OutputSize,
		InputSize,
		PipelineState.VertexShader,
		ViewInfo.StereoViewIndex,
		bUseHMDHiddenAreaMask,
		DrawRectangleFlags,
		ViewInfo.InstanceCount);
}

void AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	FScreenPassViewInfo ViewInfo,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	FIntPoint InputPosition,
	FIntPoint InputSize,
	FIntPoint OutputPosition,
	FIntPoint OutputSize,
	FRHISamplerState* Sampler)
{
	const FRDGTextureDesc& InputDesc = InputTexture->Desc;
	const FRDGTextureDesc& OutputDesc = OutputTexture->Desc;

	// Use a hardware copy if formats and sizes match.
	if (InputDesc.Format == OutputDesc.Format && InputSize == OutputSize)
	{
		return AddCopyTexturePass(GraphBuilder, InputTexture, OutputTexture, InputPosition, OutputPosition, InputSize);
	}

	if (InputSize == FIntPoint::ZeroValue)
	{
		// Copy entire input texture to output texture.
		InputSize = InputTexture->Desc.Extent;
	}

	// Don't prime color data if the whole texture is being overwritten.
	const ERenderTargetLoadAction LoadAction = (OutputPosition == FIntPoint::ZeroValue && InputSize == OutputDesc.Extent)
		? ERenderTargetLoadAction::ENoAction
		: ERenderTargetLoadAction::ELoad;

	const FScreenPassTextureViewport InputViewport(InputDesc.Extent, FIntRect(InputPosition, InputPosition + InputSize));
	const FScreenPassTextureViewport OutputViewport(OutputDesc.Extent, FIntRect(OutputPosition, OutputPosition + OutputSize));

	TShaderMapRef<FCopyRectPS> PixelShader(GetGlobalShaderMap(ViewInfo.FeatureLevel));

	FCopyRectPS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
	Parameters->InputTexture = InputTexture;
	Parameters->InputSampler = Sampler ? Sampler : TStaticSamplerState<>::GetRHI();
	Parameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, LoadAction);

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DrawTexture"), ViewInfo, OutputViewport, InputViewport, PixelShader, Parameters);
}

void AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	FScreenPassViewInfo ViewInfo,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	FIntPoint InputPosition,
	FIntPoint OutputPosition,
	FIntPoint Size,
	FRHISamplerState* Sampler)
{
	AddDrawTexturePass(
		GraphBuilder,
		ViewInfo,
		InputTexture,
		OutputTexture,
		InputPosition,
		Size,
		OutputPosition,
		Size,
		Sampler);
}

void AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	FScreenPassViewInfo ViewInfo,
	FScreenPassTexture Input,
	FScreenPassRenderTarget Output,
	uint32 RTMultiViewCount)
{
	const FScreenPassTextureViewport InputViewport(Input);
	const FScreenPassTextureViewport OutputViewport(Output);

	TShaderMapRef<FCopyRectPS> PixelShader(GetGlobalShaderMap(ViewInfo.FeatureLevel));

	FCopyRectPS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
	Parameters->InputTexture = Input.Texture;
	Parameters->InputSampler = TStaticSamplerState<>::GetRHI();
	Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	Parameters->RenderTargets.MultiViewCount = RTMultiViewCount;

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DrawTexture"), ViewInfo, OutputViewport, InputViewport, PixelShader, Parameters);
}

void AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	FScreenPassViewInfo ViewInfo,
	FScreenPassTextureSlice Input,
	FScreenPassRenderTarget Output)
{
	const FScreenPassTextureViewport InputViewport(Input);
	const FScreenPassTextureViewport OutputViewport(Output);

	TShaderMapRef<FCopyRectSrvPS> PixelShader(GetGlobalShaderMap(ViewInfo.FeatureLevel));

	FCopyRectSrvPS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyRectSrvPS::FParameters>();
	Parameters->InputTexture = Input.TextureSRV;
	Parameters->InputSampler = TStaticSamplerState<>::GetRHI();
	Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DrawTexture"), ViewInfo, OutputViewport, InputViewport, PixelShader, Parameters);
}

class FDownsampleDepthPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDownsampleDepthPS);
	SHADER_USE_PARAMETER_STRUCT(FDownsampleDepthPS, FGlobalShader);

	class FOutputMinAndMaxDepth : SHADER_PERMUTATION_BOOL("OUTPUT_MIN_AND_MAX_DEPTH");
	class FOutputMinMaxDepthFromMinMaxDepth : SHADER_PERMUTATION_BOOL("OUTPUT_MINMAXDEPTH_FROM_MINMAXDEPTH");

	using FPermutationDomain = TShaderPermutationDomain<FOutputMinAndMaxDepth, FOutputMinMaxDepthFromMinMaxDepth>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, MinMaxDepthTexture)
		SHADER_PARAMETER(FVector2f, DstToSrcPixelScale)
		SHADER_PARAMETER(FVector2f, SourceMaxUV)
		SHADER_PARAMETER(FVector2f, DestinationResolution)
		SHADER_PARAMETER(uint32, DownsampleDepthFilter)
		SHADER_PARAMETER(FIntVector4, DstPixelCoordMinAndMax)
		SHADER_PARAMETER(FIntVector4, SrcPixelCoordMinAndMax)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FOutputMinMaxDepthFromMinMaxDepth>() == true && PermutationVector.Get<FOutputMinAndMaxDepth>() == true)
		{
			return false;
		}
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDownsampleDepthPS, "/Engine/Private/DownsampleDepthPixelShader.usf", "Main", SF_Pixel);

void AddDownsampleDepthPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FScreenPassTexture Input,
	FScreenPassRenderTarget Output,
	EDownsampleDepthFilter DownsampleDepthFilter)
{
	const FScreenPassTextureViewport InputViewport(Input);
	const FScreenPassTextureViewport OutputViewport(Output);

	TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);

	const bool bIsMinAndMaxDepthFilter = DownsampleDepthFilter == EDownsampleDepthFilter::MinAndMaxDepth;
	const bool bIsMinAndMaxDepthFromMinMaxFilter = DownsampleDepthFilter == EDownsampleDepthFilter::MinAndMaxDepthFromMinAndMaxDepth;
	FDownsampleDepthPS::FPermutationDomain Permutation;
	Permutation.Set<FDownsampleDepthPS::FOutputMinAndMaxDepth>(bIsMinAndMaxDepthFilter ? 1 : 0);
	Permutation.Set<FDownsampleDepthPS::FOutputMinMaxDepthFromMinMaxDepth>(bIsMinAndMaxDepthFromMinMaxFilter ? 1 : 0);
	TShaderMapRef<FDownsampleDepthPS> PixelShader(View.ShaderMap, Permutation);

	// The lower right corner pixel whose coordinate is max considered excluded https://learn.microsoft.com/en-us/windows/win32/direct3d11/d3d11-rect
	// That is why we subtract -1 from the maximum value of the source viewport.

	FDownsampleDepthPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDownsampleDepthPS::FParameters>();
	PassParameters->View = View.GetShaderParameters();
	PassParameters->DepthTexture		= bIsMinAndMaxDepthFromMinMaxFilter ? GSystemTextures.GetDepthDummy(GraphBuilder) : Input.Texture;
	PassParameters->MinMaxDepthTexture	= bIsMinAndMaxDepthFromMinMaxFilter ? Input.Texture : GSystemTextures.GetBlackDummy(GraphBuilder);
	PassParameters->DstToSrcPixelScale = FVector2f(float(InputViewport.Extent.X) / float(OutputViewport.Extent.X), float(InputViewport.Extent.Y) / float(OutputViewport.Extent.Y));
	PassParameters->SourceMaxUV = FVector2f((float(View.ViewRect.Max.X) -1.0f - 0.51f) / InputViewport.Extent.X, (float(View.ViewRect.Max.Y) - 1.0f - 0.51f) / InputViewport.Extent.Y);
	PassParameters->DownsampleDepthFilter = (uint32)DownsampleDepthFilter;

	const int32 DownsampledSizeX = OutputViewport.Rect.Width();
	const int32 DownsampledSizeY = OutputViewport.Rect.Height();
	PassParameters->DestinationResolution = FVector2f(DownsampledSizeX, DownsampledSizeY);

	PassParameters->DstPixelCoordMinAndMax = FIntVector4(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, OutputViewport.Rect.Max.X-1, OutputViewport.Rect.Max.Y-1);
	PassParameters->SrcPixelCoordMinAndMax = FIntVector4( InputViewport.Rect.Min.X,  InputViewport.Rect.Min.Y,  InputViewport.Rect.Max.X-1,  InputViewport.Rect.Max.Y-1);

	FRHIDepthStencilState* DepthStencilState = TStaticDepthStencilState<true, CF_Always>::GetRHI();

	if (bIsMinAndMaxDepthFilter || bIsMinAndMaxDepthFromMinMaxFilter)
	{
		DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(Output.Texture, Output.LoadAction);
	}
	else
	{
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Output.Texture, Output.LoadAction, Output.LoadAction, FExclusiveDepthStencil::DepthWrite_StencilWrite);
	}

	static const TCHAR* kFilterNames[] = {
		TEXT("Point"),
		TEXT("Max"),
		TEXT("CheckerMinMax"),
		TEXT("MinAndMaxDepth"),
		TEXT("MinMaxFromMinMaxDepth"),
	};

	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("DownsampleDepth(%s) %dx%dx -> %dx%d",
			kFilterNames[int32(DownsampleDepthFilter)],
			InputViewport.Rect.Width(),
			InputViewport.Rect.Height(),
			OutputViewport.Rect.Width(),
			OutputViewport.Rect.Height()),
		View,
		OutputViewport, InputViewport,
		VertexShader, PixelShader,
		DepthStencilState,
		PassParameters);
}

FCanvas& CreateCanvasForRenderTarget(FRDGBuilder& GraphBuilder, const FSceneView& View, const FScreenPassRenderTarget& RenderTarget)
{
	const FSceneViewFamily& ViewFamily = *View.Family;
	FCanvas& Canvas = *FCanvas::Create(GraphBuilder, RenderTarget.Texture, nullptr, ViewFamily.Time, View.GetFeatureLevel(), ViewFamily.DebugDPIScale);
	Canvas.SetRenderTargetRect(RenderTarget.ViewRect);
	return Canvas;
}
