// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenSpaceDenoise.cpp: Denoise in screen space.
=============================================================================*/

#include "ScreenSpaceDenoise.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "LightSceneProxy.h"
#include "LightComponentId.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "PostProcess/SceneRenderTargets.h"
#include "SceneRenderTargetParameters.h"
#include "ClearQuad.h"
#include "PipelineStateCache.h"
#include "SceneTextureParameters.h"
#include "SystemTextures.h"
#include "Lumen/Lumen.h"
#include "SceneViewState.h"
#include "LightSceneInfo.h"

// ---------------------------------------------------- Cvars


static TAutoConsoleVariable<int32> CVarShadowReconstructionSampleCount(
	TEXT("r.Shadow.Denoiser.ReconstructionSamples"), 8,
	TEXT("Maximum number of samples for the reconstruction pass (default = 16)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowPreConvolutionCount(
	TEXT("r.Shadow.Denoiser.PreConvolution"), 1,
	TEXT("Number of pre-convolution passes (default = 1)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowTemporalAccumulation(
	TEXT("r.Shadow.Denoiser.TemporalAccumulation"), 1,
	TEXT(""),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowHistoryConvolutionSampleCount(
	TEXT("r.Shadow.Denoiser.HistoryConvolutionSamples"), 1,
	TEXT("Number of samples to use to convolve the history over time."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowDenoiserTransmission(
	TEXT("r.Shadow.Denoiser.Transmission"), 1,
	TEXT("Whether to denoise the transmission component for ray traced shadow.\n")
	TEXT(" 0: Use the raw transmission from the ray traced shadow pass.\n")
	TEXT(" 1: Denoise the transmission, might have transmission bleeding (default)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarUseReflectionDenoiser(
	TEXT("r.Reflections.Denoiser"),
	2,
	TEXT("Choose the denoising algorithm.\n")
	TEXT(" 0: Disabled;\n")
	TEXT(" 1: Forces the default denoiser of the renderer;\n")
	TEXT(" 2: GScreenSpaceDenoiser which may be overriden by a third party plugin (default)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarReflectionReconstructionSampleCount(
	TEXT("r.Reflections.Denoiser.ReconstructionSamples"), 8,
	TEXT("Maximum number of samples for the reconstruction pass (default = 8)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarReflectionPreConvolutionCount(
	TEXT("r.Reflections.Denoiser.PreConvolution"), 1,
	TEXT("Number of pre-convolution passes (default = 1)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarReflectionTemporalAccumulation(
	TEXT("r.Reflections.Denoiser.TemporalAccumulation"), 1,
	TEXT("Accumulates the samples over multiple frames."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAOReconstructionSampleCount(
	TEXT("r.AmbientOcclusion.Denoiser.ReconstructionSamples"), 16,
	TEXT("Maximum number of samples for the reconstruction pass (default = 16)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAOPreConvolutionCount(
	TEXT("r.AmbientOcclusion.Denoiser.PreConvolution"), 2,
	TEXT("Number of pre-convolution passes (default = 1)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarAOKernelSpreadFactor(
	TEXT("r.AmbientOcclusion.Denoiser.KernelSpreadFactor"), 4,
	TEXT("Spread factor of the preconvolution passes."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAOTemporalAccumulation(
	TEXT("r.AmbientOcclusion.Denoiser.TemporalAccumulation"), 1,
	TEXT("Accumulates the samples over multiple frames."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAOHistoryConvolutionSampleCount(
	TEXT("r.AmbientOcclusion.Denoiser.HistoryConvolution.SampleCount"), 1,
	TEXT("Number of samples to use for history post filter (default = 16)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarAOHistoryConvolutionKernelSpreadFactor(
	TEXT("r.AmbientOcclusion.Denoiser.HistoryConvolution.KernelSpreadFactor"), 7,
	TEXT("Multiplication factor applied on the kernel sample offset (default = 7)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarGIReconstructionSampleCount(
	TEXT("r.GlobalIllumination.Denoiser.ReconstructionSamples"), 16,
	TEXT("Maximum number of samples for the reconstruction pass (default = 16)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarGIPreConvolutionCount(
	TEXT("r.GlobalIllumination.Denoiser.PreConvolution"), 1,
	TEXT("Number of pre-convolution passes (default = 1)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarGITemporalAccumulation(
	TEXT("r.GlobalIllumination.Denoiser.TemporalAccumulation"), 1,
	TEXT("Accumulates the samples over multiple frames."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarGIHistoryConvolutionSampleCount(
	TEXT("r.GlobalIllumination.Denoiser.HistoryConvolution.SampleCount"), 1,
	TEXT("Number of samples to use for history post filter (default = 1)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarGIHistoryConvolutionKernelSpreadFactor(
	TEXT("r.GlobalIllumination.Denoiser.HistoryConvolution.KernelSpreadFactor"), 3,
	TEXT("Multiplication factor applied on the kernel sample offset (default=3)."),
	ECVF_RenderThreadSafe);

/** The maximum number of mip level supported in the denoiser. */
// TODO(Denoiser): jump to 3 because bufefr size already have a size multiple of 4.
static const int32 kMaxMipLevel = 2;

/** Maximum number of sample per pixel supported in the stackowiak sample set. */
static const int32 kStackowiakMaxSampleCountPerSet = 56;

/** The maximum number of buffers. */
static const int32 kMaxBufferProcessingCount = kMaxDenoiserBufferProcessingCount;

/** Number of texture to store compressed metadata. */
static const int32 kCompressedMetadataTextures = 2;

static_assert(IScreenSpaceDenoiser::kMaxBatchSize <= kMaxBufferProcessingCount, "Can't batch more signal than there is internal buffer in the denoiser.");


// ---------------------------------------------------- Globals

const IScreenSpaceDenoiser* GScreenSpaceDenoiser = nullptr;

DECLARE_GPU_STAT(ReflectionsDenoiser);
DECLARE_GPU_STAT(ShadowsDenoiser);
DECLARE_GPU_STAT(AmbientOcclusionDenoiser);
DECLARE_GPU_STAT(DiffuseIndirectDenoiser);

namespace
{

// ---------------------------------------------------- Enums

/** Layout for compressed meta data. */
enum class ECompressedMetadataLayout
{
	// The signal denoiser use directly depth buffer and gbuffer.
	Disabled,

	// Compress scene depth and world space normal into same render target.
	DepthAndNormal,

	// Compress scene depth and view space normal into same render target The advantage of having the normal
	// in the view space is to use much faster ScreenToView than ScreenToTranslatedWorld. But doesn't
	// support history bilateral rejection.
	DepthAndViewNormal,

	// Scene depth and shading model ID are in separate render target.
	FedDepthAndShadingModelID,

	MAX,
};

/** Different signals to denoise. */
enum class ESignalProcessing
{
	// Denoise a shadow mask.
	ShadowVisibilityMask,

	// Denoise one lighting harmonic when denoising multiple light's penumbra.
	PolychromaticPenumbraHarmonic,

	// Denoise first bounce specular.
	Reflections,

	// Denoise ambient occlusion.
	AmbientOcclusion,

	// Denoise first bounce diffuse and ambient occlusion.
	DiffuseAndAmbientOcclusion,

	// Denoise first bounce diffuse as sperical harmonic
	DiffuseSphericalHarmonic,

	// Denoise SSGI.
	ScreenSpaceDiffuseIndirect,

	// Denoise diffuse indirect hierarchy.
	IndirectProbeHierarchy,

	MAX,
};


// ---------------------------------------------------- Simple functions

static bool IsSupportedLightType(ELightComponentType LightType)
{
	return LightType == LightType_Point || LightType == LightType_Directional || LightType == LightType_Rect || LightType == LightType_Spot;
}

/** Returns whether a signal processing is supported by the constant pixel density pass layout. */
static bool UsesConstantPixelDensityPassLayout(ESignalProcessing SignalProcessing)
{
	return (
		SignalProcessing == ESignalProcessing::ShadowVisibilityMask ||
		SignalProcessing == ESignalProcessing::PolychromaticPenumbraHarmonic ||
		SignalProcessing == ESignalProcessing::Reflections ||
		SignalProcessing == ESignalProcessing::AmbientOcclusion ||
		SignalProcessing == ESignalProcessing::DiffuseAndAmbientOcclusion ||
		SignalProcessing == ESignalProcessing::DiffuseSphericalHarmonic ||
		SignalProcessing == ESignalProcessing::ScreenSpaceDiffuseIndirect ||
		SignalProcessing == ESignalProcessing::IndirectProbeHierarchy);
}

/** Returns whether a signal processing support upscaling. */
static bool SignalSupportsUpscaling(ESignalProcessing SignalProcessing)
{
	return (
		SignalProcessing == ESignalProcessing::Reflections ||
		SignalProcessing == ESignalProcessing::AmbientOcclusion ||
		SignalProcessing == ESignalProcessing::DiffuseAndAmbientOcclusion);
}

/** Returns whether a signal processing uses an injestion pass. */
static bool SignalUsesInjestion(ESignalProcessing SignalProcessing)
{
	return (
		SignalProcessing == ESignalProcessing::ShadowVisibilityMask);
}

/** Returns whether a signal processing uses a reduction pass before the reconstruction. */
static bool SignalUsesReduction(ESignalProcessing SignalProcessing)
{
	return false; //SignalProcessing == ESignalProcessing::DiffuseSphericalHarmonic;
}

/** Returns whether a signal processing uses an additional pre convolution pass. */
static bool SignalUsesPreConvolution(ESignalProcessing SignalProcessing)
{
	return
		SignalProcessing == ESignalProcessing::ShadowVisibilityMask ||
		SignalProcessing == ESignalProcessing::Reflections ||
		SignalProcessing == ESignalProcessing::AmbientOcclusion ||
		SignalProcessing == ESignalProcessing::DiffuseAndAmbientOcclusion;
}

/** Returns whether a signal processing uses a history rejection pre convolution pass. */
static bool SignalUsesRejectionPreConvolution(ESignalProcessing SignalProcessing)
{
	return (
		//SignalProcessing == ESignalProcessing::ShadowVisibilityMask ||
		//SignalProcessing == ESignalProcessing::Reflections ||
		SignalProcessing == ESignalProcessing::AmbientOcclusion);
}

/** Returns whether a signal processing uses a convolution pass after temporal accumulation pass. */
static bool SignalUsesPostConvolution(ESignalProcessing SignalProcessing)
{
	return (
		SignalProcessing == ESignalProcessing::ShadowVisibilityMask ||
		SignalProcessing == ESignalProcessing::AmbientOcclusion ||
		SignalProcessing == ESignalProcessing::DiffuseAndAmbientOcclusion);
}

/** Returns whether a signal processing uses a history rejection pre convolution pass. */
static bool SignalUsesFinalConvolution(ESignalProcessing SignalProcessing)
{
	return (
		SignalProcessing == ESignalProcessing::ShadowVisibilityMask);
}

/** Returns what meta data compression should be used when denoising a signal. */
static ECompressedMetadataLayout GetSignalCompressedMetadata(ESignalProcessing SignalProcessing)
{
	if (SignalProcessing == ESignalProcessing::ScreenSpaceDiffuseIndirect)
	{
		return ECompressedMetadataLayout::DepthAndViewNormal;
	}
	else if (SignalProcessing == ESignalProcessing::IndirectProbeHierarchy)
	{
		return ECompressedMetadataLayout::FedDepthAndShadingModelID;
	}
	return ECompressedMetadataLayout::Disabled;
}

/** Returns the number of signal that might be batched at the same time. */
static int32 SignalMaxBatchSize(ESignalProcessing SignalProcessing)
{
	if (SignalProcessing == ESignalProcessing::ShadowVisibilityMask
		)
	{
		return IScreenSpaceDenoiser::kMaxBatchSize;
	}
	else if (
		SignalProcessing == ESignalProcessing::Reflections ||
		SignalProcessing == ESignalProcessing::PolychromaticPenumbraHarmonic ||
		SignalProcessing == ESignalProcessing::AmbientOcclusion ||
		SignalProcessing == ESignalProcessing::DiffuseAndAmbientOcclusion ||
		SignalProcessing == ESignalProcessing::DiffuseSphericalHarmonic ||
		SignalProcessing == ESignalProcessing::ScreenSpaceDiffuseIndirect ||
		SignalProcessing == ESignalProcessing::IndirectProbeHierarchy)
	{
		return 1;
	}
	check(0);
	return 1;
}

/** Returns whether a signal have a code path for 1 sample per pixel. */
static bool SignalSupport1SPP(ESignalProcessing SignalProcessing)
{
	return (
		SignalProcessing == ESignalProcessing::DiffuseAndAmbientOcclusion);
}

/** Returns whether a signal can denoise multi sample per pixel. */
static bool SignalSupportMultiSPP(ESignalProcessing SignalProcessing)
{
	return (
		SignalProcessing == ESignalProcessing::ShadowVisibilityMask ||
		SignalProcessing == ESignalProcessing::PolychromaticPenumbraHarmonic ||
		SignalProcessing == ESignalProcessing::Reflections ||
		SignalProcessing == ESignalProcessing::AmbientOcclusion ||
		SignalProcessing == ESignalProcessing::DiffuseAndAmbientOcclusion ||
		SignalProcessing == ESignalProcessing::DiffuseSphericalHarmonic ||
		SignalProcessing == ESignalProcessing::ScreenSpaceDiffuseIndirect ||
		SignalProcessing == ESignalProcessing::IndirectProbeHierarchy);
}


// ---------------------------------------------------- Shaders

// Permutation dimension for the type of signal being denoised.
class FSignalProcessingDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_SIGNAL_PROCESSING", ESignalProcessing);

// Permutation dimension for the number of signal being denoised at the same time.
class FSignalBatchSizeDim : SHADER_PERMUTATION_RANGE_INT("DIM_SIGNAL_BATCH_SIZE", 1, IScreenSpaceDenoiser::kMaxBatchSize);

// Permutation dimension for denoising multiple sample at same time.
class FMultiSPPDim : SHADER_PERMUTATION_BOOL("DIM_MULTI_SPP");


const TCHAR* const kInjestResourceNames[] = {
	// ShadowVisibilityMask
	TEXT("Shadow.Denoiser.Injest0"),
	TEXT("Shadow.Denoiser.Injest1"),
	nullptr,
	nullptr,

	// PolychromaticPenumbraHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// Reflections
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// AmbientOcclusion
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// DiffuseIndirect
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// DiffuseSphericalHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// ScreenSpaceDiffuseIndirect
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// IndirectProbeHierarchy
	nullptr,
	nullptr,
	nullptr,
	nullptr,
};

const TCHAR* const kReduceResourceNames[] = {
	// ShadowVisibilityMask
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// PolychromaticPenumbraHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// Reflections
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// AmbientOcclusion
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// DiffuseIndirect
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// DiffuseSphericalHarmonic
	TEXT("DiffuseHarmonicReduce0"),
	TEXT("DiffuseHarmonicReduce1"),
	TEXT("DiffuseHarmonicReduce2"),
	TEXT("DiffuseHarmonicReduce3"),

	// ScreenSpaceDiffuseIndirect
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// IndirectProbeHierarchy
	nullptr,
	nullptr,
	nullptr,
	nullptr,
};

const TCHAR* const kReconstructionResourceNames[] = {
	// ShadowVisibilityMask
	TEXT("Shadow.Denoiser.Reconstruction0"),
	TEXT("Shadow.Denoiser.Reconstruction1"),
	TEXT("Shadow.Denoiser.Reconstruction2"),
	TEXT("Shadow.Denoiser.Reconstruction3"),

	// PolychromaticPenumbraHarmonic
	TEXT("PolychromaticPenumbraHarmonicReconstruction0"),
	TEXT("PolychromaticPenumbraHarmonicReconstruction1"),
	TEXT("PolychromaticPenumbraHarmonicReconstruction2"),
	TEXT("PolychromaticPenumbraHarmonicReconstruction3"),

	// Reflections
	TEXT("Reflections.Denoiser.Reconstruction0"),
	TEXT("Reflections.Denoiser.Reconstruction1"),
	nullptr,
	nullptr,

	// AmbientOcclusion
	TEXT("AO.Denoiser.Reconstruction0"),
	nullptr,
	nullptr,
	nullptr,

	// DiffuseIndirect
	TEXT("DiffuseIndirectReconstruction0"),
	TEXT("DiffuseIndirectReconstruction1"),
	nullptr,
	nullptr,

	// DiffuseSphericalHarmonic
	TEXT("DiffuseHarmonicReconstruction0"),
	TEXT("DiffuseHarmonicReconstruction1"),
	TEXT("DiffuseHarmonicReconstruction2"),
	TEXT("DiffuseHarmonicReconstruction3"),

	// ScreenSpaceDiffuseIndirect
	TEXT("SSGI.Denoiser.Reconstruction0"),
	TEXT("SSGI.Denoiser.Reconstruction1"),
	nullptr,
	nullptr,

	// IndirectProbeHierarchy
	nullptr,
	nullptr,
	nullptr,
	nullptr,
};

const TCHAR* const kPreConvolutionResourceNames[] = {
	// ShadowVisibilityMask
	TEXT("Shadow.Denoiser.PreConvolution0"),
	TEXT("Shadow.Denoiser.PreConvolution1"),
	TEXT("Shadow.Denoiser.PreConvolution2"),
	TEXT("Shadow.Denoiser.PreConvolution3"),

	// PolychromaticPenumbraHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// Reflections
	TEXT("Reflections.Denoiser.PreConvolution0"),
	TEXT("Reflections.Denoiser.PreConvolution1"),
	nullptr,
	nullptr,

	// AmbientOcclusion
	TEXT("AO.Denoiser.PreConvolution0"),
	nullptr,
	nullptr,
	nullptr,

	// DiffuseIndirect
	TEXT("DiffuseIndirectPreConvolution0"),
	TEXT("DiffuseIndirectPreConvolution1"),
	nullptr,
	nullptr,

	// DiffuseSphericalHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// ScreenSpaceDiffuseIndirect
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// IndirectProbeHierarchy
	nullptr,
	nullptr,
	nullptr,
	nullptr,
};

const TCHAR* const kRejectionPreConvolutionResourceNames[] = {
	// ShadowVisibilityMask
	TEXT("Shadow.Denoiser.RejectionPreConvolution0"),
	TEXT("Shadow.Denoiser.RejectionPreConvolution1"),
	TEXT("Shadow.Denoiser.RejectionPreConvolution2"),
	TEXT("Shadow.Denoiser.RejectionPreConvolution3"),

	// PolychromaticPenumbraHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// Reflections
	TEXT("Reflections.Denoiser.RejectionPreConvolution0"),
	TEXT("Reflections.Denoiser.RejectionPreConvolution1"),
	TEXT("Reflections.Denoiser.RejectionPreConvolution2"),
	nullptr,

	// AmbientOcclusion
	TEXT("AO.Denoiser.RejectionPreConvolution0"),
	nullptr,
	nullptr,
	nullptr,

	// DiffuseIndirect
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// DiffuseSphericalHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// ScreenSpaceDiffuseIndirect
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// IndirectProbeHierarchy
	nullptr,
	nullptr,
	nullptr,
	nullptr,
};

const TCHAR* const kTemporalAccumulationResourceNames[] = {
	// ShadowVisibilityMask
	TEXT("Shadow.Denoiser.TemporalAccumulation0"),
	TEXT("Shadow.Denoiser.TemporalAccumulation1"),
	TEXT("Shadow.Denoiser.TemporalAccumulation2"),
	TEXT("Shadow.Denoiser.TemporalAccumulation3"),

	// PolychromaticPenumbraHarmonic
	TEXT("PolychromaticPenumbraHistory0"),
	TEXT("PolychromaticPenumbraHistory1"),
	nullptr,
	nullptr,

	// Reflections
	TEXT("Reflections.Denoiser.TemporalAccumulation0"),
	TEXT("Reflections.Denoiser.TemporalAccumulation1"),
	nullptr,
	nullptr,

	// AmbientOcclusion
	TEXT("AO.Denoiser.TemporalAccumulation0"),
	nullptr,
	nullptr,
	nullptr,

	// DiffuseIndirect
	TEXT("DiffuseIndirect.Denoiser.TemporalAccumulation0"),
	TEXT("DiffuseIndirect.Denoiser.TemporalAccumulation1"),
	nullptr,
	nullptr,

	// DiffuseSphericalHarmonic
	TEXT("DiffuseHarmonicTemporalAccumulation0"),
	TEXT("DiffuseHarmonicTemporalAccumulation1"),
	TEXT("DiffuseHarmonicTemporalAccumulation2"),
	TEXT("DiffuseHarmonicTemporalAccumulation3"),

	// ScreenSpaceDiffuseIndirect
	TEXT("SSGI.Denoiser.TemporalAccumulation0"),
	TEXT("SSGI.Denoiser.TemporalAccumulation1"),
	nullptr,
	nullptr,

	// IndirectProbeHierarchy
	TEXT("ProbeHierarchy.TemporalAccumulation0"),
	TEXT("ProbeHierarchy.TemporalAccumulation1"),
	TEXT("ProbeHierarchy.TemporalAccumulation2"),
	nullptr,
};

const TCHAR* const kHistoryConvolutionResourceNames[] = {
	// ShadowVisibilityMask
	TEXT("Shadow.Denoiser.HistoryConvolution0"),
	TEXT("Shadow.Denoiser.HistoryConvolution1"),
	TEXT("Shadow.Denoiser.HistoryConvolution2"),
	TEXT("Shadow.Denoiser.HistoryConvolution3"),

	// PolychromaticPenumbraHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// Reflections
	TEXT("Reflections.Denoiser.HistoryConvolution0"),
	TEXT("Reflections.Denoiser.HistoryConvolution1"),
	nullptr,
	nullptr,

	// AmbientOcclusion
	TEXT("AO.Denoiser.HistoryConvolution0"),
	nullptr,
	nullptr,
	nullptr,

	// DiffuseIndirect
	TEXT("DiffuseIndirect.Denoiser.HistoryConvolution0"),
	TEXT("DiffuseIndirect.Denoiser.HistoryConvolution1"),
	nullptr,
	nullptr,

	// DiffuseSphericalHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// ScreenSpaceDiffuseIndirect
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// IndirectProbeHierarchy
	nullptr,
	nullptr,
	nullptr,
	nullptr,
};

const TCHAR* const kDenoiserOutputResourceNames[] = {
	// ShadowVisibilityMask
	TEXT("Shadow.Denoiser.DenoiserOutput0"),
	TEXT("Shadow.Denoiser.DenoiserOutput1"),
	TEXT("Shadow.Denoiser.DenoiserOutput2"),
	TEXT("Shadow.Denoiser.DenoiserOutput3"),

	// PolychromaticPenumbraHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// Reflections
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// AmbientOcclusion
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// DiffuseIndirect
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// DiffuseSphericalHarmonic
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// ScreenSpaceDiffuseIndirect
	nullptr,
	nullptr,
	nullptr,
	nullptr,

	// IndirectProbeHierarchy
	nullptr,
	nullptr,
	nullptr,
	nullptr,
};

static_assert(UE_ARRAY_COUNT(kReconstructionResourceNames) == int32(ESignalProcessing::MAX) * kMaxBufferProcessingCount, "You forgot me!");
static_assert(UE_ARRAY_COUNT(kRejectionPreConvolutionResourceNames) == int32(ESignalProcessing::MAX) * kMaxBufferProcessingCount, "You forgot me!");
static_assert(UE_ARRAY_COUNT(kTemporalAccumulationResourceNames) == int32(ESignalProcessing::MAX) * kMaxBufferProcessingCount, "You forgot me!");
static_assert(UE_ARRAY_COUNT(kHistoryConvolutionResourceNames) == int32(ESignalProcessing::MAX) * kMaxBufferProcessingCount, "You forgot me!");
static_assert(UE_ARRAY_COUNT(kDenoiserOutputResourceNames) == int32(ESignalProcessing::MAX) * kMaxBufferProcessingCount, "You forgot me!");


/** Returns whether should compile pipeline for a given shader platform.*/
static bool ShouldCompileSignalPipeline(ESignalProcessing SignalProcessing, EShaderPlatform Platform)
{
	// Explicitly disable SSD shaders on mobile as they will fail to compile at this moment.
	if (IsMobilePlatform(Platform))
	{
		return false;
	}

	if (SignalProcessing == ESignalProcessing::ScreenSpaceDiffuseIndirect)
	{
		return FDataDrivenShaderPlatformInfo::GetCompileSignalProcessingPipeline(Platform) || FDataDrivenShaderPlatformInfo::GetSupportsSSDIndirect(Platform);
	}
	else if (SignalProcessing == ESignalProcessing::Reflections)
	{
		return RHISupportsRayTracingShaders(Platform);
	}
	else if (
		SignalProcessing == ESignalProcessing::ShadowVisibilityMask ||
		SignalProcessing == ESignalProcessing::AmbientOcclusion ||
		SignalProcessing == ESignalProcessing::DiffuseAndAmbientOcclusion)
	{
		// Only for ray tracing denoising.
		return RHISupportsRayTracing(Platform);
	}
	else if (SignalProcessing == ESignalProcessing::PolychromaticPenumbraHarmonic)
	{
		return false;
	}
	else if (
		SignalProcessing == ESignalProcessing::DiffuseSphericalHarmonic ||
		SignalProcessing == ESignalProcessing::IndirectProbeHierarchy)
	{
		return DoesPlatformSupportLumenGI(Platform);
	}
	check(0);
	return false;
}


/** Shader parameter structure used for all shaders. */
BEGIN_SHADER_PARAMETER_STRUCT(FSSDCommonParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(Denoiser::FCommonShaderParameters, PublicCommonParameters)
	SHADER_PARAMETER(FIntPoint, ViewportMin)
	SHADER_PARAMETER(FIntPoint, ViewportMax)
	SHADER_PARAMETER(FVector4f, ThreadIdToBufferUV)
	SHADER_PARAMETER(FVector2f, BufferUVToOutputPixelPosition)
	SHADER_PARAMETER(FMatrix44f, ScreenToView)
	SHADER_PARAMETER(FVector2f, BufferUVBilinearCorrection)

	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)

	SHADER_PARAMETER_RDG_TEXTURE_ARRAY(Texture2D<uint>, CompressedMetadata, [kCompressedMetadataTextures])

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, TileClassificationTexture)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)

	SHADER_PARAMETER(uint32, FrameIndex)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSSDSignalSRVs, )
	SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2D, Textures, [kMaxBufferProcessingCount])
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSSDSignalUAVs, )
	SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D, UAVs, [kMaxBufferProcessingCount])
END_SHADER_PARAMETER_STRUCT()

/** Shader parameter structure to have all information to spatial filtering. */
BEGIN_SHADER_PARAMETER_STRUCT(FSSDConvolutionMetaData, )
	SHADER_PARAMETER_ARRAY(FVector4f, LightPositionAndRadius, [IScreenSpaceDenoiser::kMaxBatchSize])
	SHADER_PARAMETER_ARRAY(FVector4f, LightDirectionAndLength, [IScreenSpaceDenoiser::kMaxBatchSize])
	SHADER_PARAMETER_SCALAR_ARRAY(float, HitDistanceToWorldBluringRadius, [IScreenSpaceDenoiser::kMaxBatchSize])
	SHADER_PARAMETER_SCALAR_ARRAY(uint32, LightType, [IScreenSpaceDenoiser::kMaxBatchSize])
END_SHADER_PARAMETER_STRUCT()


FSSDSignalTextures CreateMultiplexedTextures(
	FRDGBuilder& GraphBuilder,
	int32 TextureCount,
	const TStaticArray<FRDGTextureDesc, kMaxBufferProcessingCount>& DescArray,
	const TCHAR* const* TextureNames,
	const FIntRect& Viewport)
{
	check(TextureCount <= kMaxBufferProcessingCount);
	FSSDSignalTextures SignalTextures;
	for (int32 i = 0; i < TextureCount; i++)
	{
		const TCHAR* TextureName = TextureNames[i];
		SignalTextures.Textures[i] = GraphBuilder.CreateTexture(DescArray[i], TextureName);

		SignalTextures.Textures[i]->EncloseVisualizeExtent(Viewport.Max);
	}
	return SignalTextures;
}

FSSDSignalSRVs CreateMultiplexedUintSRVs(FRDGBuilder& GraphBuilder, const FSSDSignalTextures& SignalTextures)
{
	FSSDSignalSRVs SRVs;
	for (int32 i = 0; i < kMaxBufferProcessingCount; i++)
	{
		if (SignalTextures.Textures[i])
		{
			EPixelFormat Format = SignalTextures.Textures[i]->Desc.Format;
			int32 Bytes = GPixelFormats[Format].BlockBytes;

			EPixelFormat UIntFormat = PF_Unknown;
			if (Bytes == 1)
				UIntFormat = PF_R8_UINT;
			else if (Bytes == 2)
				UIntFormat = PF_R16_UINT;
			else if (Bytes == 4)
				UIntFormat = PF_R32_UINT;
			else if (Bytes == 8)
				UIntFormat = PF_R32G32_UINT;
			else if (Bytes == 16)
				UIntFormat = PF_R32G32B32A32_UINT;
			else
			{
				check(0);
			}

			SRVs.Textures[i] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(SignalTextures.Textures[i], UIntFormat));
		}
	}
	return SRVs;
}

FSSDSignalUAVs CreateMultiplexedUAVs(FRDGBuilder& GraphBuilder, const FSSDSignalTextures& SignalTextures, int32 MipLevel = 0)
{
	FSSDSignalUAVs UAVs;
	for (int32 i = 0; i < kMaxBufferProcessingCount; i++)
	{
		if (SignalTextures.Textures[i])
			UAVs.UAVs[i] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SignalTextures.Textures[i], MipLevel));
	}
	return UAVs;
}

class FSSDCompressMetadataCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSDCompressMetadataCS);
	SHADER_USE_PARAMETER_STRUCT(FSSDCompressMetadataCS, FGlobalShader);

	class FMetadataLayoutDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_METADATA_LAYOUT", ECompressedMetadataLayout);
	using FPermutationDomain = TShaderPermutationDomain<FMetadataLayoutDim>;


	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FMetadataLayoutDim>() == ECompressedMetadataLayout::Disabled)
		{
			return false;
		}

		// Precomputed by denoiser caller.
		if (PermutationVector.Get<FMetadataLayoutDim>() == ECompressedMetadataLayout::FedDepthAndShadingModelID)
		{
			return false;
		}

		return ShouldCompileSignalPipeline(ESignalProcessing::ScreenSpaceDiffuseIndirect, Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSDCommonParameters, CommonParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<uint>, CompressedMetadataOutput, [kCompressedMetadataTextures])
	END_SHADER_PARAMETER_STRUCT()
};

class FSSDInjestCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSDInjestCS);
	SHADER_USE_PARAMETER_STRUCT(FSSDInjestCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSignalProcessingDim, FSignalBatchSizeDim, FMultiSPPDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		ESignalProcessing SignalProcessing = PermutationVector.Get<FSignalProcessingDim>();

		// Only compile this shader for signal processing that uses it.
		if (!SignalUsesInjestion(SignalProcessing))
		{
			return false;
		}

		// Not all signal processing allow to batch multiple signals at the same time.
		if (PermutationVector.Get<FSignalBatchSizeDim>() > SignalMaxBatchSize(SignalProcessing))
		{
			return false;
		}

		// Only compiler multi SPP permutation for signal that supports it.
		if (PermutationVector.Get<FMultiSPPDim>() && !SignalSupportMultiSPP(SignalProcessing))
		{
			return false;
		}

		// Compile out the shader if this permutation gets remapped.
		if (RemapPermutationVector(PermutationVector) != PermutationVector)
		{
			return false;
		}

		return ShouldCompileSignalPipeline(SignalProcessing, Parameters.Platform);
	}

	static FPermutationDomain RemapPermutationVector(FPermutationDomain PermutationVector)
	{
		ESignalProcessing SignalProcessing = PermutationVector.Get<FSignalProcessingDim>();

		// force use the multi sample per pixel code path.
		if (!SignalSupport1SPP(SignalProcessing))
		{
			PermutationVector.Set<FMultiSPPDim>(true);
		}

		return PermutationVector;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSDCommonParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSDConvolutionMetaData, ConvolutionMetaData)

		SHADER_PARAMETER_STRUCT(FSSDSignalTextures, SignalInput)
		SHADER_PARAMETER_STRUCT(FSSDSignalUAVs, SignalOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FSSDSpatialAccumulationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSDSpatialAccumulationCS);
	SHADER_USE_PARAMETER_STRUCT(FSSDSpatialAccumulationCS, FGlobalShader);

	static const uint32 kGroupSize = 8;
	
	enum class EStage
	{
		// Spatial kernel used to process raw input for the temporal accumulation.
		ReConstruction,

		// Spatial kernel to pre filter.
		PreConvolution,

		// Spatial kernel used to pre convolve history rejection.
		RejectionPreConvolution,

		// Spatial kernel used to post filter the temporal accumulation.
		PostFiltering,

		// Final spatial kernel, that may output specific buffer encoding to integrate with the rest of the renderer
		FinalOutput,

		MAX
	};

	class FStageDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_STAGE", EStage);
	class FUpscaleDim : SHADER_PERMUTATION_BOOL("DIM_UPSCALE");
	class FDenoiseTransmissionDim : SHADER_PERMUTATION_BOOL("DIM_DENOISE_TRANSMISSION");
	using FPermutationDomain = TShaderPermutationDomain<FSignalProcessingDim, FStageDim, FUpscaleDim, FSignalBatchSizeDim, FMultiSPPDim, FDenoiseTransmissionDim>;
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		ESignalProcessing SignalProcessing = PermutationVector.Get<FSignalProcessingDim>();

		// Only constant pixel density pass layout uses this shader.
		if (!UsesConstantPixelDensityPassLayout(PermutationVector.Get<FSignalProcessingDim>()))
		{
			return false;
		}

		// Not all signal processing allow to batch multiple signals at the same time.
		if (PermutationVector.Get<FSignalBatchSizeDim>() > SignalMaxBatchSize(SignalProcessing))
		{
			return false;
		}

		// Only reconstruction have upscale capability for now.
		if (PermutationVector.Get<FUpscaleDim>() && 
			PermutationVector.Get<FStageDim>() != EStage::ReConstruction)
		{
			return false;
		}

		// Only upscale is only for signal that needs it.
		if (PermutationVector.Get<FUpscaleDim>() &&
			!SignalSupportsUpscaling(SignalProcessing))
		{
			return false;
		}

		// Only compile pre convolution for signal that uses it.
		if (!SignalUsesPreConvolution(SignalProcessing) &&
			PermutationVector.Get<FStageDim>() == EStage::PreConvolution)
		{
			return false;
		}

		// Only compile rejection pre convolution for signal that uses it.
		if (!SignalUsesRejectionPreConvolution(SignalProcessing) &&
			PermutationVector.Get<FStageDim>() == EStage::RejectionPreConvolution)
		{
			return false;
		}

		// Only compile post convolution for signal that uses it.
		if (!SignalUsesPostConvolution(SignalProcessing) &&
			PermutationVector.Get<FStageDim>() == EStage::PostFiltering)
		{
			return false;
		}

		// Only compile final convolution for signal that uses it.
		if (!SignalUsesFinalConvolution(SignalProcessing) &&
			PermutationVector.Get<FStageDim>() == EStage::FinalOutput)
		{
			return false;
		}

		// Only compile multi SPP permutation for signal that supports it.
		if (PermutationVector.Get<FStageDim>() == EStage::ReConstruction &&
			PermutationVector.Get<FMultiSPPDim>() && !SignalSupportMultiSPP(SignalProcessing))
		{
			return false;
		}

		// transmission denoising permutation is only required to change for ShadowVisibilitymask 
		// FinalOutput.
		if (!(SignalProcessing == ESignalProcessing::ShadowVisibilityMask &&
			PermutationVector.Get<FStageDim>() == EStage::FinalOutput))
		{
			PermutationVector.Set<FDenoiseTransmissionDim>(false);
		}

		// Compile out the shader if this permutation gets remapped.
		if (RemapPermutationVector(PermutationVector) != PermutationVector)
		{
			return false;
		}

		return ShouldCompileSignalPipeline(SignalProcessing, Parameters.Platform);
	}

	static FPermutationDomain RemapPermutationVector(FPermutationDomain PermutationVector)
	{
		ESignalProcessing SignalProcessing = PermutationVector.Get<FSignalProcessingDim>();

		if (PermutationVector.Get<FStageDim>() == EStage::ReConstruction)
		{
			// force use the multi sample per pixel code path.
			if (!SignalSupport1SPP(SignalProcessing))
			{
				PermutationVector.Set<FMultiSPPDim>(true);
			}
		}
		else
		{
			PermutationVector.Set<FMultiSPPDim>(true);
		}

		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Dead code stripping required or else we have an unrolled loop with a non-compile time specified iteration count.
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceOptimization);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_ARRAY(FVector4f, InputBufferUVMinMax, [IScreenSpaceDenoiser::kMaxBatchSize])

		SHADER_PARAMETER(uint32, MaxSampleCount)
		SHADER_PARAMETER(uint32, PreviousCumulativeMaxSampleCount)
		SHADER_PARAMETER(int32, UpscaleFactor)
		SHADER_PARAMETER(float, KernelSpreadFactor)
		SHADER_PARAMETER(float, HarmonicPeriode)
		
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSDCommonParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSDConvolutionMetaData, ConvolutionMetaData)

		SHADER_PARAMETER_STRUCT(FSSDSignalTextures, SignalInput)
		SHADER_PARAMETER_STRUCT(FSSDSignalTextures, OriginalInput)
		SHADER_PARAMETER_STRUCT(FSSDSignalSRVs, SignalInputUint)
		SHADER_PARAMETER_STRUCT(FSSDSignalUAVs, SignalOutput)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DebugOutput) // TODO(Denoiser): remove
	END_SHADER_PARAMETER_STRUCT()
};

class FSSDTemporalAccumulationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSDTemporalAccumulationCS);
	SHADER_USE_PARAMETER_STRUCT(FSSDTemporalAccumulationCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSignalProcessingDim, FSignalBatchSizeDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		ESignalProcessing SignalProcessing = PermutationVector.Get<FSignalProcessingDim>();

		// Only constant pixel density pass layout uses this shader.
		if (!UsesConstantPixelDensityPassLayout(SignalProcessing))
		{
			return false;
		}

		// Not all signal processing allow to batch multiple signals at the same time.
		if (PermutationVector.Get<FSignalBatchSizeDim>() > SignalMaxBatchSize(SignalProcessing))
		{
			return false;
		}

		return ShouldCompileSignalPipeline(SignalProcessing, Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Dead code stripping required or else we have an unrolled loop with a non-compile time specified iteration count.
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceOptimization);
	}

	static EShaderCompileJobPriority GetOverrideJobPriority()
	{
		// FSSDTemporalAccumulationCS takes up to 23s on average
		return EShaderCompileJobPriority::ExtraHigh;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SCALAR_ARRAY(int32, bCameraCut, [IScreenSpaceDenoiser::kMaxBatchSize])
		SHADER_PARAMETER(float, HistoryPreExposureCorrection)
		SHADER_PARAMETER(FVector4f, ScreenPosToHistoryBufferUV)
		SHADER_PARAMETER(FVector4f, HistoryBufferSizeAndInvSize)
		SHADER_PARAMETER(FVector4f, HistoryBufferUVMinMax)
		SHADER_PARAMETER_ARRAY(FVector4f, HistoryBufferScissorUVMinMax, [IScreenSpaceDenoiser::kMaxBatchSize])
		SHADER_PARAMETER(FVector4f, PrevSceneBufferUVToScreenPosition)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSSDCommonParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSDConvolutionMetaData, ConvolutionMetaData)

		SHADER_PARAMETER_STRUCT(FSSDSignalTextures, SignalInput)
		SHADER_PARAMETER_STRUCT(FSSDSignalTextures, HistoryRejectionSignal)
		SHADER_PARAMETER_STRUCT(FSSDSignalUAVs, SignalHistoryOutput)

		SHADER_PARAMETER_STRUCT(FSSDSignalTextures, PrevHistory)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PrevDepthBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PrevGBufferA)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PrevGBufferB)
		SHADER_PARAMETER_RDG_TEXTURE_ARRAY(Texture2D<uint>, PrevCompressedMetadata, [kCompressedMetadataTextures])

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DebugOutput) // TODO(Denoiser): remove
	END_SHADER_PARAMETER_STRUCT()
};

class FSSDComposeHarmonicsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSDComposeHarmonicsCS);
	SHADER_USE_PARAMETER_STRUCT(FSSDComposeHarmonicsCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileSignalPipeline(ESignalProcessing::PolychromaticPenumbraHarmonic, Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_ARRAY(FSSDSignalTextures, SignalHarmonics, [IScreenSpaceDenoiser::kMultiPolychromaticPenumbraHarmonics])
		SHADER_PARAMETER_STRUCT(FSSDSignalTextures, SignalIntegrand)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSDCommonParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT(FSSDSignalUAVs, SignalOutput)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FSSDCompressMetadataCS, "/Engine/Private/ScreenSpaceDenoise/SSDCompressMetadata.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FSSDInjestCS, "/Engine/Private/ScreenSpaceDenoise/SSDInjest.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FSSDSpatialAccumulationCS, "/Engine/Private/ScreenSpaceDenoise/SSDSpatialAccumulation.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FSSDTemporalAccumulationCS, "/Engine/Private/ScreenSpaceDenoise/SSDTemporalAccumulation.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FSSDComposeHarmonicsCS, "/Engine/Private/ScreenSpaceDenoise/SSDComposeHarmonics.usf", "MainCS", SF_Compute);

} // namespace


/** PrevViewInfo and PrevFrameViewInfo pooled render targets to use for temporal storage of scene textures. */
struct FViewInfoPooledRenderTargets
{
	TRefCountPtr<IPooledRenderTarget> PrevDepthBuffer;
	TRefCountPtr<IPooledRenderTarget> PrevGBufferA;
	TRefCountPtr<IPooledRenderTarget> PrevGBufferB;
	TRefCountPtr<IPooledRenderTarget> PrevCompressedDepthViewNormal;

	TRefCountPtr<IPooledRenderTarget>* NextDepthBuffer;
	TRefCountPtr<IPooledRenderTarget>* NextGBufferA;
	TRefCountPtr<IPooledRenderTarget>* NextGBufferB;
	TRefCountPtr<IPooledRenderTarget>* NextCompressedDepthViewNormal;
};

void SetupSceneViewInfoPooledRenderTargets(
	const FViewInfo& View,
	FViewInfoPooledRenderTargets* OutViewInfoPooledRenderTargets)
{
	auto&& PrevViewInfo = View.PrevViewInfo;
	auto&& PrevFrameViewInfo = View.ViewState->PrevFrameViewInfo;

	OutViewInfoPooledRenderTargets->PrevDepthBuffer = PrevViewInfo.DepthBuffer;
	OutViewInfoPooledRenderTargets->PrevGBufferA = PrevViewInfo.GBufferA;
	OutViewInfoPooledRenderTargets->PrevGBufferB = PrevViewInfo.GBufferB;
	OutViewInfoPooledRenderTargets->PrevCompressedDepthViewNormal = PrevViewInfo.CompressedDepthViewNormal;

	OutViewInfoPooledRenderTargets->NextDepthBuffer = &PrevFrameViewInfo.DepthBuffer;
	OutViewInfoPooledRenderTargets->NextGBufferA = &PrevFrameViewInfo.GBufferA;
	OutViewInfoPooledRenderTargets->NextGBufferB = &PrevFrameViewInfo.GBufferB;
	OutViewInfoPooledRenderTargets->NextCompressedDepthViewNormal = &PrevFrameViewInfo.CompressedDepthViewNormal;
}

void Denoiser::SetupCommonShaderParameters(
	const FViewInfo& View,
	const FSceneTextureParameters& SceneTextures,
	const FIntRect DenoiserFullResViewport,
	float DenoisingResolutionFraction,
	Denoiser::FCommonShaderParameters* OutPublicCommonParameters)
{
	check(OutPublicCommonParameters);

	FIntPoint FullResBufferExtent = SceneTextures.SceneDepthTexture->Desc.Extent;

	FIntPoint DenoiserBufferExtent = FullResBufferExtent;
	FIntRect DenoiserViewport = DenoiserFullResViewport;
	if (DenoisingResolutionFraction == 0.5f)
	{
		DenoiserBufferExtent /= 2;
		DenoiserViewport = FIntRect::DivideAndRoundUp(DenoiserViewport, 2);
	}

	OutPublicCommonParameters->DenoiserBufferSizeAndInvSize = FVector4f(
		float(DenoiserBufferExtent.X),
		float(DenoiserBufferExtent.Y),
		1.0f / float(DenoiserBufferExtent.X),
		1.0f / float(DenoiserBufferExtent.Y));

	OutPublicCommonParameters->SceneBufferUVToScreenPosition.X = float(FullResBufferExtent.X) / float(View.ViewRect.Width()) * 2.0f;
	OutPublicCommonParameters->SceneBufferUVToScreenPosition.Y = -float(FullResBufferExtent.Y) / float(View.ViewRect.Height()) * 2.0f;
	OutPublicCommonParameters->SceneBufferUVToScreenPosition.Z = -float(View.ViewRect.Min.X) / float(View.ViewRect.Width()) * 2.0f - 1.0f;
	OutPublicCommonParameters->SceneBufferUVToScreenPosition.W = float(View.ViewRect.Min.Y) / float(View.ViewRect.Height()) * 2.0f + 1.0f;

	OutPublicCommonParameters->DenoiserBufferBilinearUVMinMax = FVector4f(
		float(DenoiserViewport.Min.X + 0.5f) / float(DenoiserBufferExtent.X),
		float(DenoiserViewport.Min.Y + 0.5f) / float(DenoiserBufferExtent.Y),
		float(DenoiserViewport.Max.X - 0.5f) / float(DenoiserBufferExtent.X),
		float(DenoiserViewport.Max.Y - 0.5f) / float(DenoiserBufferExtent.Y));
}


/** View specific denoise settings */
struct FSSDConstantPixelDensityViewSettings
{
	// Inputs
	FIntRect FullResViewport;
	TStaticArray<FIntRect, IScreenSpaceDenoiser::kMaxBatchSize> SignalScissor;

	// Generated in DenoiseSignalAtConstantPixelDensity
	FIntRect Viewport;
};

/** Generic settings to denoise signal at constant pixel density across the viewport. */
struct FSSDConstantPixelDensitySettings
{
	ESignalProcessing SignalProcessing;
	int32 SignalBatchSize = 1;
	float HarmonicPeriode = 1.0f;
	int32 MaxInputSPP = 1;
	float InputResolutionFraction = 1.0f;
	float DenoisingResolutionFraction = 1.0f;
	bool bEnableReconstruction = true;
	int32 ReconstructionSamples = 1;
	int32 PreConvolutionCount = 0;
	float KernelSpreadFactor = 8;
	bool bUseTemporalAccumulation = false;
	int32 HistoryConvolutionSampleCount = 1;
	float HistoryConvolutionKernelSpreadFactor = 1.0f;
	TStaticArray<const FLightSceneInfo*, IScreenSpaceDenoiser::kMaxBatchSize> LightSceneInfo;
	FRDGTextureRef CompressedDepthTexture = nullptr;
	FRDGTextureRef CompressedShadingModelTexture = nullptr;
	TArrayView<FSSDConstantPixelDensityViewSettings> ViewSettingsArray;
};

/** Denoises a signal at constant pixel density across the viewport. */
static void DenoiseSignalAtConstantPixelDensity(
	FRDGBuilder& GraphBuilder,
	TConstArrayView<FViewInfo> Views,
	const FSceneTextureParameters& SceneTextures,
	const FViewInfoPooledRenderTargets& ViewInfoPooledRenderTargets,
	const FSSDSignalTextures& InputSignal,
	FSSDConstantPixelDensitySettings Settings,
	TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevFilteringHistory,
	TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewFilteringHistory,
	FSSDSignalTextures* OutputSignal)
{
	check(UsesConstantPixelDensityPassLayout(Settings.SignalProcessing));
	check(Views.Num() == Settings.ViewSettingsArray.Num());
	
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		FSSDConstantPixelDensityViewSettings& ViewSettings = Settings.ViewSettingsArray[ViewIndex];

		// Make sure the viewport of the denoiser is within the viewport of the view.
		FIntRect Union = View.ViewRect;
		Union.Union(ViewSettings.FullResViewport);
		check(Union == View.ViewRect);

		// Initialize Viewport (defaults to full res)
		ViewSettings.Viewport = ViewSettings.FullResViewport;
	}

	ensure(Settings.InputResolutionFraction == 1.0f || Settings.InputResolutionFraction == 0.5f || Settings.InputResolutionFraction == 0.25f);
	
	auto GetResourceNames = [&](const TCHAR* const ResourceNames[])
	{
		return ResourceNames + (int32(Settings.SignalProcessing) * kMaxBufferProcessingCount);
	};

	const bool bUseMultiInputSPPShaderPath = Settings.MaxInputSPP > 1;

	FIntPoint FullResBufferExtent = SceneTextures.SceneDepthTexture->Desc.Extent;
	FIntPoint BufferExtent = FullResBufferExtent;
	if (Settings.DenoisingResolutionFraction == 0.5f)
	{
		BufferExtent /= 2;
		for (FSSDConstantPixelDensityViewSettings& ViewSettings : Settings.ViewSettingsArray)
		{
			ViewSettings.Viewport = FIntRect::DivideAndRoundUp(ViewSettings.Viewport, 2);
		}
	}

	// Get combined viewport
	FIntRect AllViewport = Settings.ViewSettingsArray[0].Viewport;
	for (int32 ViewIndex = 1; ViewIndex < Views.Num(); ViewIndex++)
	{
		AllViewport.Union(Settings.ViewSettingsArray[ViewIndex].Viewport);
	}

	// Number of signal to batch.
	int32 MaxSignalBatchSize = SignalMaxBatchSize(Settings.SignalProcessing);
	check(Settings.SignalBatchSize >= 1 && Settings.SignalBatchSize <= MaxSignalBatchSize);

	// Number of texture per batched signal.
	int32 InjestTextureCount = 0;
	int32 ReconstructionTextureCount = 0;
	int32 HistoryTextureCountPerSignal = 0;

	// Descriptor to allocate internal denoising buffer.
	bool bHasReconstructionLayoutDifferentFromHistory = false;
	TStaticArray<FRDGTextureDesc, kMaxBufferProcessingCount> InjestDescs;
	TStaticArray<FRDGTextureDesc, kMaxBufferProcessingCount> ReconstructionDescs;
	TStaticArray<FRDGTextureDesc, kMaxBufferProcessingCount> HistoryDescs;
	FRDGTextureDesc DebugDesc;
	{
		// Manually format texel in the shader to reduce VGPR pressure with overlapped texture fetched.
		const bool bManualTexelFormatting = true;

		static const EPixelFormat PixelFormatPerChannel[] = {
			PF_Unknown,
			PF_R16F,
			PF_G16R16F,
			PF_FloatRGBA, // there is no 16bits float RGB
			PF_FloatRGBA,
		};

		FRDGTextureDesc RefDesc = FRDGTextureDesc::Create2D(
			BufferExtent,
			PF_Unknown,
			FClearValueBinding::Black,
			TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);

		DebugDesc = RefDesc;
		DebugDesc.Format = PF_FloatRGBA;

		for (int32 i = 0; i < kMaxBufferProcessingCount; i++)
		{
			InjestDescs[i] = RefDesc;
			ReconstructionDescs[i] = RefDesc;
			HistoryDescs[i] = RefDesc;
		}

		if (Settings.SignalProcessing == ESignalProcessing::ShadowVisibilityMask)
		{
			check(Settings.SignalBatchSize >= 1 && Settings.SignalBatchSize <= IScreenSpaceDenoiser::kMaxBatchSize);
			for (int32 BatchedSignalId = 0; BatchedSignalId < Settings.SignalBatchSize; BatchedSignalId++)
			{
				InjestDescs[BatchedSignalId / 2].Format = (BatchedSignalId % 2) ? PF_R32G32_UINT : PF_R32_UINT;
				InjestTextureCount = BatchedSignalId / 2 + 1;
				ReconstructionDescs[BatchedSignalId].Format = PF_FloatRGBA;
				HistoryDescs[BatchedSignalId].Format = PF_FloatRGBA;
			}

			HistoryTextureCountPerSignal = 1;
			ReconstructionTextureCount = Settings.SignalBatchSize;
			bHasReconstructionLayoutDifferentFromHistory = false;
		}
		else if (Settings.SignalProcessing == ESignalProcessing::PolychromaticPenumbraHarmonic)
		{
			ReconstructionTextureCount = 4;
			ReconstructionDescs[0].Format = PF_FloatRGBA;
			ReconstructionDescs[1].Format = PF_FloatRGBA;
			ReconstructionDescs[2].Format = PF_FloatRGBA;
			ReconstructionDescs[3].Format = PF_FloatRGBA;

			HistoryTextureCountPerSignal = 2;
			HistoryDescs[0].Format = PF_FloatRGBA;
			HistoryDescs[1].Format = PF_FloatRGBA;
		}
		else if (Settings.SignalProcessing == ESignalProcessing::Reflections)
		{
			ReconstructionDescs[0].Format = HistoryDescs[0].Format = PF_FloatRGBA;
			ReconstructionDescs[1].Format = HistoryDescs[1].Format = PF_G16R16F;
			ReconstructionTextureCount = HistoryTextureCountPerSignal = 2;
			bHasReconstructionLayoutDifferentFromHistory = false;
		}
		else if (Settings.SignalProcessing == ESignalProcessing::AmbientOcclusion)
		{
			ReconstructionDescs[0].Format = HistoryDescs[0].Format = PF_FloatRGBA;
			ReconstructionTextureCount = HistoryTextureCountPerSignal = 1;
			bHasReconstructionLayoutDifferentFromHistory = false;
		}
		else if (Settings.SignalProcessing == ESignalProcessing::DiffuseAndAmbientOcclusion)
		{
			ReconstructionDescs[0].Format = PF_FloatRGBA;
			ReconstructionDescs[1].Format = PF_R16F;
			ReconstructionTextureCount = 2;

			HistoryDescs[0].Format = PF_FloatRGBA;
			HistoryDescs[1].Format = PF_R16F; //PF_FloatRGB;
			HistoryTextureCountPerSignal = 2;
			bHasReconstructionLayoutDifferentFromHistory = false;
		}
		else if (Settings.SignalProcessing == ESignalProcessing::DiffuseSphericalHarmonic)
		{
			ReconstructionDescs[0].Format = PF_FloatRGBA;
			ReconstructionDescs[1].Format = PF_FloatRGBA;
			ReconstructionTextureCount = 2;

			HistoryDescs = ReconstructionDescs;
			HistoryTextureCountPerSignal = 2;
			bHasReconstructionLayoutDifferentFromHistory = false;
		}
		else if (Settings.SignalProcessing == ESignalProcessing::ScreenSpaceDiffuseIndirect)
		{
			ReconstructionDescs[0].Format = PF_FloatR11G11B10;
			HistoryDescs[0].Format = PF_FloatR11G11B10;
			ReconstructionDescs[1].Format = PF_R8G8;
			ReconstructionTextureCount = 2;

			
			HistoryDescs[1].Format = PF_R8G8;
			HistoryTextureCountPerSignal = 2;
			bHasReconstructionLayoutDifferentFromHistory = false;
		}
		else if (Settings.SignalProcessing == ESignalProcessing::IndirectProbeHierarchy)
		{
			ReconstructionDescs[0].Format = PF_FloatR11G11B10;
			ReconstructionDescs[1].Format = PF_FloatR11G11B10;
			ReconstructionDescs[2].Format = PF_R8;
			ReconstructionTextureCount = 3;

			HistoryDescs[0].Format = PF_FloatR11G11B10;
			HistoryDescs[1].Format = PF_FloatR11G11B10;
			HistoryDescs[2].Format = PF_R8;
			HistoryTextureCountPerSignal = 3;
			bHasReconstructionLayoutDifferentFromHistory = true;
		}
		else
		{
			check(0);
		}

		check(HistoryTextureCountPerSignal > 0);
		check(ReconstructionTextureCount > 0);
	}

	// Create a UAV use to output debugging information from the shader.
	auto CreateDebugUAV = [&](const TCHAR* DebugTextureName)
	{
		return GraphBuilder.CreateUAV(GraphBuilder.CreateTexture(DebugDesc, DebugTextureName));
	};

	int32 HistoryTextureCount = HistoryTextureCountPerSignal * Settings.SignalBatchSize;

	check(HistoryTextureCount <= kMaxBufferProcessingCount);

	ECompressedMetadataLayout CompressedMetadataLayout = GetSignalCompressedMetadata(Settings.SignalProcessing);

	// Setup common shader parameters.
	TArray<FSSDCommonParameters, TInlineAllocator<6>> CommonParametersByView;
	TArray<FSSDConvolutionMetaData, TInlineAllocator<6>> ConvolutionMetaDataByView;

	CommonParametersByView.SetNum(Views.Num());
	ConvolutionMetaDataByView.SetNum(Views.Num());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		FSSDCommonParameters& CommonParameters = CommonParametersByView[ViewIndex];
		FSSDConvolutionMetaData& ConvolutionMetaData = ConvolutionMetaDataByView[ViewIndex];
		FSSDConstantPixelDensityViewSettings& ViewSettings = Settings.ViewSettingsArray[ViewIndex];

		Denoiser::SetupCommonShaderParameters(
			View, SceneTextures,
			ViewSettings.FullResViewport,
			Settings.DenoisingResolutionFraction,
			/* out */ &CommonParameters.PublicCommonParameters);

		CommonParameters.ViewportMin = ViewSettings.Viewport.Min;
		CommonParameters.ViewportMax = ViewSettings.Viewport.Max;

		CommonParameters.SceneTextures = SceneTextures;
		CommonParameters.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		CommonParameters.ViewUniformBuffer = View.ViewUniformBuffer;
		CommonParameters.EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));

		// Remove dependency of the velocity buffer on camera cut, given it's going to be ignored by the shaders.
		if (View.bCameraCut || !CommonParameters.SceneTextures.GBufferVelocityTexture)
		{
			CommonParameters.SceneTextures.GBufferVelocityTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		}

		float PixelPositionToFullResPixel = 1.0f / Settings.DenoisingResolutionFraction;
		FVector2D FullResPixelOffset = FVector2D(0.5f, 0.5f); // TODO(Denoiser).

		CommonParameters.ThreadIdToBufferUV.X = PixelPositionToFullResPixel / float(FullResBufferExtent.X);
		CommonParameters.ThreadIdToBufferUV.Y = PixelPositionToFullResPixel / float(FullResBufferExtent.Y);
		CommonParameters.ThreadIdToBufferUV.Z = (ViewSettings.Viewport.Min.X * PixelPositionToFullResPixel + FullResPixelOffset.X) / float(FullResBufferExtent.X);
		CommonParameters.ThreadIdToBufferUV.W = (ViewSettings.Viewport.Min.Y * PixelPositionToFullResPixel + FullResPixelOffset.Y) / float(FullResBufferExtent.Y);

		CommonParameters.BufferUVToOutputPixelPosition.X = BufferExtent.X;
		CommonParameters.BufferUVToOutputPixelPosition.Y = BufferExtent.Y;

		CommonParameters.ScreenToView = FMatrix44f(FMatrix(		// LWC_TODO: Precision loss
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, View.ProjectionMatrixUnadjustedForRHI.M[2][2], 1),
			FPlane(0, 0, View.ProjectionMatrixUnadjustedForRHI.M[3][2], 0))
			* View.ViewMatrices.GetClipToView());

		CommonParameters.BufferUVBilinearCorrection.X = (0.5f * PixelPositionToFullResPixel - FullResPixelOffset.X) / float(FullResBufferExtent.X);
		CommonParameters.BufferUVBilinearCorrection.Y = (0.5f * PixelPositionToFullResPixel - FullResPixelOffset.Y) / float(FullResBufferExtent.Y);

		CommonParameters.FrameIndex = View.ViewState ? View.ViewState->FrameIndex : 0;

		// Setup all the metadata to do spatial convolution.
		if (Settings.SignalProcessing == ESignalProcessing::ShadowVisibilityMask)
		{
			for (int32 BatchedSignalId = 0; BatchedSignalId < Settings.SignalBatchSize; BatchedSignalId++)
			{
				FLightSceneProxy* LightSceneProxy = Settings.LightSceneInfo[BatchedSignalId]->Proxy;

				FLightRenderParameters Parameters;
				LightSceneProxy->GetLightShaderParameters(Parameters);

				const FVector3f TranslatedWorldPosition = FVector3f(View.ViewMatrices.GetPreViewTranslation() + Parameters.WorldPosition);

				ConvolutionMetaData.LightPositionAndRadius[BatchedSignalId] = FVector4f(
					TranslatedWorldPosition, Parameters.SourceRadius);
				ConvolutionMetaData.LightDirectionAndLength[BatchedSignalId] = FVector4f(
					Parameters.Direction, Parameters.SourceLength);
				GET_SCALAR_ARRAY_ELEMENT(ConvolutionMetaData.HitDistanceToWorldBluringRadius, BatchedSignalId) =
					FMath::Tan(0.5 * FMath::DegreesToRadians(LightSceneProxy->GetLightSourceAngle()) * LightSceneProxy->GetShadowSourceAngleFactor());
				GET_SCALAR_ARRAY_ELEMENT(ConvolutionMetaData.LightType, BatchedSignalId) = LightSceneProxy->GetLightType();
			}
		}

		// Compress the meta data for lower memory bandwidth, half res for coherent memory access, and lower VGPR footprint.
		if (CompressedMetadataLayout == ECompressedMetadataLayout::FedDepthAndShadingModelID)
		{
			check(Settings.CompressedDepthTexture);
			check(Settings.CompressedShadingModelTexture);

			CommonParameters.CompressedMetadata[0] = Settings.CompressedDepthTexture;
			CommonParameters.CompressedMetadata[1] = Settings.CompressedShadingModelTexture;
		}
		else if (CompressedMetadataLayout != ECompressedMetadataLayout::Disabled)
		{
			if (CompressedMetadataLayout == ECompressedMetadataLayout::DepthAndNormal ||
				CompressedMetadataLayout == ECompressedMetadataLayout::DepthAndViewNormal)
			{
				FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
					BufferExtent,
					PF_R32_UINT,
					FClearValueBinding::Black,
					TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);

				CommonParameters.CompressedMetadata[0] = GraphBuilder.CreateTexture(Desc, TEXT("DenoiserMetadata0"));
				CommonParameters.CompressedMetadata[1] = nullptr;

				CommonParameters.CompressedMetadata[0]->EncloseVisualizeExtent(ViewSettings.Viewport.Max);
			}
			else
			{
				check(0);
			}

			FSSDCompressMetadataCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSSDCompressMetadataCS::FMetadataLayoutDim>(CompressedMetadataLayout);

			FSSDCompressMetadataCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSDCompressMetadataCS::FParameters>();
			PassParameters->CommonParameters = CommonParameters;
			for (int32 i = 0; i < kCompressedMetadataTextures; i++)
				PassParameters->CompressedMetadataOutput[i] = CommonParameters.CompressedMetadata[i] ? GraphBuilder.CreateUAV(CommonParameters.CompressedMetadata[i]) : nullptr;

			TShaderMapRef<FSSDCompressMetadataCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSD CompressMetadata %dx%d", ViewSettings.Viewport.Width(), ViewSettings.Viewport.Height()),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(ViewSettings.Viewport.Size(), FComputeShaderUtils::kGolden2DGroupSize));
		}
	}

	FSSDSignalTextures SignalHistory = InputSignal;

	// Injestion pass to precompute some values for the reconstruction pass.
	if (SignalUsesInjestion(Settings.SignalProcessing))
	{
		FSSDSignalTextures NewSignalOutput = CreateMultiplexedTextures(
			GraphBuilder,
			InjestTextureCount, InjestDescs,
			GetResourceNames(kInjestResourceNames),
			AllViewport);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			FSSDCommonParameters& CommonParameters = CommonParametersByView[ViewIndex];
			FSSDConvolutionMetaData& ConvolutionMetaData = ConvolutionMetaDataByView[ViewIndex];
			FSSDConstantPixelDensityViewSettings& ViewSettings = Settings.ViewSettingsArray[ViewIndex];

			FSSDInjestCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSDInjestCS::FParameters>();
			PassParameters->CommonParameters = CommonParameters;
			PassParameters->ConvolutionMetaData = ConvolutionMetaData;
			PassParameters->SignalInput = SignalHistory;
			PassParameters->SignalOutput = CreateMultiplexedUAVs(GraphBuilder, NewSignalOutput);
			PassParameters->DebugOutput = CreateDebugUAV(TEXT("DebugDenoiserInjest"));

			FSSDInjestCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSignalProcessingDim>(Settings.SignalProcessing);
			PermutationVector.Set<FSignalBatchSizeDim>(Settings.SignalBatchSize);
			PermutationVector.Set<FMultiSPPDim>(bUseMultiInputSPPShaderPath);
			PermutationVector = FSSDInjestCS::RemapPermutationVector(PermutationVector);

			TShaderMapRef<FSSDInjestCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSD Injest(MultiSPP=%i)",
					int32(PermutationVector.Get<FMultiSPPDim>())),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(ViewSettings.Viewport.Size(), FComputeShaderUtils::kGolden2DGroupSize));
		}

		SignalHistory = NewSignalOutput;
	}

	// Spatial reconstruction with ratio estimator to be more precise in the history rejection.
	if (Settings.bEnableReconstruction)
	{
		FSSDSignalTextures NewSignalOutput = CreateMultiplexedTextures(
			GraphBuilder,
			ReconstructionTextureCount, ReconstructionDescs,
			GetResourceNames(kReconstructionResourceNames),
			AllViewport);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			FSSDCommonParameters& CommonParameters = CommonParametersByView[ViewIndex];
			FSSDConvolutionMetaData& ConvolutionMetaData = ConvolutionMetaDataByView[ViewIndex];
			FSSDConstantPixelDensityViewSettings& ViewSettings = Settings.ViewSettingsArray[ViewIndex];

			FSSDSpatialAccumulationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSDSpatialAccumulationCS::FParameters>();
			for (int32 BatchedSignalId = 0; BatchedSignalId < Settings.SignalBatchSize; BatchedSignalId++)
			{
				FIntRect SignalScissor = ViewSettings.SignalScissor[BatchedSignalId];
				PassParameters->InputBufferUVMinMax[BatchedSignalId] = FVector4f(
					float(SignalScissor.Min.X + 0.5f) / float(BufferExtent.X),
					float(SignalScissor.Min.Y + 0.5f) / float(BufferExtent.Y),
					float(SignalScissor.Max.X - 0.5f) / float(BufferExtent.X),
					float(SignalScissor.Max.Y - 0.5f) / float(BufferExtent.Y));
			}

			PassParameters->MaxSampleCount = Settings.ReconstructionSamples;
			PassParameters->PreviousCumulativeMaxSampleCount = 1;
			PassParameters->UpscaleFactor = int32(Settings.DenoisingResolutionFraction / Settings.InputResolutionFraction);
			PassParameters->HarmonicPeriode = Settings.HarmonicPeriode;
			PassParameters->CommonParameters = CommonParameters;
			PassParameters->ConvolutionMetaData = ConvolutionMetaData;
			PassParameters->SignalInput = SignalHistory;
			//PassParameters->SignalInputUint = CreateMultiplexedUintSRVs(GraphBuilder, SignalHistory);
			PassParameters->SignalOutput = CreateMultiplexedUAVs(GraphBuilder, NewSignalOutput);

			PassParameters->DebugOutput = CreateDebugUAV(TEXT("DebugDenoiserReconstruction"));

			FSSDSpatialAccumulationCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSignalProcessingDim>(Settings.SignalProcessing);
			PermutationVector.Set<FSignalBatchSizeDim>(Settings.SignalBatchSize);
			PermutationVector.Set<FSSDSpatialAccumulationCS::FStageDim>(FSSDSpatialAccumulationCS::EStage::ReConstruction);
			PermutationVector.Set<FSSDSpatialAccumulationCS::FUpscaleDim>(PassParameters->UpscaleFactor != 1);
			PermutationVector.Set<FMultiSPPDim>(bUseMultiInputSPPShaderPath);
			PermutationVector = FSSDSpatialAccumulationCS::RemapPermutationVector(PermutationVector);

			TShaderMapRef<FSSDSpatialAccumulationCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSD Reconstruction(MaxSamples=%i Scissor=%ix%i%s%s)",
					PassParameters->MaxSampleCount,
					ViewSettings.Viewport.Width(), ViewSettings.Viewport.Height(),
					PermutationVector.Get<FSSDSpatialAccumulationCS::FUpscaleDim>() ? TEXT(" Upscale") : TEXT(""),
					PermutationVector.Get<FMultiSPPDim>() ? TEXT("") : TEXT(" 1SPP")),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(ViewSettings.Viewport.Size(), FSSDSpatialAccumulationCS::kGroupSize));
		}

		SignalHistory = NewSignalOutput;
	}

	// Spatial pre convolutions
	for (int32 PreConvolutionId = 0; PreConvolutionId < Settings.PreConvolutionCount; PreConvolutionId++)
	{
		check(SignalUsesPreConvolution(Settings.SignalProcessing));

		FSSDSignalTextures NewSignalOutput = CreateMultiplexedTextures(
			GraphBuilder,
			ReconstructionTextureCount, ReconstructionDescs,
			GetResourceNames(kPreConvolutionResourceNames),
			AllViewport);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			FSSDCommonParameters& CommonParameters = CommonParametersByView[ViewIndex];
			FSSDConvolutionMetaData& ConvolutionMetaData = ConvolutionMetaDataByView[ViewIndex];
			FSSDConstantPixelDensityViewSettings& ViewSettings = Settings.ViewSettingsArray[ViewIndex];

			FSSDSpatialAccumulationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSDSpatialAccumulationCS::FParameters>();
			PassParameters->CommonParameters = CommonParameters;
			PassParameters->ConvolutionMetaData = ConvolutionMetaData;

			PassParameters->MaxSampleCount = Settings.ReconstructionSamples;
			PassParameters->PreviousCumulativeMaxSampleCount = FMath::Pow(static_cast<float>(PassParameters->MaxSampleCount), 1 + PreConvolutionId);
			PassParameters->KernelSpreadFactor = Settings.KernelSpreadFactor * (1 << PreConvolutionId);
			PassParameters->SignalInput = SignalHistory;
			PassParameters->SignalOutput = CreateMultiplexedUAVs(GraphBuilder, NewSignalOutput);

			PassParameters->DebugOutput = CreateDebugUAV(TEXT("DebugDenoiserPreConvolution"));

			FSSDSpatialAccumulationCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSignalProcessingDim>(Settings.SignalProcessing);
			PermutationVector.Set<FSignalBatchSizeDim>(Settings.SignalBatchSize);
			PermutationVector.Set<FSSDSpatialAccumulationCS::FStageDim>(FSSDSpatialAccumulationCS::EStage::PreConvolution);
			PermutationVector.Set<FMultiSPPDim>(true);

			TShaderMapRef<FSSDSpatialAccumulationCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME(
					"SSD PreConvolution(MaxSamples=%d Spread=%f)",
					PassParameters->MaxSampleCount,
					PassParameters->KernelSpreadFactor),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(ViewSettings.Viewport.Size(), FSSDSpatialAccumulationCS::kGroupSize));
		}

		SignalHistory = NewSignalOutput;
	}

	bool bExtractSceneDepth = false;
	bool bExtractSceneGBufferA = false;
	bool bExtractSceneGBufferB = false;

	TStaticArray<bool, kCompressedMetadataTextures> bExtractCompressedMetadata;
	for (int32 i = 0; i < kCompressedMetadataTextures; i++)
		bExtractCompressedMetadata[i] = false;

	// Temporal pass.
	//
	// Note: always done even if there is no ViewState, because it is already not an idea case for the denoiser quality, therefore not really
	// care about the performance, and the reconstruction may have a different layout than temporal accumulation output.
	if (bHasReconstructionLayoutDifferentFromHistory || Settings.bUseTemporalAccumulation)
	{
		FSSDSignalTextures RejectionPreConvolutionSignal;

		// Temporal rejection might make use of a separable preconvolution.
		if (SignalUsesRejectionPreConvolution(Settings.SignalProcessing))
		{
			{
				int32 RejectionTextureCount = 1;
				TStaticArray<FRDGTextureDesc,kMaxBufferProcessingCount> RejectionSignalProcessingDescs;
				for (int32 i = 0; i < kMaxBufferProcessingCount; i++)
				{
					RejectionSignalProcessingDescs[i] = HistoryDescs[i];
				}

				if (Settings.SignalProcessing == ESignalProcessing::ShadowVisibilityMask)
				{
					for (int32 BatchedSignalId = 0; BatchedSignalId < Settings.SignalBatchSize; BatchedSignalId++)
					{
						RejectionSignalProcessingDescs[BatchedSignalId].Format = PF_FloatRGBA;
					}
					RejectionTextureCount = Settings.SignalBatchSize;
				}
				else if (Settings.SignalProcessing == ESignalProcessing::AmbientOcclusion)
				{
					RejectionSignalProcessingDescs[0].Format = PF_FloatRGBA;
				}
				else
				{
					check(0);
				}

				RejectionPreConvolutionSignal = CreateMultiplexedTextures(
					GraphBuilder,
					RejectionTextureCount, RejectionSignalProcessingDescs,
					GetResourceNames(kRejectionPreConvolutionResourceNames),
					AllViewport);
			}

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				const FViewInfo& View = Views[ViewIndex];
				FSSDCommonParameters& CommonParameters = CommonParametersByView[ViewIndex];
				FSSDConvolutionMetaData& ConvolutionMetaData = ConvolutionMetaDataByView[ViewIndex];
				FSSDConstantPixelDensityViewSettings& ViewSettings = Settings.ViewSettingsArray[ViewIndex];

				FSSDSpatialAccumulationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSDSpatialAccumulationCS::FParameters>();
				PassParameters->CommonParameters = CommonParameters;
				PassParameters->ConvolutionMetaData = ConvolutionMetaData;
				PassParameters->SignalInput = SignalHistory;
				PassParameters->SignalOutput = CreateMultiplexedUAVs(GraphBuilder, RejectionPreConvolutionSignal);

				FSSDSpatialAccumulationCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FSignalProcessingDim>(Settings.SignalProcessing);
				PermutationVector.Set<FSignalBatchSizeDim>(Settings.SignalBatchSize);
				PermutationVector.Set<FSSDSpatialAccumulationCS::FStageDim>(FSSDSpatialAccumulationCS::EStage::RejectionPreConvolution);
				PermutationVector.Set<FMultiSPPDim>(true);

				PassParameters->DebugOutput = CreateDebugUAV(TEXT("DebugDenoiserRejectionPreConvolution"));

				TShaderMapRef<FSSDSpatialAccumulationCS> ComputeShader(View.ShaderMap, PermutationVector);
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("SSD RejectionPreConvolution(MaxSamples=5)"),
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(ViewSettings.Viewport.Size(), FSSDSpatialAccumulationCS::kGroupSize));
			}
		} // if (SignalUsesRejectionPreConvolution(Settings.SignalProcessing))

		FSSDSignalTextures SignalOutput = CreateMultiplexedTextures(
			GraphBuilder,
			HistoryTextureCount, HistoryDescs,
			GetResourceNames(kTemporalAccumulationResourceNames),
			AllViewport);

		int32 LastViewIndex = Views.Num() - 1;
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			FSSDCommonParameters& CommonParameters = CommonParametersByView[ViewIndex];
			FSSDConvolutionMetaData& ConvolutionMetaData = ConvolutionMetaDataByView[ViewIndex];
			FSSDConstantPixelDensityViewSettings& ViewSettings = Settings.ViewSettingsArray[ViewIndex];

			FSSDTemporalAccumulationCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSignalProcessingDim>(Settings.SignalProcessing);
			PermutationVector.Set<FSignalBatchSizeDim>(Settings.SignalBatchSize);

			TShaderMapRef<FSSDTemporalAccumulationCS> ComputeShader(View.ShaderMap, PermutationVector);

			FSSDTemporalAccumulationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSDTemporalAccumulationCS::FParameters>();
			PassParameters->CommonParameters = CommonParameters;
			PassParameters->ConvolutionMetaData = ConvolutionMetaData;
			PassParameters->HistoryPreExposureCorrection = View.PreExposure / View.PrevViewInfo.SceneColorPreExposure;
			PassParameters->SignalInput = SignalHistory;
			PassParameters->HistoryRejectionSignal = RejectionPreConvolutionSignal;
			PassParameters->SignalHistoryOutput = CreateMultiplexedUAVs(GraphBuilder, SignalOutput);

			// Setup common previous frame data.
			PassParameters->PrevDepthBuffer = RegisterExternalTextureWithFallback(GraphBuilder, ViewInfoPooledRenderTargets.PrevDepthBuffer, GSystemTextures.BlackDummy);
			PassParameters->PrevGBufferA = RegisterExternalTextureWithFallback(GraphBuilder, ViewInfoPooledRenderTargets.PrevGBufferA, GSystemTextures.BlackDummy);
			PassParameters->PrevGBufferB = RegisterExternalTextureWithFallback(GraphBuilder, ViewInfoPooledRenderTargets.PrevGBufferB, GSystemTextures.BlackDummy);

			// The first view state's temporal render targets are used for all views in split screen
			bool bGlobalCameraCut = !Views[0].PrevViewInfo.DepthBuffer.IsValid();
			if (CompressedMetadataLayout == ECompressedMetadataLayout::DepthAndViewNormal)
			{
				PassParameters->PrevCompressedMetadata[0] = ViewInfoPooledRenderTargets.PrevCompressedDepthViewNormal
					? GraphBuilder.RegisterExternalTexture(ViewInfoPooledRenderTargets.PrevCompressedDepthViewNormal)
					: GSystemTextures.GetZeroUIntDummy(GraphBuilder);
				bGlobalCameraCut = !Views[0].PrevViewInfo.CompressedDepthViewNormal.IsValid();
			}
			else if (CompressedMetadataLayout == ECompressedMetadataLayout::FedDepthAndShadingModelID)
			{
				PassParameters->PrevCompressedMetadata[0] = RegisterExternalTextureWithFallback(
					GraphBuilder, Views[0].PrevViewInfo.CompressedOpaqueDepth, GSystemTextures.BlackDummy);
				PassParameters->PrevCompressedMetadata[1] = Views[0].PrevViewInfo.CompressedOpaqueShadingModel
					? GraphBuilder.RegisterExternalTexture(Views[0].PrevViewInfo.CompressedOpaqueShadingModel)
					: GSystemTextures.GetZeroUIntDummy(GraphBuilder);

				bGlobalCameraCut = !Views[0].PrevViewInfo.CompressedOpaqueDepth.IsValid() || !Views[0].PrevViewInfo.CompressedOpaqueShadingModel.IsValid();
			}

			// If split screen count changed, treat it as a global camera cut
			for (int32 BatchedSignalId = 0; BatchedSignalId < Settings.SignalBatchSize; BatchedSignalId++)
			{
				if (PrevFilteringHistory[BatchedSignalId] && PrevFilteringHistory[BatchedSignalId]->Scissors.Num() != Views.Num())
				{
					bGlobalCameraCut = true;
				}
			}

			FIntPoint PrevFrameBufferExtent;
			if (bGlobalCameraCut)
			{
				PassParameters->ScreenPosToHistoryBufferUV = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
				PassParameters->HistoryBufferUVMinMax = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
				PassParameters->HistoryBufferSizeAndInvSize = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
				PrevFrameBufferExtent = FIntPoint(1, 1);
			}
			else
			{
				FIntPoint ViewportOffset = View.PrevViewInfo.ViewRect.Min;
				FIntPoint ViewportExtent = View.PrevViewInfo.ViewRect.Size();

				if (PassParameters->PrevCompressedMetadata[0])
				{
					PrevFrameBufferExtent = PassParameters->PrevCompressedMetadata[0]->Desc.Extent;
				}
				else
				{
					PrevFrameBufferExtent = PassParameters->PrevDepthBuffer->Desc.Extent;
				}

				float InvBufferSizeX = 1.f / float(PrevFrameBufferExtent.X);
				float InvBufferSizeY = 1.f / float(PrevFrameBufferExtent.Y);

				PassParameters->ScreenPosToHistoryBufferUV = FVector4f(
					ViewportExtent.X * 0.5f * InvBufferSizeX,
					-ViewportExtent.Y * 0.5f * InvBufferSizeY,
					(ViewportExtent.X * 0.5f + ViewportOffset.X) * InvBufferSizeX,
					(ViewportExtent.Y * 0.5f + ViewportOffset.Y) * InvBufferSizeY);

				PassParameters->HistoryBufferUVMinMax = FVector4f(
					(ViewportOffset.X + 0.5f) * InvBufferSizeX,
					(ViewportOffset.Y + 0.5f) * InvBufferSizeY,
					(ViewportOffset.X + ViewportExtent.X - 0.5f) * InvBufferSizeX,
					(ViewportOffset.Y + ViewportExtent.Y - 0.5f) * InvBufferSizeY);

				PassParameters->HistoryBufferSizeAndInvSize = FVector4f(PrevFrameBufferExtent.X, PrevFrameBufferExtent.Y, InvBufferSizeX, InvBufferSizeY);

				PassParameters->PrevSceneBufferUVToScreenPosition.X = float(PrevFrameBufferExtent.X) / float(ViewportExtent.X) * 2.0f;
				PassParameters->PrevSceneBufferUVToScreenPosition.Y = -float(PrevFrameBufferExtent.Y) / float(ViewportExtent.Y) * 2.0f;
				PassParameters->PrevSceneBufferUVToScreenPosition.Z = -float(ViewportOffset.X) / float(ViewportExtent.X) * 2.0f - 1.0f;
				PassParameters->PrevSceneBufferUVToScreenPosition.W = float(ViewportOffset.Y) / float(ViewportExtent.Y) * 2.0f + 1.0f;
			}

			if (bGlobalCameraCut)
			{
				PassParameters->ScreenPosToHistoryBufferUV = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
				PassParameters->HistoryBufferUVMinMax = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
				PassParameters->HistoryBufferSizeAndInvSize = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
				PrevFrameBufferExtent = FIntPoint(1, 1);
			}
			else
			{
				FIntPoint ViewportOffset = View.PrevViewInfo.ViewRect.Min;
				FIntPoint ViewportExtent = View.PrevViewInfo.ViewRect.Size();

				if (PassParameters->PrevCompressedMetadata[0])
				{
					PrevFrameBufferExtent = PassParameters->PrevCompressedMetadata[0]->Desc.Extent;
				}
				else
				{
					PrevFrameBufferExtent = PassParameters->PrevDepthBuffer->Desc.Extent;
				}

				float InvBufferSizeX = 1.f / float(PrevFrameBufferExtent.X);
				float InvBufferSizeY = 1.f / float(PrevFrameBufferExtent.Y);

				PassParameters->ScreenPosToHistoryBufferUV = FVector4f(
					ViewportExtent.X * 0.5f * InvBufferSizeX,
					-ViewportExtent.Y * 0.5f * InvBufferSizeY,
					(ViewportExtent.X * 0.5f + ViewportOffset.X) * InvBufferSizeX,
					(ViewportExtent.Y * 0.5f + ViewportOffset.Y) * InvBufferSizeY);

				PassParameters->HistoryBufferUVMinMax = FVector4f(
					(ViewportOffset.X + 0.5f) * InvBufferSizeX,
					(ViewportOffset.Y + 0.5f) * InvBufferSizeY,
					(ViewportOffset.X + ViewportExtent.X - 0.5f) * InvBufferSizeX,
					(ViewportOffset.Y + ViewportExtent.Y - 0.5f) * InvBufferSizeY);

				PassParameters->HistoryBufferSizeAndInvSize = FVector4f(PrevFrameBufferExtent.X, PrevFrameBufferExtent.Y, InvBufferSizeX, InvBufferSizeY);

				PassParameters->PrevSceneBufferUVToScreenPosition.X = float(PrevFrameBufferExtent.X) / float(ViewportExtent.X) * 2.0f;
				PassParameters->PrevSceneBufferUVToScreenPosition.Y = -float(PrevFrameBufferExtent.Y) / float(ViewportExtent.Y) * 2.0f;
				PassParameters->PrevSceneBufferUVToScreenPosition.Z = -float(ViewportOffset.X) / float(ViewportExtent.X) * 2.0f - 1.0f;
				PassParameters->PrevSceneBufferUVToScreenPosition.W = float(ViewportOffset.Y) / float(ViewportExtent.Y) * 2.0f + 1.0f;
			}

			FScreenSpaceDenoiserHistory DummyPrevFrameHistory;
			DummyPrevFrameHistory.Scissors.SetNumZeroed(Views.Num());

			// Setup signals' previous frame historu buffers.
			for (int32 BatchedSignalId = 0; BatchedSignalId < Settings.SignalBatchSize; BatchedSignalId++)
			{
				FScreenSpaceDenoiserHistory* PrevFrameHistory = PrevFilteringHistory[BatchedSignalId] ? PrevFilteringHistory[BatchedSignalId] : &DummyPrevFrameHistory;

				// Use the dummy zeroed out scissors if the number of split screen views changed (this will reject the history samples)
				const auto& ScissorsToUse = PrevFrameHistory->Scissors.Num() == Views.Num() ? PrevFrameHistory->Scissors : DummyPrevFrameHistory.Scissors;

				GET_SCALAR_ARRAY_ELEMENT(PassParameters->bCameraCut, BatchedSignalId) = !PrevFrameHistory->IsValid();

				if (!(View.ViewState && Settings.bUseTemporalAccumulation) || bGlobalCameraCut)
				{
					GET_SCALAR_ARRAY_ELEMENT(PassParameters->bCameraCut, BatchedSignalId) = true;
				}

				for (int32 BufferId = 0; BufferId < HistoryTextureCountPerSignal; BufferId++)
				{
					int32 HistoryBufferId = BatchedSignalId * HistoryTextureCountPerSignal + BufferId;
					PassParameters->PrevHistory.Textures[HistoryBufferId] = RegisterExternalTextureWithFallback(
						GraphBuilder, PrevFrameHistory->RT[BufferId], GSystemTextures.BlackDummy);
				}

				PassParameters->HistoryBufferScissorUVMinMax[BatchedSignalId] = FVector4f(
					float(ScissorsToUse[ViewIndex].Min.X + 0.5f) / float(PrevFrameBufferExtent.X),
					float(ScissorsToUse[ViewIndex].Min.Y + 0.5f) / float(PrevFrameBufferExtent.Y),
					float(ScissorsToUse[ViewIndex].Max.X - 0.5f) / float(PrevFrameBufferExtent.X),
					float(ScissorsToUse[ViewIndex].Max.Y - 0.5f) / float(PrevFrameBufferExtent.Y));

				// Releases the reference on previous frame so the history's render target can be reused ASAP.
				// Need to do this for the last view when running split screen, as all views use the same render targets.
				if (ViewIndex == LastViewIndex)
				{
					PrevFrameHistory->SafeRelease();
				}
			} // for (uint32 BatchedSignalId = 0; BatchedSignalId < Settings.SignalBatchSize; BatchedSignalId++)

			PassParameters->DebugOutput = CreateDebugUAV(TEXT("DebugDenoiserTemporalAccumulation"));

			// Manually cleans the unused resource, to find out what the shader is actually going to need for next frame.
			{
				ClearUnusedGraphResources(ComputeShader, PassParameters);

				bExtractSceneDepth = PassParameters->PrevDepthBuffer != nullptr;
				bExtractSceneGBufferA = PassParameters->PrevGBufferA != nullptr;
				bExtractSceneGBufferB = PassParameters->PrevGBufferB != nullptr;

				for (int32 i = 0; i < kCompressedMetadataTextures; i++)
					bExtractCompressedMetadata[i] = PassParameters->PrevCompressedMetadata[i] != nullptr;
			}

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSD TemporalAccumulation%s",
					(!Settings.bUseTemporalAccumulation || bGlobalCameraCut) ? TEXT("(Disabled)") : TEXT("")),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(ViewSettings.Viewport.Size(), FComputeShaderUtils::kGolden2DGroupSize));
		}

		SignalHistory = SignalOutput;
	} // if (View.ViewState && Settings.bUseTemporalAccumulation)
	
	// Spatial filter, to converge history faster.
	int32 MaxPostFilterSampleCount = FMath::Clamp(Settings.HistoryConvolutionSampleCount, 1, kStackowiakMaxSampleCountPerSet);
	if (MaxPostFilterSampleCount > 1)
	{
		FSSDSignalTextures SignalOutput = CreateMultiplexedTextures(
			GraphBuilder,
			HistoryTextureCount, HistoryDescs,
			GetResourceNames(kHistoryConvolutionResourceNames),
			AllViewport);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			FSSDCommonParameters& CommonParameters = CommonParametersByView[ViewIndex];
			FSSDConvolutionMetaData& ConvolutionMetaData = ConvolutionMetaDataByView[ViewIndex];
			FSSDConstantPixelDensityViewSettings& ViewSettings = Settings.ViewSettingsArray[ViewIndex];

			FSSDSpatialAccumulationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSDSpatialAccumulationCS::FParameters>();
			PassParameters->MaxSampleCount = FMath::Clamp(MaxPostFilterSampleCount, 1, kStackowiakMaxSampleCountPerSet);
			PassParameters->KernelSpreadFactor = Settings.HistoryConvolutionKernelSpreadFactor;
			PassParameters->CommonParameters = CommonParameters;
			PassParameters->ConvolutionMetaData = ConvolutionMetaData;
			PassParameters->SignalInput = SignalHistory;
			PassParameters->SignalOutput = CreateMultiplexedUAVs(GraphBuilder, SignalOutput);

			FSSDSpatialAccumulationCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSignalProcessingDim>(Settings.SignalProcessing);
			PermutationVector.Set<FSignalBatchSizeDim>(Settings.SignalBatchSize);
			PermutationVector.Set<FSSDSpatialAccumulationCS::FStageDim>(FSSDSpatialAccumulationCS::EStage::PostFiltering);
			PermutationVector.Set<FMultiSPPDim>(true);

			PassParameters->DebugOutput = CreateDebugUAV(TEXT("DebugDenoiserPostfilter"));

			TShaderMapRef<FSSDSpatialAccumulationCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSD HistoryConvolution(MaxSamples=%i)", MaxPostFilterSampleCount),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(ViewSettings.Viewport.Size(), FSSDSpatialAccumulationCS::kGroupSize));
		}

		SignalHistory = SignalOutput;
	} // if (MaxPostFilterSampleCount > 1)

	if (!Views[0].bStatePrevViewInfoIsReadOnly && Settings.bUseTemporalAccumulation)
	{
		check(Views[0].ViewState);

		// Keep depth buffer and GBuffer around for next frame if the temporal accumulation needs it.
		// We only need to keep track of render target history for the first view with multi-view split screen.
		{
			// Might requires the depth.
			if (bExtractSceneDepth)
			{
				GraphBuilder.QueueTextureExtraction(SceneTextures.SceneDepthTexture, ViewInfoPooledRenderTargets.NextDepthBuffer);
			}

			// Might requires the world normal that are in GBuffer A.
			if (bExtractSceneGBufferA)
			{
				GraphBuilder.QueueTextureExtraction(SceneTextures.GBufferATexture, ViewInfoPooledRenderTargets.NextGBufferA);
			}

			// Might need the roughness that is in GBuffer B.
			if (bExtractSceneGBufferB)
			{
				GraphBuilder.QueueTextureExtraction(SceneTextures.GBufferBTexture, ViewInfoPooledRenderTargets.NextGBufferB);
			}

			// Extract the compressed scene texture to make te history re-projection faster.
			for (int32 i = 0; i < kCompressedMetadataTextures; i++)
			{
				TRefCountPtr<IPooledRenderTarget>* Dest = nullptr;

				if (CompressedMetadataLayout == ECompressedMetadataLayout::DepthAndViewNormal)
				{
					if (i == 0)
					{
						Dest = ViewInfoPooledRenderTargets.NextCompressedDepthViewNormal;
					}
				}
				else if (CompressedMetadataLayout == ECompressedMetadataLayout::FedDepthAndShadingModelID)
				{
					if (i == 0)
					{
						Dest = &Views[0].ViewState->PrevFrameViewInfo.CompressedOpaqueDepth;
					}
					else // if (i == 1)
					{
						Dest = &Views[0].ViewState->PrevFrameViewInfo.CompressedOpaqueShadingModel;
					}
				}

				check((CommonParametersByView[0].CompressedMetadata[i] != nullptr) == (Dest != nullptr));

				if (Dest)
				{
					check(CommonParametersByView[0].CompressedMetadata[i]);
					GraphBuilder.QueueTextureExtraction(CommonParametersByView[0].CompressedMetadata[i], Dest);
				}
			}
		}

		// Saves signal histories.
		for (int32 BatchedSignalId = 0; BatchedSignalId < Settings.SignalBatchSize; BatchedSignalId++)
		{
			FScreenSpaceDenoiserHistory* NewHistory = NewFilteringHistory[BatchedSignalId];
			check(NewHistory);

			for (int32 BufferId = 0; BufferId < HistoryTextureCountPerSignal; BufferId++)
			{
				int32 HistoryBufferId = BatchedSignalId * HistoryTextureCountPerSignal + BufferId;
				GraphBuilder.QueueTextureExtraction(SignalHistory.Textures[HistoryBufferId], &NewHistory->RT[BufferId]);
			}

			NewHistory->Scissors.SetNumUninitialized(Views.Num());
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				FSSDConstantPixelDensityViewSettings& ViewSettings = Settings.ViewSettingsArray[ViewIndex];

				NewHistory->Scissors[ViewIndex] = ViewSettings.FullResViewport;
			}
		} // for (uint32 BatchedSignalId = 0; BatchedSignalId < Settings.SignalBatchSize; BatchedSignalId++)
	}

	// Final convolution / output to correct
	if (SignalUsesFinalConvolution(Settings.SignalProcessing))
	{
		TStaticArray<FRDGTextureDesc, kMaxBufferProcessingCount> OutputDescs;
		for (int32 i = 0; i < kMaxBufferProcessingCount; i++)
		{
			OutputDescs[i] = HistoryDescs[i];
		}

		if (Settings.SignalProcessing == ESignalProcessing::ShadowVisibilityMask)
		{
			for (int32 BatchedSignalId = 0; BatchedSignalId < Settings.SignalBatchSize; BatchedSignalId++)
			{
				OutputDescs[BatchedSignalId].Format = PF_FloatRGBA;
			}
		}
		else
		{
			check(0);
		}

		*OutputSignal = CreateMultiplexedTextures(
			GraphBuilder,
			Settings.SignalBatchSize, OutputDescs,
			GetResourceNames(kDenoiserOutputResourceNames),
			AllViewport);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			FSSDCommonParameters& CommonParameters = CommonParametersByView[ViewIndex];
			FSSDConvolutionMetaData& ConvolutionMetaData = ConvolutionMetaDataByView[ViewIndex];
			FSSDConstantPixelDensityViewSettings& ViewSettings = Settings.ViewSettingsArray[ViewIndex];

			FSSDSpatialAccumulationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSDSpatialAccumulationCS::FParameters>();
			PassParameters->CommonParameters = CommonParameters;
			PassParameters->ConvolutionMetaData = ConvolutionMetaData;
			PassParameters->SignalInput = SignalHistory;
			PassParameters->OriginalInput = InputSignal;
			PassParameters->SignalOutput = CreateMultiplexedUAVs(GraphBuilder, *OutputSignal);

			FSSDSpatialAccumulationCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSignalProcessingDim>(Settings.SignalProcessing);
			PermutationVector.Set<FSignalBatchSizeDim>(Settings.SignalBatchSize);
			PermutationVector.Set<FSSDSpatialAccumulationCS::FStageDim>(FSSDSpatialAccumulationCS::EStage::FinalOutput);
			PermutationVector.Set<FMultiSPPDim>(true);
			PermutationVector.Set<FSSDSpatialAccumulationCS::FDenoiseTransmissionDim>(CVarShadowDenoiserTransmission.GetValueOnRenderThread()!= 0);

			TShaderMapRef<FSSDSpatialAccumulationCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSD SpatialAccumulation(Final)"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(ViewSettings.Viewport.Size(), FSSDSpatialAccumulationCS::kGroupSize));
		}
	}
	else
	{
		*OutputSignal = SignalHistory;
	}
} // DenoiseSignalAtConstantPixelDensity()

// static
IScreenSpaceDenoiser::FHarmonicTextures IScreenSpaceDenoiser::CreateHarmonicTextures(FRDGBuilder& GraphBuilder, FIntPoint Extent, const TCHAR* DebugName)
{
	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
		Extent,
		PF_FloatRGBA,
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV);

	FHarmonicTextures HarmonicTextures;
	for (int32 HarmonicBorderId = 0; HarmonicBorderId < kHarmonicBordersCount; HarmonicBorderId++)
	{
		HarmonicTextures.Harmonics[HarmonicBorderId] = GraphBuilder.CreateTexture(Desc, DebugName);
	}
	return HarmonicTextures;
}

// static
IScreenSpaceDenoiser::FHarmonicUAVs IScreenSpaceDenoiser::CreateUAVs(FRDGBuilder& GraphBuilder, const FHarmonicTextures& Textures)
{
	FHarmonicUAVs UAVs;
	for (int32 HarmonicBorderId = 0; HarmonicBorderId < kHarmonicBordersCount; HarmonicBorderId++)
	{
		UAVs.Harmonics[HarmonicBorderId] = GraphBuilder.CreateUAV(Textures.Harmonics[HarmonicBorderId]);
	}
	return UAVs;
}

// static
IScreenSpaceDenoiser::FDiffuseIndirectHarmonicUAVs IScreenSpaceDenoiser::CreateUAVs(FRDGBuilder& GraphBuilder, const FDiffuseIndirectHarmonic& Textures)
{
	FDiffuseIndirectHarmonicUAVs UAVs;
	for (int32 HarmonicBorderId = 0; HarmonicBorderId < kSphericalHarmonicTextureCount; HarmonicBorderId++)
	{
		UAVs.SphericalHarmonic[HarmonicBorderId] = GraphBuilder.CreateUAV(Textures.SphericalHarmonic[HarmonicBorderId]);
	}
	return UAVs;
}

/** The implementation of the default denoiser of the renderer. */
class FDefaultScreenSpaceDenoiser : public IScreenSpaceDenoiser
{
public:
	const TCHAR* GetDebugName() const override
	{
		return TEXT("ScreenSpaceDenoiser");
	}

	virtual EShadowRequirements GetShadowRequirements(
		const FViewInfo& View,
		const FLightSceneInfo& LightSceneInfo,
		const FShadowRayTracingConfig& RayTracingConfig) const override
	{
		check(SignalSupportMultiSPP(ESignalProcessing::ShadowVisibilityMask));
		return IScreenSpaceDenoiser::EShadowRequirements::PenumbraAndClosestOccluder;
	}

	virtual void DenoiseShadowVisibilityMasks(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const TStaticArray<FShadowVisibilityParameters, IScreenSpaceDenoiser::kMaxBatchSize>& InputParameters,
		const int32 InputParameterCount,
		TStaticArray<FShadowVisibilityOutputs, IScreenSpaceDenoiser::kMaxBatchSize>& Outputs) const
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, ShadowsDenoiser, "DenoiseShadowVisibilityMasks");

		FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
		SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

		FSSDSignalTextures InputSignal;

		FSSDConstantPixelDensitySettings Settings;
		FSSDConstantPixelDensityViewSettings ViewSettings;
		Settings.SignalProcessing = ESignalProcessing::ShadowVisibilityMask;
		Settings.InputResolutionFraction = 1.0f;
		Settings.ReconstructionSamples = FMath::Clamp(CVarShadowReconstructionSampleCount.GetValueOnRenderThread(), 1, kStackowiakMaxSampleCountPerSet);
		Settings.PreConvolutionCount = CVarShadowPreConvolutionCount.GetValueOnRenderThread();
		Settings.bUseTemporalAccumulation = CVarShadowTemporalAccumulation.GetValueOnRenderThread() != 0;
		Settings.HistoryConvolutionSampleCount = CVarShadowHistoryConvolutionSampleCount.GetValueOnRenderThread();
		Settings.SignalBatchSize = InputParameterCount;

		for (int32 BatchedSignalId = 0; BatchedSignalId < InputParameterCount; BatchedSignalId++)
		{
			Settings.MaxInputSPP = FMath::Max(Settings.MaxInputSPP, InputParameters[BatchedSignalId].RayTracingConfig.RayCountPerPixel);
		}

		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories = {};
		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories  = {};
		for (int32 BatchedSignalId = 0; BatchedSignalId < InputParameterCount; BatchedSignalId++)
		{
			const FShadowVisibilityParameters& Parameters = InputParameters[BatchedSignalId];
			const FLightSceneProxy* Proxy = Parameters.LightSceneInfo->Proxy;
			
			// Scissor the denoiser.
			{
				FIntRect LightScissorRect;
				if (Proxy->GetScissorRect(/* out */ LightScissorRect, View, View.ViewRect))
				{

				}
				else
				{
					LightScissorRect = View.ViewRect;
				}

				if (BatchedSignalId == 0)
				{
					ViewSettings.FullResViewport = LightScissorRect;
				}
				else
				{
					ViewSettings.FullResViewport.Union(LightScissorRect);
				}

				ViewSettings.SignalScissor[BatchedSignalId] = LightScissorRect;
			}

			ensure(IsSupportedLightType(ELightComponentType(Proxy->GetLightType())));

			Settings.LightSceneInfo[BatchedSignalId] = Parameters.LightSceneInfo;
			// Get the packed penumbra and hit distance in Penumbra texture.
			InputSignal.Textures[BatchedSignalId] = Parameters.InputTextures.Mask;

			const FLightComponentId& LightComponentId = Settings.LightSceneInfo[BatchedSignalId]->Proxy->GetLightComponentId();
			TSharedPtr<FScreenSpaceDenoiserHistory>* PrevHistoryEntry = PreviousViewInfos->ShadowHistories.Find(LightComponentId);
			PrevHistories[BatchedSignalId] = PrevHistoryEntry ? PrevHistoryEntry->Get() : nullptr;
			NewHistories[BatchedSignalId] = nullptr;
				
			if (!View.bStatePrevViewInfoIsReadOnly)
			{
				check(View.ViewState);
				TSharedPtr<FScreenSpaceDenoiserHistory>* NewHistoryEntry = View.ViewState->PrevFrameViewInfo.ShadowHistories.Find(LightComponentId);
				if (NewHistoryEntry == nullptr)
				{
					FScreenSpaceDenoiserHistory* NewHistory = new FScreenSpaceDenoiserHistory;
					View.ViewState->PrevFrameViewInfo.ShadowHistories.Emplace(LightComponentId, NewHistory);
					NewHistories[BatchedSignalId] = NewHistory;
				}
				else
				{
					NewHistories[BatchedSignalId] = NewHistoryEntry->Get();
				}
			}
		}

		// Force viewport to be a multiple of 2, to avoid over frame interference between TAA jitter of the frame, and Stackowiack's SampleTrackId.
		{
			ViewSettings.FullResViewport.Min.X &= ~1;
			ViewSettings.FullResViewport.Min.Y &= ~1;
		}

		Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);

		FSSDSignalTextures SignalOutput;
		DenoiseSignalAtConstantPixelDensity(
			GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures, ViewInfoPooledRenderTargets,
			InputSignal, Settings,
			PrevHistories,
			NewHistories,
			&SignalOutput);

		for (int32 BatchedSignalId = 0; BatchedSignalId < InputParameterCount; BatchedSignalId++)
		{
			Outputs[BatchedSignalId].Mask = SignalOutput.Textures[BatchedSignalId];
		}
	}

	FPolychromaticPenumbraOutputs DenoisePolychromaticPenumbraHarmonics(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FPolychromaticPenumbraHarmonics& Inputs) const override
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, ShadowsDenoiser, "DenoisePolychromaticPenumbraHarmonics");

		FRDGTextureRef BlackDummy = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		FRDGTextureRef WhiteDummy = GraphBuilder.RegisterExternalTexture(GSystemTextures.WhiteDummy);

		FSSDComposeHarmonicsCS::FParameters* ComposePassParameters = GraphBuilder.AllocParameters<FSSDComposeHarmonicsCS::FParameters>();

		// Harmonic 0 doesn't need any reconstruction given it's the highest frequency details.
		{
			const int32 HarmonicId = 0;
			ComposePassParameters->SignalHarmonics[HarmonicId].Textures[0] = Inputs.Diffuse.Harmonics[0];
			ComposePassParameters->SignalHarmonics[HarmonicId].Textures[1] = Inputs.Diffuse.Harmonics[1];
			ComposePassParameters->SignalHarmonics[HarmonicId].Textures[2] = Inputs.Specular.Harmonics[0];
			ComposePassParameters->SignalHarmonics[HarmonicId].Textures[3] = Inputs.Specular.Harmonics[1];
		}

		// Reconstruct each harmonic independently
		for (int32 HarmonicId = 1; HarmonicId < IScreenSpaceDenoiser::kMultiPolychromaticPenumbraHarmonics; HarmonicId++)
		{
			int32 Periode = 1 << HarmonicId;

			FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
			SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

			FSSDConstantPixelDensitySettings Settings;
			FSSDConstantPixelDensityViewSettings ViewSettings;
			ViewSettings.FullResViewport = View.ViewRect;
			Settings.SignalProcessing = ESignalProcessing::PolychromaticPenumbraHarmonic;
			Settings.HarmonicPeriode = Periode;
			Settings.ReconstructionSamples = Periode * Periode; // TODO(Denoiser): should use preconvolution instead for harmonic 3
			Settings.bUseTemporalAccumulation = false;
			Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);

			TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
			TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
			PrevHistories[0] = nullptr;
			NewHistories[0] = nullptr;

			FSSDSignalTextures InputSignal;
			InputSignal.Textures[0] = Inputs.Diffuse.Harmonics[HarmonicId + 0];
			InputSignal.Textures[1] = Inputs.Diffuse.Harmonics[HarmonicId + 1];
			InputSignal.Textures[2] = Inputs.Specular.Harmonics[HarmonicId + 0];
			InputSignal.Textures[3] = Inputs.Specular.Harmonics[HarmonicId + 1];

			FSSDSignalTextures SignalOutput;
			DenoiseSignalAtConstantPixelDensity(
				GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures, ViewInfoPooledRenderTargets,
				InputSignal, Settings,
				PrevHistories, NewHistories,
				/* out */ &SignalOutput);

			ComposePassParameters->SignalHarmonics[HarmonicId] = SignalOutput;
		}

		// Denoise the entire integrand signal.
		// TODO(Denoiser): this assume all the lights are going into lowest frequency harmonic.
		#if (1)
		{
			const int32 HarmonicId = IScreenSpaceDenoiser::kMultiPolychromaticPenumbraHarmonics - 1;

			int32 Periode = 1 << HarmonicId;

			FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
			SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

			FSSDConstantPixelDensitySettings Settings;
			FSSDConstantPixelDensityViewSettings ViewSettings;
			ViewSettings.FullResViewport = View.ViewRect;
			Settings.SignalProcessing = ESignalProcessing::PolychromaticPenumbraHarmonic;
			Settings.HarmonicPeriode = Periode;
			Settings.ReconstructionSamples = Periode * Periode; // TODO(Denoiser): should use preconvolution instead for harmonic 3
			Settings.bUseTemporalAccumulation = false;
			Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);

			TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
			TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
			PrevHistories[0] = nullptr;
			NewHistories[0] = nullptr;

			// TODO(Denoiser): pipeline permutation to be faster.
			FSSDSignalTextures InputSignal;
			InputSignal.Textures[0] = Inputs.Diffuse.Harmonics[0];
			InputSignal.Textures[1] = BlackDummy;
			InputSignal.Textures[2] = Inputs.Specular.Harmonics[0];
			InputSignal.Textures[3] = BlackDummy;

			DenoiseSignalAtConstantPixelDensity(
				GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures, ViewInfoPooledRenderTargets,
				InputSignal, Settings,
				PrevHistories, NewHistories,
				/* out */ &ComposePassParameters->SignalIntegrand);
		}
		#else
		{
			ComposePassParameters->SignalIntegrand.Textures[0] = WhiteDummy;
			ComposePassParameters->SignalIntegrand.Textures[1] = BlackDummy;
			ComposePassParameters->SignalIntegrand.Textures[2] = WhiteDummy;
			ComposePassParameters->SignalIntegrand.Textures[3] = BlackDummy;
		}
		#endif

		// Merges the different harmonics.
		FSSDSignalTextures ComposedHarmonics;
		{
			FIntPoint BufferExtent = SceneTextures.SceneDepthTexture->Desc.Extent;

			{
				FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
					BufferExtent,
					PF_FloatRGBA,
					FClearValueBinding::Black,
					TexCreate_ShaderResource | TexCreate_UAV);

				ComposedHarmonics.Textures[0] = GraphBuilder.CreateTexture(Desc, TEXT("PolychromaticPenumbraComposition0"));
				ComposedHarmonics.Textures[1] = GraphBuilder.CreateTexture(Desc, TEXT("PolychromaticPenumbraComposition1"));

				ComposedHarmonics.Textures[0]->EncloseVisualizeExtent(View.ViewRect.Max);
				ComposedHarmonics.Textures[1]->EncloseVisualizeExtent(View.ViewRect.Max);
			}

			ComposePassParameters->CommonParameters.ViewUniformBuffer = View.ViewUniformBuffer;
			ComposePassParameters->CommonParameters.SceneTextures = SceneTextures;
			ComposePassParameters->CommonParameters.ViewportMin = View.ViewRect.Min;
			ComposePassParameters->CommonParameters.ViewportMax = View.ViewRect.Max;
			ComposePassParameters->CommonParameters.PublicCommonParameters.DenoiserBufferBilinearUVMinMax = FVector4f(
				float(View.ViewRect.Min.X + 0.5f) / float(BufferExtent.X),
				float(View.ViewRect.Min.Y + 0.5f) / float(BufferExtent.Y),
				float(View.ViewRect.Max.X - 0.5f) / float(BufferExtent.X),
				float(View.ViewRect.Max.Y - 0.5f) / float(BufferExtent.Y));

			ComposePassParameters->SignalOutput = CreateMultiplexedUAVs(GraphBuilder, ComposedHarmonics);

			{
				FRDGTextureDesc DebugDesc = FRDGTextureDesc::Create2D(
					SceneTextures.SceneDepthTexture->Desc.Extent,
					PF_FloatRGBA,
					FClearValueBinding::Black,
					TexCreate_ShaderResource | TexCreate_UAV);

				FRDGTextureRef DebugTexture = GraphBuilder.CreateTexture(DebugDesc, TEXT("DebugHarmonicComposition"));
				ComposePassParameters->DebugOutput = GraphBuilder.CreateUAV(DebugTexture);
			}

			TShaderMapRef<FSSDComposeHarmonicsCS> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSD ComposeHarmonics"),
				ComputeShader, ComposePassParameters,
				FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), FSSDSpatialAccumulationCS::kGroupSize));
		}

		FPolychromaticPenumbraOutputs Outputs;
		{
			FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
			SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

			FSSDConstantPixelDensitySettings Settings;
			FSSDConstantPixelDensityViewSettings ViewSettings;
			ViewSettings.FullResViewport = View.ViewRect;
			Settings.SignalProcessing = ESignalProcessing::PolychromaticPenumbraHarmonic;
			Settings.bEnableReconstruction = false;
			Settings.bUseTemporalAccumulation = CVarShadowTemporalAccumulation.GetValueOnRenderThread() != 0;
			Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);

			TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
			TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
			PrevHistories[0] = &PreviousViewInfos->PolychromaticPenumbraHarmonicsHistory;
			NewHistories[0] = View.ViewState ? &View.ViewState->PrevFrameViewInfo.PolychromaticPenumbraHarmonicsHistory : nullptr;

			FSSDSignalTextures SignalOutput;
			DenoiseSignalAtConstantPixelDensity(
				GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures, ViewInfoPooledRenderTargets,
				ComposedHarmonics, Settings,
				PrevHistories, NewHistories,
				/* out */ &SignalOutput);

			Outputs.Diffuse = SignalOutput.Textures[0];
			Outputs.Specular = SignalOutput.Textures[1];
		}

		return Outputs;
	}

	FReflectionsOutputs DenoiseReflections(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FReflectionsInputs& ReflectionInputs,
		const FReflectionsRayTracingConfig RayTracingConfig) const override
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, ReflectionsDenoiser, "DenoiseReflections");

		FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
		SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

		FSSDSignalTextures InputSignal;
		InputSignal.Textures[0] = ReflectionInputs.Color;
		InputSignal.Textures[1] = ReflectionInputs.RayHitDistance;

		FSSDConstantPixelDensitySettings Settings;
		FSSDConstantPixelDensityViewSettings ViewSettings;
		ViewSettings.FullResViewport = View.ViewRect;
		Settings.SignalProcessing = ESignalProcessing::Reflections;
		Settings.InputResolutionFraction = RayTracingConfig.ResolutionFraction;
		Settings.ReconstructionSamples = CVarReflectionReconstructionSampleCount.GetValueOnRenderThread();
		Settings.PreConvolutionCount = CVarReflectionPreConvolutionCount.GetValueOnRenderThread();
		Settings.bUseTemporalAccumulation = CVarReflectionTemporalAccumulation.GetValueOnRenderThread() != 0;
		Settings.MaxInputSPP = RayTracingConfig.RayCountPerPixel;
		Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);

		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
		PrevHistories[0] = &PreviousViewInfos->ReflectionsHistory;
		NewHistories[0] = View.ViewState ? &View.ViewState->PrevFrameViewInfo.ReflectionsHistory : nullptr;

		FSSDSignalTextures SignalOutput;
		DenoiseSignalAtConstantPixelDensity(
			GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures, ViewInfoPooledRenderTargets,
			InputSignal, Settings,
			PrevHistories,
			NewHistories,
			&SignalOutput);

		FReflectionsOutputs ReflectionsOutput;
		ReflectionsOutput.Color = SignalOutput.Textures[0];
		return ReflectionsOutput;
	}

	FReflectionsOutputs DenoiseWaterReflections(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FReflectionsInputs& ReflectionInputs,
		const FReflectionsRayTracingConfig RayTracingConfig) const override
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, ReflectionsDenoiser, "DenoiseWaterReflections");
		
		FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
		SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

		FSSDSignalTextures InputSignal;
		InputSignal.Textures[0] = ReflectionInputs.Color;
		InputSignal.Textures[1] = ReflectionInputs.RayHitDistance;

		FSSDConstantPixelDensitySettings Settings;
		FSSDConstantPixelDensityViewSettings ViewSettings;
		ViewSettings.FullResViewport = View.ViewRect;
		Settings.SignalProcessing = ESignalProcessing::Reflections; // TODO: water reflection to denoise only water pixels
		Settings.InputResolutionFraction = RayTracingConfig.ResolutionFraction;
		Settings.ReconstructionSamples = CVarReflectionReconstructionSampleCount.GetValueOnRenderThread();
		Settings.PreConvolutionCount = CVarReflectionPreConvolutionCount.GetValueOnRenderThread();
		Settings.bUseTemporalAccumulation = CVarReflectionTemporalAccumulation.GetValueOnRenderThread() != 0;
		Settings.MaxInputSPP = RayTracingConfig.RayCountPerPixel;
		Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);

		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
		PrevHistories[0] = &PreviousViewInfos->WaterReflectionsHistory;
		NewHistories[0] = View.ViewState ? &View.ViewState->PrevFrameViewInfo.WaterReflectionsHistory : nullptr;

		FSSDSignalTextures SignalOutput;
		DenoiseSignalAtConstantPixelDensity(
			GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures,
			ViewInfoPooledRenderTargets,
			InputSignal, Settings,
			PrevHistories,
			NewHistories,
			&SignalOutput);

		FReflectionsOutputs ReflectionsOutput;
		ReflectionsOutput.Color = SignalOutput.Textures[0];
		return ReflectionsOutput;
	}
	
	FAmbientOcclusionOutputs DenoiseAmbientOcclusion(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FAmbientOcclusionInputs& ReflectionInputs,
		const FAmbientOcclusionRayTracingConfig RayTracingConfig) const override
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, AmbientOcclusionDenoiser, "DenoiseAmbientOcclusion");

		FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
		SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

		FSSDSignalTextures InputSignal;
		InputSignal.Textures[0] = ReflectionInputs.Mask;
		InputSignal.Textures[1] = ReflectionInputs.RayHitDistance;
		
		FSSDConstantPixelDensitySettings Settings;
		FSSDConstantPixelDensityViewSettings ViewSettings;
		ViewSettings.FullResViewport = View.ViewRect;
		Settings.SignalProcessing = ESignalProcessing::AmbientOcclusion;
		Settings.InputResolutionFraction = RayTracingConfig.ResolutionFraction;
		Settings.ReconstructionSamples = FMath::Clamp(CVarAOReconstructionSampleCount.GetValueOnRenderThread(), 1, kStackowiakMaxSampleCountPerSet);
		Settings.PreConvolutionCount = CVarAOPreConvolutionCount.GetValueOnRenderThread();
		Settings.KernelSpreadFactor = CVarAOKernelSpreadFactor.GetValueOnRenderThread();
		Settings.bUseTemporalAccumulation = CVarAOTemporalAccumulation.GetValueOnRenderThread() != 0;
		Settings.HistoryConvolutionSampleCount = CVarAOHistoryConvolutionSampleCount.GetValueOnRenderThread();
		Settings.HistoryConvolutionKernelSpreadFactor = CVarAOHistoryConvolutionKernelSpreadFactor.GetValueOnRenderThread();
		Settings.MaxInputSPP = RayTracingConfig.RayCountPerPixel;
		Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);

		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
		PrevHistories[0] = &PreviousViewInfos->AmbientOcclusionHistory;
		NewHistories[0] = View.ViewState ? &View.ViewState->PrevFrameViewInfo.AmbientOcclusionHistory : nullptr;

		FSSDSignalTextures SignalOutput;
		DenoiseSignalAtConstantPixelDensity(
			GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures, ViewInfoPooledRenderTargets,
			InputSignal, Settings,
			PrevHistories,
			NewHistories,
			&SignalOutput);

		FAmbientOcclusionOutputs AmbientOcclusionOutput;
		AmbientOcclusionOutput.AmbientOcclusionMask = SignalOutput.Textures[0];
		return AmbientOcclusionOutput;
	}

	FSSDSignalTextures DenoiseDiffuseIndirect(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FDiffuseIndirectInputs& Inputs,
		const FAmbientOcclusionRayTracingConfig Config) const override
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, DiffuseIndirectDenoiser, "DenoiseDiffuseIndirect");

		FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
		SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

		FSSDSignalTextures InputSignal;
		InputSignal.Textures[0] = Inputs.Color;
		InputSignal.Textures[1] = Inputs.RayHitDistance;

		FSSDConstantPixelDensitySettings Settings;
		FSSDConstantPixelDensityViewSettings ViewSettings;
		ViewSettings.FullResViewport = View.ViewRect;
		Settings.SignalProcessing = ESignalProcessing::DiffuseAndAmbientOcclusion;
		Settings.InputResolutionFraction = Config.ResolutionFraction;
		Settings.ReconstructionSamples = FMath::Clamp(CVarGIReconstructionSampleCount.GetValueOnRenderThread(), 1, kStackowiakMaxSampleCountPerSet);
		Settings.PreConvolutionCount = CVarGIPreConvolutionCount.GetValueOnRenderThread();
		Settings.bUseTemporalAccumulation = CVarGITemporalAccumulation.GetValueOnRenderThread() != 0;
		Settings.HistoryConvolutionSampleCount = CVarGIHistoryConvolutionSampleCount.GetValueOnRenderThread();
		Settings.HistoryConvolutionKernelSpreadFactor = CVarGIHistoryConvolutionKernelSpreadFactor.GetValueOnRenderThread();
		Settings.MaxInputSPP = Config.RayCountPerPixel;
		Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);

		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
		PrevHistories[0] = &PreviousViewInfos->DiffuseIndirectHistory;
		NewHistories[0] = View.ViewState ? &View.ViewState->PrevFrameViewInfo.DiffuseIndirectHistory : nullptr;

		FSSDSignalTextures SignalOutput;
		DenoiseSignalAtConstantPixelDensity(
			GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures, ViewInfoPooledRenderTargets,
			InputSignal, Settings,
			PrevHistories,
			NewHistories,
			&SignalOutput);

		return SignalOutput;
	}

	FDiffuseIndirectOutputs DenoiseSkyLight(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FDiffuseIndirectInputs& Inputs,
		const FAmbientOcclusionRayTracingConfig Config) const override
	{
		return IScreenSpaceDenoiser::DenoiseSkyLight(
			GraphBuilder,
			TConstArrayView<FViewInfo>(&View, 1),
			PreviousViewInfos,
			SceneTextures,
			Inputs,
			Config);
	}
	
	FSSDSignalTextures DenoiseDiffuseIndirectHarmonic(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FDiffuseIndirectHarmonic& Inputs,
		const HybridIndirectLighting::FCommonParameters& CommonDiffuseParameters) const override
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, DiffuseIndirectDenoiser, "DenoiseDiffuseIndirectHarmonic");

		FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
		SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

		FSSDSignalTextures InputSignal;
		for (int32 i = 0; i < IScreenSpaceDenoiser::kSphericalHarmonicTextureCount; i++)
			InputSignal.Textures[i] = Inputs.SphericalHarmonic[i];

		FSSDConstantPixelDensitySettings Settings;
		FSSDConstantPixelDensityViewSettings ViewSettings;
		ViewSettings.FullResViewport = View.ViewRect;
		Settings.SignalProcessing = ESignalProcessing::DiffuseSphericalHarmonic;
		Settings.InputResolutionFraction = 1.0f / float(CommonDiffuseParameters.DownscaleFactor);
		Settings.ReconstructionSamples = CVarGIReconstructionSampleCount.GetValueOnRenderThread();
		Settings.bUseTemporalAccumulation = CVarGITemporalAccumulation.GetValueOnRenderThread() != 0;
		Settings.MaxInputSPP = CommonDiffuseParameters.RayCountPerPixel;
		Settings.DenoisingResolutionFraction = 1.0f / float(CommonDiffuseParameters.DownscaleFactor);
		Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);

		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
		PrevHistories[0] = &PreviousViewInfos->DiffuseIndirectHistory;
		NewHistories[0] = View.ViewState ? &View.ViewState->PrevFrameViewInfo.DiffuseIndirectHistory : nullptr;

		FSSDSignalTextures SignalOutput;
		DenoiseSignalAtConstantPixelDensity(
			GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures, ViewInfoPooledRenderTargets,
			InputSignal, Settings,
			PrevHistories,
			NewHistories,
			&SignalOutput);

		return SignalOutput;
	}

	bool SupportsScreenSpaceDiffuseIndirectDenoiser(EShaderPlatform Platform) const override
	{
		return ShouldCompileSignalPipeline(ESignalProcessing::ScreenSpaceDiffuseIndirect, Platform);
	}

	FSSDSignalTextures DenoiseScreenSpaceDiffuseIndirect(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FDiffuseIndirectInputs& Inputs,
		const FAmbientOcclusionRayTracingConfig Config) const override
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, DiffuseIndirectDenoiser, "DenoiseScreenSpaceDiffuseIndirect");

		FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
		SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

		FSSDSignalTextures InputSignal;
		InputSignal.Textures[0] = Inputs.Color;
		InputSignal.Textures[1] = Inputs.AmbientOcclusionMask;

		FSSDConstantPixelDensitySettings Settings;
		FSSDConstantPixelDensityViewSettings ViewSettings;
		ViewSettings.FullResViewport = View.ViewRect;
		Settings.SignalProcessing = ESignalProcessing::ScreenSpaceDiffuseIndirect;
		Settings.InputResolutionFraction = Config.ResolutionFraction;
		Settings.DenoisingResolutionFraction = Config.ResolutionFraction;
		Settings.ReconstructionSamples = 8;
		Settings.bUseTemporalAccumulation = CVarGITemporalAccumulation.GetValueOnRenderThread() != 0;
		Settings.MaxInputSPP = Config.RayCountPerPixel;
		Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);

		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
		TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
		PrevHistories[0] = &PreviousViewInfos->DiffuseIndirectHistory;
		NewHistories[0] = View.ViewState ? &View.ViewState->PrevFrameViewInfo.DiffuseIndirectHistory : nullptr;

		FSSDSignalTextures SignalOutput;
		DenoiseSignalAtConstantPixelDensity(
			GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures, ViewInfoPooledRenderTargets,
			InputSignal, Settings,
			PrevHistories,
			NewHistories,
			&SignalOutput);

		return SignalOutput;
	}
}; // class FDefaultScreenSpaceDenoiser

// static
FSSDSignalTextures IScreenSpaceDenoiser::DenoiseIndirectProbeHierarchy(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FPreviousViewInfo* PreviousViewInfos,
	const FSceneTextureParameters& SceneTextures,
	const FSSDSignalTextures& InputSignal,
	FRDGTextureRef CompressedDepthTexture,
	FRDGTextureRef CompressedShadingModelTexture)
{
	FSSDConstantPixelDensitySettings Settings;
	FSSDConstantPixelDensityViewSettings ViewSettings;
	ViewSettings.FullResViewport = View.ViewRect;
	Settings.SignalProcessing = ESignalProcessing::IndirectProbeHierarchy;
	Settings.bEnableReconstruction = false;
	Settings.bUseTemporalAccumulation = CVarGITemporalAccumulation.GetValueOnRenderThread() != 0;
	Settings.MaxInputSPP = 8;
	Settings.CompressedDepthTexture = CompressedDepthTexture;
	Settings.CompressedShadingModelTexture = CompressedShadingModelTexture;
	Settings.ViewSettingsArray = TArrayView<FSSDConstantPixelDensityViewSettings>(&ViewSettings, 1);
	TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
	TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
	PrevHistories[0] = &PreviousViewInfos->DiffuseIndirectHistory;
	NewHistories[0] = View.ViewState ? &View.ViewState->PrevFrameViewInfo.DiffuseIndirectHistory : nullptr;
	FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
	SetupSceneViewInfoPooledRenderTargets(View, &ViewInfoPooledRenderTargets);

	FSSDSignalTextures SignalOutput;
	DenoiseSignalAtConstantPixelDensity(
		GraphBuilder, TConstArrayView<FViewInfo>(&View, 1), SceneTextures, ViewInfoPooledRenderTargets,
		InputSignal, Settings, 
		PrevHistories,
		NewHistories,
		&SignalOutput);

	return SignalOutput;
}

// static
const IScreenSpaceDenoiser* IScreenSpaceDenoiser::GetDefaultDenoiser()
{
	static IScreenSpaceDenoiser* GDefaultDenoiser = new FDefaultScreenSpaceDenoiser;
	return GDefaultDenoiser;
}

int GetReflectionsDenoiserMode()
{
	return CVarUseReflectionDenoiser.GetValueOnRenderThread();
}

// static
IScreenSpaceDenoiser::EMode IScreenSpaceDenoiser::GetDenoiserMode(const TAutoConsoleVariable<int32>& CVar)
{
	int32 CVarSettings = CVar.GetValueOnRenderThread();
	if (CVarSettings == 0)
	{
		return EMode::Disabled;
	}
	else if (CVarSettings == 1 || GScreenSpaceDenoiser == GetDefaultDenoiser())
	{
		return EMode::DefaultDenoiser;
	}
	return EMode::ThirdPartyDenoiser;
}

IScreenSpaceDenoiser::FDiffuseIndirectOutputs IScreenSpaceDenoiser::DenoiseSkyLight(
	FRDGBuilder& GraphBuilder,
	TConstArrayView<FViewInfo> Views,
	FPreviousViewInfo* PreviousViewInfos,
	const FSceneTextureParameters& SceneTextures,
	const FDiffuseIndirectInputs& Inputs,
	const FAmbientOcclusionRayTracingConfig Config)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, DiffuseIndirectDenoiser, "DenoiseSkyLight");

	FViewInfoPooledRenderTargets ViewInfoPooledRenderTargets;
	SetupSceneViewInfoPooledRenderTargets(Views[0], &ViewInfoPooledRenderTargets);

	FSSDSignalTextures InputSignal;
	InputSignal.Textures[0] = Inputs.Color;
	InputSignal.Textures[1] = Inputs.RayHitDistance;

	FSSDConstantPixelDensitySettings Settings;
	Settings.SignalProcessing = ESignalProcessing::DiffuseAndAmbientOcclusion;
	Settings.InputResolutionFraction = Config.ResolutionFraction;
	Settings.ReconstructionSamples = FMath::Clamp(CVarGIReconstructionSampleCount.GetValueOnRenderThread(), 1, kStackowiakMaxSampleCountPerSet);
	Settings.PreConvolutionCount = CVarGIPreConvolutionCount.GetValueOnRenderThread();
	Settings.bUseTemporalAccumulation = CVarGITemporalAccumulation.GetValueOnRenderThread() != 0;
	Settings.HistoryConvolutionSampleCount = CVarGIHistoryConvolutionSampleCount.GetValueOnRenderThread();
	Settings.HistoryConvolutionKernelSpreadFactor = CVarGIHistoryConvolutionKernelSpreadFactor.GetValueOnRenderThread();
	Settings.MaxInputSPP = Config.RayCountPerPixel;

	TArray<FSSDConstantPixelDensityViewSettings, TInlineAllocator<6>> ViewSettingsArray;
	ViewSettingsArray.SetNum(Views.Num());
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		ViewSettingsArray[ViewIndex].FullResViewport = Views[ViewIndex].ViewRect;
	}
	Settings.ViewSettingsArray = ViewSettingsArray;

	TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> PrevHistories;
	TStaticArray<FScreenSpaceDenoiserHistory*, IScreenSpaceDenoiser::kMaxBatchSize> NewHistories;
	PrevHistories[0] = &PreviousViewInfos->SkyLightHistory;
	NewHistories[0] = Views[0].ViewState ? &Views[0].ViewState->PrevFrameViewInfo.SkyLightHistory : nullptr;

	FSSDSignalTextures SignalOutput;
	DenoiseSignalAtConstantPixelDensity(
		GraphBuilder, Views, SceneTextures, ViewInfoPooledRenderTargets,
		InputSignal, Settings,
		PrevHistories,
		NewHistories,
		&SignalOutput);

	FDiffuseIndirectOutputs GlobalIlluminationOutputs;
	GlobalIlluminationOutputs.Color = SignalOutput.Textures[0];
	return GlobalIlluminationOutputs;
}
