// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraph.h"
#include "IndirectLightRendering.h"


class FViewInfo;
struct FPreviousViewInfo;
class FLightSceneInfo;
class FSceneTextureParameters;


// TODO(Denoiser): namespace.

/** The maximum number of buffers. */
static const int32 kMaxDenoiserBufferProcessingCount = 4;


namespace Denoiser
{

/** Public shader parameter structure to be able to execute bilateral kernel of the denoiser in external shaders. */
BEGIN_SHADER_PARAMETER_STRUCT(FCommonShaderParameters, )
	SHADER_PARAMETER(FVector4f, DenoiserBufferSizeAndInvSize)
	SHADER_PARAMETER(FVector4f, DenoiserBufferBilinearUVMinMax)
	SHADER_PARAMETER(FVector4f, SceneBufferUVToScreenPosition) // TODO: move to view uniform buffer
END_SHADER_PARAMETER_STRUCT()

void SetupCommonShaderParameters(
	const FViewInfo& View,
	const FSceneTextureParameters& SceneTextures,
	const FIntRect DenoiserFullResViewport,
	float DenoisingResolutionFraction,
	FCommonShaderParameters* OutPublicCommonParameters);

} // namespace Denoiser


/** Shader parameter structure use to bind all signal generically. */
BEGIN_SHADER_PARAMETER_STRUCT(FSSDSignalTextures, )
	SHADER_PARAMETER_RDG_TEXTURE_ARRAY(Texture2D, Textures, [kMaxDenoiserBufferProcessingCount])
END_SHADER_PARAMETER_STRUCT()


/** Interface for denoiser to have all hook in the renderer. */
class IScreenSpaceDenoiser
{
public:
	/** Maximum number a denoiser might be able to denoise at the same time. */
	static const int32 kMaxBatchSize = 4;

	// Number of screen space harmonic to be feed when denoising multiple lights.
	static constexpr int32 kMultiPolychromaticPenumbraHarmonics = 4;

	// Number of border between screen space harmonics are used to denoise harmonical signal.
	static constexpr int32 kHarmonicBordersCount = kMultiPolychromaticPenumbraHarmonics + 1;

	// Number of texture to store spherical harmonics.
	static constexpr int32 kSphericalHarmonicTextureCount = 2;


	/** Mode to run denoiser. */
	enum class EMode
	{
		// Denoising is disabled.
		Disabled,

		// Using default denoiser of the renderer.
		DefaultDenoiser,

		// Using a denoiser from a third party.
		ThirdPartyDenoiser,
	};


	/** All the inputs of the shadow denoiser. */
	BEGIN_SHADER_PARAMETER_STRUCT(FShadowVisibilityInputs, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Mask)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ClosestOccluder)
	END_SHADER_PARAMETER_STRUCT()

	/** All the outputs of the shadow denoiser may generate. */
	BEGIN_SHADER_PARAMETER_STRUCT(FShadowVisibilityOutputs, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Mask)
	END_SHADER_PARAMETER_STRUCT()
		
	/** Screen space harmonic decomposition a signal to denoise. */
	BEGIN_SHADER_PARAMETER_STRUCT(FHarmonicTextures, )
		SHADER_PARAMETER_RDG_TEXTURE_ARRAY(Texture2D, Harmonics, [kHarmonicBordersCount])
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FHarmonicUAVs, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(Texture2D, Harmonics, [kHarmonicBordersCount])
	END_SHADER_PARAMETER_STRUCT()
		
	/** All the inputs to denoise polychromatic penumbra of multiple lights. */
	BEGIN_SHADER_PARAMETER_STRUCT(FPolychromaticPenumbraHarmonics, )
		SHADER_PARAMETER_STRUCT(FHarmonicTextures, Diffuse)
		SHADER_PARAMETER_STRUCT(FHarmonicTextures, Specular)
	END_SHADER_PARAMETER_STRUCT()

	/** All the outputs when denoising polychromatic penumbra. */
	BEGIN_SHADER_PARAMETER_STRUCT(FPolychromaticPenumbraOutputs, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Diffuse)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Specular)
	END_SHADER_PARAMETER_STRUCT()
		
	/** All the inputs of the reflection denoiser. */
	BEGIN_SHADER_PARAMETER_STRUCT(FReflectionsInputs, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Color)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RayHitDistance)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RayImaginaryDepth)
	END_SHADER_PARAMETER_STRUCT()

	/** All the outputs of the reflection denoiser may generate. */
	BEGIN_SHADER_PARAMETER_STRUCT(FReflectionsOutputs, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Color)
	END_SHADER_PARAMETER_STRUCT()
		
	/** All the inputs of the AO denoisers. */
	BEGIN_SHADER_PARAMETER_STRUCT(FAmbientOcclusionInputs, )
		// TODO: Merge this back to MaskAndRayHitDistance into RG texture for performance improvement of denoiser's reconstruction pass. May also support RayDistanceOnly for 1spp AO ray tracing.
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Mask)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RayHitDistance)
	END_SHADER_PARAMETER_STRUCT()

	/** All the outputs of the AO denoiser may generate. */
	BEGIN_SHADER_PARAMETER_STRUCT(FAmbientOcclusionOutputs, )
		// Ambient occlusion mask stored in the red channel as [0; 1].
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AmbientOcclusionMask)
	END_SHADER_PARAMETER_STRUCT()
		
	/** All the inputs of the GI denoisers. */
	BEGIN_SHADER_PARAMETER_STRUCT(FDiffuseIndirectInputs, )
		// Irradiance in RGB, AO mask in alpha.
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Color)

		// Ambient occlusion mask stored in the red channel as [0; 1].
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AmbientOcclusionMask)

		// Hit distance in world space.
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RayHitDistance)
	END_SHADER_PARAMETER_STRUCT()

	/** All the outputs of the GI denoiser may generate. */
	BEGIN_SHADER_PARAMETER_STRUCT(FDiffuseIndirectOutputs, )
		// Irradiance in RGB, AO mask in alpha.
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Color)
		
		// Ambient occlusion mask stored in the red channel as [0; 1].
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AmbientOcclusionMask)
	END_SHADER_PARAMETER_STRUCT()
		
	/** All the inputs and outputs for spherical harmonic denoising. */
	BEGIN_SHADER_PARAMETER_STRUCT(FDiffuseIndirectHarmonic, )
		SHADER_PARAMETER_RDG_TEXTURE_ARRAY(Texture2D, SphericalHarmonic, [kSphericalHarmonicTextureCount])
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FDiffuseIndirectHarmonicUAVs, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D, SphericalHarmonic, [kSphericalHarmonicTextureCount])
	END_SHADER_PARAMETER_STRUCT()

		
	/** What the shadow ray tracing needs to output */
	enum class EShadowRequirements
	{
		// Denoiser is unable to denoise that configuration.
		Bailout,

		// Denoiser only need ray hit distance and the diffuse mask of the penumbra.
		// FShadowPenumbraInputs::Penumbra: average diffuse penumbra mask in [0; 1]
		// FShadowPenumbraInputs::ClosestOccluder:
		//   -1: invalid sample
		//   >0: average hit distance of occluding geometry
		PenumbraAndAvgOccluder,

		PenumbraAndClosestOccluder,
	};

	/** The configuration of the reflection ray tracing. */
	struct FShadowRayTracingConfig
	{
		// Number of rays per pixels.
		int32 RayCountPerPixel = 1;
	};

	/** Structure that contains all the parameters the denoiser needs to denoise one shadow. */
	struct FShadowVisibilityParameters
	{
		const FLightSceneInfo* LightSceneInfo = nullptr;
		FShadowRayTracingConfig RayTracingConfig;
		FShadowVisibilityInputs InputTextures;
	};

	/** The configuration of the reflection ray tracing. */
	struct FReflectionsRayTracingConfig
	{
		// Resolution fraction the ray tracing is being traced at.
		float ResolutionFraction = 1.0f;
		
		// Number of rays per pixels.
		int32 RayCountPerPixel = 1;
	};
	
	/** The configuration of the reflection ray tracing. */
	struct FAmbientOcclusionRayTracingConfig
	{
		// Resolution fraction the ray tracing is being traced at.
		float ResolutionFraction = 1.0f;

		// Number of rays per pixels.
		float RayCountPerPixel = 1.0f;
	};
	

	static RENDERER_API FHarmonicTextures CreateHarmonicTextures(FRDGBuilder& GraphBuilder, FIntPoint Extent, const TCHAR* DebugName);
	static RENDERER_API FHarmonicUAVs CreateUAVs(FRDGBuilder& GraphBuilder, const FHarmonicTextures& Textures);
	static RENDERER_API FDiffuseIndirectHarmonicUAVs CreateUAVs(FRDGBuilder& GraphBuilder, const FDiffuseIndirectHarmonic& Textures);



	virtual ~IScreenSpaceDenoiser() = default;

	/** Debug name of the denoiser for draw event. */
	virtual const TCHAR* GetDebugName() const = 0;

	/** Returns the ray tracing configuration that should be done for denoiser. */
	virtual EShadowRequirements GetShadowRequirements(
		const FViewInfo& View,
		const FLightSceneInfo& LightSceneInfo,
		const FShadowRayTracingConfig& RayTracingConfig) const = 0;

	/** Entry point to denoise the visibility mask of multiple shadows at the same time. */
	// TODO(Denoiser): Denoise specular occlusion. But requires refactor of direct lighting code.
	virtual void DenoiseShadowVisibilityMasks(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const TStaticArray<FShadowVisibilityParameters, IScreenSpaceDenoiser::kMaxBatchSize>& InputParameters,
		const int32 InputParameterCount,
		TStaticArray<FShadowVisibilityOutputs, IScreenSpaceDenoiser::kMaxBatchSize>& Outputs) const = 0;

	/** Entry point to denoise polychromatic penumbra of multiple light. */
	virtual FPolychromaticPenumbraOutputs DenoisePolychromaticPenumbraHarmonics(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FPolychromaticPenumbraHarmonics& Inputs) const = 0;

	/** Entry point to denoise reflections. */
	virtual FReflectionsOutputs DenoiseReflections(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FReflectionsInputs& ReflectionInputs,
		const FReflectionsRayTracingConfig RayTracingConfig) const = 0;

	/** Entry point to denoise water reflections. */
	virtual FReflectionsOutputs DenoiseWaterReflections(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FReflectionsInputs& ReflectionInputs,
		const FReflectionsRayTracingConfig RayTracingConfig) const = 0;
	
	/** Entry point to denoise reflections. */
	virtual FAmbientOcclusionOutputs DenoiseAmbientOcclusion(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FAmbientOcclusionInputs& ReflectionInputs,
		const FAmbientOcclusionRayTracingConfig RayTracingConfig) const = 0;

	/** Entry point to denoise diffuse indirect and AO. */
	virtual FSSDSignalTextures DenoiseDiffuseIndirect(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FDiffuseIndirectInputs& Inputs,
		const FAmbientOcclusionRayTracingConfig Config) const = 0;

	/** Entry point to denoise SkyLight diffuse indirect. */
	virtual FDiffuseIndirectOutputs DenoiseSkyLight(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FDiffuseIndirectInputs& Inputs,
		const FAmbientOcclusionRayTracingConfig Config) const = 0;

	/** Entry point to denoise spherical harmonic for diffuse indirect. */
	virtual FSSDSignalTextures DenoiseDiffuseIndirectHarmonic(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FDiffuseIndirectHarmonic& Inputs,
		const HybridIndirectLighting::FCommonParameters& CommonDiffuseParameters) const = 0;

	/** Entry point to denoise SSGI. */
	virtual bool SupportsScreenSpaceDiffuseIndirectDenoiser(EShaderPlatform Platform) const = 0;
	virtual FSSDSignalTextures DenoiseScreenSpaceDiffuseIndirect(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FDiffuseIndirectInputs& Inputs,
		const FAmbientOcclusionRayTracingConfig Config) const = 0;

	/** Entry point to denoise diffuse indirect probe hierarchy. */
	static RENDERER_API FSSDSignalTextures DenoiseIndirectProbeHierarchy(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FSSDSignalTextures& InputSignal,
		FRDGTextureRef CompressedDepthTexture,
		FRDGTextureRef CompressedShadingModelTexture);

	/** Returns the interface of the default denoiser of the renderer. */
	static RENDERER_API const IScreenSpaceDenoiser* GetDefaultDenoiser();

	/** Returns the denoising mode. */
	static RENDERER_API EMode GetDenoiserMode(const TAutoConsoleVariable<int32>& CVar);

	/** Variations of API that more efficiently support multiple split screen views. */
	static RENDERER_API FDiffuseIndirectOutputs DenoiseSkyLight(
		FRDGBuilder& GraphBuilder,
		TConstArrayView<FViewInfo> Views,
		FPreviousViewInfo* PreviousViewInfos,
		const FSceneTextureParameters& SceneTextures,
		const FDiffuseIndirectInputs& Inputs,
		const FAmbientOcclusionRayTracingConfig Config);

}; // class IScreenSpaceDenoiser


// The interface for the renderer to denoise what it needs, Plugins can come and point this to custom interface.
extern RENDERER_API const IScreenSpaceDenoiser* GScreenSpaceDenoiser;

extern int GetReflectionsDenoiserMode();
