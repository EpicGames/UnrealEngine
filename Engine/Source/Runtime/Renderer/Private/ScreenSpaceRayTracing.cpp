// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScreenSpaceRayTracing.h"
#include "CoreMinimal.h"
#include "CommonRenderResources.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/EngineTypes.h"
#include "Engine/Scene.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "HZB.h"
#include "IndirectLightRendering.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Misc/LargeWorldCoordinates.h"
#include "PipelineStateCache.h"
#include "PixelShaderUtils.h"
#include "RayTracing/RaytracingOptions.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "SceneTexturesConfig.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "ScreenSpaceDenoise.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "VariableRateShadingImageManager.h"
#include "SceneViewState.h"

static TAutoConsoleVariable<int32> CVarSSRQuality(
	TEXT("r.SSR.Quality"),
	3,
	TEXT("Whether to use screen space reflections and at what quality setting.\n")
	TEXT("(limits the setting in the post process settings which has a different scale)\n")
	TEXT("(costs performance, adds more visual realism but the technique has limits)\n")
	TEXT(" 0: off (default)\n")
	TEXT(" 1: low (no glossy)\n")
	TEXT(" 2: medium (no glossy)\n")
	TEXT(" 3: high (glossy/using roughness, few samples)\n")
	TEXT(" 4: very high (likely too slow for real-time)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

int32 GSSRHalfResSceneColor = 0;
FAutoConsoleVariableRef CVarSSRHalfResSceneColor(
	TEXT("r.SSR.HalfResSceneColor"),
	GSSRHalfResSceneColor,
	TEXT("Use half res scene color as input for SSR. Improves performance without much of a visual quality loss."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSSRTemporal(
	TEXT("r.SSR.Temporal"),
	0,
	TEXT("Defines if we use the temporal smoothing for the screen space reflection\n")
	TEXT(" 0 is off (for debugging), 1 is on (default)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSSRStencil(
	TEXT("r.SSR.Stencil"),
	0,
	TEXT("Defines if we use the stencil prepass for the screen space reflection\n")
	TEXT(" 0 is off (default), 1 is on"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSSRCompute(
	TEXT("r.SSR.Compute"), 0,
	TEXT("Use compute for SSR if possible. Only available for non tiled SSR with no stencil prepass currently\n")
	TEXT(" 0 is PS (default), 1 is sync compute, 2 is async compute"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSSGILeakFreeReprojection(
	FCVarDeprecationArgs{ TEXT("5.8") },
	TEXT("r.SSGI.LeakFreeReprojection"), 1,
	TEXT("[DEPRECATED] Whether use a more expensive but leak free reprojection of previous frame's scene color.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSSGIQuality(
	TEXT("r.SSGI.Quality"), 4,
	TEXT("[DEPRECATED] Quality setting to control number of ray shot with SSGI, between 1 and 4 (defaults to 4).\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSSGIMinimumLuminance(
	FCVarDeprecationArgs{ TEXT("5.8") },
	TEXT("r.SSGI.MinimumLuminance"), 0.5f,
	TEXT("[DEPRECATED] "),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSSGIRejectUncertainRays(
	FCVarDeprecationArgs{ TEXT("5.8") },
	TEXT("r.SSGI.RejectUncertainRays"), 1,
	TEXT("[DEPRECATED] Rejects the screen space ray if it was uncertain due to going behind screen geometry."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSSGITerminateCertainRay(
	FCVarDeprecationArgs{ TEXT("5.8") },
	TEXT("r.SSGI.TerminateCertainRay"), 1,
	TEXT("[DEPRECATED] Optimisations that if the screen space ray is certain and didn't find any geometry, don't fallback on otehr tracing technic."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSSGISkyDistance(
	FCVarDeprecationArgs{ TEXT("5.8") },
	TEXT("r.SSGI.SkyDistance"), 10000000,
	TEXT("[DEPRECATED] Distance of the sky in KM."),
	ECVF_Scalability | ECVF_RenderThreadSafe);


DECLARE_GPU_DRAWCALL_STAT_NAMED(ScreenSpaceReflections, TEXT("ScreenSpace Reflections"));
DECLARE_GPU_STAT_NAMED(ScreenSpaceDiffuseIndirect, TEXT("Screen Space Diffuse Indirect"));


static bool IsScreenSpaceDiffuseIndirectSupported(EShaderPlatform ShaderPlatform)
{
	if (IsForwardShadingEnabled(ShaderPlatform))
	{
		return false;
	}
	return IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM5);
}

static bool SupportScreenSpaceDiffuseIndirect(const FViewInfo& View)
{
	if (View.FinalPostProcessSettings.DynamicGlobalIlluminationMethod != EDynamicGlobalIlluminationMethod::ScreenSpace)
	{
		return false;
	}

	int Quality = CVarSSGIQuality.GetValueOnRenderThread();

	if (Quality <= 0)
	{
		return false;
	}

	if (!IsScreenSpaceDiffuseIndirectSupported(View.GetShaderPlatform()))
	{
		return false;
	}

	return View.ViewState != nullptr;
}

namespace ScreenSpaceRayTracing
{

bool ShouldKeepBleedFreeSceneColor(const FViewInfo& View)
{
	// TODO(Guillaume): SSR as well.
	return CVarSSGILeakFreeReprojection.GetValueOnRenderThread() != 0;
}

bool ShouldRenderScreenSpaceReflections(const FViewInfo& View)
{
	if(!View.Family->EngineShowFlags.ScreenSpaceReflections 
		// Note: intentionally allow falling back to SSR from other reflection methods, which may be disabled by scalability
		|| View.FinalPostProcessSettings.ReflectionMethod == EReflectionMethod::None 
		|| HasRayTracedOverlay(*View.Family)
		|| View.bIsReflectionCapture)
	{
		return false;
	}

	if(!View.State)
	{
		// not view state (e.g. thumbnail rendering?), no HZB (no screen space reflections or occlusion culling)
		return false;
	}

	int SSRQuality = CVarSSRQuality.GetValueOnRenderThread();

	if(SSRQuality <= 0)
	{
		return false;
	}

	if(View.FinalPostProcessSettings.ScreenSpaceReflectionIntensity < 1.0f)
	{
		return false;
	}

	const EShaderPlatform ShaderPlatform = View.GetShaderPlatform();
	if (IsForwardShadingEnabled(ShaderPlatform) && !IsMobilePlatform(ShaderPlatform))
	{
		return false;
	}

	return true;
}

bool IsScreenSpaceDiffuseIndirectSupported(const FViewInfo& View)
{
	if (!SupportScreenSpaceDiffuseIndirect(View))
	{
		return false;
	}

	return View.PrevViewInfo.ScreenSpaceRayTracingInput.IsValid();
}

bool IsSSRTemporalPassRequired(const FViewInfo& View)
{
	check(ShouldRenderScreenSpaceReflections(View) || ShouldRenderScreenSpaceReflectionsWater(View));

	if (!View.State)
	{
		return false;
	}
	return !IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod) || CVarSSRTemporal.GetValueOnRenderThread() != 0;
}


FRDGTextureUAV* CreateScreenSpaceRayTracingDebugUAV(FRDGBuilder& GraphBuilder, const FRDGTextureDesc& Desc, const TCHAR* Name, bool bClear = false)
#if (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)
{
	FRDGTextureDesc DebugDesc = FRDGTextureDesc::Create2D(
		Desc.Extent,
		PF_FloatRGBA,
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV);
	FRDGTexture* DebugTexture = GraphBuilder.CreateTexture(DebugDesc, Name);
	FRDGTextureUAVRef DebugOutput = GraphBuilder.CreateUAV(DebugTexture);
	if (bClear)
		AddClearUAVPass(GraphBuilder, DebugOutput, FLinearColor::Transparent);
	return DebugOutput;
}
#else
{
	return nullptr;
}
#endif

void SetupCommonScreenSpaceRayParameters(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const ScreenSpaceRayTracing::FPrevSceneColorMip& PrevSceneColor,
	const FViewInfo& View,
	FCommonScreenSpaceRayParameters* OutParameters)
{
	{
		// float2 SceneBufferUV;
		// float2 PixelPos = SceneBufferUV * View.BufferSizeAndInvSize.xy - View.ViewRect.Min;
		// PixelPos *= 0.5 // ReducedSceneColor is half resolution.
		// float2 ReducedSceneColorUV = PixelPos / ReducedSceneColor->Extent;

		OutParameters->ColorBufferScaleBias = FVector4f(
			0.5f * SceneTextures.SceneDepthTexture->Desc.Extent.X / float(PrevSceneColor.SceneColor->Desc.Extent.X),
			0.5f * SceneTextures.SceneDepthTexture->Desc.Extent.Y / float(PrevSceneColor.SceneColor->Desc.Extent.Y),
			-0.5f * View.ViewRect.Min.X / float(PrevSceneColor.SceneColor->Desc.Extent.X),
			-0.5f * View.ViewRect.Min.Y / float(PrevSceneColor.SceneColor->Desc.Extent.Y));

		OutParameters->ReducedColorUVMax = FVector2f(
			(0.5f * View.ViewRect.Width() - 0.5f) / float(PrevSceneColor.SceneColor->Desc.Extent.X),
			(0.5f * View.ViewRect.Height() - 0.5f) / float(PrevSceneColor.SceneColor->Desc.Extent.Y));
	}

	OutParameters->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);
	OutParameters->ColorTexture = PrevSceneColor.SceneColor;
	OutParameters->ColorTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();

	OutParameters->AlphaTexture = PrevSceneColor.SceneAlpha;
	OutParameters->AlphaTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();

	OutParameters->ViewUniformBuffer = View.ViewUniformBuffer;

	OutParameters->DebugOutput = CreateScreenSpaceRayTracingDebugUAV(GraphBuilder, SceneTextures.SceneDepthTexture->Desc, TEXT("DebugSSRT"));

	OutParameters->bRejectUncertainRays = CVarSSGIRejectUncertainRays.GetValueOnRenderThread() ? 1 : 0;
	OutParameters->bTerminateCertainRay = CVarSSGITerminateCertainRay.GetValueOnRenderThread() ? 1 : 0;
} // SetupCommonScreenSpaceRayParameters()

void SetupCommonScreenSpaceRayParameters(
	FRDGBuilder& GraphBuilder,
	const HybridIndirectLighting::FCommonParameters& CommonDiffuseParameters,
	const ScreenSpaceRayTracing::FPrevSceneColorMip& PrevSceneColor,
	const FViewInfo& View,
	FCommonScreenSpaceRayParameters* OutParameters)
{
	OutParameters->CommonDiffuseParameters = CommonDiffuseParameters;

	if (CommonDiffuseParameters.DownscaleFactor == 2.0f)
	{
		OutParameters->PixelPositionToFullResPixel = 2.0f;
		OutParameters->FullResPixelOffset = FVector2f(0.5f, 0.5f); // TODO.
	}
	else if (CommonDiffuseParameters.DownscaleFactor == 1.0f)
	{
		OutParameters->PixelPositionToFullResPixel = 1.0f;
		OutParameters->FullResPixelOffset = FVector2f(0.5f, 0.5f);
	}
	else
	{
		unimplemented();
	}

	SetupCommonScreenSpaceRayParameters(
		GraphBuilder,  CommonDiffuseParameters.SceneTextures,
		PrevSceneColor, View,
		/* inout */ OutParameters);
} // SetupCommonScreenSpaceRayParameters()

static float ComputeRoughnessMaskScale(const FViewInfo& View, ESSRQuality SSRQuality)
{
	float MaxRoughness = FMath::Clamp(View.FinalPostProcessSettings.ScreenSpaceReflectionMaxRoughness, 0.01f, 1.0f);

	// f(x) = x * Scale + Bias
	// f(MaxRoughness) = 0
	// f(MaxRoughness/2) = 1

	float RoughnessMaskScale = -2.0f / MaxRoughness;
	return RoughnessMaskScale * (int32(SSRQuality) < 3 ? 2.0f : 1.0f);
}

FLinearColor ComputeSSRParams(const FViewInfo& View, ESSRQuality SSRQuality, bool bEnableDiscard)
{
	float RoughnessMaskScale = ComputeRoughnessMaskScale(View, SSRQuality);

	float FrameRandom = 0;

	if (View.ViewState)
	{
		bool bTemporalAAIsOn = IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod);

		if (bTemporalAAIsOn)
		{
			// usually this number is in the 0..7 range but it depends on the TemporalAA quality
			FrameRandom = View.ViewState->GetCurrentTemporalAASampleIndex() * 1551;
		}
		else
		{
			// 8 aligns with the temporal smoothing, larger number will do more flickering (power of two for best performance)
			FrameRandom = View.ViewState->GetFrameIndex(8) * 1551;
		}
	}

	return FLinearColor(
		FMath::Clamp(View.FinalPostProcessSettings.ScreenSpaceReflectionIntensity * 0.01f, 0.0f, 1.0f),
		RoughnessMaskScale,
		(float)bEnableDiscard,	// TODO 
		FrameRandom);
}

} // namespace ScreenSpaceRayTracing

bool UseSingleLayerWaterIndirectDraw(EShaderPlatform ShaderPlatform);

namespace
{

BEGIN_SHADER_PARAMETER_STRUCT(FSSRTTileClassificationParameters, )
	SHADER_PARAMETER(FIntPoint, TileBufferExtent)
	SHADER_PARAMETER(int32, ViewTileCount)
	SHADER_PARAMETER(int32, MaxTileCount)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float>, TileClassificationBuffer)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSSRCommonParameters, )
	SHADER_PARAMETER(FLinearColor, SSRParams)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSSRPassCommonParameters, )
	SHADER_PARAMETER(FVector4f, PrevScreenPositionScaleBias)
	SHADER_PARAMETER(float, PrevSceneColorPreExposureCorrection)
	SHADER_PARAMETER(uint32, ShouldReflectOnlyWater)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SceneColor)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ScreenSpaceRayTracingDebugOutput)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
END_SHADER_PARAMETER_STRUCT()

enum class ELightingTerm
{
	Diffuse,
	Specular,
	MAX
};

class FSSRQualityDim : SHADER_PERMUTATION_ENUM_CLASS("SSR_QUALITY", ESSRQuality);
class FSSROutputForDenoiser : SHADER_PERMUTATION_BOOL("SSR_OUTPUT_FOR_DENOISER");
class FLightingTermDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_LIGHTING_TERM", ELightingTerm);

class FSSRTPrevFrameReductionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSRTPrevFrameReductionCS);
	SHADER_USE_PARAMETER_STRUCT(FSSRTPrevFrameReductionCS, FGlobalShader);

	class FLowerMips : SHADER_PERMUTATION_BOOL("DIM_LOWER_MIPS");
	class FLeakFree : SHADER_PERMUTATION_BOOL("DIM_LEAK_FREE");

	using FPermutationDomain = TShaderPermutationDomain<FLowerMips, FLeakFree>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, PrevBufferBilinearUVMinMax)
		SHADER_PARAMETER(FVector4f, PrevScreenPositionScaleBias)
		SHADER_PARAMETER(FVector2f, ReducedSceneColorSize)
		SHADER_PARAMETER(FVector2f, ReducedSceneColorTexelSize)
		SHADER_PARAMETER(FVector2f, HigherMipBufferBilinearMax)
		SHADER_PARAMETER(float, PrevSceneColorPreExposureCorrection)
		SHADER_PARAMETER(float, MinimumLuminance)
		SHADER_PARAMETER(float, HigherMipDownScaleFactor)
		SHADER_PARAMETER(float, SkyDistance)
		
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, PrevSceneColor)
		SHADER_PARAMETER_SAMPLER(SamplerState, PrevSceneColorSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PrevSceneDepth)
		SHADER_PARAMETER_SAMPLER(SamplerState, PrevSceneDepthSampler)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, HigherMipTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, HigherAlphaMipTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, HigherMipTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, HigherAlphaMipTextureSampler)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)

		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<float4>, ReducedSceneColorOutput, [3])
		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<float>, ReducedSceneAlphaOutput, [3])
	END_SHADER_PARAMETER_STRUCT()
};

class FSSRTDiffuseTileClassificationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSRTDiffuseTileClassificationCS);
	SHADER_USE_PARAMETER_STRUCT(FSSRTDiffuseTileClassificationCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSRTTileClassificationParameters, TileClassificationParameters)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float>, TileClassificationBufferOutput)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FScreenSpaceReflectionsStencilPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScreenSpaceReflectionsStencilPS);
	SHADER_USE_PARAMETER_STRUCT(FScreenSpaceReflectionsStencilPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSSROutputForDenoiser>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SSR_QUALITY"), uint32(0));
	}
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSRCommonParameters, CommonParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

class FScreenSpaceReflectionsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScreenSpaceReflectionsPS);
	SHADER_USE_PARAMETER_STRUCT(FScreenSpaceReflectionsPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSSRQualityDim, FSSROutputForDenoiser>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUPPORTS_ANISOTROPIC_MATERIALS"), FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Parameters.Platform));
	}
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSRCommonParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSRPassCommonParameters, SSRPassCommonParameter)
		RDG_BUFFER_ACCESS(IndirectDrawParameter, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileListData)		// FScreenSpaceReflectionsTileVS
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

class FScreenSpaceReflectionsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScreenSpaceReflectionsCS);
	SHADER_USE_PARAMETER_STRUCT(FScreenSpaceReflectionsCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSSRQualityDim, FSSROutputForDenoiser>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUPPORTS_ANISOTROPIC_MATERIALS"), FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Parameters.Platform));
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GetThreadGroupSize().X);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GetThreadGroupSize().Y);
		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 1);
	}

	static FIntPoint GetThreadGroupSize()
	{
		return 8;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSRCommonParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSSRPassCommonParameters, SSRPassCommonParameter)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SSRColorOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, SSRHitDistanceOutput)
	END_SHADER_PARAMETER_STRUCT()
};

// This is duplicated from FWaterTileVS because vertex shader should share Parameters structure for everything to be registered correctly in a RDG pass.
class FScreenSpaceReflectionsTileVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScreenSpaceReflectionsTileVS);
	SHADER_USE_PARAMETER_STRUCT(FScreenSpaceReflectionsTileVS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;
		
	using FParameters = FScreenSpaceReflectionsPS::FParameters; // Sharing parameters for proper registration with RDG
		
	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}
		
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ::UseSingleLayerWaterIndirectDraw(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TILE_VERTEX_SHADER"), 1.0f);
		OutEnvironment.SetDefine(TEXT("WORK_TILE_SIZE"), 8);
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

class FVisualizeTiledScreenSpaceReflectionsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeTiledScreenSpaceReflectionsPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeTiledScreenSpaceReflectionsPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSSRQualityDim, FSSROutputForDenoiser>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FScreenSpaceReflectionsPS::FParameters, CommonParameters)
	END_SHADER_PARAMETER_STRUCT()
};

class FVisualizeTiledScreenSpaceReflectionsVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeTiledScreenSpaceReflectionsVS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeTiledScreenSpaceReflectionsVS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	using FParameters = FVisualizeTiledScreenSpaceReflectionsPS::FParameters; // Sharing parameters for proper registration with RDG

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TILE_VERTEX_SHADER"), 1.0f);
		OutEnvironment.SetDefine(TEXT("WORK_TILE_SIZE"), 8);
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};


class FScreenSpaceCastStandaloneRayCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScreenSpaceCastStandaloneRayCS);
	SHADER_USE_PARAMETER_STRUCT(FScreenSpaceCastStandaloneRayCS, FGlobalShader)

	class FQualityDim : SHADER_PERMUTATION_RANGE_INT("QUALITY", 1, 4);
	using FPermutationDomain = TShaderPermutationDomain<FQualityDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)
		SHADER_PARAMETER_STRUCT_INCLUDE(FCommonScreenSpaceRayParameters, CommonParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, IndirectDiffuseOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, AmbientOcclusionOutput)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsScreenSpaceDiffuseIndirectSupported(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSSRTPrevFrameReductionCS, "/Engine/Private/SSRT/SSRTPrevFrameReduction.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FSSRTDiffuseTileClassificationCS, "/Engine/Private/SSRT/SSRTTileClassification.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FScreenSpaceReflectionsPS,        "/Engine/Private/SSRT/SSRTReflections.usf", "ScreenSpaceReflectionsPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FScreenSpaceReflectionsCS, "/Engine/Private/SSRT/SSRTReflections.usf", "ScreenSpaceReflectionsCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FScreenSpaceReflectionsTileVS,    "/Engine/Private/SingleLayerWaterComposite.usf", "WaterTileVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FVisualizeTiledScreenSpaceReflectionsPS, "/Engine/Private/SSRT/SSRTReflections.usf", "VisualizeTiledScreenSpaceReflectionsPS", SF_Pixel)
IMPLEMENT_GLOBAL_SHADER(FVisualizeTiledScreenSpaceReflectionsVS, "/Engine/Private/SingleLayerWaterComposite.usf", "WaterTileVS", SF_Vertex)
IMPLEMENT_GLOBAL_SHADER(FScreenSpaceReflectionsStencilPS, "/Engine/Private/SSRT/SSRTReflections.usf", "ScreenSpaceReflectionsStencilPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FScreenSpaceCastStandaloneRayCS,  "/Engine/Private/SSRT/SSRTDiffuseIndirect.usf", "MainCS", SF_Compute);

void GetSSRShaderOptionsForQuality(ESSRQuality Quality, IScreenSpaceDenoiser::FReflectionsRayTracingConfig* OutRayTracingConfigs)
{
	if (Quality == ESSRQuality::VisualizeSSR)
	{
		OutRayTracingConfigs->RayCountPerPixel = 12;
	}
	else if (Quality == ESSRQuality::Epic)
	{
		OutRayTracingConfigs->RayCountPerPixel = 12;
	}
	else if (Quality == ESSRQuality::High)
	{
		OutRayTracingConfigs->RayCountPerPixel = 4;
	}
	else if (Quality == ESSRQuality::Medium)
	{
		OutRayTracingConfigs->RayCountPerPixel = 1;
	}
	else if (Quality == ESSRQuality::Low)
	{
		OutRayTracingConfigs->RayCountPerPixel = 1;
	}
	else
	{
		check(0);
	}
}

FIntPoint GetSSRTGroupSizeForSampleCount(int32 RayCountPerPixel)
{
	FIntPoint GroupCount(1, 1);

	if (RayCountPerPixel == 4)
	{
		GroupCount = FIntPoint(8, 8);
	}
	else if (RayCountPerPixel == 8)
	{
		GroupCount = FIntPoint(8, 4);
	}
	else if (RayCountPerPixel == 16)
	{
		GroupCount = FIntPoint(4, 4);
	}
	else if (RayCountPerPixel == 32)
	{
		GroupCount = FIntPoint(4, 2);
	}
	else
	{
		check(0);
	}

	check(GroupCount.X * GroupCount.Y * RayCountPerPixel == 256);

	return GroupCount;
}

void GetSSRTGIShaderOptionsForQuality(int32 Quality, int32* OutRayCountPerPixel)
{
	if (Quality == 1)
	{
		*OutRayCountPerPixel = 4;
	}
	else if (Quality == 2)
	{
		*OutRayCountPerPixel = 8;
	}
	else if (Quality == 3)
	{
		*OutRayCountPerPixel = 16;
	}
	else if (Quality == 4)
	{
		*OutRayCountPerPixel = 32;
	}
	else
	{
		check(0);
	}
}

} // namespace

namespace ScreenSpaceRayTracing
{

void GetSSRQualityForView(const FViewInfo& View, ESSRQuality* OutQuality, IScreenSpaceDenoiser::FReflectionsRayTracingConfig* OutRayTracingConfigs)
{
	check(ShouldRenderScreenSpaceReflections(View) || ShouldRenderScreenSpaceReflectionsWater(View));
	
	int32 SSRQualityCVar = FMath::Clamp(CVarSSRQuality.GetValueOnRenderThread(), 0, int32(ESSRQuality::MAX) - 1);
	
	if (View.Family->EngineShowFlags.VisualizeSSR)
	{
		*OutQuality = ESSRQuality::VisualizeSSR;
		return;
	}
	else if (View.FinalPostProcessSettings.ScreenSpaceReflectionQuality >= 80.0f && SSRQualityCVar >= 4)
	{
		*OutQuality = ESSRQuality::Epic;
	}
	else if (View.FinalPostProcessSettings.ScreenSpaceReflectionQuality >= 60.0f && SSRQualityCVar >= 3)
	{
		*OutQuality = ESSRQuality::High;
	}
	else if (View.FinalPostProcessSettings.ScreenSpaceReflectionQuality >= 40.0f && SSRQualityCVar >= 2)
	{
		*OutQuality = ESSRQuality::Medium;
	}
	else
	{
		*OutQuality = ESSRQuality::Low;
	}

	GetSSRShaderOptionsForQuality(*OutQuality, OutRayTracingConfigs);
}


FPrevSceneColorMip ReducePrevSceneColorMip(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FViewInfo& View)
{
	RDG_EVENT_SCOPE(GraphBuilder, "SSGI SceneColorReduction");

	// Number of mip skipped at the begining of the mip chain.
	const int32 DownSamplingMip = 1;

	// Number of mip in the mip chain
	const int32 kNumMips = 5;

	bool bUseLeakFree = View.PrevViewInfo.ScreenSpaceRayTracingInput != nullptr && View.PrevViewInfo.DepthBuffer != nullptr;
	check(bUseLeakFree == true);

	// Allocate FPrevSceneColorMip.
	FPrevSceneColorMip PrevSceneColorMip;
	{
		FIntPoint RequiredSize = SceneTextures.SceneDepthTexture->Desc.Extent / (1 << DownSamplingMip);

		int32 QuantizeMultiple = 1 << (kNumMips - 1);
		FIntPoint QuantizedSize = FIntPoint::DivideAndRoundUp(RequiredSize, QuantizeMultiple);

		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			FIntPoint(QuantizeMultiple * QuantizedSize.X, QuantizeMultiple * QuantizedSize.Y),
			PF_FloatR11G11B10,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV);
		Desc.NumMips = kNumMips;

		PrevSceneColorMip.SceneColor = GraphBuilder.CreateTexture(Desc, TEXT("SSRTReducedSceneColor"));

		if (bUseLeakFree)
		{
			Desc.Format = PF_R8;
			PrevSceneColorMip.SceneAlpha = GraphBuilder.CreateTexture(Desc, TEXT("SSRTReducedSceneAlpha"));
		}
	}

	FSSRTPrevFrameReductionCS::FParameters DefaultPassParameters;
	{
		DefaultPassParameters.SceneTextures = SceneTextures;
		DefaultPassParameters.View = View.ViewUniformBuffer;

		DefaultPassParameters.ReducedSceneColorSize = FVector2f(
			PrevSceneColorMip.SceneColor->Desc.Extent.X, PrevSceneColorMip.SceneColor->Desc.Extent.Y);
		DefaultPassParameters.ReducedSceneColorTexelSize = FVector2f(
			1.0f / float(PrevSceneColorMip.SceneColor->Desc.Extent.X), 1.0f / float(PrevSceneColorMip.SceneColor->Desc.Extent.Y));
	}

	{
		FSSRTPrevFrameReductionCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSRTPrevFrameReductionCS::FParameters>();
		*PassParameters = DefaultPassParameters;

		FIntPoint ViewportOffset;
		FIntPoint ViewportExtent;
		FIntPoint BufferSize;

		if (bUseLeakFree)
		{
			BufferSize = View.PrevViewInfo.ScreenSpaceRayTracingInput->GetDesc().Extent;
			ViewportOffset = View.PrevViewInfo.ViewRect.Min;
			ViewportExtent = View.PrevViewInfo.ViewRect.Size();

			PassParameters->PrevSceneColor = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(
				GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.ScreenSpaceRayTracingInput)));
			PassParameters->PrevSceneColorSampler = TStaticSamplerState<SF_Point>::GetRHI();

			PassParameters->PrevSceneDepth = GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.DepthBuffer);
			PassParameters->PrevSceneDepthSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		}
		else
		{
			BufferSize = View.PrevViewInfo.TemporalAAHistory.ReferenceBufferSize;
			ViewportOffset = View.PrevViewInfo.TemporalAAHistory.ViewportRect.Min;
			ViewportExtent = View.PrevViewInfo.TemporalAAHistory.ViewportRect.Size();

			FRDGTextureRef TemporalAAHistoryTexture = GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.TemporalAAHistory.RT[0]);
			PassParameters->PrevSceneColorSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
			PassParameters->PrevSceneColor = GraphBuilder.CreateSRV(TemporalAAHistoryTexture->Desc.IsTextureArray()
				? FRDGTextureSRVDesc::CreateForSlice(TemporalAAHistoryTexture, View.PrevViewInfo.TemporalAAHistory.OutputSliceIndex)
				: FRDGTextureSRVDesc(TemporalAAHistoryTexture));
		}

		float InvBufferSizeX = 1.f / float(BufferSize.X);
		float InvBufferSizeY = 1.f / float(BufferSize.Y);

		PassParameters->PrevBufferBilinearUVMinMax = FVector4f(
			(ViewportOffset.X + 0.5f) * InvBufferSizeX,
			(ViewportOffset.Y + 0.5f) * InvBufferSizeY,
			(ViewportOffset.X + ViewportExtent.X - 0.5f) * InvBufferSizeX,
			(ViewportOffset.Y + ViewportExtent.Y - 0.5f) * InvBufferSizeY);

		PassParameters->PrevSceneColorPreExposureCorrection = View.PreExposure / View.PrevViewInfo.SceneColorPreExposure;
		PassParameters->MinimumLuminance = CVarSSGIMinimumLuminance.GetValueOnRenderThread();
		PassParameters->SkyDistance = CVarSSGISkyDistance.GetValueOnRenderThread();

		PassParameters->PrevScreenPositionScaleBias = FVector4f(
			ViewportExtent.X * 0.5f / BufferSize.X,
			-ViewportExtent.Y * 0.5f / BufferSize.Y,
			(ViewportExtent.X * 0.5f + ViewportOffset.X) / BufferSize.X,
			(ViewportExtent.Y * 0.5f + ViewportOffset.Y) / BufferSize.Y);

		for (int32 MipLevel = 0; MipLevel < (PassParameters->ReducedSceneColorOutput.Num() - DownSamplingMip); MipLevel++)
		{
			PassParameters->ReducedSceneColorOutput[DownSamplingMip + MipLevel] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(PrevSceneColorMip.SceneColor, MipLevel));
			if (PrevSceneColorMip.SceneAlpha)
				PassParameters->ReducedSceneAlphaOutput[DownSamplingMip + MipLevel] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(PrevSceneColorMip.SceneAlpha, MipLevel));
		}

		FSSRTPrevFrameReductionCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FSSRTPrevFrameReductionCS::FLowerMips>(false);
		PermutationVector.Set<FSSRTPrevFrameReductionCS::FLeakFree>(bUseLeakFree);

		TShaderMapRef<FSSRTPrevFrameReductionCS> ComputeShader(View.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("PrevFrameReduction(LeakFree=%i) %dx%d",
				bUseLeakFree ? 1 : 0,
				View.ViewRect.Width(), View.ViewRect.Height()),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), 8));
	}

	for (int32 i = 0; i < 1; i++)
	{
		int32 SrcMip = i * 3 + 2 - DownSamplingMip;
		int32 StartDestMip = SrcMip + 1;
		int32 Divisor = 1 << (StartDestMip + DownSamplingMip);

		FSSRTPrevFrameReductionCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSRTPrevFrameReductionCS::FParameters>();
		*PassParameters = DefaultPassParameters;

		PassParameters->HigherMipTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(PrevSceneColorMip.SceneColor, SrcMip));
		if (bUseLeakFree)
		{
			PassParameters->HigherMipTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
			PassParameters->HigherAlphaMipTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(PrevSceneColorMip.SceneAlpha, SrcMip));
			PassParameters->HigherAlphaMipTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		}
		else
		{
			PassParameters->HigherMipTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		}

		PassParameters->HigherMipDownScaleFactor = 1 << (DownSamplingMip + SrcMip);

		PassParameters->HigherMipBufferBilinearMax = FVector2f(
			(0.5f * View.ViewRect.Width() - 0.5f) / float(PrevSceneColorMip.SceneColor->Desc.Extent.X),
			(0.5f * View.ViewRect.Height() - 0.5f) / float(PrevSceneColorMip.SceneColor->Desc.Extent.Y));

		PassParameters->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);

		for (int32 MipLevel = 0; MipLevel < PassParameters->ReducedSceneColorOutput.Num(); MipLevel++)
		{
			PassParameters->ReducedSceneColorOutput[MipLevel] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(PrevSceneColorMip.SceneColor, StartDestMip + MipLevel));
			if (PrevSceneColorMip.SceneAlpha)
				PassParameters->ReducedSceneAlphaOutput[MipLevel] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(PrevSceneColorMip.SceneAlpha, StartDestMip + MipLevel));
		}

		FSSRTPrevFrameReductionCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FSSRTPrevFrameReductionCS::FLowerMips>(true);
		PermutationVector.Set<FSSRTPrevFrameReductionCS::FLeakFree>(bUseLeakFree);

		TShaderMapRef<FSSRTPrevFrameReductionCS> ComputeShader(View.ShaderMap, PermutationVector);
		ClearUnusedGraphResources(ComputeShader, PassParameters);
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("PrevFrameReduction(LeakFree=%i) %dx%d",
				bUseLeakFree ? 1 : 0,
				View.ViewRect.Width() / Divisor, View.ViewRect.Height() / Divisor),
			PassParameters,
			ERDGPassFlags::Compute,
			[PassParameters, ComputeShader, &View, Divisor](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *PassParameters, FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), 8 * Divisor));
		});
	}

	return PrevSceneColorMip;
}

FSSRTTileClassificationParameters RenderHorizonTileClassification(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FViewInfo& View)
{
	FIntPoint SceneTexturesExtent = SceneTextures.SceneDepthTexture->Desc.Extent;

	FSSRTTileClassificationParameters ClassificationParameters;
	{
		FRDGBufferRef TileClassificationBuffer;

		{
			FIntPoint MaxTileBufferExtent = FIntPoint::DivideAndRoundUp(SceneTexturesExtent, 8);
			int32 MaxTileCount = MaxTileBufferExtent.X * MaxTileBufferExtent.Y;

			ClassificationParameters.TileBufferExtent = FIntPoint::DivideAndRoundUp(View.ViewRect.Size(), 8);
			ClassificationParameters.ViewTileCount = ClassificationParameters.TileBufferExtent.X * ClassificationParameters.TileBufferExtent.Y;

			TileClassificationBuffer = GraphBuilder.CreateBuffer(
				FRDGBufferDesc::CreateStructuredDesc(sizeof(float) * 16, MaxTileCount), TEXT("SSRTTileClassification"));

			ClassificationParameters.TileClassificationBuffer = GraphBuilder.CreateSRV(TileClassificationBuffer);
		}

		FIntPoint ThreadCount = ClassificationParameters.TileBufferExtent;

		FSSRTDiffuseTileClassificationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSRTDiffuseTileClassificationCS::FParameters>();
		PassParameters->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::All);
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->TileClassificationParameters = ClassificationParameters;
		PassParameters->TileClassificationBufferOutput = GraphBuilder.CreateUAV(TileClassificationBuffer);

		{
			FRDGTextureDesc DebugDesc = FRDGTextureDesc::Create2D(
				FIntPoint::DivideAndRoundUp(SceneTexturesExtent, 8),
				PF_FloatRGBA,
				FClearValueBinding::Transparent,
				TexCreate_ShaderResource | TexCreate_UAV);

			PassParameters->DebugOutput = GraphBuilder.CreateUAV(GraphBuilder.CreateTexture(DebugDesc, TEXT("DebugSSRTTiles")));
		}

		TShaderMapRef<FSSRTDiffuseTileClassificationCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ScreenSpaceDiffuseClassification %dx%d", ClassificationParameters.TileBufferExtent.X, ClassificationParameters.TileBufferExtent.Y),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(ClassificationParameters.TileBufferExtent, 8));
	}

	return ClassificationParameters;
}


void RenderScreenSpaceReflections(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FRDGTextureRef CurrentSceneColor,
	const FViewInfo& View,
	ESSRQuality SSRQuality,
	bool bDenoiser,
	IScreenSpaceDenoiser::FReflectionsInputs* DenoiserInputs,
	bool bSingleLayerWater,
	FTiledReflection* TiledScreenSpaceReflection)
{
	FRDGTextureSRVRef InputColor = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(CurrentSceneColor));
	if (SSRQuality != ESSRQuality::VisualizeSSR)
	{
		if (View.PrevViewInfo.CustomSSRInput.IsValid())
		{
			InputColor = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(
				GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.CustomSSRInput.RT[0])));
		}
		else if (GSSRHalfResSceneColor && View.PrevViewInfo.HalfResTemporalAAHistory.IsValid())
		{
			FRDGTextureRef HalfResTemporalAAHistory = GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.HalfResTemporalAAHistory);

			if (HalfResTemporalAAHistory->Desc.Dimension == ETextureDimension::Texture2DArray)
			{
				InputColor = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForSlice(HalfResTemporalAAHistory, /* SliceIndex = */ 0));
			}
			else
			{
				InputColor = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(HalfResTemporalAAHistory));
			}
		}
		else if (View.PrevViewInfo.TemporalAAHistory.IsValid())
		{
			FRDGTextureRef TemporalAAHistoryTexture = GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.TemporalAAHistory.RT[0]);
			InputColor = GraphBuilder.CreateSRV(TemporalAAHistoryTexture->Desc.IsTextureArray()
				? FRDGTextureSRVDesc::CreateForSlice(TemporalAAHistoryTexture, View.PrevViewInfo.TemporalAAHistory.OutputSliceIndex)
				: FRDGTextureSRVDesc(TemporalAAHistoryTexture));
		}
	}

	const bool SSRStencilPrePass = CVarSSRStencil.GetValueOnRenderThread() != 0 && SSRQuality != ESSRQuality::VisualizeSSR && TiledScreenSpaceReflection == nullptr;
	
	// Alloc inputs for denoising.
	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			View.GetSceneTexturesConfig().Extent,
			PF_FloatRGBA, FClearValueBinding(FLinearColor(0, 0, 0, 0)),
			TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV | TexCreate_NoFastClear);

		Desc.Flags |= GFastVRamConfig.SSR;

		DenoiserInputs->Color = GraphBuilder.CreateTexture(Desc, TEXT("ScreenSpaceReflections"));

		if (bDenoiser)
		{
			Desc.Format = PF_R16F;
			DenoiserInputs->RayHitDistance = GraphBuilder.CreateTexture(Desc, TEXT("ScreenSpaceReflectionsHitDistance"));
		}
	}

	IScreenSpaceDenoiser::FReflectionsRayTracingConfig RayTracingConfigs;
	GetSSRShaderOptionsForQuality(SSRQuality, &RayTracingConfigs);
		
	FSSRCommonParameters CommonParameters;
	CommonParameters.SSRParams = ComputeSSRParams(View, SSRQuality, false);
	CommonParameters.ViewUniformBuffer = View.ViewUniformBuffer;
	CommonParameters.SceneTextures = SceneTextures;
	// Pipe down a mid grey texture when not using TAA's history to avoid wrongly reprojecting current scene color as if previous frame's TAA history.
	if (InputColor->Desc.Texture == CurrentSceneColor || !CommonParameters.SceneTextures.GBufferVelocityTexture)
	{
		// Technically should be 32767.0f / 65535.0f to perfectly null out DecodeVelocityFromTexture(), but 0.5f is good enough.
		CommonParameters.SceneTextures.GBufferVelocityTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.MidGreyDummy);
	}
	
	FRenderTargetBindingSlots RenderTargets;
	RenderTargets[0] = FRenderTargetBinding(DenoiserInputs->Color, ERenderTargetLoadAction::ENoAction);

	if (bDenoiser)
	{
		RenderTargets[1] = FRenderTargetBinding(DenoiserInputs->RayHitDistance, ERenderTargetLoadAction::ENoAction);
	}

	// Do a pre pass that output 0, or set a stencil mask to run the more expensive pixel shader.
	if (SSRStencilPrePass)
	{
		// Also bind the depth buffer
		RenderTargets.DepthStencil = FDepthStencilBinding(
			SceneTextures.SceneDepthTexture,
			ERenderTargetLoadAction::ENoAction,
			ERenderTargetLoadAction::ELoad,
			FExclusiveDepthStencil::DepthNop_StencilWrite);

		FScreenSpaceReflectionsStencilPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FSSROutputForDenoiser>(bDenoiser);

		FScreenSpaceReflectionsStencilPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FScreenSpaceReflectionsStencilPS::FParameters>();
		PassParameters->CommonParameters = CommonParameters;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->RenderTargets = RenderTargets;
		
		TShaderMapRef<FScreenSpaceReflectionsStencilPS> PixelShader(View.ShaderMap, PermutationVector);
		ClearUnusedGraphResources(PixelShader, PassParameters);

		RDG_EVENT_SCOPE_STAT(GraphBuilder, ScreenSpaceReflections, "ScreenSpaceReflections");

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("SSR StencilSetup %dx%d", View.ViewRect.Width(), View.ViewRect.Height()),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, &View, PixelShader](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
		
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			FPixelShaderUtils::InitFullscreenPipelineState(RHICmdList, View.ShaderMap, PixelShader, /* out */ GraphicsPSOInit);
			// Clobers the stencil to pixel that should not compute SSR
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always, true, CF_Always, SO_Replace, SO_Replace, SO_Replace>::GetRHI();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0x80);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			FPixelShaderUtils::DrawFullscreenTriangle(RHICmdList);
		});
	}

	// Adds SSR pass.
	auto SetSSRParameters = [&](auto* PassParameters)
	{
		{
			FIntPoint ViewportOffset = View.ViewRect.Min;
			FIntPoint ViewportExtent = View.ViewRect.Size();
			FIntPoint BufferSize = SceneTextures.SceneDepthTexture->Desc.Extent;

			if (View.PrevViewInfo.CustomSSRInput.IsValid())
			{
				ViewportOffset = View.PrevViewInfo.CustomSSRInput.ViewportRect.Min;
				ViewportExtent = View.PrevViewInfo.CustomSSRInput.ViewportRect.Size();
				BufferSize = View.PrevViewInfo.CustomSSRInput.ReferenceBufferSize;
				ensure(ViewportExtent.X > 0 && ViewportExtent.Y > 0);
				ensure(BufferSize.X > 0 && BufferSize.Y > 0);
			}
			else if (View.PrevViewInfo.TemporalAAHistory.IsValid())
			{
				ViewportOffset = View.PrevViewInfo.TemporalAAHistory.ViewportRect.Min;
				ViewportExtent = View.PrevViewInfo.TemporalAAHistory.ViewportRect.Size();
				BufferSize = View.PrevViewInfo.TemporalAAHistory.ReferenceBufferSize;
				ensure(ViewportExtent.X > 0 && ViewportExtent.Y > 0);
				ensure(BufferSize.X > 0 && BufferSize.Y > 0);
			}

			FVector2D InvBufferSize(1.0f / float(BufferSize.X), 1.0f / float(BufferSize.Y));

			PassParameters->PrevScreenPositionScaleBias = FVector4f(
				ViewportExtent.X * 0.5f * InvBufferSize.X,
				-ViewportExtent.Y * 0.5f * InvBufferSize.Y,
				(ViewportExtent.X * 0.5f + ViewportOffset.X) * InvBufferSize.X,
				(ViewportExtent.Y * 0.5f + ViewportOffset.Y) * InvBufferSize.Y);

			PassParameters->ScreenSpaceRayTracingDebugOutput = CreateScreenSpaceRayTracingDebugUAV(GraphBuilder, DenoiserInputs->Color->Desc, TEXT("DebugSSR"), true);
		}
		PassParameters->PrevSceneColorPreExposureCorrection = InputColor->Desc.Texture != CurrentSceneColor ? View.PreExposure / View.PrevViewInfo.SceneColorPreExposure : 1.0f;
		PassParameters->ShouldReflectOnlyWater = bSingleLayerWater ? 1u : 0u;
		
		PassParameters->SceneColor = InputColor;
		PassParameters->SceneColorSampler = GSSRHalfResSceneColor ? TStaticSamplerState<SF_Bilinear>::GetRHI() : TStaticSamplerState<SF_Point>::GetRHI();
		
		PassParameters->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);
	};

	const int32 CVarSSRComputeValue = CVarSSRCompute.GetValueOnRenderThread();
	// Compute path is only implemented for non-tiled SSR currently
	const bool bCompute = CVarSSRComputeValue > 0 && !TiledScreenSpaceReflection && !SSRStencilPrePass && !bSingleLayerWater;
	const bool bAsyncCompute = bCompute && CVarSSRComputeValue == 2;

	FScreenSpaceReflectionsPS::FPermutationDomain PermutationVectorPS;
	FScreenSpaceReflectionsCS::FPermutationDomain PermutationVectorCS;
	FScreenSpaceReflectionsPS::FParameters* PassParametersPS = nullptr;
	FScreenSpaceReflectionsCS::FParameters* PassParametersCS = nullptr;

	if (bCompute)
	{
		PermutationVectorCS.Set<FSSRQualityDim>(SSRQuality);
		PermutationVectorCS.Set<FSSROutputForDenoiser>(bDenoiser);

		PassParametersCS = GraphBuilder.AllocParameters<FScreenSpaceReflectionsCS::FParameters>();
		PassParametersCS->CommonParameters = CommonParameters;
		SetSSRParameters(&PassParametersCS->SSRPassCommonParameter);
		PassParametersCS->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParametersCS->SSRColorOutput = GraphBuilder.CreateUAV(DenoiserInputs->Color);
		if (bDenoiser)
		{
			PassParametersCS->SSRHitDistanceOutput = GraphBuilder.CreateUAV(DenoiserInputs->RayHitDistance);
		}
	}
	else
	{
		PermutationVectorPS.Set<FSSRQualityDim>(SSRQuality);
		PermutationVectorPS.Set<FSSROutputForDenoiser>(bDenoiser);

		PassParametersPS = GraphBuilder.AllocParameters<FScreenSpaceReflectionsPS::FParameters>();
		PassParametersPS->CommonParameters = CommonParameters;
		SetSSRParameters(&PassParametersPS->SSRPassCommonParameter);
		PassParametersPS->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParametersPS->RenderTargets = RenderTargets;
		PassParametersPS->RenderTargets.ShadingRateTexture = GVRSImageManager.GetVariableRateShadingImage(GraphBuilder, View, FVariableRateShadingImageManager::EVRSPassType::SSR);
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, ScreenSpaceReflections, "ScreenSpaceReflections");

	static const auto CVarSSRTiledCompositeVisualize = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.SSR.TiledComposite.Visualize"));
	const bool bVisualizeTiledScreenSpaceReflection = CVarSSRTiledCompositeVisualize ? CVarSSRTiledCompositeVisualize->GetValueOnRenderThread() : false;

	if (TiledScreenSpaceReflection == nullptr)
	{
		if (bCompute)
		{
			TShaderMapRef<FScreenSpaceReflectionsCS> ComputeShader(View.ShaderMap, PermutationVectorCS);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSR RayMarchCS(Quality=%d RayPerPixel=%d%s) %dx%d",
					SSRQuality, RayTracingConfigs.RayCountPerPixel, bDenoiser ? TEXT(" DenoiserOutput") : TEXT(""),
					View.ViewRect.Width(), View.ViewRect.Height()),
				bAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
				ComputeShader,
				PassParametersCS,
				FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), FScreenSpaceReflectionsCS::GetThreadGroupSize()));
		}
		else
		{
			TShaderMapRef<FScreenSpaceReflectionsPS> PixelShader(View.ShaderMap, PermutationVectorPS);

			ClearUnusedGraphResources(PixelShader, PassParametersPS);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("SSR RayMarch(Quality=%d RayPerPixel=%d%s) %dx%d",
					SSRQuality, RayTracingConfigs.RayCountPerPixel, bDenoiser ? TEXT(" DenoiserOutput") : TEXT(""),
					View.ViewRect.Width(), View.ViewRect.Height()),
				PassParametersPS,
				ERDGPassFlags::Raster,
				[PassParametersPS, &View, PixelShader, SSRStencilPrePass](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				FPixelShaderUtils::InitFullscreenPipelineState(RHICmdList, View.ShaderMap, PixelShader, /* out */ GraphicsPSOInit);
				if (SSRStencilPrePass)
				{
					// Clobers the stencil to pixel that should not compute SSR
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always, true, CF_NotEqual, SO_Keep, SO_Keep, SO_Keep>::GetRHI();
				}

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0x80);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParametersPS);

				FPixelShaderUtils::DrawFullscreenTriangle(RHICmdList);
			});
		}
	}
	else if (!bVisualizeTiledScreenSpaceReflection)
	{
		check(TiledScreenSpaceReflection->TileSize == 8); // WORK_TILE_SIZE

		FScreenSpaceReflectionsTileVS::FPermutationDomain VsPermutationVector;
		TShaderMapRef<FScreenSpaceReflectionsTileVS> VertexShader(View.ShaderMap, VsPermutationVector);

		PassParametersPS->TileListData = TiledScreenSpaceReflection->TileListDataBufferSRV;
		PassParametersPS->IndirectDrawParameter = TiledScreenSpaceReflection->DrawIndirectParametersBuffer;

		TShaderMapRef<FScreenSpaceReflectionsPS> PixelShader(View.ShaderMap, PermutationVectorPS);

		ClearUnusedGraphResources(VertexShader, PixelShader, PassParametersPS);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("SSR RayMarch(Quality=%d RayPerPixel=%d%s) %dx%d",
				SSRQuality, RayTracingConfigs.RayCountPerPixel, bDenoiser ? TEXT(" DenoiserOutput") : TEXT(""),
				View.ViewRect.Width(), View.ViewRect.Height()),
			PassParametersPS,
			ERDGPassFlags::Raster,
			[PassParametersPS, &View, VertexShader, PixelShader, SSRStencilPrePass](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			FPixelShaderUtils::InitFullscreenPipelineState(RHICmdList, View.ShaderMap, PixelShader, /* out */ GraphicsPSOInit);
			if (SSRStencilPrePass)
			{
				// Clobers the stencil to pixel that should not compute SSR
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always, true, CF_NotEqual, SO_Keep, SO_Keep, SO_Keep>::GetRHI();
			}
			GraphicsPSOInit.PrimitiveType = GRHISupportsRectTopology ? PT_RectList : PT_TriangleList;
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0x80);
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParametersPS);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParametersPS);

			PassParametersPS->IndirectDrawParameter->MarkResourceAsUsed();

			RHICmdList.DrawPrimitiveIndirect(PassParametersPS->IndirectDrawParameter->GetIndirectRHICallBuffer(), 0);
		});
	}
	else
	{
		// Visualize tiled screen space reflection
		check(TiledScreenSpaceReflection->TileSize == 8); // WORK_TILE_SIZE

		FVisualizeTiledScreenSpaceReflectionsVS::FPermutationDomain VsPermutationVector;
		TShaderMapRef<FVisualizeTiledScreenSpaceReflectionsVS> VertexShader(View.ShaderMap, VsPermutationVector);

		FVisualizeTiledScreenSpaceReflectionsPS::FPermutationDomain VisualizePermutationVector;
		VisualizePermutationVector.Set<FSSRQualityDim>(SSRQuality);
		VisualizePermutationVector.Set<FSSROutputForDenoiser>(bDenoiser);

		TShaderMapRef<FVisualizeTiledScreenSpaceReflectionsPS> VisualizePixelShader(View.ShaderMap, VisualizePermutationVector);

		FVisualizeTiledScreenSpaceReflectionsPS::FParameters* VisualizePassParameters = GraphBuilder.AllocParameters<FVisualizeTiledScreenSpaceReflectionsPS::FParameters>();

		VisualizePassParameters->CommonParameters = *PassParametersPS;
		VisualizePassParameters->CommonParameters.TileListData = TiledScreenSpaceReflection->TileListDataBufferSRV;
		VisualizePassParameters->CommonParameters.IndirectDrawParameter = TiledScreenSpaceReflection->DrawIndirectParametersBuffer;

		ClearUnusedGraphResources(VertexShader, VisualizePixelShader, VisualizePassParameters);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("SSR RayMarch(Visualize Tiles%s) %dx%d",
				bDenoiser ? TEXT(" DenoiserOutput") : TEXT(""), View.ViewRect.Width(), View.ViewRect.Height()),
			VisualizePassParameters,
			ERDGPassFlags::Raster,
			[VisualizePassParameters, &View, VertexShader, VisualizePixelShader, SSRStencilPrePass](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				FPixelShaderUtils::InitFullscreenPipelineState(RHICmdList, View.ShaderMap, VisualizePixelShader, /* out */ GraphicsPSOInit);
				if (SSRStencilPrePass)
				{
					// Clobers the stencil to pixel that should not compute SSR
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always, true, CF_NotEqual, SO_Keep, SO_Keep, SO_Keep>::GetRHI();
				}
				GraphicsPSOInit.PrimitiveType = GRHISupportsRectTopology ? PT_RectList : PT_TriangleList;
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = VisualizePixelShader.GetPixelShader();

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0x80);
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *VisualizePassParameters);
				SetShaderParameters(RHICmdList, VisualizePixelShader, VisualizePixelShader.GetPixelShader(), *VisualizePassParameters);

				VisualizePassParameters->CommonParameters.IndirectDrawParameter->MarkResourceAsUsed();

				RHICmdList.DrawPrimitiveIndirect(VisualizePassParameters->CommonParameters.IndirectDrawParameter->GetIndirectRHICallBuffer(), 0);
			});
	}
} // RenderScreenSpaceReflections()


int32 GetSSGIRayCountPerTracingPixel()
{
	const int32 Quality = FMath::Clamp(CVarSSGIQuality.GetValueOnRenderThread(), 1, 4);

	int32 RayCountPerPixel;
	GetSSRTGIShaderOptionsForQuality(Quality, /* out */ &RayCountPerPixel);

	return RayCountPerPixel;
}


IScreenSpaceDenoiser::FDiffuseIndirectInputs CastStandaloneDiffuseIndirectRays(
	FRDGBuilder& GraphBuilder,
	const HybridIndirectLighting::FCommonParameters& CommonParameters,
	const FPrevSceneColorMip& PrevSceneColor,
	const FViewInfo& View)
{
	const int32 Quality = FMath::Clamp(CVarSSGIQuality.GetValueOnRenderThread(), 1, 4);

	int32 RayCountPerPixel;
	GetSSRTGIShaderOptionsForQuality(Quality, &RayCountPerPixel);
	check(RayCountPerPixel == CommonParameters.RayCountPerPixel);

	FIntPoint GroupSize = GetSSRTGroupSizeForSampleCount(CommonParameters.RayCountPerPixel);

	// Alloc output for the denoiser.
	IScreenSpaceDenoiser::FDiffuseIndirectInputs DenoiserInputs;
	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			CommonParameters.SceneTextures.SceneDepthTexture->Desc.Extent / CommonParameters.DownscaleFactor,
			PF_FloatRGBA,
			FClearValueBinding::Transparent,
			TexCreate_ShaderResource | TexCreate_UAV);

		DenoiserInputs.Color = GraphBuilder.CreateTexture(Desc, TEXT("SSRTDiffuseIndirect"));

		Desc.Format = PF_R16F;
		Desc.Flags |= TexCreate_RenderTargetable;
		DenoiserInputs.AmbientOcclusionMask = GraphBuilder.CreateTexture(Desc, TEXT("SSRTAmbientOcclusion"));
	}

	FScreenSpaceCastStandaloneRayCS::FParameters* PassParameters =
		GraphBuilder.AllocParameters<FScreenSpaceCastStandaloneRayCS::FParameters>();
	ScreenSpaceRayTracing::SetupCommonScreenSpaceRayParameters(GraphBuilder, CommonParameters, PrevSceneColor, View, /* out */ &PassParameters->CommonParameters);
	PassParameters->IndirectDiffuseOutput = GraphBuilder.CreateUAV(DenoiserInputs.Color);
	PassParameters->AmbientOcclusionOutput = GraphBuilder.CreateUAV(DenoiserInputs.AmbientOcclusionMask);

	FScreenSpaceCastStandaloneRayCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FScreenSpaceCastStandaloneRayCS::FQualityDim>(Quality);

	TShaderMapRef<FScreenSpaceCastStandaloneRayCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("SSGI Standalone(Quality=%d RayPerPixel=%d) %dx%d",
			Quality, CommonParameters.RayCountPerPixel, CommonParameters.TracingViewportSize.X, CommonParameters.TracingViewportSize.Y),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(CommonParameters.TracingViewportSize, GroupSize));

	return DenoiserInputs;
}

} // namespace ScreenSpaceRayTracing
