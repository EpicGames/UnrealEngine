// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraph.h"
#include "SceneView.h"
#include "ScreenSpaceDenoise.h"
#include "IndirectLightRendering.h"
#include "HZB.h"

class FViewInfo;
class FSceneTextureParameters;


enum class ESSRQuality
{
	VisualizeSSR,

	Low,
	Medium,
	High,
	Epic,

	MAX
};

struct FTiledReflection
{
	FRDGBufferRef DrawIndirectParametersBuffer;
	FRDGBufferRef DispatchIndirectParametersBuffer;
	FRDGBufferRef DispatchClearIndirectParametersBuffer;
	FRDGBufferRef DispatchDownsampledIndirectParametersBuffer;
	FRDGBufferRef DispatchDownsampledClearIndirectParametersBuffer;
	FRDGBufferSRVRef TileListDataBufferSRV;
	FRDGBufferSRVRef ClearTileListDataBufferSRV;
	FRDGBufferSRVRef DownsampledTileListDataBufferSRV;
	FRDGBufferSRVRef DownsampledClearTileListDataBufferSRV;
	uint32 TileSize;
};

BEGIN_SHADER_PARAMETER_STRUCT(FCommonScreenSpaceRayParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(HybridIndirectLighting::FCommonParameters, CommonDiffuseParameters)

	SHADER_PARAMETER(FVector4f, ColorBufferScaleBias)
	SHADER_PARAMETER(FVector2f, ReducedColorUVMax)
	SHADER_PARAMETER(FVector2f, FullResPixelOffset)

	SHADER_PARAMETER(float, PixelPositionToFullResPixel)

	SHADER_PARAMETER(int32, bRejectUncertainRays)
	SHADER_PARAMETER(int32, bTerminateCertainRay)

	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ColorTextureSampler)

	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AlphaTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, AlphaTextureSampler)

	SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)

	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ScreenSpaceRayTracingDebugOutput)
END_SHADER_PARAMETER_STRUCT()

namespace ScreenSpaceRayTracing
{

BEGIN_SHADER_PARAMETER_STRUCT(FPrevSceneColorMip, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColor)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, SceneAlpha)
END_SHADER_PARAMETER_STRUCT()

bool ShouldKeepBleedFreeSceneColor(const FViewInfo& View);

bool ShouldRenderScreenSpaceReflections(const FViewInfo& View);
bool ShouldRenderScreenSpaceReflectionsWater(const FViewInfo& View);


void ProcessForNextFrameScreenSpaceRayTracing(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FRDGTextureRef CurrentSceneColor,
	const FViewInfo& View);

void GetSSRQualityForView(const FViewInfo& View, ESSRQuality* OutQuality, IScreenSpaceDenoiser::FReflectionsRayTracingConfig* OutRayTracingConfigs);

bool IsSSRTemporalPassRequired(const FViewInfo& View);

int32 GetSSGIRayCountPerTracingPixel();


FPrevSceneColorMip ReducePrevSceneColorMip(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FViewInfo& View);


void RenderScreenSpaceReflections(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FRDGTextureRef CurrentSceneColor,
	const FViewInfo& View,
	ESSRQuality SSRQuality,
	bool bDenoiser,
	IScreenSpaceDenoiser::FReflectionsInputs* DenoiserInputs,
	bool bSingleLayerWater = false,
	FTiledReflection* TiledScreenSpaceReflection = nullptr);

bool IsScreenSpaceDiffuseIndirectSupported(const FViewInfo& View);

IScreenSpaceDenoiser::FDiffuseIndirectInputs CastStandaloneDiffuseIndirectRays(
	FRDGBuilder& GraphBuilder, 
	const HybridIndirectLighting::FCommonParameters& CommonParameters,
	const FPrevSceneColorMip& PrevSceneColor,
	const FViewInfo& View);

void SetupCommonScreenSpaceRayParameters(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const ScreenSpaceRayTracing::FPrevSceneColorMip& PrevSceneColor,
	const FViewInfo& View,
	FCommonScreenSpaceRayParameters* OutParameters);

FLinearColor ComputeSSRParams(const FViewInfo& View, ESSRQuality SSRQuality, bool bEnableDiscard);

} // namespace ScreenSpaceRayTracing
