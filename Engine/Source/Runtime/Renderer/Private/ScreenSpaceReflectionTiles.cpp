// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScreenSpaceReflectionTiles.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"

static TAutoConsoleVariable<int32> CVarSSRTiledComposite(
	TEXT("r.SSR.TiledComposite"), 0,
	TEXT("Enable tiled optimization of the screen space reflection."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSSRTiledCompositeOverrideMaxRoughness(
	TEXT("r.SSR.TiledComposite.OverrideMaxRoughness"), -1.0f,
	TEXT("Ignore pixels with roughness larger than this value.")
	TEXT("<0: use derived value from ScreenSpaceReflectionMaxRoughness of FinalPostProcessSettings."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSSRTiledCompositeMinSpecular(
	TEXT("r.SSR.TiledComposite.MinSpecular"), 0.0f,
	TEXT("Ignore pixels with very small specular contribution in case max roughness cannot filter them out"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarSSRTiledCompositeTwoSidedFoliage(
	TEXT("r.SSR.TiledComposite.TwoSidedFoliage"), 0,
	TEXT("0: diable SSR for foliage if tiling is enabled."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<bool> CVarSSRTiledCompositeVisualize(
	TEXT("r.SSR.TiledComposite.Visualize"), 0,
	TEXT("1: Visualize the tiling region."),
	ECVF_RenderThreadSafe | ECVF_Scalability);


static float GetScreenSpaceReflectionMaxRoughnessScale(const FViewInfo& View)
{
	float MaxRoughness = CVarSSRTiledCompositeOverrideMaxRoughness.GetValueOnRenderThread();

	if (MaxRoughness < 0)
	{
		MaxRoughness = FMath::Clamp(View.FinalPostProcessSettings.ScreenSpaceReflectionMaxRoughness, 0.01f, 1.0f);
	}
	return MaxRoughness;
}

bool UseSSRIndirectDraw(EShaderPlatform ShaderPlatform)
{
	return IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM5)
		// todo: check if this is true for screenspacereflection: Vulkan gives error with SSRTileCatergorisationMarkCS usage of atomic, and Metal does not play nice, either.
		&& !IsVulkanMobilePlatform(ShaderPlatform);
		//&& FDataDrivenShaderPlatformInfo::GetSupportsWaterIndirectDraw(ShaderPlatform);
}

bool ShouldVisualizeTiledScreenSpaceReflection()
{
	return CVarSSRTiledCompositeVisualize.GetValueOnRenderThread() != 0;
}


class FSSRTileCategorisationMarkCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSRTileCategorisationMarkCS);
	SHADER_USE_PARAMETER_STRUCT(FSSRTileCategorisationMarkCS, FGlobalShader)

		class FUsePrepassStencil : SHADER_PERMUTATION_BOOL("USE_SSR_PRE_PASS_STENCIL");
	using FPermutationDomain = TShaderPermutationDomain<FUsePrepassStencil>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SSRDepthStencilTexture)
		SHADER_PARAMETER(FIntPoint, TiledViewRes)
		SHADER_PARAMETER(float, MaxRoughness)
		SHADER_PARAMETER(float, MinSpecular)
		SHADER_PARAMETER(int32, bEnableTwoSidedFoliage)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, TileMaskBufferOut)
		END_SHADER_PARAMETER_STRUCT()

		static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return UseSSRIndirectDraw(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TILE_CATERGORISATION_SHADER"), 1);
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSSRTileCategorisationMarkCS, "/Engine/Private/DefaultSSRTiles.usf", "SSRTileCategorisationMarkCS", SF_Compute);

class FSSRTileClassificationBuildListsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSRTileClassificationBuildListsCS);
	SHADER_USE_PARAMETER_STRUCT(FSSRTileClassificationBuildListsCS, FGlobalShader)

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(uint32, VertexCountPerInstanceIndirect)
		SHADER_PARAMETER(FIntPoint, TiledViewRes)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, DrawIndirectDataUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, DispatchIndirectDataUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, SSRTileListDataUAV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileMaskBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return UseSSRIndirectDraw(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return SSR_TILE_SIZE_XY;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("TILE_CATERGORISATION_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FSSRTileClassificationBuildListsCS, "/Engine/Private/DefaultSSRTiles.usf", "SSRTileClassificationBuildListsCS", SF_Compute);

bool IsDefaultSSRTileEnabled(const FViewInfo& View)
{
	return UseSSRIndirectDraw(View.GetShaderPlatform()) && CVarSSRTiledComposite.GetValueOnRenderThread();
}
/**
 * Build lists of 8x8 tiles used by SSR pixels
 * Mark and build list steps are separated in order to build a more coherent list (z-ordered over a larger region), which is important for the performance of future passes like ray traced Lumen reflections
 */
FScreenSpaceReflectionTileClassification ClassifySSRTiles(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FSceneTextures& SceneTextures, const FRDGTextureRef& DepthPrepassTexture)
{
	FScreenSpaceReflectionTileClassification Result;
	const bool bRunTiled = UseSSRIndirectDraw(View.GetShaderPlatform()) && CVarSSRTiledComposite.GetValueOnRenderThread();
	if (bRunTiled)
	{
		FIntPoint ViewRes(View.ViewRect.Width(), View.ViewRect.Height());
		Result.TiledViewRes = FIntPoint::DivideAndRoundUp(ViewRes, SSR_TILE_SIZE_XY);

		Result.TiledReflection.DrawIndirectParametersBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndirectParameters>(), TEXT("SSR.IndirectDrawParameters"));
		Result.TiledReflection.DispatchIndirectParametersBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("SSR.IndirectDispatchParameters"));

		FRDGBufferRef TileListDataBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Result.TiledViewRes.X * Result.TiledViewRes.Y), TEXT("SSR.TileListDataBuffer"));
		Result.TiledReflection.TileListDataBufferSRV = GraphBuilder.CreateSRV(TileListDataBuffer, PF_R32_UINT);

		FRDGBufferUAVRef DrawIndirectParametersBufferUAV = GraphBuilder.CreateUAV(Result.TiledReflection.DrawIndirectParametersBuffer);
		FRDGBufferUAVRef DispatchIndirectParametersBufferUAV = GraphBuilder.CreateUAV(Result.TiledReflection.DispatchIndirectParametersBuffer);

		// Allocate buffer with 1 bit / tile
		Result.TileMaskBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FMath::DivideAndRoundUp(Result.TiledViewRes.X * Result.TiledViewRes.Y, 32)), TEXT("SSR.TileMaskBuffer"));
		FRDGBufferUAVRef TileMaskBufferUAV = GraphBuilder.CreateUAV(Result.TileMaskBuffer);
		AddClearUAVPass(GraphBuilder, TileMaskBufferUAV, 0);

		// Clear DrawIndirectParametersBuffer
		AddClearUAVPass(GraphBuilder, DrawIndirectParametersBufferUAV, 0);
		AddClearUAVPass(GraphBuilder, DispatchIndirectParametersBufferUAV, 0);

		// Mark used tiles based on SHADING_MODEL_ID, roughness
		{
			typedef FSSRTileCategorisationMarkCS SHADER;
			SHADER::FPermutationDomain PermutationVector;
			PermutationVector.Set<SHADER::FUsePrepassStencil>(DepthPrepassTexture != nullptr);
			TShaderMapRef<SHADER> ComputeShader(View.ShaderMap, PermutationVector);

			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
			PassParameters->View = View.GetShaderParameters();
			PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
			PassParameters->TiledViewRes = Result.TiledViewRes;
			PassParameters->SSRDepthStencilTexture = DepthPrepassTexture ? GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(DepthPrepassTexture, PF_X24_G8)) : nullptr;
			PassParameters->TileMaskBufferOut = TileMaskBufferUAV;
			PassParameters->MaxRoughness = GetScreenSpaceReflectionMaxRoughnessScale(View);
			PassParameters->MinSpecular = FMath::Clamp(CVarSSRTiledCompositeMinSpecular.GetValueOnRenderThread(), -0.001f, 1.001f);
			PassParameters->bEnableTwoSidedFoliage = CVarSSRTiledCompositeTwoSidedFoliage.GetValueOnRenderThread() != 0;

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSR::TileCategorisationMarkTiles"),
				ComputeShader,
				PassParameters,
				FIntVector(Result.TiledViewRes.X, Result.TiledViewRes.Y, 1)
			);
		}

		// Build compacted and coherent z-order tile lists from bit-marked tiles
		{
			typedef FSSRTileClassificationBuildListsCS SHADER;
			TShaderMapRef<SHADER> ComputeShader(View.ShaderMap);

			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();

			PassParameters->View = View.GetShaderParameters();
			PassParameters->TiledViewRes = Result.TiledViewRes;
			PassParameters->VertexCountPerInstanceIndirect = GRHISupportsRectTopology ? 3 : 6;
			PassParameters->DrawIndirectDataUAV = DrawIndirectParametersBufferUAV;
			PassParameters->DispatchIndirectDataUAV = DispatchIndirectParametersBufferUAV;
			PassParameters->SSRTileListDataUAV = GraphBuilder.CreateUAV(TileListDataBuffer, PF_R32_UINT);
			PassParameters->TileMaskBuffer = GraphBuilder.CreateSRV(Result.TileMaskBuffer);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSR::TileCategorisationBuildList"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(Result.TiledViewRes, SHADER::GetGroupSize())
			);
		}
	}
	return Result;
}