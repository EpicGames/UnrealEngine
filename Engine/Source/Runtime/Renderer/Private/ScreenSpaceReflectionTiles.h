// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraph.h"
#include "ScreenSpaceDenoise.h"
#include "IndirectLightRendering.h"
#include "ScreenSpaceRayTracing.h"

class FViewInfo;

#define SSR_TILE_SIZE_XY (8U)

struct FScreenSpaceReflectionTileClassification
{
	FTiledReflection TiledReflection = FTiledReflection{ nullptr, nullptr, nullptr, nullptr, nullptr, nullptr, nullptr, nullptr, nullptr, SSR_TILE_SIZE_XY};
	FRDGBufferRef TileMaskBuffer = nullptr;
	FIntPoint TiledViewRes = FIntPoint{ 0, 0 };
};

bool IsDefaultSSRTileEnabled(const FViewInfo& View);
bool ShouldVisualizeTiledScreenSpaceReflection();

FScreenSpaceReflectionTileClassification ClassifySSRTiles(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View, 
	const FSceneTextures& SceneTextures, 
	const FRDGTextureRef& DepthPrepassTexture);
