// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderBaseClasses.cpp: Shader base classes
=============================================================================*/

#include "ShaderBaseClasses.h"
#include "Materials/MaterialRenderProxy.h"
#include "PostProcess/SceneRenderTargets.h"
#include "LogRenderer.h"
#include "ScenePrivate.h"
#include "ParameterCollection.h"
#include "SceneViewState.h"
#include "Materials/MaterialInterface.h"

IMPLEMENT_TYPE_LAYOUT(FMaterialShader);
IMPLEMENT_TYPE_LAYOUT(FMeshMaterialShader);
IMPLEMENT_TYPE_LAYOUT(FDebugUniformExpressionSet);


void FMeshMaterialShaderElementData::InitializeMeshMaterialData()
{
	FadeUniformBuffer = GDistanceCullFadedInUniformBuffer.GetUniformBufferRHI();
	DitherUniformBuffer = GDitherFadedInUniformBuffer.GetUniformBufferRHI();
}

void FMeshMaterialShaderElementData::InitializeMeshMaterialData(const FSceneView* SceneView, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, bool bDitheredLODTransition, bool bAllowStencilDither)
{
	InitializeMeshMaterialData();

	if (SceneView)
	{
		checkSlow(SceneView->bIsViewInfo);
		const FViewInfo* ViewInfo = (FViewInfo*)SceneView;

		if (StaticMeshId >= 0 && bDitheredLODTransition && !(bAllowStencilDither && ViewInfo->bAllowStencilDither))
		{
			if (ViewInfo->StaticMeshFadeOutDitheredLODMap[StaticMeshId])
			{
				DitherUniformBuffer = ViewInfo->DitherFadeOutUniformBuffer;
			}
			else if (ViewInfo->StaticMeshFadeInDitheredLODMap[StaticMeshId])
			{
				DitherUniformBuffer = ViewInfo->DitherFadeInUniformBuffer;
			}
		}

		if (PrimitiveSceneProxy)
		{
			int32 const PrimitiveIndex = PrimitiveSceneProxy->GetPrimitiveSceneInfo()->GetIndex();

			if (ViewInfo->PrimitiveFadeUniformBufferMap.IsValidIndex(PrimitiveIndex) && ViewInfo->PrimitiveFadeUniformBufferMap[PrimitiveIndex])
			{
				FadeUniformBuffer = ViewInfo->PrimitiveFadeUniformBuffers[PrimitiveIndex];
			}
		}
	}
}

void FMeshMaterialShaderElementData::InitializeMeshMaterialData(const FSceneView* SceneView, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, const FMeshBatch& RESTRICT MeshBatch, int32 StaticMeshId, bool bAllowStencilDither)
{
	InitializeMeshMaterialData(SceneView, PrimitiveSceneProxy, StaticMeshId, MeshBatch.bDitheredLODTransition, bAllowStencilDither);
}

FDebugUniformExpressionSet::FDebugUniformExpressionSet()
	: NumPreshaders(0)
{
	FMemory::Memzero(NumTextureExpressions);
}

FDebugUniformExpressionSet::FDebugUniformExpressionSet(const FUniformExpressionSet& InUniformExpressionSet)
{
	InitFromExpressionSet(InUniformExpressionSet);
}

/** Initialize from a uniform expression set. */
void FDebugUniformExpressionSet::InitFromExpressionSet(const FUniformExpressionSet& InUniformExpressionSet)
{
	NumPreshaders = InUniformExpressionSet.UniformPreshaders.Num();
	for (uint32 TypeIndex = 0u; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
	{
		NumTextureExpressions[TypeIndex] = InUniformExpressionSet.UniformTextureParameters[TypeIndex].Num();
	}
}

/** Returns true if the number of uniform expressions matches those with which the debug set was initialized. */
bool FDebugUniformExpressionSet::Matches(const FUniformExpressionSet& InUniformExpressionSet) const
{
	for (uint32 TypeIndex = 0u; TypeIndex < NumMaterialTextureParameterTypes; ++TypeIndex)
	{
		if (NumTextureExpressions[TypeIndex] != InUniformExpressionSet.UniformTextureParameters[TypeIndex].Num())
		{
			return false;
		}
	}
	return NumPreshaders == InUniformExpressionSet.UniformPreshaders.Num();
}

FName FMaterialShader::UniformBufferLayoutName(TEXT("Material"));

FMaterialShader::FMaterialShader() = default;

FMaterialShader::FMaterialShader(const FMaterialShaderType::CompiledShaderInitializerType& Initializer)
:	FShader(Initializer)
#if WITH_EDITORONLY_DATA
,	DebugUniformExpressionSet(Initializer.UniformExpressionSet)
,	DebugDescription(Initializer.DebugDescription)
#endif // WITH_EDITORONLY_DATA
{
#if WITH_EDITORONLY_DATA
	check(UE_BUILD_SHIPPING || !DebugDescription.IsEmpty());
	DebugUniformExpressionUBLayout.CopyFrom(Initializer.UniformExpressionSet.GetUniformBufferLayoutInitializer());
#endif // WITH_EDITORONLY_DATA

	// Bind the material uniform buffer parameter.
	MaterialUniformBuffer.Bind(Initializer.ParameterMap,TEXT("Material"));
	MaterialUniformBufferVS.Bind(Initializer.ParameterMap,TEXT("MaterialVS"));

	for (int32 CollectionIndex = 0; CollectionIndex < Initializer.UniformExpressionSet.ParameterCollections.Num(); CollectionIndex++)
	{
		FShaderUniformBufferParameter CollectionParameter;
		CollectionParameter.Bind(Initializer.ParameterMap,*FString::Printf(TEXT("MaterialCollection%u"), CollectionIndex));
		ParameterCollectionUniformBuffers.Add(CollectionParameter);
	}
}

FRHIUniformBuffer* FMaterialShader::GetParameterCollectionBuffer(const FGuid& Id, const FSceneInterface* SceneInterface) const
{
	const FScene* Scene = (const FScene*)SceneInterface;
	FRHIUniformBuffer* UniformBuffer = Scene ? Scene->GetParameterCollectionBuffer(Id) : nullptr;

	if (!UniformBuffer)
	{
		FMaterialParameterCollectionInstanceResource** CollectionResource = GDefaultMaterialParameterCollectionInstances.Find(Id);
		if (CollectionResource && *CollectionResource)
		{
			UniformBuffer = (*CollectionResource)->GetUniformBuffer();
		}
	}

	return UniformBuffer;
}

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING || !WITH_EDITOR)
template<typename TLayout>
static void DumpUB(const TLayout& Layout)
{
	UE_LOGF(LogShaders, Warning, "Layout %ls, Hash %08x", *Layout.GetDebugName(), Layout.GetHash());
	FString ResourcesString;
	for (int32 Index = 0; Index < Layout.Resources.Num(); ++Index)
	{
		ResourcesString += FString::Printf(TEXT("%d "), (uint8)Layout.Resources[Index].MemberType);
	}
	UE_LOGF(LogShaders, Warning, "Layout CB Size %d %d Resources: %ls", Layout.ConstantBufferSize, Layout.Resources.Num(), *ResourcesString);
};

void FMaterialShader::VerifyExpressionAndShaderMaps(const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial& Material, const FUniformExpressionCache* UniformExpressionCache) const
{
	// Validate that the shader is being used for a material that matches the uniform expression set the shader was compiled for.
	FMaterialShaderMap* ShaderMap = Material.GetRenderingThreadShaderMap();
	if (ShaderMap->GetShaderMapId().IsCookedId())
	{
		// If this is a cooked shadermap, the debug uniform expression set won't be properly initialized...so skip the verification
		return;
	}

	const FUniformExpressionSet& MaterialUniformExpressionSet = ShaderMap->GetUniformExpressionSet();
	bool bUniformExpressionSetMismatch = !DebugUniformExpressionSet.Matches(MaterialUniformExpressionSet)
		|| UniformExpressionCache->CachedUniformExpressionShaderMap != ShaderMap;
	if (!bUniformExpressionSetMismatch)
	{
		if (DebugUniformExpressionUBLayout.GetHash() != UniformExpressionCache->UniformBuffer->GetLayout().GetHash())
		{
			UE_LOGF(LogShaders, Warning, "Material Expression UB mismatch!");
			DumpUB(DebugUniformExpressionUBLayout);
			DumpUB(UniformExpressionCache->UniformBuffer->GetLayout());
			bUniformExpressionSetMismatch = true;
		}
	}
	if (bUniformExpressionSetMismatch)
	{
		const FShaderType* ShaderType = GetType(ShaderMap->GetPointerTable());
		const FString ProxyName = *MaterialRenderProxy->GetFriendlyName();
		const FString MaterialName = *Material.GetFriendlyName();
		const TCHAR* ShaderMapDesc = ShaderMap->GetDebugDescription();
		UE_LOG(
			LogShaders,
			Warning,//Fatal,
			TEXT("%s shader uniform expression set mismatch for material %s/%s.\n")
			TEXT("Shader compilation info:                %s\n")
			TEXT("Material render proxy compilation info: %s\n")
			TEXT("Shader uniform expression set:   %u preshaders, %u 2D textures, %u cube textures, %u array textures, %u 3D textures, %u virtual textures, shader map %p\n")
			TEXT("Material uniform expression set: %u preshaders, %u 2D textures, %u cube textures, %u array textures, %u 3D textures, %u virtual textures, shader map %p\n"),
			ShaderType->GetName(),
			*ProxyName,
			*MaterialName,
			*DebugDescription,
			ShaderMapDesc,
			DebugUniformExpressionSet.NumPreshaders,
			DebugUniformExpressionSet.NumTextureExpressions[(uint32)EMaterialTextureParameterType::Standard2D],
			DebugUniformExpressionSet.NumTextureExpressions[(uint32)EMaterialTextureParameterType::Cube],
			DebugUniformExpressionSet.NumTextureExpressions[(uint32)EMaterialTextureParameterType::Array2D],
			DebugUniformExpressionSet.NumTextureExpressions[(uint32)EMaterialTextureParameterType::Volume],
			DebugUniformExpressionSet.NumTextureExpressions[(uint32)EMaterialTextureParameterType::Virtual],
			UniformExpressionCache->CachedUniformExpressionShaderMap,
			MaterialUniformExpressionSet.UniformPreshaders.Num(),
			MaterialUniformExpressionSet.UniformTextureParameters[(uint32)EMaterialTextureParameterType::Standard2D].Num(),
			MaterialUniformExpressionSet.UniformTextureParameters[(uint32)EMaterialTextureParameterType::Cube].Num(),
			MaterialUniformExpressionSet.UniformTextureParameters[(uint32)EMaterialTextureParameterType::Array2D].Num(),
			MaterialUniformExpressionSet.UniformTextureParameters[(uint32)EMaterialTextureParameterType::Volume].Num(),
			MaterialUniformExpressionSet.UniformTextureParameters[(uint32)EMaterialTextureParameterType::Virtual].Num(),
			ShaderMap
		);
	}
}
#endif

void FMaterialShader::SetViewParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer)
{
	const auto& ViewUniformBufferParameter = GetUniformBufferParameter<FViewUniformShaderParameters>();
	SetUniformBufferParameter(BatchedParameters, ViewUniformBufferParameter, ViewUniformBuffer);

	if (View.bShouldBindInstancedViewUB)
	{
		// When drawing an instanced stereo scene, the instanced view UB should be taken from the same view where it will contains a copy of both left and eye values (see FViewInfo::CreateViewUniformBuffers).
		const auto& InstancedViewUniformBufferParameter = GetUniformBufferParameter<FInstancedViewUniformShaderParameters>();
		SetUniformBufferParameter(BatchedParameters, InstancedViewUniformBufferParameter, View.GetInstancedViewUniformBuffer());
	}
}

void FMaterialShader::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial& Material, const FSceneView& View)
{
	SetParameters(BatchedParameters, MaterialRenderProxy, Material, View.Family->Scene);
}

static void DumpRegisteredMaterialParameterCollections(const FMaterial& Material, const FGuid& ParameterCollectionGuid)
{
	// Dump the currently registered parameter collections and the ID we failed to find.
	// In a cooked project these numbers are persistent so we can track back to the original
	// parameter collection that was being referenced and no longer exists
	FString InstancesString;
	TMultiMap<FGuid, FMaterialParameterCollectionInstanceResource*>::TIterator Iter = GDefaultMaterialParameterCollectionInstances.CreateIterator();
	while (Iter)
	{
		FMaterialParameterCollectionInstanceResource* Instance = Iter.Value();
		InstancesString += FString::Printf(TEXT("\n0x%p: %s: %s"),
			Instance, Instance ? *Instance->GetOwnerName().ToString() : TEXT("None"), *Iter.Key().ToString());
		++Iter;
	}

	UE_LOGF(LogRenderer, Fatal, "Failed to find parameter collection buffer with GUID '%ls' for %ls.\n"
		"Currently %i listed default instances: %ls",
		*ParameterCollectionGuid.ToString(),
		*Material.GetFullPath(),
		GDefaultMaterialParameterCollectionInstances.Num(), *InstancesString);
}

void FMaterialShader::SetParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FMaterial& Material,
	const FSceneInterface* Scene)
{
	const ERHIFeatureLevel::Type FeatureLevel = Scene ? Scene->GetFeatureLevel() : GMaxRHIFeatureLevel;
	FMaterialShaderMap* ShaderMap = Material.GetRenderingThreadShaderMap();
	checkf(ShaderMap, TEXT("RenderingThreadShaderMap: %i"), ShaderMap ? 1 : 0);
	checkf(ShaderMap->IsValidForRendering(true) && Material.GetFeatureLevel() == FeatureLevel, TEXT("IsValid:%i, MaterialFeatureLevel:%i, FeatureLevel:%i"), ShaderMap->IsValidForRendering() ? 1 : 0, (ERHIFeatureLevel::Type)Material.GetFeatureLevel(), FeatureLevel);

	FUniformExpressionCache* UniformExpressionCache = &MaterialRenderProxy->UniformExpressionCache[FeatureLevel];
	bool bUniformExpressionCacheNeedsDelete = false;

#if DO_CHECK
	if (UniformExpressionCache->CachedUniformExpressionShaderMap != ShaderMap)
	{
		UMaterialInterface* MtlInterface = Material.GetMaterialInterface();
		UMaterialInterface* ProxyInterface = MaterialRenderProxy->GetMaterialInterface();

		const FShaderType* ShaderType = GetType(ShaderMap->GetPointerTable());
		checkf(false,
			TEXT("'%s' shader uniform expression set mismatched shader map for material '%s/%s'.\n")
			TEXT("Material: '%s'\n")
			TEXT("Proxy: '%s'\n"),
			ShaderType->GetName(),
			*MaterialRenderProxy->GetFriendlyName(), *Material.GetFriendlyName(),
			MtlInterface ? *MtlInterface->GetFullName() : TEXT("nullptr"),
			ProxyInterface ? *ProxyInterface->GetFullName() : TEXT("nullptr"));
	}
#endif

	SetUniformBufferParameter(BatchedParameters, MaterialUniformBuffer, UniformExpressionCache->UniformBuffer);
	if (UniformExpressionCache->UniformBufferVS)
	{
		SetUniformBufferParameter(BatchedParameters, MaterialUniformBufferVS, UniformExpressionCache->UniformBufferVS);
	}

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING || !WITH_EDITOR)
	VerifyExpressionAndShaderMaps(MaterialRenderProxy, Material, UniformExpressionCache);
#endif

	{
		const TArray<FGuid>& ParameterCollections = UniformExpressionCache->ParameterCollections;
		const int32 ParameterCollectionsNum = ParameterCollections.Num();

		// For shipping and test builds the assert above will be compiled out, but we're trying to verify that this condition is never hit.
		if (ParameterCollectionUniformBuffers.Num() < ParameterCollectionsNum)
		{
			UE_LOGF(LogRenderer, Warning,
				"ParameterCollectionUniformBuffers.Num() [%u] < ParameterCollectionsNum [%u], this would crash below on SetUniformBufferParameter.\n" "RenderProxy=%ls Material=%ls",
				ParameterCollectionUniformBuffers.Num(),
				ParameterCollectionsNum,
				*MaterialRenderProxy->GetFriendlyName(),
				*Material.GetFriendlyName()
				);
		}

		check(ParameterCollectionUniformBuffers.Num() >= ParameterCollectionsNum);
		int32 NumToSet = FMath::Min(ParameterCollectionUniformBuffers.Num(), ParameterCollections.Num());

		// Find each referenced parameter collection's uniform buffer in the scene and set the parameter
		for (int32 CollectionIndex = 0; CollectionIndex < NumToSet; CollectionIndex++)
		{
			FRHIUniformBuffer* UniformBuffer = GetParameterCollectionBuffer(ParameterCollections[CollectionIndex], Scene);

			if (!UniformBuffer)
			{
				DumpRegisteredMaterialParameterCollections(Material, ParameterCollections[CollectionIndex]);
			}

			SetUniformBufferParameter(BatchedParameters, ParameterCollectionUniformBuffers[CollectionIndex], UniformBuffer);
		}
	}

	if (bUniformExpressionCacheNeedsDelete)
	{
		delete UniformExpressionCache;
	}
}

void FMaterialShader::GetShaderBindings(
	const FSceneInterface* Scene,
	const ERHIFeatureLevel::Type FeatureLevel,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	FMeshDrawSingleShaderBindings& ShaderBindings) const
{
	check(Material.GetRenderingThreadShaderMap() && Material.GetRenderingThreadShaderMap()->IsValidForRendering() && Material.GetFeatureLevel() == FeatureLevel);

	const FUniformExpressionCache& UniformExpressionCache = MaterialRenderProxy.UniformExpressionCache[FeatureLevel];

	checkf(UniformExpressionCache.CachedUniformExpressionShaderMap == Material.GetRenderingThreadShaderMap(), TEXT("UniformExpressionCache should be up to date, RenderProxy=%s Material=%s FeatureLevel=%d"), *MaterialRenderProxy.GetFriendlyName(), *Material.GetFriendlyName(), FeatureLevel);
	checkf(UniformExpressionCache.UniformBuffer, TEXT("NULL UniformBuffer, RenderProxy=%s Material=%s FeatureLevel=%d"), *MaterialRenderProxy.GetFriendlyName(), *Material.GetFriendlyName(), FeatureLevel);

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING || !WITH_EDITOR)
	VerifyExpressionAndShaderMaps(&MaterialRenderProxy, Material, &UniformExpressionCache);
#endif

	ShaderBindings.Add(MaterialUniformBuffer, UniformExpressionCache.UniformBuffer);
	if (UniformExpressionCache.UniformBufferVS)
	{
		ShaderBindings.Add(MaterialUniformBufferVS, UniformExpressionCache.UniformBufferVS);
	}

	{
		const TArray<FGuid>& ParameterCollections = UniformExpressionCache.ParameterCollections;
		const int32 ParameterCollectionsNum = ParameterCollections.Num();

		// For shipping and test builds the assert above will be compiled out, but we're trying to verify that this condition is never hit.
		if (ParameterCollectionUniformBuffers.Num() < ParameterCollectionsNum)
		{
			UE_LOGF(LogRenderer, Warning,
				"ParameterCollectionUniformBuffers.Num() [%u] < ParameterCollectionsNum [%u], this would crash below on SetUniformBufferParameter.\n" "RenderProxy=%ls Material=%ls",
				ParameterCollectionUniformBuffers.Num(),
				ParameterCollectionsNum,
				*MaterialRenderProxy.GetFriendlyName(),
				*Material.GetFriendlyName()
				);
		}

		check(ParameterCollectionUniformBuffers.Num() >= ParameterCollectionsNum);

		const int32 NumToSet = FMath::Min(ParameterCollectionUniformBuffers.Num(), ParameterCollections.Num());

		// Find each referenced parameter collection's uniform buffer in the scene and set the parameter
		for (int32 CollectionIndex = 0; CollectionIndex < NumToSet; CollectionIndex++)
		{			
			FRHIUniformBuffer* UniformBuffer = GetParameterCollectionBuffer(ParameterCollections[CollectionIndex], Scene);

			if (!UniformBuffer)
			{
				DumpRegisteredMaterialParameterCollections(Material, ParameterCollections[CollectionIndex]);
			}

			ShaderBindings.Add(ParameterCollectionUniformBuffers[CollectionIndex], UniformBuffer);		
		}
	}
}

#if 0
bool FMaterialShader::Serialize(FArchive& Ar)
{
	const bool bShaderHasOutdatedParameters = FShader::Serialize(Ar);
	Ar << MaterialUniformBuffer;
	Ar << ParameterCollectionUniformBuffers;

#if !ALLOW_SHADERMAP_DEBUG_DATA
	FDebugUniformExpressionSet	DebugUniformExpressionSet;
	static FName DebugUniformExpressionUB("DebugUniformExpressionUB");
	FRHIUniformBufferLayout		DebugUniformExpressionUBLayout(DebugUniformExpressionUB);
	FString						DebugDescription;
#endif

	Ar << DebugUniformExpressionSet;
	if (Ar.IsLoading())
	{
		FName LayoutName;
		Ar << LayoutName;
		DebugUniformExpressionUBLayout = FRHIUniformBufferLayout(LayoutName);
		Ar << DebugUniformExpressionUBLayout.ConstantBufferSize;

		TArray<uint16> ResourceOffsets;
		TArray<uint8> ResourceTypes;
		Ar << ResourceOffsets;
		Ar << ResourceTypes;

#if ALLOW_SHADERMAP_DEBUG_DATA
		DebugUniformExpressionUBLayout.Resources.Reserve(ResourceOffsets.Num());
		for (int32 i = 0; i < ResourceOffsets.Num(); i++)
		{
			DebugUniformExpressionUBLayout.Resources.Emplace(FRHIUniformBufferResource{ ResourceOffsets[i], EUniformBufferBaseType(ResourceTypes[i]) });
		}
		DebugUniformExpressionUBLayout.ComputeHash();
#endif
	}
	else
	{
		FName LayoutName = DebugUniformExpressionUBLayout.GetDebugName();
		Ar << LayoutName;
		Ar << DebugUniformExpressionUBLayout.ConstantBufferSize;

		TArray<uint16> ResourceOffsets;
		TArray<uint8> ResourceTypes;

		ResourceOffsets.Reserve(DebugUniformExpressionUBLayout.Resources.Num());
		ResourceTypes.Reserve(DebugUniformExpressionUBLayout.Resources.Num());
		for (int32 i = 0; i < DebugUniformExpressionUBLayout.Resources.Num(); i++)
		{
			ResourceOffsets.Emplace(DebugUniformExpressionUBLayout.Resources[i].MemberOffset);
			ResourceTypes.Emplace(uint8(DebugUniformExpressionUBLayout.Resources[i].MemberType));
		}

		Ar << ResourceOffsets;
		Ar << ResourceTypes;
	}
	Ar << DebugDescription;

	return bShaderHasOutdatedParameters;
}
#endif

FMeshMaterialShader::FMeshMaterialShader(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
	: FMaterialShader(Initializer)
{
	VertexFactoryParameters = Initializer.VertexFactoryType->CreateShaderParameters(Initializer.Target.GetFrequency(), Initializer.ParameterMap);
}

void FMeshMaterialShader::GetShaderBindings(
	const FScene* Scene,
	ERHIFeatureLevel::Type FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	const FMeshMaterialShaderElementData& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings) const
{
	FMaterialShader::GetShaderBindings(Scene, FeatureLevel, MaterialRenderProxy, Material, ShaderBindings);
	ShaderBindings.Add(GetUniformBufferParameter<FDistanceCullFadeUniformShaderParameters>(), ShaderElementData.FadeUniformBuffer);
	ShaderBindings.Add(GetUniformBufferParameter<FDitherUniformShaderParameters>(), ShaderElementData.DitherUniformBuffer);
}

void FMeshMaterialShader::GetElementShaderBindings(
	const FShaderMapPointerTable& PointerTable,
	const FScene* Scene, 
	const FSceneView* ViewIfDynamicMeshCommand, 
	const FVertexFactory* VertexFactory,
	const EVertexInputStreamType InputStreamType,
	const FStaticFeatureLevel FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMeshBatch& MeshBatch,
	const FMeshBatchElement& BatchElement, 
	const FMeshMaterialShaderElementData& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams) const
{
	const FVertexFactoryType* VertexFactoryType = GetVertexFactoryType(PointerTable);
	if (VertexFactoryType)
	{
		const FVertexFactoryShaderParameters* VFParameters = VertexFactoryParameters.Get();
		if (VFParameters)
		{
			VertexFactoryType->GetShaderParameterElementShaderBindings(GetFrequency(), VFParameters, Scene, ViewIfDynamicMeshCommand, this, InputStreamType, FeatureLevel, VertexFactory, BatchElement, ShaderBindings, VertexStreams);
		}
	}

	EShaderPlatform ShaderPlatform = Scene ? Scene->GetShaderPlatform() : GMaxRHIShaderPlatform;
	bool PlatformUsesPrimitiveUB = false;
	if (PlatformGPUSceneUsesUniformBufferView(ShaderPlatform))
	{
		// On platforms that use "UniformView" GPUScene, some VF might bind primitive UB
		// this includes ISM VFs and mesh particles
		PlatformUsesPrimitiveUB = (BatchElement.bForceInstanceCulling || (PrimitiveSceneProxy && PrimitiveSceneProxy->DoesVFRequirePrimitiveUniformBuffer()));
	}
		
	if (UseGPUScene(ShaderPlatform) && VertexFactory->GetPrimitiveIdStreamIndex(FeatureLevel, InputStreamType) >= 0 && !PlatformUsesPrimitiveUB)
	{
		const FShaderType* ShaderType = GetType(PointerTable);
		ensureMsgf(!GetUniformBufferParameter<FPrimitiveUniformShaderParameters>().IsBound(), TEXT("Shader %s attempted to bind the Primitive uniform buffer even though Vertex Factory computes a PrimitiveId per-instance.  This will break auto-instancing.  Shaders should use GetPrimitiveData(PrimitiveId).Member instead of Primitive.Member."), ShaderType->GetName());
		// Some primitives may use several VFs with a mixed support for a GPUScene. In this case all mesh batches get Primitive UB assigned regardless of VF type 
		ensureMsgf(!BatchElement.PrimitiveUniformBuffer || (PrimitiveSceneProxy && PrimitiveSceneProxy->DoesVFRequirePrimitiveUniformBuffer()), TEXT("FMeshBatchElement was assigned a PrimitiveUniformBuffer even though Vertex Factory %s fetches primitive shader data through a Scene buffer.  The assigned PrimitiveUniformBuffer cannot be respected.  Use PrimitiveUniformBufferResource instead for dynamic primitive data."), ShaderType->GetName());
	}
	else
	{
		if (BatchElement.PrimitiveUniformBuffer)
		{
			ShaderBindings.Add(GetUniformBufferParameter<FPrimitiveUniformShaderParameters>(), BatchElement.PrimitiveUniformBuffer);
		}
		else
		{
			const FShaderType* ShaderType = GetType(PointerTable);
			checkf(BatchElement.PrimitiveUniformBufferResource, TEXT("%s expected a primitive uniform buffer but none was set on BatchElement.PrimitiveUniformBuffer or BatchElement.PrimitiveUniformBufferResource"), ShaderType->GetName());
			ShaderBindings.Add(GetUniformBufferParameter<FPrimitiveUniformShaderParameters>(), BatchElement.PrimitiveUniformBufferResource->GetUniformBufferRHI());
		}
	}
}

void FMeshMaterialShader::WriteFrozenVertexFactoryParameters(FMemoryImageWriter& Writer, const TMemoryImagePtr<FVertexFactoryShaderParameters>& InVertexFactoryParameters) const
{
	const FVertexFactoryType* VertexFactoryType = GetVertexFactoryType(Writer.TryGetPrevPointerTable());
	InVertexFactoryParameters.WriteMemoryImageWithDerivedType(Writer, VertexFactoryType->GetShaderParameterLayout(GetFrequency()));
}
