// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderBaseClasses.h: Shader base classes
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UniformBuffer.h"
#include "Shader.h"
#include "MeshMaterialShader.h"

class FPrimitiveSceneProxy;
struct FMeshBatchElement;
struct FMeshDrawingRenderState;

/** The uniform shader parameters associated with a distance cull fade. */
// This was moved out of ScenePrivate.h to workaround MSVC vs clang template issue (it's used in this header file, so needs to be declared earlier)
// Z is the dither fade value (-1 = just fading in, 0 no fade, 1 = just faded out)
// W is unused and zero
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FDistanceCullFadeUniformShaderParameters,)
	SHADER_PARAMETER_EX(FVector2f,FadeTimeScaleBias, EShaderPrecisionModifier::Half)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef< FDistanceCullFadeUniformShaderParameters > FDistanceCullFadeUniformBufferRef;

/** The uniform shader parameters associated with a LOD dither fade. */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FDitherUniformShaderParameters, )
	SHADER_PARAMETER_EX(float, LODFactor, EShaderPrecisionModifier::Half)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef< FDitherUniformShaderParameters > FDitherUniformBufferRef;
