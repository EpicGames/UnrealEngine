// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderPrint.h"
#include "ShaderPrintParameters.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderParameterStruct.h"
#include "CommonRenderResources.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "Engine/Engine.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "RenderGraphBuilder.h"
#include "SceneRendering.h"
#include "ScreenPass.h"
#include "SystemTextures.h"
#include "RenderGraphUtils.h"
#include "RHIResourceUtils.h"
#include "SceneViewState.h"

namespace ShaderPrint
{
	//////////////////////////////////////////////////////////////////////////////////////////////////
	// Console variables

	static int32 GEnabled = false;
	static FAutoConsoleVariableRef CVarEnable(
		TEXT("r.ShaderPrint"),
		GEnabled,
		TEXT("ShaderPrint debugging toggle.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarFontSize(
		TEXT("r.ShaderPrint.FontSize"),
		8,
		TEXT("ShaderPrint font size.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarFontSpacingX(
		TEXT("r.ShaderPrint.FontSpacingX"),
		0,
		TEXT("ShaderPrint horizontal spacing between symbols.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarFontSpacingY(
		TEXT("r.ShaderPrint.FontSpacingY"),
		8,
		TEXT("ShaderPrint vertical spacing between symbols.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarMaxCharacterCount(
		TEXT("r.ShaderPrint.MaxCharacters"),
		2048,
		TEXT("ShaderPrint output buffer size.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarMaxWidgetCount(
		TEXT("r.ShaderPrint.MaxWidget"),
		32,
		TEXT("ShaderPrint max widget count.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarMaxLineCount(
		TEXT("r.ShaderPrint.MaxLine"),
		1024,
		TEXT("ShaderPrint max line count.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarMaxTriangleCount(
		TEXT("r.ShaderPrint.MaxTriangle"),
		1024,
		TEXT("ShaderPrint max triangle count.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarDrawLock(
		TEXT("r.ShaderPrint.Lock"),
		0,
		TEXT("Lock the line drawing.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarDrawOccludedLines(
		TEXT("r.ShaderPrint.DrawOccludedLines"),
		1,
		TEXT("Whether to draw occluded lines using checkboarding and lower opacity.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarDrawZoomEnable(
		TEXT("r.ShaderPrint.Zoom"),
		0,
		TEXT("Enable zoom magnification around the mouse cursor.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarDrawZoomPixel(
		TEXT("r.ShaderPrint.Zoom.Pixel"),
		16,
		TEXT("Number of pixels magnified around the mouse cursor.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarDrawZoomFactor(
		TEXT("r.ShaderPrint.Zoom.Factor"),
		8,
		TEXT("Zoom factor for magnification around the mouse cursor.\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<int32> CVarDrawZoomCorner(
		TEXT("r.ShaderPrint.Zoom.Corner"),
		3,
		TEXT("Select in which corner the zoom magnifer is displayed (0:top-left, 1:top-right, 2:bottom-right, 3:bottom-left).\n"),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	//////////////////////////////////////////////////////////////////////////////////////////////////
	// Global states

	struct FShaderPrintRequest
	{
		uint32 WidgetCount 		= 0;
		uint32 CharacterCount 	= 0;
		uint32 LineCount 		= 0;
		uint32 TriangleCount 	= 0;
	};

	static FShaderPrintRequest MaxRequests(const FShaderPrintRequest& A, const FShaderPrintRequest& B)
	{
		FShaderPrintRequest Out;
		Out.CharacterCount 	= FMath::Max(A.CharacterCount, 	B.CharacterCount);
		Out.LineCount 		= FMath::Max(A.LineCount, 		B.LineCount);
		Out.TriangleCount 	= FMath::Max(A.TriangleCount, 	B.TriangleCount);
		Out.WidgetCount 	= FMath::Max(A.WidgetCount, 	B.WidgetCount);
		return Out;
	}

	static FShaderPrintRequest GActiveShaderPrintRequest = FShaderPrintRequest();
	static FShaderPrintRequest GCachedShaderPrintMaxRequest = FShaderPrintRequest();
	static FViewInfo* GDefaultView = nullptr;

	struct FQueuedRenderItem
	{
		FFrozenShaderPrintData Payload;
		FSceneInterface const* Scene = nullptr;
		uint32 FrameForGC = 0;
	};
	static TArray<FQueuedRenderItem> GQueuedRenderItems;
	
	//////////////////////////////////////////////////////////////////////////////////////////////////
	// Struct & Functions

	static uint32 GetMaxCharacterCount()
	{
		return FMath::Max(CVarMaxCharacterCount.GetValueOnAnyThread() + int32(GCachedShaderPrintMaxRequest.CharacterCount), 0);
	}

	static uint32 GetMaxWidgetCount()
	{
		return FMath::Max(CVarMaxWidgetCount.GetValueOnAnyThread() + int32(GCachedShaderPrintMaxRequest.WidgetCount), 0);
	}

	static uint32 GetMaxLineCount()
	{
		return FMath::Max(CVarMaxLineCount.GetValueOnAnyThread() + int32(GCachedShaderPrintMaxRequest.LineCount), 0);
	}

	static uint32 GetMaxTriangleCount()
	{
		return FMath::Max(CVarMaxTriangleCount.GetValueOnAnyThread() + int32(GCachedShaderPrintMaxRequest.TriangleCount), 0);
	}

	// Returns the number of uints used for counters, a line element, and a triangle elements
	static uint32 GetCountersUintSize()       { return 4; }
	static uint32 GetPackedLineUintSize()     { return 8; }
	static uint32 GetPackedTriangleUintSize() { return 12; }
	static uint32 GetPackedSymbolUintSize()   { return 4; }

	// Get symbol buffer size
	// This is some multiple of the character buffer size to allow for maximum character->symbol expansion
	static uint32 GetMaxSymbolCountFromCharacterCount(uint32 InMaxCharacterCount)
	{
		return InMaxCharacterCount * 12u;
	}

	static bool IsDrawLocked()
	{
		return CVarDrawLock.GetValueOnRenderThread() > 0;
	}

	// Empty buffer for binding when ShaderPrint is disabled
	class FEmptyBuffer : public FBufferWithRDG
	{
	public:
		void InitRHI(FRHICommandListBase&) override
		{
			Buffer = AllocatePooledBuffer(FRDGBufferDesc::CreateBufferDesc(4, GetCountersUintSize()), TEXT("ShaderPrint.EmptyValueBuffer"));
		}
	};

	FBufferWithRDG* GEmptyBuffer = new TGlobalResource<FEmptyBuffer>();

	//////////////////////////////////////////////////////////////////////////////////////////////////
	// Global fixed index buffers

	static const uint16 GLineIndexBufferData[2] = { 0, 1 };

	class FLineIndexBuffer : public FIndexBuffer
	{
	public:
		void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FLineIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(GLineIndexBufferData));
		}
	};
	TGlobalResource<FLineIndexBuffer, FRenderResource::EInitPhase::Pre> GLineIndexBuffer;

	static const uint16 GTriangleIndexBufferData[3] = { 0, 1, 2 };

	class FTriangleIndexBuffer : public FIndexBuffer
	{
	public:
		void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FTriangleIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(GTriangleIndexBufferData));
		}
	};
	TGlobalResource<FTriangleIndexBuffer, FRenderResource::EInitPhase::Pre> GTriangleIndexBuffer;

		//////////////////////////////////////////////////////////////////////////////////////////////////
	// Uniform buffer
	
	// ShaderPrint uniform buffer
	IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FShaderPrintCommonParameters, "ShaderPrintData");
	
	// Fill the uniform buffer parameters
	void GetParameters(FShaderPrintSetup const& InSetup, FShaderPrintCommonParameters& OutParameters)
	{
		const FVector2D ViewSize(FMath::Max(InSetup.ViewRect.Size().X, 1), FMath::Max(InSetup.ViewRect.Size().Y, 1));
		const float FontWidth = float(InSetup.FontSize.X) * InSetup.DPIScale / ViewSize.X;
		const float FontHeight = float(InSetup.FontSize.Y) * InSetup.DPIScale / ViewSize.Y;
		const float SpaceWidth = float(InSetup.FontSpacing.X) * InSetup.DPIScale / ViewSize.X;
		const float SpaceHeight = float(InSetup.FontSpacing.Y) * InSetup.DPIScale / ViewSize.Y;

		OutParameters.FontSize = FVector2f(FontWidth, FontHeight);
		OutParameters.FontSpacing = FVector2f(FontWidth + SpaceWidth, FontHeight + SpaceHeight);
		OutParameters.Resolution = InSetup.ViewRect.Size();
		OutParameters.CursorCoord = InSetup.CursorCoord;
		OutParameters.MaxCharacterCount = InSetup.MaxCharacterCount;
		OutParameters.MaxSymbolCount = GetMaxSymbolCountFromCharacterCount(InSetup.MaxCharacterCount);
		OutParameters.MaxStateCount = InSetup.MaxStateCount;
		OutParameters.MaxLineCount = InSetup.MaxLineCount;
		OutParameters.MaxTriangleCount = InSetup.MaxTriangleCount;
		OutParameters.IsDrawLocked = InSetup.bIsDrawLocked ? 1 : 0;
		OutParameters.TranslatedWorldOffset = FVector3f(InSetup.PreViewTranslation);
	}

	// Return a uniform buffer with values filled and with single frame lifetime
	static TUniformBufferRef<ShaderPrint::FShaderPrintCommonParameters> CreateUniformBuffer(const FShaderPrintSetup& InSetup)
	{		
		FShaderPrintCommonParameters Parameters;
		GetParameters(InSetup, Parameters);

		return TUniformBufferRef<ShaderPrint::FShaderPrintCommonParameters>::CreateUniformBufferImmediate(Parameters, UniformBuffer_SingleFrame);
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////
	// Accessors

	// Fill the FShaderParameters parameters
	void SetParameters(FRDGBuilder& GraphBuilder, const FShaderPrintData& InData, FShaderParameters& OutParameters)
	{
		OutParameters.Common = InData.UniformBuffer;
		OutParameters.ShaderPrint_StateBuffer = GraphBuilder.CreateSRV(InData.ShaderPrintStateBuffer, PF_R32_UINT);
		OutParameters.ShaderPrint_RWEntryBuffer = GraphBuilder.CreateUAV(InData.ShaderPrintEntryBuffer, PF_R32_UINT);
	}

	void SetParameters(FRDGBuilder& GraphBuilder, FShaderParameters& OutParameters)
	{
		if (ensure(GDefaultView != nullptr))
		{
			SetParameters(GraphBuilder, GDefaultView->ShaderPrintData, OutParameters);
		}
	}

	bool IsSupported(EShaderPlatform InShaderPlatform)
	{
		// Avoid FXC since it struggles pretty hard with the shaders
		if (IsD3DPlatform(InShaderPlatform) && !FDataDrivenShaderPlatformInfo::GetSupportsDxc(InShaderPlatform))
		{
			return false;
		}

		return true;
	}

	void ModifyCompilationEnvironment(const EShaderPlatform Platform, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SHADER_PRINT_EXPLICIT_BINDING"), 1);
	}

	void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
	}

	bool IsEnabled()
	{
		return GEnabled != 0;
	}

	void SetEnabled(bool bInEnabled)
	{
		GEnabled = bInEnabled ? 1 : 0;
	}

	bool IsValid(FShaderPrintData const& InShaderPrintData)
	{
		// Assume that if UniformBuffer is valid then all other buffers are.
		return InShaderPrintData.UniformBuffer.IsValid();
	}

	bool IsEnabled(FShaderPrintData const& InShaderPrintData)
	{
		return InShaderPrintData.Setup.bEnabled;
	}

	bool IsDefaultViewValid()
	{
		return GDefaultView != nullptr && IsValid(GDefaultView->ShaderPrintData);
	}

	bool IsDefaultViewEnabled()
	{
		return GDefaultView != nullptr && IsEnabled(GDefaultView->ShaderPrintData);
	}

	void RequestSpaceForCharacters(uint32 InCount)
	{
		GActiveShaderPrintRequest.CharacterCount += InCount;
	}

	void RequestSpaceForLines(uint32 InCount)
	{
		GActiveShaderPrintRequest.LineCount += InCount;
	}

	void RequestSpaceForTriangles(uint32 InCount)
	{
		GActiveShaderPrintRequest.TriangleCount += InCount;
	}

	void SubmitShaderPrintData(FFrozenShaderPrintData& InData, FSceneInterface const* InScene)
	{
		// Queue with a frame number so that we can garbage collect if no matching view ever renders.
		GQueuedRenderItems.Add({InData, InScene, GFrameNumberRenderThread + 10});
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////
	// Common Shaders

	// Upload ShaderPrint parmeters into diagnostic buffer
	class FShaderPrintUploadCS: public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FShaderPrintUploadCS);
		SHADER_USE_PARAMETER_STRUCT(FShaderPrintUploadCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, Common)
		END_SHADER_PARAMETER_STRUCT()


		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("SHADER_UPLOAD"), 1);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderPrintUploadCS, "/Engine/Private/ShaderPrintDraw.usf", "UploadCS", SF_Compute);

	
	// Upload ShaderPrint parmeters into diagnostic buffer
	class FShaderPrintCopyCS: public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FShaderPrintCopyCS);
		SHADER_USE_PARAMETER_STRUCT(FShaderPrintCopyCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, Common)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWValuesBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("SHADER_COPY"), 1);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderPrintCopyCS, "/Engine/Private/ShaderPrintDraw.usf", "CopyCS", SF_Compute);

	//////////////////////////////////////////////////////////////////////////////////////////////////
	// Widget/Characters Shaders
	 
	// Shader to initialize the output value buffer
	class FShaderPrintClearCounterCS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FShaderPrintClearCounterCS);
		SHADER_USE_PARAMETER_STRUCT(FShaderPrintClearCounterCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWValuesBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderPrintClearCounterCS, "/Engine/Private/ShaderPrintDraw.usf", "ClearCounterCS", SF_Compute);

	// Shader to fill the indirect parameter arguments ready for the value->symbol compute pass
	class FShaderBuildIndirectDispatchArgsCS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FShaderBuildIndirectDispatchArgsCS);
		SHADER_USE_PARAMETER_STRUCT(FShaderBuildIndirectDispatchArgsCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, Common)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ValuesBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWSymbolsBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDispatchArgsBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderBuildIndirectDispatchArgsCS, "/Engine/Private/ShaderPrintDraw.usf", "BuildIndirectDispatchArgsCS", SF_Compute);

	// Shader to clean & compact widget state
	class FShaderCompactStateBufferCS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FShaderCompactStateBufferCS);
		SHADER_USE_PARAMETER_STRUCT(FShaderCompactStateBufferCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(uint32, FrameIndex)
			SHADER_PARAMETER(uint32, FrameThreshold)
			SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, Common)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWStateBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderCompactStateBufferCS, "/Engine/Private/ShaderPrintDraw.usf", "CompactStateBufferCS", SF_Compute);

	// Shader to read the values buffer and convert to the symbols buffer
	class FShaderBuildSymbolBufferCS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FShaderBuildSymbolBufferCS);
		SHADER_USE_PARAMETER_STRUCT(FShaderBuildSymbolBufferCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(uint32, FrameIndex)
			SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, Common)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ValuesBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWSymbolsBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWStateBuffer)
			RDG_BUFFER_ACCESS(IndirectDispatchArgsBuffer, ERHIAccess::IndirectArgs)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderBuildSymbolBufferCS, "/Engine/Private/ShaderPrintDraw.usf", "BuildSymbolBufferCS", SF_Compute);

	// Shader to fill the indirect parameter arguments ready for draw pass
	class FShaderBuildIndirectDrawArgsCS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FShaderBuildIndirectDrawArgsCS);
		SHADER_USE_PARAMETER_STRUCT(FShaderBuildIndirectDrawArgsCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, Common)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SymbolsBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDrawArgsBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderBuildIndirectDrawArgsCS, "/Engine/Private/ShaderPrintDraw.usf", "BuildIndirectDrawArgsCS", SF_Compute);

	// Shader for draw pass to render each symbol
	class FShaderDrawSymbols : public FGlobalShader
	{
	public:
		FShaderDrawSymbols()
		{}

		FShaderDrawSymbols(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
			: FGlobalShader(Initializer)
		{}

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			RENDER_TARGET_BINDING_SLOTS()
			SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, Common)
			SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SymbolsBuffer)
			RDG_BUFFER_ACCESS(IndirectDrawArgsBuffer, ERHIAccess::IndirectArgs)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}
	};

	class FShaderDrawSymbolsVS : public FShaderDrawSymbols
	{
		DECLARE_GLOBAL_SHADER(FShaderDrawSymbolsVS);
		SHADER_USE_PARAMETER_STRUCT(FShaderDrawSymbolsVS, FShaderDrawSymbols);
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderDrawSymbolsVS, "/Engine/Private/ShaderPrintDraw.usf", "DrawSymbolsVS", SF_Vertex);

	class FShaderDrawSymbolsPS : public FShaderDrawSymbols
	{
		DECLARE_GLOBAL_SHADER(FShaderDrawSymbolsPS);
		SHADER_USE_PARAMETER_STRUCT(FShaderDrawSymbolsPS, FShaderDrawSymbols);
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderDrawSymbolsPS, "/Engine/Private/ShaderPrintDraw.usf", "DrawSymbolsPS", SF_Pixel);

	//////////////////////////////////////////////////////////////////////////
	// Line Shaders

	class FShaderDrawDebugCopyCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FShaderDrawDebugCopyCS);
		SHADER_USE_PARAMETER_STRUCT(FShaderDrawDebugCopyCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, ElementBuffer)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, RWIndirectArgs)
			SHADER_PARAMETER(uint32, PrimitiveType)
			SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, ShaderPrintData)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("GPU_DEBUG_RENDERING"), 1);
			OutEnvironment.SetDefine(TEXT("GPU_DEBUG_RENDERING_COPY_CS"), 1);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderDrawDebugCopyCS, "/Engine/Private/ShaderPrintDrawPrimitive.usf", "ShaderDrawDebugCopyCS", SF_Compute);

	class FShaderDrawDebugVS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FShaderDrawDebugVS);
		SHADER_USE_PARAMETER_STRUCT(FShaderDrawDebugVS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, Common)
			SHADER_PARAMETER(FVector3f, TranslatedWorldOffsetConversion)
			SHADER_PARAMETER(FMatrix44f, TranslatedWorldToClip)
			SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ShaderDrawDebugPrimitive)
		END_SHADER_PARAMETER_STRUCT()

		class FPrimitiveType : SHADER_PERMUTATION_INT("PERMUTATION_PRIMITIVE_TYPE", 2);
		using FPermutationDomain = TShaderPermutationDomain<FPrimitiveType>;

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("GPU_DEBUG_RENDERING"), 1);
			FPermutationDomain PermutationVector(Parameters.PermutationId);
			if (PermutationVector.Get<FPrimitiveType>() == 0)
			{
				OutEnvironment.SetDefine(TEXT("GPU_DEBUG_RENDERING_LINE_VS"), 1);
			}
			else
			{
				OutEnvironment.SetDefine(TEXT("GPU_DEBUG_RENDERING_TRIANGLE_VS"), 1);
			}
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderDrawDebugVS, "/Engine/Private/ShaderPrintDrawPrimitive.usf", "ShaderDrawDebugVS", SF_Vertex);

	class FShaderDrawDebugPS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FShaderDrawDebugPS);
		SHADER_USE_PARAMETER_STRUCT(FShaderDrawDebugPS, FGlobalShader);
		
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(FVector2f, OutputInvResolution)
			SHADER_PARAMETER(FVector2f, OriginalViewRectMin)
			SHADER_PARAMETER(FVector2f, OriginalViewSize)
			SHADER_PARAMETER(FVector2f, OriginalBufferInvSize)
			SHADER_PARAMETER(uint32, bCheckerboardEnabled)
			SHADER_PARAMETER(uint32, bDrawOccludedLines)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, DepthSampler)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

		using FPermutationDomain = TShaderPermutationDomain<>;

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("GPU_DEBUG_RENDERING"), 1);
			OutEnvironment.SetDefine(TEXT("GPU_DEBUG_RENDERING_PS"), 1);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderDrawDebugPS, "/Engine/Private/ShaderPrintDrawPrimitive.usf", "ShaderDrawDebugPS", SF_Pixel);

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderDrawVSPSParameters , )
		SHADER_PARAMETER_STRUCT_INCLUDE(FShaderDrawDebugVS::FParameters, VS)
		SHADER_PARAMETER_STRUCT_INCLUDE(FShaderDrawDebugPS::FParameters, PS)
		RDG_BUFFER_ACCESS(IndirectBuffer, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()


	// Shader to zoom the final output
	class FShaderZoomCS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FShaderZoomCS);
		SHADER_USE_PARAMETER_STRUCT(FShaderZoomCS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(uint32, PixelExtent)
			SHADER_PARAMETER(uint32, ZoomFactor)
			SHADER_PARAMETER(uint32, Corner)
			SHADER_PARAMETER(FIntPoint, Resolution)
			SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, Common)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, InTexture)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutTexture)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(FGlobalShaderPermutationParameters const& Parameters)
		{
			return IsSupported(Parameters.Platform);
		}
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("SHADER_ZOOM"), 1);
		}
	};

	IMPLEMENT_GLOBAL_SHADER(FShaderZoomCS, "/Engine/Private/ShaderPrintDraw.usf", "DrawZoomCS", SF_Compute);

	//////////////////////////////////////////////////////////////////////////////////////////////////
	// Setup render data

	FShaderPrintSetup::FShaderPrintSetup(FIntRect InViewRect)
	{
		bEnabled = IsEnabled();

		ViewRect = InViewRect;

		FontSize = FIntPoint(FMath::Max(CVarFontSize.GetValueOnAnyThread(), 1), FMath::Max(CVarFontSize.GetValueOnAnyThread(), 1));
		FontSpacing = FIntPoint(FMath::Max(CVarFontSpacingX.GetValueOnAnyThread(), 1), FMath::Max(CVarFontSpacingY.GetValueOnAnyThread(), 1));
		MaxCharacterCount = bEnabled ? GetMaxCharacterCount() : 0;
		MaxStateCount = bEnabled ? GetMaxWidgetCount() : 0;
		MaxLineCount = bEnabled ? GetMaxLineCount() : 0;
		MaxTriangleCount = bEnabled ? GetMaxTriangleCount(): 0;
		bIsDrawLocked = false;
	}

	FShaderPrintSetup::FShaderPrintSetup(FSceneView const& View)
	{
		bEnabled = IsEnabled() && IsSupported(View.GetShaderPlatform()) && View.Family->EngineShowFlags.ShaderPrint;

		ViewRect = View.UnconstrainedViewRect;
		CursorCoord = View.CursorPos;
		PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
		DPIScale = View.Family->DebugDPIScale;

		FontSize = FIntPoint(FMath::Max(CVarFontSize.GetValueOnAnyThread(), 1), FMath::Max(CVarFontSize.GetValueOnAnyThread(), 1));
		FontSpacing = FIntPoint(FMath::Max(CVarFontSpacingX.GetValueOnAnyThread(), 1), FMath::Max(CVarFontSpacingY.GetValueOnAnyThread(), 1));
		MaxCharacterCount = bEnabled ? GetMaxCharacterCount() : 0;
		MaxStateCount = bEnabled ? GetMaxWidgetCount() : 0;
		MaxLineCount = bEnabled ? GetMaxLineCount() : 0;
		MaxTriangleCount = bEnabled ? GetMaxTriangleCount() : 0;
		bIsDrawLocked = View.State ? ((const FSceneViewState*)View.State)->ShaderPrintStateData.bIsLocked : false;
	}

	static uint32 GetRequestedEntryBufferSizeInUint(const FShaderPrintSetup& In)
	{
		const uint32 UintElementCount = 
			GetCountersUintSize() + 
			GetPackedSymbolUintSize() * In.MaxCharacterCount +
			GetPackedLineUintSize() * In.MaxLineCount + 
			GetPackedTriangleUintSize() * In.MaxTriangleCount;
		return UintElementCount;
	}

	FShaderPrintData CreateShaderPrintData(FRDGBuilder& GraphBuilder, FShaderPrintSetup const& InSetup, FSceneViewState* InViewState)
	{
		FShaderPrintData ShaderPrintData;

		// Common uniform buffer
		ShaderPrintData.Setup = InSetup;
		ShaderPrintData.UniformBuffer = CreateUniformBuffer(InSetup);

		// Early out if system is disabled.
		// Note that we still bind dummy buffers.
		// This is in case some debug shader code is still active and accessing the buffer.
		if (!InSetup.bEnabled)
		{
			ShaderPrintData.ShaderPrintEntryBuffer = GraphBuilder.RegisterExternalBuffer(GEmptyBuffer->Buffer);
			ShaderPrintData.ShaderPrintStateBuffer = GraphBuilder.RegisterExternalBuffer(GEmptyBuffer->Buffer);
			return ShaderPrintData;
		}

		// Characters/Widgets/Primitives/Lines
		{
			const bool bLockBufferThisFrame = IsDrawLocked() && InViewState != nullptr && !InViewState->ShaderPrintStateData.bIsLocked;
			ERDGBufferFlags Flags = bLockBufferThisFrame ? ERDGBufferFlags::MultiFrame : ERDGBufferFlags::None;

			const uint32 UintElementCount = GetRequestedEntryBufferSizeInUint(InSetup);
			ShaderPrintData.ShaderPrintEntryBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, UintElementCount), TEXT("ShaderPrint.EntryBuffer"), Flags);

			// State buffer is retrieved from the view state, or created if it does not exist
			if (InViewState != nullptr)
			{
				if (InViewState->ShaderPrintStateData.StateBuffer)
				{
					ShaderPrintData.ShaderPrintStateBuffer = GraphBuilder.RegisterExternalBuffer(InViewState->ShaderPrintStateData.StateBuffer);
				}
				else
				{
					ShaderPrintData.ShaderPrintStateBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, (3 * InSetup.MaxStateCount) + 1), TEXT("ShaderPrint.StateBuffer"));
					AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(ShaderPrintData.ShaderPrintStateBuffer, PF_R32_UINT), 0u);
					InViewState->ShaderPrintStateData.StateBuffer = GraphBuilder.ConvertToExternalBuffer(ShaderPrintData.ShaderPrintStateBuffer);
				}
			}
			else
			{
				ShaderPrintData.ShaderPrintStateBuffer = GraphBuilder.RegisterExternalBuffer(GEmptyBuffer->Buffer);
			}

			// Clear counters
			{
				FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
				TShaderMapRef<FShaderPrintClearCounterCS> ComputeShader(GlobalShaderMap);

				FShaderPrintClearCounterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FShaderPrintClearCounterCS::FParameters>();
				PassParameters->RWValuesBuffer = GraphBuilder.CreateUAV(ShaderPrintData.ShaderPrintEntryBuffer, PF_R32_UINT);
				ClearUnusedGraphResources(ComputeShader, PassParameters);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("ShaderPrint::ClearCounters"),
					PassParameters,
					ERDGPassFlags::Compute,
					[PassParameters, ComputeShader](FRDGAsyncTask, FRHICommandList& RHICmdList)
					{
						FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *PassParameters, FIntVector(1, 1, 1));
					});
			}

			if (InViewState != nullptr)
			{
				if (IsDrawLocked() && !InViewState->ShaderPrintStateData.bIsLocked)
				{
					InViewState->ShaderPrintStateData.EntryBuffer = GraphBuilder.ConvertToExternalBuffer(ShaderPrintData.ShaderPrintEntryBuffer);
					InViewState->ShaderPrintStateData.PreViewTranslation = InSetup.PreViewTranslation;
					InViewState->ShaderPrintStateData.bIsLocked = true;
				}

				if (!IsDrawLocked() && InViewState->ShaderPrintStateData.bIsLocked)
				{
					InViewState->ShaderPrintStateData.EntryBuffer = nullptr;
					InViewState->ShaderPrintStateData.PreViewTranslation = FVector::ZeroVector;
					InViewState->ShaderPrintStateData.bIsLocked = false;
				}
			}
		}

		return ShaderPrintData;
	}

	FShaderPrintData CreateShaderPrintData(FRDGBuilder& GraphBuilder, FShaderPrintSetup const& InSetup)
	{
		return CreateShaderPrintData(GraphBuilder, InSetup, nullptr);
	}

	FFrozenShaderPrintData FreezeShaderPrintData(FRDGBuilder& GraphBuilder, FShaderPrintData& ShaderPrintData)
	{
		FFrozenShaderPrintData Out;

		Out.Setup = ShaderPrintData.Setup;

		Out.ShaderPrintEntryBuffer = GraphBuilder.ConvertToExternalBuffer(ShaderPrintData.ShaderPrintEntryBuffer);
		Out.ShaderPrintStateBuffer = GraphBuilder.ConvertToExternalBuffer(ShaderPrintData.ShaderPrintStateBuffer);

		return Out;
	}

	FShaderPrintData UnFreezeShaderPrintData(FRDGBuilder& GraphBuilder, FFrozenShaderPrintData& FrozenShaderPrintData)
	{
		FShaderPrintData Out;

		Out.Setup = FrozenShaderPrintData.Setup;
		Out.UniformBuffer = CreateUniformBuffer(Out.Setup);

		Out.ShaderPrintEntryBuffer = GraphBuilder.RegisterExternalBuffer(FrozenShaderPrintData.ShaderPrintEntryBuffer);
		Out.ShaderPrintStateBuffer = GraphBuilder.RegisterExternalBuffer(FrozenShaderPrintData.ShaderPrintStateBuffer);

		return Out;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////
	// Drawing/Rendering API

	static void InternalUploadParameters(FRDGBuilder& GraphBuilder, const FShaderPrintData& ShaderPrintData)
	{
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FShaderPrintUploadCS> ComputeShader(GlobalShaderMap);
		FShaderPrintUploadCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FShaderPrintUploadCS::FParameters>();
		PassParameters->Common = ShaderPrintData.UniformBuffer;
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ShaderPrint::UploadParameters"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			ComputeShader, PassParameters, FIntVector(1,1,1));
	}

	static void InternalCopyParameters(FRDGBuilder& GraphBuilder, const FShaderPrintData& ShaderPrintData)
	{
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FShaderPrintCopyCS> ComputeShader(GlobalShaderMap);
		FShaderPrintCopyCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FShaderPrintCopyCS::FParameters>();
		PassParameters->Common = ShaderPrintData.UniformBuffer;
		PassParameters->RWValuesBuffer = GraphBuilder.CreateUAV(ShaderPrintData.ShaderPrintEntryBuffer, PF_R32_UINT);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ShaderPrint::CopyParameters"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			ComputeShader, PassParameters, FIntVector(1,1,1));
	}

	void BeginView(FRDGBuilder& GraphBuilder, FViewInfo& View)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ShaderPrint::BeginView);

		// Create the render data and store on the view.
		FShaderPrintSetup ShaderPrintSetup(View);
		View.ShaderPrintData = CreateShaderPrintData(GraphBuilder, ShaderPrintSetup, View.ViewState);

		if (IsSupported(View.GetShaderPlatform()))
		{
			// Upload/Copy ShaderPrint parameters into UEDiagnostic buffer
			InternalUploadParameters(GraphBuilder, View.ShaderPrintData);
		}
	}

	void BeginViews(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views)
	{
		ensure(GDefaultView == nullptr);
		if (Views.Num() > 0)
		{
			GDefaultView = &Views[0];
		}

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			BeginView(GraphBuilder, View);
		}

		// * Track the max requests accross frame to avoid flickering when switching between views/renderer having different requests
		// * Then Reset counters which are read on the next BeginViews().
		GCachedShaderPrintMaxRequest = MaxRequests(GCachedShaderPrintMaxRequest, GActiveShaderPrintRequest);
		GActiveShaderPrintRequest = FShaderPrintRequest();
	}

	static void InternalDrawView_Characters(
		FRDGBuilder& GraphBuilder, 
		FShaderPrintData const& ShaderPrintData, 
		FIntRect ViewRect, 
		int32 FrameNumber, 
		FScreenPassTexture OutputTexture)
	{
		// Initialize graph managed resources
		const uint32 UintElementCount = GetCountersUintSize() + GetPackedSymbolUintSize() * GetMaxSymbolCountFromCharacterCount(ShaderPrintData.Setup.MaxCharacterCount);
		FRDGBufferRef SymbolBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, UintElementCount), TEXT("ShaderPrint.SymbolBuffer"));
		FRDGBufferRef IndirectDispatchArgsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("ShaderPrint.IndirectDispatchArgs"));
		FRDGBufferRef IndirectDrawArgsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(5), TEXT("ShaderPrint.IndirectDrawArgs"));
		FRDGBufferUAVRef SymbolBufferUAV = GraphBuilder.CreateUAV(SymbolBuffer, PF_R32_UINT);
		FRDGBufferSRVRef SymbolBufferSRV = GraphBuilder.CreateSRV(SymbolBuffer, PF_R32_UINT);

		// Non graph managed resources
		FRDGBufferSRVRef ValueBuffer = GraphBuilder.CreateSRV(ShaderPrintData.ShaderPrintEntryBuffer, PF_R32_UINT);
		FRDGBufferSRVRef StateBuffer = GraphBuilder.CreateSRV(ShaderPrintData.ShaderPrintStateBuffer, PF_R32_UINT);
		FTextureRHIRef FontTexture = GSystemTextures.AsciiTexture->GetRHI();

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		// BuildIndirectDispatchArgs
		{
			typedef FShaderBuildIndirectDispatchArgsCS SHADER;
			TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);

			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			PassParameters->Common = ShaderPrintData.UniformBuffer;
			PassParameters->ValuesBuffer = ValueBuffer;
			PassParameters->RWSymbolsBuffer = SymbolBufferUAV;
			PassParameters->RWIndirectDispatchArgsBuffer = GraphBuilder.CreateUAV(IndirectDispatchArgsBuffer, PF_R32_UINT);

			FComputeShaderUtils::AddPass(
				GraphBuilder, 
				RDG_EVENT_NAME("ShaderPrint::BuildIndirectDispatchArgs"), 
				ComputeShader, PassParameters,
				FIntVector(1, 1, 1));
		}

		// BuildSymbolBuffer
		{
			typedef FShaderBuildSymbolBufferCS SHADER;
			TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);

			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			PassParameters->FrameIndex = FrameNumber;
			PassParameters->Common = ShaderPrintData.UniformBuffer;
			PassParameters->ValuesBuffer = ValueBuffer;
			PassParameters->RWSymbolsBuffer = SymbolBufferUAV;
			PassParameters->RWStateBuffer = GraphBuilder.CreateUAV(ShaderPrintData.ShaderPrintStateBuffer, PF_R32_UINT);
			PassParameters->IndirectDispatchArgsBuffer = IndirectDispatchArgsBuffer;

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ShaderPrint::BuildSymbolBuffer"),
				ComputeShader, PassParameters,
				IndirectDispatchArgsBuffer, 0);
		}

		// CompactStateBuffer
		#if 0
		{
			typedef FShaderCompactStateBufferCS SHADER;
			TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);

			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			PassParameters->FrameIndex = FrameNumber;
			PassParameters->FrameThreshold = 300u;
			PassParameters->Common = ShaderPrintData.UniformBuffer;
			PassParameters->RWStateBuffer = GraphBuilder.CreateUAV(ShaderPrintData.ShaderPrintStateBuffer, PF_R32_UINT);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ShaderPrint::CompactStateBuffer"),
				ComputeShader, PassParameters, FIntVector(1,1,1));
		}
		#endif

		// BuildIndirectDrawArgs
		{
			typedef FShaderBuildIndirectDrawArgsCS SHADER;
			TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);

			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			PassParameters->Common = ShaderPrintData.UniformBuffer;
			PassParameters->SymbolsBuffer = SymbolBufferSRV;
			PassParameters->RWIndirectDrawArgsBuffer = GraphBuilder.CreateUAV(IndirectDrawArgsBuffer, PF_R32_UINT);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ShaderPrint::BuildIndirectDrawArgs"),
				ComputeShader, PassParameters,
				FIntVector(1, 1, 1));
		}

		// DrawSymbols
		{
			typedef FShaderDrawSymbols SHADER;
			TShaderMapRef< FShaderDrawSymbolsVS > VertexShader(GlobalShaderMap);
			TShaderMapRef< FShaderDrawSymbolsPS > PixelShader(GlobalShaderMap);

			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture.Texture, ERenderTargetLoadAction::ELoad);
			PassParameters->Common = ShaderPrintData.UniformBuffer;
			PassParameters->MiniFontTexture = FontTexture;
			PassParameters->SymbolsBuffer = SymbolBufferSRV;
			PassParameters->IndirectDrawArgsBuffer = IndirectDrawArgsBuffer;

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("ShaderPrint::DrawSymbols"),
				PassParameters,
				ERDGPassFlags::Raster,
				[VertexShader, PixelShader, PassParameters, ViewRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				
				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParameters);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

				RHICmdList.DrawIndexedPrimitiveIndirect(GTwoTrianglesIndexBuffer.IndexBufferRHI, PassParameters->IndirectDrawArgsBuffer->GetIndirectRHICallBuffer(), 0);
			});
		}
	}

	static void InternalDrawView_Primitives(
		FRDGBuilder& GraphBuilder,
		const FShaderPrintData& ShaderPrintData,
		FRDGBufferSRVRef ShaderPrintPrimitiveBufferSRV,
		const FIntRect& ViewRect,
		const FIntRect& UnscaledViewRect,
		const FMatrix & TranslatedWorldToClip,
		const FVector& TranslatedWorldOffsetConversion,
		const bool bLines,
		const bool bLocked,
		FRDGTextureRef OutputTexture,
		FRDGTextureRef DepthTexture)
	{
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		FRDGBufferRef IndirectBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndexedIndirectParameters>(1), TEXT("ShaderDraw.IndirectBuffer"), ERDGBufferFlags::None);
		{
			FShaderDrawDebugCopyCS::FParameters* Parameters = GraphBuilder.AllocParameters<FShaderDrawDebugCopyCS::FParameters>();
			Parameters->ElementBuffer = ShaderPrintPrimitiveBufferSRV;
			Parameters->RWIndirectArgs = GraphBuilder.CreateUAV(IndirectBuffer, PF_R32_UINT);
			Parameters->ShaderPrintData = ShaderPrintData.UniformBuffer;
			Parameters->PrimitiveType = bLines ? 0u : 1u;

			TShaderMapRef<FShaderDrawDebugCopyCS> ComputeShader(GlobalShaderMap);
			ClearUnusedGraphResources(ComputeShader, Parameters);
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("ShaderPrint::CopyLineArgs(%s%s)", bLines ? TEXT("Lines") : TEXT("Triangles"), bLocked ? TEXT(",Locked") : TEXT("")),
				Parameters,
				ERDGPassFlags::Compute,
				[Parameters, ComputeShader](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *Parameters, FIntVector(1, 1, 1));
				});
		}

		FShaderDrawDebugVS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FShaderDrawDebugVS::FPrimitiveType>(bLines ? 0u : 1u);
		TShaderMapRef<FShaderDrawDebugVS> VertexShader(GlobalShaderMap, PermutationVector);
		TShaderMapRef<FShaderDrawDebugPS> PixelShader(GlobalShaderMap);

		// Create a transient depth texture which allows to depth test filled primitive between themselves. These primitives are not culled against the scene depth texture, but only 'checkerboarded'.
		FRDGTextureRef TransientDepthTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputTexture->Desc.Extent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource), TEXT("ShaderPrint.DepthTexture"));

		FShaderDrawVSPSParameters* PassParameters = GraphBuilder.AllocParameters<FShaderDrawVSPSParameters >();
		PassParameters->VS.TranslatedWorldOffsetConversion = FVector3f(TranslatedWorldOffsetConversion);
		PassParameters->VS.TranslatedWorldToClip = FMatrix44f(TranslatedWorldToClip);
		PassParameters->PS.RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ELoad);
		PassParameters->PS.RenderTargets.DepthStencil = FDepthStencilBinding(TransientDepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);
		PassParameters->PS.OutputInvResolution = FVector2f(1.f / UnscaledViewRect.Width(), 1.f / UnscaledViewRect.Height());
		PassParameters->PS.OriginalViewRectMin = FVector2f(ViewRect.Min);
		PassParameters->PS.OriginalViewSize = FVector2f(ViewRect.Width(), ViewRect.Height());
		PassParameters->PS.OriginalBufferInvSize = FVector2f(1.f / DepthTexture->Desc.Extent.X, 1.f / DepthTexture->Desc.Extent.Y);
		PassParameters->PS.DepthTexture = DepthTexture;
		PassParameters->PS.DepthSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->PS.bCheckerboardEnabled = bLines ? 1u : 0u;
		PassParameters->PS.bDrawOccludedLines = CVarDrawOccludedLines.GetValueOnRenderThread() != 0 ? 1 : 0;
		PassParameters->VS.ShaderDrawDebugPrimitive = ShaderPrintPrimitiveBufferSRV;
		PassParameters->VS.Common = ShaderPrintData.UniformBuffer;
		PassParameters->IndirectBuffer = IndirectBuffer;

		ValidateShaderParameters(PixelShader, PassParameters->PS);
		ClearUnusedGraphResources(PixelShader, &PassParameters->PS);
		ValidateShaderParameters(VertexShader, PassParameters->VS);
		ClearUnusedGraphResources(VertexShader, &PassParameters->VS);

		const FIntRect Viewport = UnscaledViewRect;
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("ShaderPrint::Draw(%s%s)", bLines ? TEXT("Lines") : TEXT("Triangles"), bLocked ? TEXT(",Locked") : TEXT("")),
			PassParameters,
			ERDGPassFlags::Raster,
			[VertexShader, PixelShader, PassParameters, IndirectBuffer, Viewport, bLines](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				// Marks the indirect draw parameter as used by the pass, given it's not used directly by any of the shaders.
				PassParameters->IndirectBuffer->MarkResourceAsUsed();

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI(); // Premultiplied-alpha composition
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.PrimitiveType = bLines ? PT_LineList : PT_TriangleList;
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

				// Marks the indirect draw parameter as used by the pass, given it's not used directly by any of the shaders.
				FRHIBuffer* IndirectBufferRHI = PassParameters->IndirectBuffer->GetIndirectRHICallBuffer();
				check(IndirectBufferRHI != nullptr);
				RHICmdList.DrawIndexedPrimitiveIndirect(bLines ? GLineIndexBuffer.IndexBufferRHI : GTriangleIndexBuffer.IndexBufferRHI, IndirectBufferRHI, 0);
			});
	}

	static void InternalDrawZoom(FRDGBuilder& GraphBuilder, const FShaderPrintData& ShaderPrintData, const FScreenPassTexture& OutputTexture)
	{
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		FRDGTextureDesc Desc = OutputTexture.Texture->Desc;
		Desc.Flags |= ETextureCreateFlags::UAV;
		FRDGTextureRef OutZoomTexture = GraphBuilder.CreateTexture(Desc, TEXT("ShaderPrint.OutZoomTexture"));
		AddCopyTexturePass(GraphBuilder, OutputTexture.Texture, OutZoomTexture);

		TShaderMapRef<FShaderZoomCS> ComputeShader(GlobalShaderMap);

		FShaderZoomCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FShaderZoomCS::FParameters>();
		PassParameters->PixelExtent = FMath::Clamp(CVarDrawZoomPixel.GetValueOnRenderThread(), 2, 128);
		PassParameters->ZoomFactor  = FMath::Clamp(CVarDrawZoomFactor.GetValueOnRenderThread(), 1, 10);
		PassParameters->Resolution = OutputTexture.Texture->Desc.Extent;
		PassParameters->Corner = FMath::Clamp(CVarDrawZoomCorner.GetValueOnRenderThread(), 0, 3);
		PassParameters->Common = ShaderPrintData.UniformBuffer;
		PassParameters->InTexture = OutputTexture.Texture;
		PassParameters->OutTexture = GraphBuilder.CreateUAV(OutZoomTexture);

		const uint32 SrcPixelCount = PassParameters->PixelExtent * 2 + 1;
		const uint32 OutPixelCount = PassParameters->ZoomFactor * SrcPixelCount;
		const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(FIntPoint(PassParameters->PixelExtent * 2 + 1), FIntPoint(8));
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ShaderPrint::DrawZoom"),
			ComputeShader, PassParameters, GroupCount);

		AddCopyTexturePass(GraphBuilder, OutZoomTexture, OutputTexture.Texture);
	}

	static void InternalDrawView(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FShaderPrintData& ShaderPrintData, const FScreenPassTexture& OutputTexture, const FScreenPassTexture& DepthTexture)
	{
		if (!ensure(OutputTexture.IsValid()))
		{
			return;
		}

		RDG_EVENT_SCOPE(GraphBuilder, "ShaderPrint::DrawView");

		// ShaderPrintPrimitiveBuffer is a StructuredBuffer, but we need to read its content in a vertex shader. For certain platforms, this results in UAV reads, which is not always supported in a vertex shader.
		// To work around this issue, we instead create a typed buffer here (readable in VS on all platforms) and copy ShaderPrintPrimitiveBuffer into it.
		auto CreateTypedBufferFromStructured = [](FRDGBuilder& GraphBuilder, FRDGBufferRef Input, const TCHAR* ResultBufferName) -> FRDGBufferRef
		{
			FRDGBufferRef Result = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(Input->Desc.BytesPerElement, Input->Desc.NumElements), ResultBufferName);
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Result, PF_R32_UINT), 0u);
			AddCopyBufferPass(GraphBuilder, Result, Input);
			return Result;
		};

		const FIntRect SourceViewRect = View.ViewRect;
		const FIntRect OutputViewRect = OutputTexture.ViewRect;
		const FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation() - ShaderPrintData.Setup.PreViewTranslation;

		// Copy/merge data (when using UEDiagnostic buffer as storage)
		{
			InternalCopyParameters(GraphBuilder, View.ShaderPrintData);
		}

		FRDGBufferRef ShaderPrintEntryTypedBuffer = CreateTypedBufferFromStructured(GraphBuilder, ShaderPrintData.ShaderPrintEntryBuffer, TEXT("ShaderDraw.EntryBufferTyped"));
		FRDGBufferSRVRef ShaderPrintEntryTypedBufferSRV = GraphBuilder.CreateSRV(ShaderPrintEntryTypedBuffer, PF_R32_UINT);

		// Lines
		{
			FRDGBufferSRVRef DataBuffer = ShaderPrintEntryTypedBufferSRV;
			InternalDrawView_Primitives(GraphBuilder, ShaderPrintData, DataBuffer, SourceViewRect, OutputViewRect, View.ViewMatrices.GetTranslatedWorldToClip(), PreViewTranslation, true /*bLines*/, false /*bLocked*/, OutputTexture.Texture, DepthTexture.Texture);
		}

		// Triangles
		{
			FRDGBufferSRVRef DataBuffer = ShaderPrintEntryTypedBufferSRV;
			InternalDrawView_Primitives(GraphBuilder, ShaderPrintData, DataBuffer, SourceViewRect, OutputViewRect, View.ViewMatrices.GetTranslatedWorldToClip(), PreViewTranslation, false /*bLines*/, false /*bLocked*/, OutputTexture.Texture, DepthTexture.Texture);
		}

		// Locked Lines/Triangles
		if (View.ViewState && View.ViewState->ShaderPrintStateData.bIsLocked)
		{
			const FVector LockedPreViewTranslation = View.ViewMatrices.GetPreViewTranslation() - View.ViewState->ShaderPrintStateData.PreViewTranslation;
			FRDGBufferRef DataBufferStructured = GraphBuilder.RegisterExternalBuffer(View.ViewState->ShaderPrintStateData.EntryBuffer);
			FRDGBufferRef DataBufferTyped = CreateTypedBufferFromStructured(GraphBuilder, DataBufferStructured, TEXT("ShaderDraw.LockedEntryBufferTyped"));
			FRDGBufferSRVRef DataBuffer = GraphBuilder.CreateSRV(DataBufferTyped, PF_R32_UINT);
			InternalDrawView_Primitives(GraphBuilder, ShaderPrintData, DataBuffer, SourceViewRect, OutputViewRect, View.ViewMatrices.GetTranslatedWorldToClip(), LockedPreViewTranslation, true  /*bLines*/, true/*bLocked*/, OutputTexture.Texture, DepthTexture.Texture);
			InternalDrawView_Primitives(GraphBuilder, ShaderPrintData, DataBuffer, SourceViewRect, OutputViewRect, View.ViewMatrices.GetTranslatedWorldToClip(), LockedPreViewTranslation, false /*bLines*/, true/*bLocked*/, OutputTexture.Texture, DepthTexture.Texture);
		}

		// Characters
		{
			const int32 FrameNumber = View.Family ? View.Family->FrameNumber : 0u;
			InternalDrawView_Characters(GraphBuilder, ShaderPrintData, OutputViewRect, FrameNumber, OutputTexture);
		}

		// Zoom
		const bool bZoom = CVarDrawZoomEnable.GetValueOnRenderThread() > 0;
		if (bZoom)
		{
			InternalDrawZoom(GraphBuilder, ShaderPrintData, OutputTexture);
		}
	}

	void DrawView(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FScreenPassTexture& OutputTexture, const FScreenPassTexture& DepthTexture)
	{
		// Draw the shader print data for the view.
		InternalDrawView(GraphBuilder, View, View.ShaderPrintData, OutputTexture, DepthTexture);

		// Draw any externally enqueued shader print data.
		FSceneInterface const* Scene = View.Family != nullptr ? View.Family->Scene : nullptr;
		for (FQueuedRenderItem& ShaderPrintDataToRender : GQueuedRenderItems)
		{
			if (ShaderPrintDataToRender.Scene == nullptr || ShaderPrintDataToRender.Scene == Scene)
			{
				FShaderPrintData ShaderPrintData = UnFreezeShaderPrintData(GraphBuilder, ShaderPrintDataToRender.Payload);
				InternalDrawView(GraphBuilder, View, ShaderPrintData, OutputTexture, DepthTexture);
			}
		}
	}

	void EndViews(TArrayView<FViewInfo> Views)
	{
		// Clear the shader print data owned by the views.
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			Views[ViewIndex].ShaderPrintData = FShaderPrintData();
		}

		// Remove all externally enqueud shader print data for the matching Scene, since we will have drawn it in the calls to DrawView().
		if (Views.Num())
		{
			FSceneInterface const* Scene = Views[0].Family != nullptr ? Views[0].Family->Scene : nullptr;
			GQueuedRenderItems.RemoveAll([Scene](FQueuedRenderItem& Data) { return Data.Scene == nullptr || Data.Scene == Scene || Data.FrameForGC < GFrameNumberRenderThread; });
		}

		GDefaultView = nullptr;
	}


	FStrings::FStrings(uint32 InMaxEntryCount, uint32 InMaxStringLength)
	{
		static_assert(sizeof(FEntryInfo) == 8);
		Chars.Reserve(InMaxEntryCount * InMaxStringLength);
	}
	
	void FStrings::Add(const FString& In, uint32 EntryID)
	{
		const uint32 Offset = Chars.Num();
		const uint32 Length = In.Len();
		for (TCHAR C : In)
		{
			Chars.Add(uint8(C));
		}

		FEntryInfo& Info = Infos.AddDefaulted_GetRef();
		Info.EntryID = EntryID;
		Info.Length = Length;
		Info.Offset = Offset;
	}

	void FStrings::Add(const FString& In)
	{
		Add(In, Infos.Num());
	}
	
	FStrings::FShaderParameters FStrings::GetParameters(FRDGBuilder& GraphBuilder)
	{
		if (Infos.IsEmpty())
		{
			FEntryInfo& Info = Infos.AddDefaulted_GetRef();
			Info.EntryID = ~0;
			Info.Length = 4;
			Info.Offset = 0;
			Chars.Add(uint8('N'));
			Chars.Add(uint8('o'));
			Chars.Add(uint8('n'));
			Chars.Add(uint8('e'));
		}
	
		FShaderParameters Out;
		Out.InfoCount = Infos.Num();
		Out.CharCount = Chars.Num();
		Out.InfoBuffer = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("ShaderPrint.Strings.Infos"), Infos));
		Out.CharBuffer = GraphBuilder.CreateSRV(CreateVertexBuffer(GraphBuilder, TEXT("ShaderPrint.Strings.Chars"), FRDGBufferDesc::CreateBufferDesc(1, Chars.Num()), Chars.GetData(), Chars.Num()), PF_R8_UINT);

		return Out;
	}
}