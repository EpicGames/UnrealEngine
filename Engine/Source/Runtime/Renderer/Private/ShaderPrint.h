// Copyright Epic Games, Inc. All Rights Reserved.

// Use ShaderPrint to debug print from any shader type.
// Call BeginViews() at the start of a view group to init the debug output buffer.
// Call DrawView() to composite the debug data to the final render target.
// Call EndViews() at the end of the view group to tidy up.

// The shader HLSL code should include the ShaderPrintCommon.ush and use the ShaderPrint*() functions.
// The shader C++ code needs to bind the buffers required to capture the debug print. See ShaderPrintParameters.h for this. 

#pragma once

#include "CoreMinimal.h"

class FRDGBuilder;
class FViewInfo;
struct FScreenPassTexture;

namespace ShaderPrint
{
	// Allocate the debug print buffer associated with the view
	void BeginViews(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views);
	// Draw info from the debug print buffer to the given output target
	void DrawView(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FScreenPassTexture& OutputTexture, const FScreenPassTexture& DepthTexture);
	// Release the debug print buffer associated with the view
	void EndViews(TArrayView<FViewInfo> Views);
}
