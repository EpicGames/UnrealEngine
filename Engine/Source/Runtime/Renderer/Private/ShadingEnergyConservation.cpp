// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadingEnergyConservation.cpp: private energy conservation implementation
=============================================================================*/

#include "ShadingEnergyConservation.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderGraphUtils.h"
#include "SceneRendering.h"
#include "PixelShaderUtils.h"
#include "Substrate/Substrate.h"
#include "Engine/VolumeTexture.h"
#include "Engine/Engine.h"
#include "RenderTargetPool.h"

static TAutoConsoleVariable<int32> CVarShadingEnergyConservation(
	TEXT("r.Shading.EnergyConservation"),
	1,
	TEXT("0 to disable energy conservation on shading models.\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadingEnergyConservation_Preservation(
	TEXT("r.Shading.EnergyPreservation"),
	1,
	TEXT("0 to disable energy preservation on shading models, i.e. the energy attenuation on diffuse lighting caused by the specular reflection. Require energy conservation to be enabled\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadingFurnaceTest(
	TEXT("r.Shading.FurnaceTest"),
	0,
	TEXT("Enable/disable furnace for shading validation."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadingFurnaceTest_EnableForProject(
	TEXT("r.Shading.FurnaceTest.EnableForProject"),
	0,
	TEXT("Enable FurnaceTest rendering and shader code."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadingFurnaceTest_SampleCount(
	TEXT("r.Shading.FurnaceTest.SampleCount"),
	64,
	TEXT("Number of sampler per pixel used for furnace tests."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadingEnergyConservation_TableFormat(
	TEXT("r.Shading.EnergyConservation.Format"),
	1,
	TEXT("Energy conservation table format 0: 16bits, 1: 32bits."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadingEnergyConservation_TableResolution(
	TEXT("r.Shading.EnergyConservation.Resolution"),
	32,
	TEXT("Energy conservation table resolution. Used only when using runtime generated tables."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadingEnergyConservation_RuntimeGeneration(
	TEXT("r.Shading.EnergyConservation.RuntimeGeneration"),
	0,
	TEXT("Enable Energy conservation tables generation at runtime instead of relying on precomputed tables."),
	ECVF_RenderThreadSafe);

// Transition render settings that will disapear when Substrate gets enabled

static TAutoConsoleVariable<int32> CVarMaterialEnergyConservation(
	TEXT("r.Material.EnergyConservation"),
	0,
	TEXT("Enable energy conservation for legacy materials (project settings, read only). Please note that when Substrate is enabled, energy conservation is forced to enabled."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

#define SHADING_ENERGY_CONSERVATION_TABLE_RESOLUTION 32

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FShadingEnergyConservationSettings
{
	bool bIsEnergyConservationEnabled = false;
	bool bIsEnergyPreservationEnabled = false;
	bool bNeedData = false;
};

static FShadingEnergyConservationSettings GetSettings(const FViewInfo& View)
{
	FShadingEnergyConservationSettings Out;

	// Enabled based on settings
	const bool bMaterialEnergyConservationEnabled = CVarMaterialEnergyConservation.GetValueOnRenderThread() > 0;
	Out.bIsEnergyConservationEnabled = CVarShadingEnergyConservation.GetValueOnRenderThread() > 0;
	Out.bIsEnergyPreservationEnabled = CVarShadingEnergyConservation_Preservation.GetValueOnRenderThread() > 0;

	// Build/bind table if energy conservation is enabled or if Substrate is enabled in order to have 
	// the correct tables built & bound. Even if we are not using energy conservation, we want to 
	// have access to directional albedo information for env. lighting for instance)
	Out.bNeedData = (bMaterialEnergyConservationEnabled || Substrate::IsSubstrateEnabled() || (View.Family->EngineShowFlags.PathTracing)) && (Out.bIsEnergyPreservationEnabled || Out.bIsEnergyConservationEnabled);

	return Out;
}

class FShadingEnergyConservationResources : public FRenderResource
{
public:
	FShadingEnergyConservationResources() : FRenderResource()
	{}

	virtual void ReleaseRHI()
	{
		GGXSpecEnergyTexture.SafeRelease();
		GGXGlassEnergyTexture.SafeRelease();
		ClothEnergyTexture.SafeRelease();
		DiffuseEnergyTexture.SafeRelease();
		GGXMaxSpecEnergyTexture.SafeRelease();
	}

	EPixelFormat Format = PF_Unknown;
	TRefCountPtr<IPooledRenderTarget> GGXSpecEnergyTexture = nullptr;
	TRefCountPtr<IPooledRenderTarget> GGXGlassEnergyTexture = nullptr;
	TRefCountPtr<IPooledRenderTarget> ClothEnergyTexture = nullptr;
	TRefCountPtr<IPooledRenderTarget> DiffuseEnergyTexture = nullptr;
	TRefCountPtr<IPooledRenderTarget> GGXMaxSpecEnergyTexture = nullptr;
};

/** The global energy conservation data.textures used for scene rendering. */
TGlobalResource<FShadingEnergyConservationResources> GShadingEnergyConservationResources;

namespace ShadingEnergyConservationData
{
	static bool IsTextureDataValid(UTexture2D* In)
	{
		return In && In->GetPlatformData() && In->GetCPUCopy();
	}

	static TRefCountPtr<IPooledRenderTarget> CreateTexture2D(FRHICommandListImmediate& RHICmdList, TObjectPtr<class UTexture2D>& InCPUTexture, EPixelFormat InFormat, const TCHAR* InName)
	{
		if (!IsTextureDataValid(InCPUTexture))
		{
			return nullptr;
		}

		check(InCPUTexture->Availability == ETextureAvailability::CPU);
		FSharedImageConstRef Data = InCPUTexture->GetCPUCopy();
		check(Data && Data->Format == ERawImageFormat::RGBA16F);
		const TArrayView64<const FFloat16Color> DataView = Data->AsRGBA16F();

		const FIntPoint DataSize(Data->SizeX, Data->SizeY);
		TRefCountPtr<IPooledRenderTarget> OutTexture = GRenderTargetPool.FindFreeElement(FRDGTextureDesc::Create2D(DataSize, InFormat, FClearValueBinding::None, TexCreate_ShaderResource), InName);

		FTextureRHIRef Texture = OutTexture->GetRHI();

		check(InFormat == PF_A16B16G16R16 || InFormat == PF_R8G8B8A8 || InFormat == PF_G16R16 || InFormat == PF_R8G8 || InFormat == PF_G16 || InFormat == PF_R8);
		const uint32 ComponentCount = (InFormat == PF_A16B16G16R16 || InFormat == PF_R8G8B8A8) ? 4u : ((InFormat == PF_G16R16 || InFormat == PF_R8G8) ? 2u: 1u);
		const bool b8bit = InFormat == PF_R8G8B8A8 || InFormat == PF_R8G8 || InFormat == PF_R8;
		const uint32 DstBytesPerPixel = (b8bit? 1u : 2u) * ComponentCount;

		// Write the contents of the texture with transcoding
		uint32 DestStride;
		uint8* DestBuffer = (uint8*)RHICmdList.LockTexture2D(Texture, 0, RLM_WriteOnly, DestStride, false);
		for (int32 y = 0; y < DataSize.Y; ++y)
		{
			for (int32 x = 0; x < DataSize.X; ++x)
			{		
				const FFloat16Color Value = DataView[x + y * DataSize.X];
				if (b8bit)
				{
					uint8* Dest  = (uint8*)(DestBuffer + x * DstBytesPerPixel + y * DestStride);
					{
						Dest[0] = uint8(FMath::Clamp(float(Value.R) * 0xFF, 0u, 0xFF));
					}
					if (ComponentCount > 1)
					{
						Dest[1] = uint8(FMath::Clamp(float(Value.G) * 0xFF, 0u, 0xFF));
					}
					if (ComponentCount > 2)
					{
						Dest[2] = uint8(FMath::Clamp(float(Value.B) * 0xFF, 0u, 0xFF));
						Dest[3] = uint8(FMath::Clamp(float(Value.A) * 0xFF, 0u, 0xFF));
					}
				}
				else
				{
					uint16* Dest = (uint16*)(DestBuffer + x * DstBytesPerPixel + y * DestStride);
					{
						Dest[0] = uint16(FMath::Clamp(float(Value.R) * 0xFFFF, 0u, 0xFFFF));
					}
					if (ComponentCount > 1)
					{
						Dest[1] = uint16(FMath::Clamp(float(Value.G) * 0xFFFF, 0u, 0xFFFF));
					}
					if (ComponentCount > 2)
					{
						Dest[2] = uint16(FMath::Clamp(float(Value.B) * 0xFFFF, 0u, 0xFFFF));
						Dest[3] = uint16(FMath::Clamp(float(Value.A) * 0xFFFF, 0u, 0xFFFF));
					}
				}
			}
		}
		RHICmdList.UnlockTexture2D(Texture, 0, false);

		// Release CPU data which are no longer needed
		#if !WITH_EDITORONLY_DATA
		InCPUTexture->RemoveFromRoot();
		InCPUTexture = nullptr;
		#endif

		return OutTexture;
	}

	static TRefCountPtr<IPooledRenderTarget> CreateTexture3D(FRHICommandListImmediate& RHICmdList, TObjectPtr<class UTexture2D>& InCPUTexture, EPixelFormat InFormat, const TCHAR* InName)
	{
		if (!IsTextureDataValid(InCPUTexture))
		{
			return nullptr;
		}

		check(InCPUTexture->Availability == ETextureAvailability::CPU);
		FSharedImageConstRef Data = InCPUTexture->GetCPUCopy();
		#if 0
		typedef FFloat16Color TColorType;
		check(Data && Data->Format == ERawImageFormat::RGBA16F);
		const TArrayView64<const FFloat16Color> DataView = Data->AsRGBA16F();
		#else
		typedef FLinearColor TColorType;
		check(Data && Data->Format == ERawImageFormat::RGBA32F);
		const TArrayView64<const FLinearColor> DataView = Data->AsRGBA32F();
		#endif

		// Stored as an array of 2D slices
		const FIntVector DataSize(Data->SizeX, Data->SizeX, Data->SizeY / Data->SizeX);
		TRefCountPtr<IPooledRenderTarget> OutTexture = GRenderTargetPool.FindFreeElement(FRDGTextureDesc::Create3D(DataSize, InFormat, FClearValueBinding::None, TexCreate_ShaderResource), InName);

		FTextureRHIRef Texture = OutTexture->GetRHI();

		check(InFormat == PF_G16R16 || InFormat == PF_R8G8 || InFormat == PF_G16 || InFormat == PF_R8);
		const uint32 ComponentCount = InFormat == PF_G16R16 || InFormat == PF_R8G8 ? 2u: 1u;
		const bool b8bit = InFormat == PF_R8G8 || InFormat == PF_R8;
		const uint32 DstBytesPerPixel = (b8bit? 1u : 2u) * ComponentCount;

		// Transcoded data before uploading data to the GPU
		TArray<uint8> TranscodedData;
		TranscodedData.SetNum(DataSize.X * DataSize.Y * DataSize.Z * DstBytesPerPixel);
		uint32 DestStrideY = DataSize.X * DstBytesPerPixel;
		uint32 DestStrideZ = DataSize.X * DataSize.Y * DstBytesPerPixel;
		uint8* DestBuffer = (uint8*)TranscodedData.GetData();
		for (int32 z = 0; z < DataSize.Z; ++z)
		{
			for (int32 y = 0; y < DataSize.Y; ++y)
			{
				for (int32 x = 0; x < DataSize.X; ++x)
				{		
					const TColorType Value = DataView[x + y * DataSize.X + z * DataSize.X * DataSize.Y];
					if (b8bit)
					{
						uint8* Dest  = (uint8*)(DestBuffer + x * DstBytesPerPixel + y * DestStrideY + z * DestStrideZ);
						{
							Dest[0] = uint8(FMath::Clamp(float(Value.R) * 0xFF, 0u, 0xFF));
						}
						if (ComponentCount > 1)
						{
							Dest[1] = uint8(FMath::Clamp(float(Value.G) * 0xFF, 0u, 0xFF));
						}
					}
					else
					{
						uint16* Dest = (uint16*)(DestBuffer + x * DstBytesPerPixel + y * DestStrideY + z * DestStrideZ);
						{
							Dest[0] = uint16(FMath::Clamp(float(Value.R) * 0xFFFF, 0u, 0xFFFF));
						}
						if (ComponentCount > 1)
						{
							Dest[1] = uint16(FMath::Clamp(float(Value.G) * 0xFFFF, 0u, 0xFFFF));
						}
					}
				}
			}
		}

		FUpdateTextureRegion3D Region(0, 0, 0, 0, 0, 0, DataSize.X, DataSize.Y, DataSize.Z);
		RHICmdList.UpdateTexture3D(Texture, 0, Region, DataSize.X * DstBytesPerPixel, DataSize.X * DataSize.Y * DstBytesPerPixel, TranscodedData.GetData());

		// UpdateTexture3D before and after state is currently undefined
		RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::Unknown, ERHIAccess::SRVMask));

		// Release CPU data which are no longer needed
		#if !WITH_EDITORONLY_DATA
		InCPUTexture->RemoveFromRoot();
		InCPUTexture = nullptr;
		#endif

		return OutTexture;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FShadingFurnaceTestPassPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FShadingFurnaceTestPassPS);
	SHADER_USE_PARAMETER_STRUCT(FShadingFurnaceTestPassPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER(uint32, NumSamplesPerSet)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool IsSupported(const EShaderPlatform& InPlaform)
	{
		return CVarShadingFurnaceTest_EnableForProject.GetValueOnAnyThread() && GetMaxSupportedFeatureLevel(InPlaform) >= ERHIFeatureLevel::SM5;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsSupported(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_FURNACE_ANALYTIC"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FShadingFurnaceTestPassPS, "/Engine/Private/ShadingFurnaceTest.usf", "MainPS", SF_Pixel);

static void AddShadingFurnacePass(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View, 
	TRDGUniformBufferRef<FSceneTextureUniformParameters>& SceneTexturesUniformBuffer,
	FRDGTextureRef OutTexture)
{
	if (!FShadingFurnaceTestPassPS::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	TShaderMapRef<FShadingFurnaceTestPassPS> PixelShader(View.ShaderMap);
	FShadingFurnaceTestPassPS::FParameters* Parameters = GraphBuilder.AllocParameters<FShadingFurnaceTestPassPS::FParameters>();
	Parameters->ViewUniformBuffer				= View.ViewUniformBuffer;
	Parameters->SceneTexturesStruct				= SceneTexturesUniformBuffer;
	Parameters->NumSamplesPerSet				= FMath::Clamp(CVarShadingFurnaceTest_SampleCount.GetValueOnAnyThread(), 16, 2048);
	Parameters->RenderTargets[0]				= FRenderTargetBinding(OutTexture, ERenderTargetLoadAction::ELoad);
	if (Substrate::IsSubstrateEnabled())
	{
		Parameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	}

	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintUniformBuffer);

	FPixelShaderUtils::AddFullscreenPass<FShadingFurnaceTestPassPS>(
		GraphBuilder,
		View.ShaderMap,
		RDG_EVENT_NAME("ShadingEnergyConservation::FurnaceTest"),
		PixelShader,
		Parameters,
		View.ViewRect);
}

////////////////////////////////////////////////////////////////////////////////////////////////

class FBuildShadingEnergyConservationTableCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBuildShadingEnergyConservationTableCS)
	SHADER_USE_PARAMETER_STRUCT(FBuildShadingEnergyConservationTableCS, FGlobalShader)

	enum class EEnergyTableType 
	{
		GGXSpecular = 0,
		GGXGlass = 1,
		Cloth = 2,
		Diffuse = 3,
		GGXMaxSpecular = 4,
		MAX
	};

	class FEnergyTableDim : SHADER_PERMUTATION_ENUM_CLASS("BUILD_ENERGY_TABLE", EEnergyTableType);

	using FPermutationDomain = TShaderPermutationDomain<FEnergyTableDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumSamples)
		SHADER_PARAMETER(uint32, EnergyTableResolution)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, Output1Texture2D)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, Output2Texture2D)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, Output3Texture2D)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, OutputTexture3D)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_SHADER_TYPE(, FBuildShadingEnergyConservationTableCS, TEXT("/Engine/Private/ShadingEnergyConservationTable.usf"), TEXT("BuildEnergyTableCS"), SF_Compute);

////////////////////////////////////////////////////////////////////////////////////////////////

namespace ShadingEnergyConservation
{

void Init(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& Views)
{
	FShadingEnergyConservationResources& Out = GShadingEnergyConservationResources;

	// Change this to true in order to regenerate the energy tables, and manually copy the coefficients into ShadingEnergyConservationData.h
	const bool bRuntimeGeneration = CVarShadingEnergyConservation_RuntimeGeneration.GetValueOnRenderThread() > 0;
	const int Size2D = FMath::Clamp(CVarShadingEnergyConservation_TableResolution.GetValueOnRenderThread(), 16, 512);
	const int Size2DMaxLUT = 32; // Must match GGX_MAX_LUT_RESOLUTION

	bool bNeedEnergyConservationLutData = false;
	for (auto& View : Views)
	{
		bNeedEnergyConservationLutData |= GetSettings(View).bNeedData;
		if (bNeedEnergyConservationLutData)
		{
			break;
		}
	}

	const bool bR16Supported = GPixelFormats[PF_G16].Supported && UE::PixelFormat::HasCapabilities(PF_G16, EPixelFormatCapabilities::TextureFilterable);
	const bool bRG16Supported = GPixelFormats[PF_G16R16].Supported && UE::PixelFormat::HasCapabilities(PF_G16R16, EPixelFormatCapabilities::TextureFilterable);
	const bool bRGBA16Supported = GPixelFormats[PF_A16B16G16R16].Supported && UE::PixelFormat::HasCapabilities(PF_A16B16G16R16, EPixelFormatCapabilities::TextureFilterable);

	const EPixelFormat SpecFormat = bRuntimeGeneration && CVarShadingEnergyConservation_TableFormat.GetValueOnRenderThread() > 0 ? PF_G32R32F : (bRG16Supported ? PF_G16R16 : PF_R8G8);
	const EPixelFormat MaxSpecFormat = bRuntimeGeneration ? PF_FloatRGBA : (bRGBA16Supported ? PF_A16B16G16R16 : PF_R8G8B8A8);
	const EPixelFormat DiffFormat = bR16Supported ? PF_G16 : PF_R8;

	const bool bBuildTable =
		Out.GGXMaxSpecEnergyTexture == nullptr ||	// This texture is always loaded on GPU as it needs to always be available for Toon shading.
		bRuntimeGeneration ||						// A developer is running runtime generation
		(bNeedEnergyConservationLutData && 
			(
			Out.Format != SpecFormat ||
			Out.GGXSpecEnergyTexture == nullptr ||
			Out.GGXGlassEnergyTexture == nullptr ||
			Out.ClothEnergyTexture == nullptr ||
			Out.DiffuseEnergyTexture == nullptr ||
			Out.GGXMaxSpecEnergyTexture == nullptr ||
			Out.GGXSpecEnergyTexture->GetDesc().Extent.X != Size2D
			)
		);

	{
		if (bBuildTable)
		{
			Out.Format = SpecFormat;

			if (bRuntimeGeneration)
			{
				const int Size3D = SHADING_ENERGY_CONSERVATION_TABLE_RESOLUTION;
				FRDGTextureRef GGXSpecEnergyTexture		= GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(Size2D, Size2D),         SpecFormat,    FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV), TEXT("Shading.GGXSpecEnergy"),   ERDGTextureFlags::MultiFrame);
				FRDGTextureRef GGXGlassEnergyTexture	= GraphBuilder.CreateTexture(FRDGTextureDesc::Create3D(FIntVector(Size3D, Size3D, Size3D),SpecFormat,    FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV), TEXT("Shading.GGXGlassEnergy"),  ERDGTextureFlags::MultiFrame);
				FRDGTextureRef ClothEnergyTexture		= GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(Size2D, Size2D),         SpecFormat,    FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV), TEXT("Shading.ClothSpecEnergy"), ERDGTextureFlags::MultiFrame);
				FRDGTextureRef DiffuseEnergyTexture		= GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(Size2D, Size2D),         DiffFormat,    FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV), TEXT("Shading.DiffuseEnergy"),   ERDGTextureFlags::MultiFrame);
				FRDGTextureRef GGXMaxSpecEnergyTexture	= GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(Size2D, Size2D),         MaxSpecFormat, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV), TEXT("Shading.GGXMaxSpecEnergy"),ERDGTextureFlags::MultiFrame);
			
				const uint32 NumSamples = 1u << 14u; 
				auto GlobalShaderMap = GetGlobalShaderMap(GMaxRHIShaderPlatform);

				// GGX
				{
					FBuildShadingEnergyConservationTableCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FBuildShadingEnergyConservationTableCS::FEnergyTableDim>(FBuildShadingEnergyConservationTableCS::EEnergyTableType::GGXSpecular);
					TShaderMapRef<FBuildShadingEnergyConservationTableCS> ComputeShader(GlobalShaderMap, PermutationVector);
					FBuildShadingEnergyConservationTableCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildShadingEnergyConservationTableCS::FParameters>();
					PassParameters->NumSamples = NumSamples;
					PassParameters->EnergyTableResolution = Size2D;
					PassParameters->Output2Texture2D = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(GGXSpecEnergyTexture, 0));
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("ShadingEnergyConservation::BuildTable(GGXSpec)"),
						ComputeShader,
						PassParameters,
						FComputeShaderUtils::GetGroupCount(FIntPoint(Size2D, Size2D), FComputeShaderUtils::kGolden2DGroupSize));
				}

				// GGX (Reflection + Transmission) indexed by IOR
				{
					FBuildShadingEnergyConservationTableCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FBuildShadingEnergyConservationTableCS::FEnergyTableDim>(FBuildShadingEnergyConservationTableCS::EEnergyTableType::GGXGlass);
					TShaderMapRef<FBuildShadingEnergyConservationTableCS> ComputeShader(GlobalShaderMap, PermutationVector);
					FBuildShadingEnergyConservationTableCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildShadingEnergyConservationTableCS::FParameters>();
					PassParameters->NumSamples = NumSamples;
					PassParameters->EnergyTableResolution = Size3D;
					PassParameters->OutputTexture3D = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(GGXGlassEnergyTexture, 0));
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("ShadingEnergyConservation::BuildTable(GGXGlass)"),
						ComputeShader,
						PassParameters,
						FComputeShaderUtils::GetGroupCount(FIntVector(Size3D, Size3D, Size3D), FIntVector(FComputeShaderUtils::kGolden2DGroupSize, FComputeShaderUtils::kGolden2DGroupSize, 1)));
				}

				// Cloth
				{
					FBuildShadingEnergyConservationTableCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FBuildShadingEnergyConservationTableCS::FEnergyTableDim>(FBuildShadingEnergyConservationTableCS::EEnergyTableType::Cloth);
					TShaderMapRef<FBuildShadingEnergyConservationTableCS> ComputeShader(GlobalShaderMap, PermutationVector);
					FBuildShadingEnergyConservationTableCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildShadingEnergyConservationTableCS::FParameters>();
					PassParameters->NumSamples = NumSamples;
					PassParameters->EnergyTableResolution = Size2D;
					PassParameters->Output2Texture2D = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(ClothEnergyTexture, 0));
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("ShadingEnergyConservation::BuildTable(Cloth)"),
						ComputeShader,
						PassParameters,
						FComputeShaderUtils::GetGroupCount(FIntPoint(Size2D, Size2D), FComputeShaderUtils::kGolden2DGroupSize));
				}

				// Diffuse
				{
					FBuildShadingEnergyConservationTableCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FBuildShadingEnergyConservationTableCS::FEnergyTableDim>(FBuildShadingEnergyConservationTableCS::EEnergyTableType::Diffuse);
					TShaderMapRef<FBuildShadingEnergyConservationTableCS> ComputeShader(GlobalShaderMap, PermutationVector);
					FBuildShadingEnergyConservationTableCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildShadingEnergyConservationTableCS::FParameters>();
					PassParameters->NumSamples = NumSamples;
					PassParameters->EnergyTableResolution = Size2D;
					PassParameters->Output1Texture2D = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(DiffuseEnergyTexture, 0));
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("ShadingEnergyConservation::BuildTable(Diffuse)"),
						ComputeShader,
						PassParameters,
						FComputeShaderUtils::GetGroupCount(FIntPoint(Size2D, Size2D), FComputeShaderUtils::kGolden2DGroupSize));
				}

				// Max of GGX
				{
					FBuildShadingEnergyConservationTableCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FBuildShadingEnergyConservationTableCS::FEnergyTableDim>(FBuildShadingEnergyConservationTableCS::EEnergyTableType::GGXMaxSpecular);
					TShaderMapRef<FBuildShadingEnergyConservationTableCS> ComputeShader(GlobalShaderMap, PermutationVector);
					FBuildShadingEnergyConservationTableCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildShadingEnergyConservationTableCS::FParameters>();
					PassParameters->NumSamples = NumSamples;
					PassParameters->EnergyTableResolution = Size2DMaxLUT;
					PassParameters->Output3Texture2D = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(GGXMaxSpecEnergyTexture, 0));
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("ShadingEnergyConservation::BuildTable(GGXMaxSpec)"),
						ComputeShader,
						PassParameters,
						FComputeShaderUtils::GetGroupCount(FIntPoint(Size2DMaxLUT, Size2DMaxLUT), FComputeShaderUtils::kGolden2DGroupSize));
				}

				Out.GGXSpecEnergyTexture  = GraphBuilder.ConvertToExternalTexture(GGXSpecEnergyTexture);
				Out.GGXGlassEnergyTexture = GraphBuilder.ConvertToExternalTexture(GGXGlassEnergyTexture);
				Out.ClothEnergyTexture    = GraphBuilder.ConvertToExternalTexture(ClothEnergyTexture);
				Out.DiffuseEnergyTexture  = GraphBuilder.ConvertToExternalTexture(DiffuseEnergyTexture);
				Out.GGXMaxSpecEnergyTexture = GraphBuilder.ConvertToExternalTexture(GGXMaxSpecEnergyTexture);
			}
			else
			{
				// Precomputed data are stored as float16
				check(SpecFormat == PF_G16R16 || SpecFormat == PF_R8G8);
				check(DiffFormat == PF_G16 || DiffFormat == PF_R8);
				check(MaxSpecFormat == PF_A16B16G16R16 || MaxSpecFormat == PF_R8G8B8A8);

				if (bNeedEnergyConservationLutData)
				{
					Out.GGXSpecEnergyTexture    = ShadingEnergyConservationData::CreateTexture2D(GraphBuilder.RHICmdList, GEngine->GGXReflectionEnergyTexture,   SpecFormat,    TEXT("Shading.GGXReflectionEnergy"));
					Out.GGXGlassEnergyTexture   = ShadingEnergyConservationData::CreateTexture3D(GraphBuilder.RHICmdList, GEngine->GGXTransmissionEnergyTexture, SpecFormat,    TEXT("Shading.GGXTransmissionEnergy"));
					Out.ClothEnergyTexture      = ShadingEnergyConservationData::CreateTexture2D(GraphBuilder.RHICmdList, GEngine->SheenEnergyTexture,           SpecFormat,    TEXT("Shading.SheenEnergy"));
					Out.DiffuseEnergyTexture    = ShadingEnergyConservationData::CreateTexture2D(GraphBuilder.RHICmdList, GEngine->DiffuseEnergyTexture,         DiffFormat,    TEXT("Shading.DiffuseEnergy"));
				}

				// LUTs that are always loaded.
				Out.GGXMaxSpecEnergyTexture = ShadingEnergyConservationData::CreateTexture2D(GraphBuilder.RHICmdList, GEngine->GGXMaxSpecEnergyTexture,      MaxSpecFormat, TEXT("Shading.GGXMaxSpecEnergyTexture"));

				// Fallback
				if (!Out.GGXSpecEnergyTexture )   { Out.GGXSpecEnergyTexture    = GSystemTextures.BlackDummy; }
				if (!Out.GGXGlassEnergyTexture)   { Out.GGXGlassEnergyTexture   = GSystemTextures.VolumetricBlackDummy; }
				if (!Out.ClothEnergyTexture   )   { Out.ClothEnergyTexture      = GSystemTextures.BlackDummy; }
				if (!Out.DiffuseEnergyTexture )   { Out.DiffuseEnergyTexture    = GSystemTextures.BlackDummy; }
				if (!Out.GGXMaxSpecEnergyTexture) { Out.GGXMaxSpecEnergyTexture = GSystemTextures.WhiteDummy; }
			}
		}
	}
}

FShadingEnergyConservationData GetData(const FViewInfo& View)
{
	const FShadingEnergyConservationSettings Settings = GetSettings(View);

	FShadingEnergyConservationData Out;
	Out.bEnergyConservation     = Settings.bIsEnergyConservationEnabled;
	Out.bEnergyPreservation     = Settings.bIsEnergyPreservationEnabled;
	Out.GGXSpecEnergyTexture    = GShadingEnergyConservationResources.GGXSpecEnergyTexture;
	Out.GGXGlassEnergyTexture   = GShadingEnergyConservationResources.GGXGlassEnergyTexture;
	Out.ClothEnergyTexture      = GShadingEnergyConservationResources.ClothEnergyTexture;
	Out.DiffuseEnergyTexture    = GShadingEnergyConservationResources.DiffuseEnergyTexture;
	Out.GGXMaxSpecEnergyTexture = GShadingEnergyConservationResources.GGXMaxSpecEnergyTexture;
	return Out;
}

void Debug(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSceneTextures& SceneTextures)
{
	if (CVarShadingFurnaceTest.GetValueOnAnyThread() > 0)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "ShadingEnergyConservation::FurnaceTest");
		AddShadingFurnacePass(GraphBuilder, View, SceneTextures.UniformBuffer, SceneTextures.Color.Target);
	}
}

}