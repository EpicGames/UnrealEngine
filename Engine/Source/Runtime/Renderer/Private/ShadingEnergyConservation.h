// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadingEnergyConservation.h: private energy conservation related data
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"

class FViewInfo;

struct FShadingEnergyConservationData
{
	bool bEnergyConservation = false;
	bool bEnergyPreservation = false;

	TRefCountPtr<IPooledRenderTarget> GGXSpecEnergyTexture = nullptr;
	TRefCountPtr<IPooledRenderTarget> GGXGlassEnergyTexture = nullptr;
	TRefCountPtr<IPooledRenderTarget> ClothEnergyTexture = nullptr;
	TRefCountPtr<IPooledRenderTarget> DiffuseEnergyTexture = nullptr;
	TRefCountPtr<IPooledRenderTarget> GGXMaxSpecEnergyTexture = nullptr;
};

namespace ShadingEnergyConservation
{
	void Init(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& Views);
	void Debug(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSceneTextures& SceneTextures);
	FShadingEnergyConservationData GetData(const FViewInfo& View);
}
