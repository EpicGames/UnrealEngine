// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowDepthRendering.cpp: Shadow depth rendering implementation
=============================================================================*/

#include "CoreMinimal.h"
#include "ClearQuad.h"
#include "CommonRenderResources.h"
#include "Containers/ArrayView.h"
#include "Containers/SparseSet.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DecalRenderingShared.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "GlobalShader.h"
#include "GPUScene.h"
#include "HAL/IConsoleManager.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "LightRendering.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "Logging/LogMacros.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "MeshBatch.h"
#include "MeshDrawCommands.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "Misc/LargeWorldCoordinates.h"
#include "MultiGPU.h"
#include "Nanite/Nanite.h"
#include "Nanite/NaniteCullRaster.h"
#include "Nanite/NaniteShared.h"
#include "NaniteSceneProxy.h"
#include "PipelineStateCache.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveViewRelevance.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "PSOPrecache.h"
#include "PSOPrecacheMaterial.h"
#include "ReadOnlyCVARCache.h"
#include "RenderCore.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/NaniteStreamingManager.h"
#include "RenderResource.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIContext.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneExtensions.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextureReductions.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneTypes.h"
#include "SceneUniformBuffer.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "ScreenRendering.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderPlatformCachedIniValue.h"
#include "ShadowRendering.h"
#include "Shadows/ShadowSceneRenderer.h"
#include "Shadows/CSMSceneExtension.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "SystemTextures.h"
#include "Tasks/Pipe.h"
#include "Tasks/Task.h"
#include "UnrealEngine.h"
#include "VertexFactory.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"

static float GShadowLODDistanceFactor = 1;
static FAutoConsoleVariableRef CVarShadowScreenMultiple(
	TEXT("r.Shadow.LODDistanceFactor"),
	GShadowLODDistanceFactor,
	TEXT("Multiplier for LOD selection distance when rendering regular shadows (global)"),
	ECVF_RenderThreadSafe);

static float GShadowLODDistanceFactorCascadeScale = 0.0f;
static FAutoConsoleVariableRef CVarShadowScreenMultipleCascadeScale(
	TEXT("r.Shadow.LODDistanceFactor.CascadeScale"),
	GShadowLODDistanceFactorCascadeScale,
	TEXT("Multiplier for LOD selection distance when rendering regular shadows (scales with cascade index)"),
	ECVF_RenderThreadSafe);

static float GFarShadowLODDistanceFactor = 1;
static FAutoConsoleVariableRef CVarFarShadowScreenMultiple(
	TEXT("r.Shadow.FarShadow.LODDistanceFactor"),
	GFarShadowLODDistanceFactor,
	TEXT("Multiplier for LOD selection distance when rendering far shadows"),
	ECVF_RenderThreadSafe);

static bool GShadowOnePassPointLightUseCachedMDCs = true;
static FAutoConsoleVariableRef CVarShadowOnePassPointLightUseCachedMDCs(
	TEXT("r.Shadow.OnePassPointLight.UseCachedMDCs"),
	GShadowOnePassPointLightUseCachedMDCs,
	TEXT("Controls whether to use a dedicated cached mesh draw command pass for one pass point lights. This increases the cost of caching / storing mesh draw commands but improves the CPU rendering time of point lights."),
	ECVF_ReadOnly);

DEFINE_GPU_DRAWCALL_STAT(ShadowDepths);

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FShadowDepthPassUniformParameters, "ShadowDepthPass", SceneTextures);
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FMobileShadowDepthPassUniformParameters, "MobileShadowDepthPass", SceneTextures);

template<bool bUsingVertexLayers = false>
class TScreenVSForGS : public FScreenVS
{
public:
	DECLARE_SHADER_TYPE(TScreenVSForGS, Global);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (IsMobilePlatform(Parameters.Platform))
		{
			return bUsingVertexLayers;
		}
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FScreenVS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USING_LAYERS"), (uint32)(bUsingVertexLayers ? 1 : 0));
		if (!bUsingVertexLayers)
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_VertexToGeometryShader);
		}
	}

	TScreenVSForGS() = default;
	TScreenVSForGS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FScreenVS(Initializer)
	{}
};

IMPLEMENT_SHADER_TYPE(template<>, TScreenVSForGS<false>, TEXT("/Engine/Private/ScreenVertexShader.usf"), TEXT("MainForGS"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(template<>, TScreenVSForGS<true>, TEXT("/Engine/Private/ScreenVertexShader.usf"), TEXT("MainForGS"), SF_Vertex);


static TAutoConsoleVariable<int32> CVarShouldBeginDeferredCullingAfterShadowRendering(
	TEXT("r.Shadow.ShouldBeginDeferredCullingAfterShadowRendering"),
	1,
	TEXT("If turned on (default), a new deferred scope of instance culling will be started directly after shadow depth rendering (iff instance occlusion culling is enabled).\n")\
	TEXT("  This has a mild performance benefit on some platforms."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarNaniteShadows(
	TEXT("r.Shadow.Nanite"),
	1,
	TEXT("Enables shadows from Nanite meshes."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarNaniteShadowsUseHZB(
	TEXT("r.Shadow.NaniteUseHZB"),
	1,
	TEXT("Enables HZB for Nanite shadows."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarNaniteShadowsUpdateStreaming(
	TEXT("r.Shadow.NaniteUpdateStreaming"),
	1,
	TEXT("Produce Nanite geometry streaming requests from shadow map rendering."),
	ECVF_RenderThreadSafe);

extern int32 GNaniteShowStats;

namespace Nanite
{
	extern bool IsStatFilterActive(const FString& FilterName);
	extern bool IsStatFilterActiveForLight(const FLightSceneProxy* LightProxy);
	extern FString GetFilterNameForLight(const FLightSceneProxy* LightProxy);
}

void SetupShadowDepthPassUniformBuffer(
	const FProjectedShadowInfo* ShadowInfo,
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FShadowDepthPassUniformParameters& ShadowDepthPassParameters)
{
	static const auto CSMCachingCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Shadow.CSMCaching"));
	const bool bCSMCachingEnabled = CSMCachingCVar && CSMCachingCVar->GetValueOnAnyThread() != 0;

	SetupSceneTextureUniformParameters(GraphBuilder, View.GetSceneTexturesChecked(), View.FeatureLevel, ESceneTextureSetupMode::None, ShadowDepthPassParameters.SceneTextures);

	ShadowDepthPassParameters.ProjectionMatrix = FTranslationMatrix44f(FVector3f(ShadowInfo->PreShadowTranslation - View.ViewMatrices.GetPreViewTranslation())) * ShadowInfo->TranslatedWorldToClipOuterMatrix;		// LWC_TDOO: Precision loss?
	ShadowDepthPassParameters.ViewMatrix = FMatrix44f(ShadowInfo->TranslatedWorldToView);	// LWC_TODO: Precision loss

	// Disable the SlopeDepthBias because we couldn't reconstruct the depth offset if it is not 0.0f when scrolling the cached shadow map.
	ShadowDepthPassParameters.ShadowParams = FVector4f(ShadowInfo->GetShaderDepthBias(), bCSMCachingEnabled ? 0.0f : ShadowInfo->GetShaderSlopeDepthBias(), ShadowInfo->GetShaderMaxSlopeDepthBias(), ShadowInfo->bOnePassPointLightShadow ? 1 : ShadowInfo->InvMaxSubjectDepth);
	ShadowDepthPassParameters.bClampToNearPlane = ShadowInfo->ShouldClampToNearPlane() ? 1.0f : 0.0f;

	ShadowDepthPassParameters.ShadowDepthType = (uint32)ShadowInfo->GetShadowDepthType();

	if (ShadowInfo->bOnePassPointLightShadow)
	{
		check(ShadowInfo->BorderSize == 0);

		// offset from translated world space to (pre translated) shadow space 
		const FMatrix Translation = FTranslationMatrix(ShadowInfo->PreShadowTranslation - View.ViewMatrices.GetPreViewTranslation());

		for (int32 FaceIndex = 0; FaceIndex < 6; FaceIndex++)
		{
			ShadowDepthPassParameters.ShadowViewProjectionMatrices[FaceIndex] = FMatrix44f(Translation * ShadowInfo->OnePassShadowViewProjectionMatrices[FaceIndex]);		// LWC_TODO: Precision loss?
			ShadowDepthPassParameters.ShadowViewMatrices[FaceIndex] = FMatrix44f(Translation * ShadowInfo->OnePassShadowViewMatrices[FaceIndex]);
		}
	}

	// Reference dummy data in the UB initially
	const uint32 DummyPageTableElement = 0xFFFFFFFF;
	ShadowDepthPassParameters.VirtualSmPageTable = GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R32_UINT, DummyPageTableElement);

	ShadowDepthPassParameters.PackedNaniteViews = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(Nanite::FPackedView)));
	ShadowDepthPassParameters.AllocatedPageRectBounds = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FIntVector4)));
	ShadowDepthPassParameters.UncachedPageRectBounds = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FIntVector4)));

	FRDGTextureRef DepthBufferArray = GraphBuilder.CreateTexture( FRDGTextureDesc::Create2DArray( FIntPoint(4,4), PF_R32_UINT, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV, 1 ), TEXT("Dummy-OutDepthBuffer") );
	
	ShadowDepthPassParameters.OutDepthBufferArray = GraphBuilder.CreateUAV( DepthBufferArray );
}

void SetupShadowDepthPassUniformBuffer(
	const FProjectedShadowInfo* ShadowInfo,
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FMobileShadowDepthPassUniformParameters& ShadowDepthPassParameters)
{
	SetupMobileSceneTextureUniformParameters(GraphBuilder, View.GetSceneTexturesChecked(), EMobileSceneTextureSetupMode::None, ShadowDepthPassParameters.SceneTextures);

	ShadowDepthPassParameters.ProjectionMatrix = FTranslationMatrix44f(FVector3f(ShadowInfo->PreShadowTranslation - View.ViewMatrices.GetPreViewTranslation())) * ShadowInfo->TranslatedWorldToClipOuterMatrix;		// LWC_TODO: Precision loss
	ShadowDepthPassParameters.ViewMatrix = FMatrix44f(ShadowInfo->TranslatedWorldToView);

	ShadowDepthPassParameters.ShadowParams = FVector4f(ShadowInfo->GetShaderDepthBias(), ShadowInfo->GetShaderSlopeDepthBias(), ShadowInfo->GetShaderMaxSlopeDepthBias(), ShadowInfo->InvMaxSubjectDepth);
	ShadowDepthPassParameters.bClampToNearPlane = ShadowInfo->ShouldClampToNearPlane() ? 1.0f : 0.0f;

	ShadowDepthPassParameters.ShadowDepthType = (uint32)ShadowInfo->GetShadowDepthType();

	if (ShadowInfo->bOnePassPointLightShadow)
	{
		check(ShadowInfo->BorderSize == 0);

		// offset from translated world space to (pre translated) shadow space 
		const FMatrix Translation = FTranslationMatrix(ShadowInfo->PreShadowTranslation - View.ViewMatrices.GetPreViewTranslation());
		for (int32 FaceIndex = 0; FaceIndex < 6; FaceIndex++)
		{
			ShadowDepthPassParameters.ShadowViewProjectionMatrices[FaceIndex] = FMatrix44f(Translation * ShadowInfo->OnePassShadowViewProjectionMatrices[FaceIndex]);		// LWC_TODO: Precision loss?
		}
	}
}

void AddClearShadowDepthPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture)
{
	// Clear atlas depth, but ignore stencil.
	auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Texture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);
	GraphBuilder.AddPass(RDG_EVENT_NAME("ClearShadowDepth"), PassParameters, ERDGPassFlags::Raster, [](FRDGAsyncTask, FRHICommandList&) {});
}

void AddClearShadowDepthPass(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture, const FProjectedShadowInfo* ProjectedShadowInfo)
{
	// Clear atlas depth, but ignore stencil.
	auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Texture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);
	GraphBuilder.AddPass(RDG_EVENT_NAME("ClearShadowDepthTile"), PassParameters, ERDGPassFlags::Raster, [ProjectedShadowInfo](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		ProjectedShadowInfo->ClearDepth(RHICmdList);
	});
}

class FShadowDepthShaderElementData : public FMeshMaterialShaderElementData
{
public:
	int32 LayerId;
	int32 bUseGpuSceneInstancing;
};

/**
* A vertex shader for rendering the depth of a mesh.
*/
class FShadowDepthVS : public FMeshMaterialShader
{
public:
	DECLARE_INLINE_TYPE_LAYOUT(FShadowDepthVS, NonVirtual);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return false;
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FShadowDepthShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		ShaderBindings.Add(LayerId, ShaderElementData.LayerId);
		ShaderBindings.Add(bUseGpuSceneInstancing, ShaderElementData.bUseGpuSceneInstancing);
	}

	FShadowDepthVS() = default;
	FShadowDepthVS(const ShaderMetaType::CompiledShaderInitializerType & Initializer) :
		FMeshMaterialShader(Initializer)
	{
		LayerId.Bind(Initializer.ParameterMap, TEXT("LayerId"));
		bUseGpuSceneInstancing.Bind(Initializer.ParameterMap, TEXT("bUseGpuSceneInstancing"));
	}

private:
	LAYOUT_FIELD(FShaderParameter, LayerId);
	LAYOUT_FIELD(FShaderParameter, bUseGpuSceneInstancing);
};

static TAutoConsoleVariable<int32> CVarSupportPointLightWholeSceneShadows(
	TEXT("r.SupportPointLightWholeSceneShadows"),
	1,
	TEXT("Enables shadowcasting point lights."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static bool MobileUsesPerspectiveCorrectShadowPermutation(EShaderPlatform ShaderPlatform)
{
	// Required only for spotlight shadows on mobile
	static FShaderPlatformCachedIniValue<bool> MobileEnableMovableSpotlightShadowsIniValue(TEXT("r.Mobile.EnableMovableSpotlightsShadow"));
	const bool bMobileEnableMovableSpotlightShadows = (MobileEnableMovableSpotlightShadowsIniValue.Get(ShaderPlatform) != 0);
	return bMobileEnableMovableSpotlightShadows;
}

static TAutoConsoleVariable<int32> CVarDetectVertexShaderLayerAtRuntime(
	TEXT("r.Shadow.DetectVertexShaderLayerAtRuntime"),
	0,
	TEXT("Forces the compilation of the vslayer shader permutation even if the platform (RHI) does not declare compile-time support through RHISupportsVertexShaderLayer.")
	TEXT("Enabled by default for windows/SM5 as DX11 almost universally supports this at runtime."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

enum class EShadowDepthShaderMode
{
	PerspectiveCorrect,
	OutputDepth,
	OnePassPointLight,
	VirtualShadowMap,
	Combined,
};

// Returns bitfield of shadow depth types supported by the given shader mode
// NOTE: By default any modes covered by the "Combined" mode will not be included as separate shader permutations,
// and instead that mode will be used for any passes requesting the individual modes.
static constexpr EShadowDepthType GetShadowDepthTypesForMode(EShadowDepthShaderMode Mode)
{
	switch (Mode)
	{
		case EShadowDepthShaderMode::PerspectiveCorrect: 	return EShadowDepthType::Spot;
		case EShadowDepthShaderMode::OutputDepth: 			return EShadowDepthType::Directional;
		case EShadowDepthShaderMode::OnePassPointLight: 	return EShadowDepthType::OnePassPoint;
		case EShadowDepthShaderMode::VirtualShadowMap: 		return EShadowDepthType::VSM;
		case EShadowDepthShaderMode::Combined: 				return EShadowDepthType::VSM | EShadowDepthType::OnePassPoint;
		default: return EShadowDepthType::None;
	}
}

static constexpr bool IsShadowDepthTypeInCombinedMode(EShadowDepthType Types)
{
	EShadowDepthType CombinedShadowDepthTypes = GetShadowDepthTypesForMode(EShadowDepthShaderMode::Combined);
	// NOTE: This is slightly messy, but the shadow depth types for mesh passes get set to "VSM | Spot" or similar
	// in the VSM case. In these cases we only need the VSM permutation, so using "Any" allows them to correctly
	// route down to the combined permutation.
	return EnumHasAnyFlags(CombinedShadowDepthTypes, Types);
}

static constexpr EShadowDepthType GetShadowDepthTypesThatRequirePS()
{
	return EShadowDepthType::VSM | EShadowDepthType::Spot;
}

// Returns a bitfield of shadow depth types supported on the given platform
// TODO: Cache result for current platform...?
EShadowDepthType GetPlatformSupportedShadowDepthTypes(EShaderPlatform Platform)
{
	// Compile VS layer permutation if RHI supports it unconditionally OR we have forced it on (default for DX11&12 at SM5)
	static FShaderPlatformCachedIniValue<bool> DetectVertexShaderLayerRuntimeIniValue(TEXT("r.Shadow.DetectVertexShaderLayerAtRuntime"));
	const bool bShouldCompileVSLayer = RHISupportsVertexShaderLayer(Platform) || DetectVertexShaderLayerRuntimeIniValue.Get(Platform) != 0;
	
	EShadowDepthType Types = EShadowDepthType::None;

	if (!IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5))
	{
		// Mobile only needs OutputDepth, and optionally PerspectiveCorrect and OnePassPointLight
		Types |= EShadowDepthType::Directional
			| (MobileUsesPerspectiveCorrectShadowPermutation(Platform) ? EShadowDepthType::Spot : EShadowDepthType::None)
			| (FReadOnlyCVARCache::EnablePointLightShadows(Platform) && bShouldCompileVSLayer ? EShadowDepthType::OnePassPoint : EShadowDepthType::None);
	}
	else
	{
		Types |= EShadowDepthType::Directional
			| EShadowDepthType::Spot
			| (bShouldCompileVSLayer ? EShadowDepthType::OnePassPoint : EShadowDepthType::None)
			| (DoesPlatformSupportNonNaniteVirtualShadowMaps(Platform) ? EShadowDepthType::VSM : EShadowDepthType::None);
	}

	// If classic shadow maps are disabled at the platform/project level, remove those permutations
	if (!DoesPlatformSupportClassicShadowMaps(Platform))
	{
		Types &= ~(EShadowDepthType::Directional | EShadowDepthType::Spot | EShadowDepthType::OnePassPoint);
	}

	return Types;
}

// Get the "effective" supported shadow depth types for a given mode given both the mode and platform support.
EShadowDepthType GetEffectiveShadowDepthTypes(EShaderPlatform Platform, EShadowDepthShaderMode Mode)
{
	return GetShadowDepthTypesForMode(Mode) & GetPlatformSupportedShadowDepthTypes(Platform);
}

void SetEnvironmentDefinesForShadowDepthTypes(FShaderCompilerEnvironment& OutEnvironment, EShadowDepthType Types)
{
	OutEnvironment.SetDefine(TEXT("SUPPORT_SHADOW_DEPTH_VSM"), 			EnumHasAnyFlags(Types, EShadowDepthType::VSM));
	OutEnvironment.SetDefine(TEXT("SUPPORT_SHADOW_DEPTH_DIRECTIONAL"), 	EnumHasAnyFlags(Types, EShadowDepthType::Directional));
	OutEnvironment.SetDefine(TEXT("SUPPORT_SHADOW_DEPTH_SPOT"), 		EnumHasAnyFlags(Types, EShadowDepthType::Spot));
	OutEnvironment.SetDefine(TEXT("SUPPORT_SHADOW_DEPTH_POINT"), 		EnumHasAnyFlags(Types, EShadowDepthType::OnePassPoint));
}

/**
* A vertex shader for rendering the depth of a mesh.
*/
template <EShadowDepthShaderMode ShaderMode, bool bUsePositionOnlyStream>
class TShadowDepthVS : public FShadowDepthVS
{
public:
	DECLARE_SHADER_TYPE(TShadowDepthVS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		EShadowDepthType ShadowDepthTypes = GetEffectiveShadowDepthTypes(Parameters.Platform, ShaderMode);
		if (ShadowDepthTypes == EShadowDepthType::None)
		{
			return false;
		}

		// Don't include this permutation if it is not the combined mode, but the depth types are covered by it
		if (ShaderMode != EShadowDepthShaderMode::Combined && IsShadowDepthTypeInCombinedMode(ShadowDepthTypes))
		{
			return false;
		}

		// Note: This logic needs to stay in sync with UseDefaultMaterialForShadowDepth!
		return (Parameters.MaterialParameters.bIsSpecialEngineMaterial
			// Masked and WPO materials need their shaders but cannot be used with a position only stream.
			|| ((!Parameters.MaterialParameters.bWritesEveryPixelShadowPass || Parameters.MaterialParameters.bMaterialMayModifyMeshPosition) && !bUsePositionOnlyStream))
			// Only compile position-only shaders for vertex factories that support it. (Note: this assumes that a vertex factor which supports PositionOnly, supports also PositionAndNormalOnly)
			&& (!bUsePositionOnlyStream || Parameters.VertexFactoryType->SupportsPositionOnly())
			// Don't render ShadowDepth for translucent unlit materials
			&& Parameters.MaterialParameters.bShouldCastDynamicShadows
			&& !Parameters.VertexFactoryType->SupportsNaniteRendering();
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FShadowDepthVS::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		EShadowDepthType Types = GetEffectiveShadowDepthTypes(Parameters.Platform, ShaderMode);
		check(Types != EShadowDepthType::None);		// Should have been killed by ShouldCompilePermutation
		SetEnvironmentDefinesForShadowDepthTypes(OutEnvironment, Types);
		
		OutEnvironment.SetDefine(TEXT("POSITION_ONLY"), (uint32)bUsePositionOnlyStream);

		if (EnumHasAnyFlags(Types, EShadowDepthType::VSM))
		{
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}

		if (EnumHasAnyFlags(Types, EShadowDepthType::OnePassPoint))
		{
			// TODO: This is possibly harmful for other shadow depth modes in the combined case
			OutEnvironment.CompilerFlags.Add(CFLAG_VertexUseAutoCulling);
		}
	}

	TShadowDepthVS() = default;
	TShadowDepthVS(const ShaderMetaType::CompiledShaderInitializerType & Initializer)
		: FShadowDepthVS(Initializer)
	{}
};

#define IMPLEMENT_SHADOW_DEPTH_SHADERMODE_SHADERS(ShaderMode) \
	typedef TShadowDepthVS<EShadowDepthShaderMode::ShaderMode, false> TShadowDepthVS##ShaderMode; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TShadowDepthVS##ShaderMode, TEXT("/Engine/Private/ShadowDepthVertexShader.usf"), TEXT("Main"), SF_Vertex); \
	typedef TShadowDepthVS<EShadowDepthShaderMode::ShaderMode, true> TShadowDepthVSPositionOnly##ShaderMode; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TShadowDepthVSPositionOnly##ShaderMode, TEXT("/Engine/Private/ShadowDepthVertexShader.usf"), TEXT("PositionOnlyMain"), SF_Vertex);

IMPLEMENT_SHADOW_DEPTH_SHADERMODE_SHADERS(PerspectiveCorrect);
IMPLEMENT_SHADOW_DEPTH_SHADERMODE_SHADERS(OutputDepth);
IMPLEMENT_SHADOW_DEPTH_SHADERMODE_SHADERS(OnePassPointLight);
IMPLEMENT_SHADOW_DEPTH_SHADERMODE_SHADERS(VirtualShadowMap);
IMPLEMENT_SHADOW_DEPTH_SHADERMODE_SHADERS(Combined);

/**
* A pixel shader for rendering the depth of a mesh.
*/
class FShadowDepthBasePS : public FMeshMaterialShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FShadowDepthBasePS, NonVirtual);
public:

	FShadowDepthBasePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		const ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel((EShaderPlatform)Initializer.Target.Platform);

		if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Deferred)
		{
			PassUniformBuffer.Bind(Initializer.ParameterMap, FShadowDepthPassUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		}

		if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Mobile)
		{
			PassUniformBuffer.Bind(Initializer.ParameterMap, FMobileShadowDepthPassUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		}
	}

	FShadowDepthBasePS() = default;
};

template <EShadowDepthShaderMode ShaderMode>
class TShadowDepthPS : public FShadowDepthBasePS
{
	DECLARE_SHADER_TYPE(TShadowDepthPS, MeshMaterial);
public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		EShadowDepthType ShadowDepthTypes = GetEffectiveShadowDepthTypes(Parameters.Platform, ShaderMode);
		if (ShadowDepthTypes == EShadowDepthType::None)
		{
			return false;
		}

		// Don't include this permutation if it is not the combined mode, but the depth types are covered by it
		if (ShaderMode != EShadowDepthShaderMode::Combined && IsShadowDepthTypeInCombinedMode(ShadowDepthTypes))
		{
			return false;
		}

		const bool bModeRequiresPS = EnumHasAnyFlags(GetShadowDepthTypesThatRequirePS(), ShadowDepthTypes);

		//Note: This logic needs to stay in sync with UseDefaultMaterialForShadowDepth!
		return (Parameters.MaterialParameters.bIsSpecialEngineMaterial
			// Only compile for masked or lit translucent materials
			|| !Parameters.MaterialParameters.bWritesEveryPixelShadowPass
			|| (Parameters.MaterialParameters.bMaterialMayModifyMeshPosition && Parameters.MaterialParameters.bIsUsedWithInstancedStaticMeshes)
			// This mode needs a pixel shader and WPO materials can't be overridden with default material.
			|| (bModeRequiresPS && Parameters.MaterialParameters.bMaterialMayModifyMeshPosition))
			// Don't render ShadowDepth for translucent unlit materials
			&& Parameters.MaterialParameters.bShouldCastDynamicShadows
			&& !Parameters.VertexFactoryType->SupportsNaniteRendering();
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FShadowDepthBasePS::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		EShadowDepthType Types = GetEffectiveShadowDepthTypes(Parameters.Platform, ShaderMode);
		check(Types != EShadowDepthType::None);		// Should have been killed by ShouldCompilePermutation
		SetEnvironmentDefinesForShadowDepthTypes(OutEnvironment, Types);		

		if (EnumHasAnyFlags(Types, EShadowDepthType::VSM))
		{
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}
	}

	TShadowDepthPS()
	{
	}

	TShadowDepthPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FShadowDepthBasePS(Initializer)
	{
	}
};

// typedef required to get around macro expansion failure due to commas in template argument list for TShadowDepthPixelShader
#define IMPLEMENT_SHADOWDEPTHPASS_PIXELSHADER_TYPE(ShaderMode) \
	typedef TShadowDepthPS<EShadowDepthShaderMode::ShaderMode> TShadowDepthPS##ShaderMode; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TShadowDepthPS##ShaderMode,TEXT("/Engine/Private/ShadowDepthPixelShader.usf"),TEXT("Main"),SF_Pixel);

IMPLEMENT_SHADOWDEPTHPASS_PIXELSHADER_TYPE(PerspectiveCorrect);
IMPLEMENT_SHADOWDEPTHPASS_PIXELSHADER_TYPE(OutputDepth);
IMPLEMENT_SHADOWDEPTHPASS_PIXELSHADER_TYPE(OnePassPointLight);
IMPLEMENT_SHADOWDEPTHPASS_PIXELSHADER_TYPE(VirtualShadowMap);
IMPLEMENT_SHADOWDEPTHPASS_PIXELSHADER_TYPE(Combined);

/**
 * Check if default material can be used for shadow depth rendering 
 * Overriding in this manner can reduce state switches and the number of shaders that have to be compiled.
 * This logic needs to stay in sync with shadow depth shader ShouldCache logic.
 */
static bool UseDefaultMaterialForShadowDepth(
	const FMaterial& Material,
	bool bVFTypeSupportsNullPixelShader,
	bool MaterialModifiesMeshPosition)
{
	const bool bShadowPass = true;
	return (Material.WritesEveryPixel(bShadowPass, bVFTypeSupportsNullPixelShader) &&
		!MaterialModifiesMeshPosition);
}

bool GetShadowDepthPassShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	EShadowDepthType ShadowDepthType,
	bool bSupportsPositionAndNormalOnlyStream,
	bool bMaterialModifiesMeshPosition,
	TShaderRef<FShadowDepthVS>& VertexShader,
	TShaderRef<FShadowDepthBasePS>& PixelShader)
{
	const bool bVFTypeSupportsNullPixelShader = VertexFactoryType->SupportsNullPixelShader();
	const bool bPositionOnlyVS =
		bSupportsPositionAndNormalOnlyStream
		&& Material.WritesEveryPixel(true, bVFTypeSupportsNullPixelShader)
		&& !bMaterialModifiesMeshPosition;

	const EShaderPlatform Platform = GShaderPlatformForFeatureLevel[FeatureLevel];

	FMaterialShaderTypes ShaderTypes;

	const bool bUseCombinedMode = IsShadowDepthTypeInCombinedMode(ShadowDepthType);

	// Vertex shaders
	if (bUseCombinedMode)
	{
		if (bPositionOnlyVS)
		{
			ShaderTypes.AddShaderType<TShadowDepthVS<EShadowDepthShaderMode::Combined, true>>();
		}
		else
		{
			ShaderTypes.AddShaderType<TShadowDepthVS<EShadowDepthShaderMode::Combined, false>>();
		}
	}
	else if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::VSM))
	{
		if (bPositionOnlyVS)
		{
			ShaderTypes.AddShaderType<TShadowDepthVS<EShadowDepthShaderMode::VirtualShadowMap, true>>();
		}
		else
		{
			ShaderTypes.AddShaderType<TShadowDepthVS<EShadowDepthShaderMode::VirtualShadowMap, false>>();
		}
	}
	else
	{
		// Classic shadow map techniques. If disabled, we bail as we can't render this batch.
		if (!DoesPlatformSupportClassicShadowMaps(Platform))
		{
			return false;
		}

		if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::OnePassPoint))
		{
			if (DoesRuntimeSupportOnePassPointLightShadows(Platform))
			{
				if (bPositionOnlyVS)
				{
					ShaderTypes.AddShaderType<TShadowDepthVS<EShadowDepthShaderMode::OnePassPointLight, true>>();
				}
				else
				{
					ShaderTypes.AddShaderType<TShadowDepthVS<EShadowDepthShaderMode::OnePassPointLight, false>>();
				}
			}
			else
			{
				return false;
			}
		}
		else if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::Spot))
		{
			if (bPositionOnlyVS)
			{
				ShaderTypes.AddShaderType<TShadowDepthVS<EShadowDepthShaderMode::PerspectiveCorrect, true>>();
			}
			else
			{
				ShaderTypes.AddShaderType<TShadowDepthVS<EShadowDepthShaderMode::PerspectiveCorrect, false>>();
			}
		}
		else if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::Directional))
		{
			if (bPositionOnlyVS)
			{
				ShaderTypes.AddShaderType<TShadowDepthVS<EShadowDepthShaderMode::OutputDepth, true>>();
			}
			else
			{
				ShaderTypes.AddShaderType<TShadowDepthVS<EShadowDepthShaderMode::OutputDepth, false>>();
			}
		}
		else
		{
			return false;
		}
	}

	// Pixel shaders
	const bool bNullPixelShader = 
		bVFTypeSupportsNullPixelShader &&
		Material.WritesEveryPixel(true, bVFTypeSupportsNullPixelShader) &&
		!EnumHasAnyFlags(GetShadowDepthTypesThatRequirePS(), ShadowDepthType);

	if (!bNullPixelShader)
	{
		// TODO: Move this around to the same branch above, just respect NullPixelShader as appropriate
		if (bUseCombinedMode)
		{
			ShaderTypes.AddShaderType<TShadowDepthPS<EShadowDepthShaderMode::Combined>>();
		}
		else if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::VSM))
		{
			ShaderTypes.AddShaderType<TShadowDepthPS<EShadowDepthShaderMode::VirtualShadowMap>>();
		}
		else if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::OnePassPoint))
		{
			ShaderTypes.AddShaderType<TShadowDepthPS<EShadowDepthShaderMode::OnePassPointLight>>();
		}
		else if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::Spot))
		{
			ShaderTypes.AddShaderType<TShadowDepthPS<EShadowDepthShaderMode::PerspectiveCorrect>>();
		}
		else
		{
			ShaderTypes.AddShaderType<TShadowDepthPS<EShadowDepthShaderMode::OutputDepth>>();
		}
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}

/*-----------------------------------------------------------------------------
FProjectedShadowInfo
-----------------------------------------------------------------------------*/

void FProjectedShadowInfo::ClearDepth(FRHICommandList& RHICmdList) const
{
	check(RHICmdList.IsInsideRenderPass());

	const uint32 ViewportMinX = X;
	const uint32 ViewportMinY = Y;
	const float ViewportMinZ = 0.0f;
	const uint32 ViewportMaxX = X + BorderSize * 2 + ResolutionX;
	const uint32 ViewportMaxY = Y + BorderSize * 2 + ResolutionY;
	const float ViewportMaxZ = 1.0f;

	// Clear depth only.
	const int32 NumClearColors = 1;
	const bool bClearColor = false;
	const FLinearColor Colors[] = { FLinearColor::White };

	// Translucent shadows use draw call clear
	check(!bTranslucentShadow);

	RHICmdList.SetViewport(
		ViewportMinX,
		ViewportMinY,
		ViewportMinZ,
		ViewportMaxX,
		ViewportMaxY,
		ViewportMaxZ
	);

	DrawClearQuadMRT(RHICmdList, bClearColor, NumClearColors, Colors, true, 1.0f, false, 0);
}

void FProjectedShadowInfo::SetStateForView(FRHICommandList& RHICmdList, bool bRequiresBorderClamp) const
{
	check(bAllocated);

	RHICmdList.SetViewport(
		X,
		Y,
		0.0f,
		X + ResolutionX + 2 * BorderSize,
		Y + ResolutionY + 2 * BorderSize,
		1.0f
	);

	if (ShouldUseCSMScissorOptim())
	{
		RHICmdList.SetScissorRect(
			true,
			ScissorRectOptim.Min.X + X,
			ScissorRectOptim.Min.Y + Y,
			ScissorRectOptim.Max.X + X,
			ScissorRectOptim.Max.Y + Y
		);
	}
	else if (bRequiresBorderClamp)
	{
		// Platforms without the required stencil bits need to prevent shadow stretching by
		// restricting the rendering to inside the texture borders
		RHICmdList.SetScissorRect(
			true,
			X + BorderSize,
			Y + BorderSize,
			X + ResolutionX + BorderSize,
			Y + ResolutionY + BorderSize
		);
	}
}

void SetStateForShadowDepth(EShadowDepthType ShadowDepthType, FMeshPassProcessorRenderState& DrawRenderState)
{
	// Disable color writes
	DrawRenderState.SetBlendState(TStaticBlendState<CW_NONE>::GetRHI());

	if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::VSM))
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
	}
	else if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::OnePassPoint))
	{
		// Point lights use reverse Z depth maps
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
	}
	else
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_LessEqual>::GetRHI());
	}
}

static TAutoConsoleVariable<int32> CVarParallelShadows(
	TEXT("r.ParallelShadows"),
	1,
	TEXT("Toggles parallel shadow rendering. Parallel rendering must be enabled for this to have an effect."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarParallelShadowsNonWholeScene(
	TEXT("r.ParallelShadowsNonWholeScene"),
	0,
	TEXT("Toggles parallel shadow rendering for non whole-scene shadows. r.ParallelShadows must be enabled for this to have an effect."),
	ECVF_RenderThreadSafe
);

class FCopyShadowMapsCubeGS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopyShadowMapsCubeGS);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return RHISupportsGeometryShaders(Parameters.Platform) && IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	FCopyShadowMapsCubeGS() = default;
	FCopyShadowMapsCubeGS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

IMPLEMENT_GLOBAL_SHADER(FCopyShadowMapsCubeGS, "/Engine/Private/CopyShadowMaps.usf", "CopyCubeDepthGS", SF_Geometry);

class FCopyShadowMapsCubePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopyShadowMapsCubePS);
	SHADER_USE_PARAMETER_STRUCT(FCopyShadowMapsCubePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(TextureCube, ShadowDepthCubeTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowDepthSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCopyShadowMapsCubePS, "/Engine/Private/CopyShadowMaps.usf", "CopyCubeDepthPS", SF_Pixel);

class FCopyShadowMaps2DPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopyShadowMaps2DPS);
	SHADER_USE_PARAMETER_STRUCT(FCopyShadowMaps2DPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowDepthSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCopyShadowMaps2DPS, "/Engine/Private/CopyShadowMaps.usf", "Copy2DDepthPS", SF_Pixel);

/** */
class FScrollingShadowMaps2DPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FScrollingShadowMaps2DPS);
	SHADER_USE_PARAMETER_STRUCT(FScrollingShadowMaps2DPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowDepthSampler)
		SHADER_PARAMETER(FVector4f, DepthOffsetScale)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

IMPLEMENT_GLOBAL_SHADER(FScrollingShadowMaps2DPS, "/Engine/Private/CopyShadowMaps.usf", "Scrolling2DDepthPS", SF_Pixel);

void FProjectedShadowInfo::CopyCachedShadowMap(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSceneRenderer* SceneRenderer,
	const FRenderTargetBindingSlots& RenderTargetBindingSlots,
	const FMeshPassProcessorRenderState& DrawRenderState)
{
	check(CacheMode == SDCM_MovablePrimitivesOnly || CacheMode == SDCM_CSMScrolling);
	const FCachedShadowMapData& CachedShadowMapData = SceneRenderer->Scene->GetExtension<FCSMSceneExtension>().GetCachedShadowMapDataRef(GetLightSceneInfo().Id, FMath::Max(CascadeSettings.ShadowSplitIndex, 0));

	if (CachedShadowMapData.bCachedShadowMapHasPrimitives && CachedShadowMapData.ShadowMap.IsValid())
	{
		FRDGTextureRef ShadowDepthTexture = GraphBuilder.RegisterExternalTexture(CachedShadowMapData.ShadowMap.DepthTarget);
		const FIntPoint ShadowDepthExtent = ShadowDepthTexture->Desc.Extent;

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		DrawRenderState.ApplyToPSO(GraphicsPSOInit);
		const uint32 StencilRef = DrawRenderState.GetStencilRef();

		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
		// No depth tests, so we can replace the clear
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_Always>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		extern TGlobalResource<FFilterVertexDeclaration, FRenderResource::EInitPhase::Pre> GFilterVertexDeclaration;
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;

		if (CacheMode == SDCM_MovablePrimitivesOnly)
		{
			if (bOnePassPointLightShadow)
			{
				TShaderRef<FScreenVS> ScreenVertexShader;
				TShaderMapRef<FCopyShadowMapsCubePS> PixelShader(View.ShaderMap);
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

				int32 InstanceCount = 1;

	#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
				if (RHISupportsGeometryShaders(GShaderPlatformForFeatureLevel[SceneRenderer->FeatureLevel]))
				{
					TShaderMapRef<TScreenVSForGS<false>> VertexShader(View.ShaderMap);
					TShaderMapRef<FCopyShadowMapsCubeGS> GeometryShader(View.ShaderMap);
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.SetGeometryShader(GeometryShader.GetGeometryShader());
					ScreenVertexShader = VertexShader;
				}
				else
	#endif
				{
					check((RHISupportsVertexShaderLayer(SceneRenderer->ShaderPlatform) && GRHISupportsArrayIndexFromAnyShader));
					TShaderMapRef<TScreenVSForGS<true>> VertexShader(View.ShaderMap);
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					ScreenVertexShader = VertexShader;

					InstanceCount = 6;
				}

				auto* PassParameters = GraphBuilder.AllocParameters<FCopyShadowMapsCubePS::FParameters>();
				PassParameters->RenderTargets = RenderTargetBindingSlots;
				PassParameters->ShadowDepthCubeTexture = ShadowDepthTexture;
				PassParameters->ShadowDepthSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("CopyCachedShadowMap"),
					PassParameters,
					ERDGPassFlags::Raster,
					[this, ScreenVertexShader, PixelShader, GraphicsPSOInit, PassParameters, ShadowDepthExtent, InstanceCount, StencilRef](FRDGAsyncTask, FRHICommandList& RHICmdList) mutable
				{
					SetStateForView(RHICmdList);
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

					FIntPoint ResolutionWithBorder = FIntPoint(ResolutionX + 2 * BorderSize, ResolutionY + 2 * BorderSize);

					DrawRectangle(
						RHICmdList,
						0, 0,
						ResolutionWithBorder.X, ResolutionWithBorder.Y,
						0, 0,
						ResolutionWithBorder.X, ResolutionWithBorder.Y,
						ResolutionWithBorder,
						ShadowDepthExtent,
						ScreenVertexShader,
						EDRF_Default,
						InstanceCount);
				});
			}
			else
			{
				TShaderMapRef<FScreenVS> ScreenVertexShader(View.ShaderMap);
				TShaderMapRef<FCopyShadowMaps2DPS> PixelShader(View.ShaderMap);
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = ScreenVertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

				auto* PassParameters = GraphBuilder.AllocParameters<FCopyShadowMaps2DPS::FParameters>();
				PassParameters->RenderTargets = RenderTargetBindingSlots;
				PassParameters->ShadowDepthTexture = ShadowDepthTexture;
				PassParameters->ShadowDepthSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("CopyCachedShadowMap"),
					PassParameters,
					ERDGPassFlags::Raster,
					[this, ScreenVertexShader, PixelShader, GraphicsPSOInit, PassParameters, ShadowDepthExtent, StencilRef](FRDGAsyncTask, FRHICommandList& RHICmdList) mutable
				{
					SetStateForView(RHICmdList);
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

					FIntPoint ResolutionWithBorder = FIntPoint(ResolutionX + 2 * BorderSize, ResolutionY + 2 * BorderSize);

					DrawRectangle(
						RHICmdList,
						0, 0,
						ResolutionWithBorder.X, ResolutionWithBorder.Y,
						0, 0,
						ResolutionWithBorder.X, ResolutionWithBorder.Y,
						ResolutionWithBorder,
						ShadowDepthExtent,
						ScreenVertexShader,
						EDRF_Default);
				});
			}
		}
		else // CacheMode == SDCM_CSMScrolling
		{
			TShaderMapRef<FScreenVS> ScreenVertexShader(View.ShaderMap);
			TShaderMapRef<FScrollingShadowMaps2DPS> PixelShader(View.ShaderMap);
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = ScreenVertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			auto* PassParameters = GraphBuilder.AllocParameters<FScrollingShadowMaps2DPS::FParameters>();
			PassParameters->RenderTargets = RenderTargetBindingSlots;
			PassParameters->ShadowDepthTexture = ShadowDepthTexture;
			PassParameters->ShadowDepthSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			/** According to the implementation in ShadowDepthVertexShader.usf, the formula is DeviceZ = 1 - ((MaxZ - SceneDepth) / (MaxZ - MinZ)) * InvMaxSubjectDepth + SlopeDepthBias * Slope + ConstantDepthBias;
			For short C0 = InvMaxSubjectDepth; C1 = 1 + SlopeDepthBias * Slope + ConstantDepthBias;
			So the SceneDepth0 = MaxZ0 - (C1 - DeviceZ0) * (MaxZ0 - MinZ0) / C0 ;
			SceneDepth1 = SceneDepth0 + ZOffset;
			The reconstruct DeviceZ1 = C1 - ((MaxZ1 - SceneDepth1) / (MaxZ1 - MinZ1)) * C0;
			So DeviceZ1 = DeviceZ0 * (MaxZ0 - MinZ0) / (MaxZ1 - MinZ1) + (C0 * (MaxZ0 + ZOffset - MaxZ1) - C1 * (MaxZ0 - MinZ0)) / (MaxZ1 - MinZ1) + C1;
			*/
			float MaxZ0MinusMinZ0 = CachedShadowMapData.MaxSubjectZ - CachedShadowMapData.MinSubjectZ;
			float MaxZ1MinusMinZ1 = MaxSubjectZ - MinSubjectZ;
			float MaxZ0PlusZOffsetMinusMaxZ1 = CachedShadowMapData.MaxSubjectZ + CSMScrollingZOffset - MaxSubjectZ;
			float C1 = 1 + GetShaderDepthBias();
			PassParameters->DepthOffsetScale = FVector4f((InvMaxSubjectDepth * MaxZ0PlusZOffsetMinusMaxZ1 - C1 * MaxZ0MinusMinZ0) / MaxZ1MinusMinZ1 + C1, MaxZ0MinusMinZ0 / MaxZ1MinusMinZ1, 0.0f, 0.0f);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("ScrollingCachedWholeSceneDirectionalShadowMap"),
				PassParameters,
				ERDGPassFlags::Raster,
				[this, ScreenVertexShader, PixelShader, GraphicsPSOInit, PassParameters, ShadowDepthExtent, StencilRef](FRDGAsyncTask, FRHICommandList& RHICmdList) mutable
			{
				checkSlow(OverlappedUVOnCachedShadowMap != FVector4f(-1.0f, -1.0f, -1.0f, -1.0f));
				checkSlow(OverlappedUVOnCurrentShadowMap != FVector4f(-1.0f, -1.0f, -1.0f, -1.0f));

				FIntPoint ResolutionWithBorder = FIntPoint(ResolutionX + 2 * BorderSize, ResolutionY + 2 * BorderSize);

				uint32 UStart = OverlappedUVOnCachedShadowMap.X * ResolutionWithBorder.X + 0.5f;
				uint32 USize = (OverlappedUVOnCachedShadowMap.Z - OverlappedUVOnCachedShadowMap.X) * ResolutionWithBorder.X + 0.5f;

				uint32 VStart = OverlappedUVOnCachedShadowMap.Y * ResolutionWithBorder.Y + 0.5f;
				uint32 VSize = (OverlappedUVOnCachedShadowMap.W - OverlappedUVOnCachedShadowMap.Y) * ResolutionWithBorder.Y + 0.5f;

				FIntVector4 OutputViewport = FIntVector4(OverlappedUVOnCurrentShadowMap.X * ResolutionWithBorder.X + 0.5f, OverlappedUVOnCurrentShadowMap.Y * ResolutionWithBorder.Y + 0.5f, OverlappedUVOnCurrentShadowMap.Z * ResolutionWithBorder.X + 0.5f, OverlappedUVOnCurrentShadowMap.W * ResolutionWithBorder.Y + 0.5f);

				SetStateForView(RHICmdList);
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

				DrawRectangle(
					RHICmdList,
					OutputViewport.X, OutputViewport.Y,
					OutputViewport.Z - OutputViewport.X, OutputViewport.W - OutputViewport.Y,
					UStart, VStart,
					USize, VSize,
					ResolutionWithBorder,
					ShadowDepthExtent,
					ScreenVertexShader,
					EDRF_Default);
			});
		}
	}
}

void FProjectedShadowInfo::BeginRenderView(FRDGBuilder& GraphBuilder, FScene* Scene)
{
	if (DependentView)
	{
		const ERHIFeatureLevel::Type FeatureLevel = ShadowDepthView->FeatureLevel;
		if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Deferred)
		{
			extern TSet<IPersistentViewUniformBufferExtension*> PersistentViewUniformBufferExtensions;

			for (IPersistentViewUniformBufferExtension* Extension : PersistentViewUniformBufferExtensions)
			{
				Extension->BeginRenderView(DependentView);
			}
		}
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FShadowDepthPassParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileShadowDepthPassUniformParameters, MobilePassUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FShadowDepthPassUniformParameters, DeferredPassUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

#if WITH_MGPU
void CopyCachedShadowMapCrossGPU(FRHICommandList& RHICmdList, FRHITexture* ShadowDepthTexture, FRHIGPUMask SourceGPUMask)
{
	if (SourceGPUMask != FRHIGPUMask::All())
	{
		uint32 SourceGPUIndex = SourceGPUMask.GetFirstIndex();

		TArray<FTransferResourceParams, TFixedAllocator<MAX_NUM_GPUS>> CrossGPUTransferBuffers;
		for (uint32 DestGPUIndex : FRHIGPUMask::All())
		{
			if (!SourceGPUMask.Contains(DestGPUIndex))
			{
				CrossGPUTransferBuffers.Add(FTransferResourceParams(ShadowDepthTexture, SourceGPUIndex, DestGPUIndex, false, false));
			}
		}

		RHICmdList.TransferResources(CrossGPUTransferBuffers);
	}
}
#endif  // WITH_MGPU

void FProjectedShadowInfo::RenderDepth(
	FRDGBuilder& GraphBuilder,
	const FSceneRenderer* SceneRenderer,
	FRDGTextureRef ShadowDepthTexture,
	bool bDoParallelDispatch,
	bool bDoCrossGPUCopy)
{
#if WANTS_DRAW_MESH_EVENTS
	FString EventName;

	if (GetEmitDrawEvents())
	{
		GetShadowTypeNameForDrawEvent(EventName);
		EventName += FString(" ") + FString::FromInt(ResolutionX) + TEXT("x") + FString::FromInt(ResolutionY);
	}

	RDG_EVENT_SCOPE(GraphBuilder, "%s", *EventName);
#endif

	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_RenderWholeSceneShadowDepthsTime, bWholeSceneShadow);
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_RenderPerObjectShadowDepthsTime, !bWholeSceneShadow);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RenderShadowDepth);

	FScene* Scene = SceneRenderer->Scene;
	const ERHIFeatureLevel::Type FeatureLevel = ShadowDepthView->FeatureLevel;
	BeginRenderView(GraphBuilder, Scene);

	FShadowDepthPassParameters* PassParameters = GraphBuilder.AllocParameters<FShadowDepthPassParameters>();
	PassParameters->View = ShadowDepthView->ViewUniformBuffer;
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
		ShadowDepthTexture,
		ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ENoAction,
		FExclusiveDepthStencil::DepthWrite_StencilNop);

	if (CacheMode == SDCM_MovablePrimitivesOnly || CacheMode == SDCM_CSMScrolling)
	{
		// Copy in depths of static primitives before we render movable primitives.
		FMeshPassProcessorRenderState DrawRenderState;
		SetStateForShadowDepth(GetShadowDepthType(), DrawRenderState);
		CopyCachedShadowMap(GraphBuilder, *ShadowDepthView, SceneRenderer, PassParameters->RenderTargets, DrawRenderState);
	}

	PassParameters->VirtualShadowMap = SceneRenderer->VirtualShadowMapArray.GetUniformBuffer(0);

	switch (GetFeatureLevelShadingPath(FeatureLevel))
	{
	case EShadingPath::Deferred:
	{
		auto* ShadowDepthPassParameters = GraphBuilder.AllocParameters<FShadowDepthPassUniformParameters>();
		SetupShadowDepthPassUniformBuffer(this, GraphBuilder, *ShadowDepthView, *ShadowDepthPassParameters);
		PassParameters->DeferredPassUniformBuffer = GraphBuilder.CreateUniformBuffer(ShadowDepthPassParameters);
	}
	break;
	case EShadingPath::Mobile:
	{
		auto* ShadowDepthPassParameters = GraphBuilder.AllocParameters<FMobileShadowDepthPassUniformParameters>();
		SetupShadowDepthPassUniformBuffer(this, GraphBuilder, *ShadowDepthView, *ShadowDepthPassParameters);
		PassParameters->MobilePassUniformBuffer = GraphBuilder.CreateUniformBuffer(ShadowDepthPassParameters);
	}
	break;
	default:
		checkNoEntry();
	}

	ShadowDepthPass.BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

	bool bRequiresBorderClamp = !SupportsShadowStencilCulling(SceneRenderer->ShaderPlatform);

	if (bDoParallelDispatch)
	{
		GraphBuilder.AddDispatchPass(
			RDG_EVENT_NAME("ShadowDepthPassParallel"),
			PassParameters,
			ERDGPassFlags::Raster,
			[this, PassParameters, bRequiresBorderClamp](FRDGDispatchPassBuilder& DispatchPassBuilder)
		{
			ShadowDepthPass.Dispatch(DispatchPassBuilder, &PassParameters->InstanceCullingDrawParams, [this, bRequiresBorderClamp] (FRHICommandList& RHICmdList)
			{
				SetStateForView(RHICmdList, bRequiresBorderClamp);
			});
		});
	}
	else
	{
			GraphBuilder.AddPass(
			RDG_EVENT_NAME("ShadowDepthPass"),
			PassParameters,
			ERDGPassFlags::Raster,
			[this, PassParameters, bRequiresBorderClamp](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			SetStateForView(RHICmdList, bRequiresBorderClamp);
			ShadowDepthPass.Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
		});
	}

#if WITH_MGPU
	if (bDoCrossGPUCopy)
	{
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("CopyCachedShadowMapCrossGPU"),
			PassParameters,
			ERDGPassFlags::Raster,
			[ShadowDepthTexture, GPUMask = GraphBuilder.RHICmdList.GetGPUMask()](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			CopyCachedShadowMapCrossGPU(RHICmdList, ShadowDepthTexture->GetRHI(), GPUMask);
		});
	}
#endif
}

float FProjectedShadowInfo::GetLODDistanceFactor() const
{
	float Result = GFarShadowLODDistanceFactor;
	if (CascadeSettings.bFarShadowCascade)
	{
		Result = GFarShadowLODDistanceFactor;
	}
	else
	{
		Result = GShadowLODDistanceFactor;
		// Make sure values are in a reasonable range (Local lights set SplitIndex to -1 and VSM to 1000)
		if (CascadeSettings.ShadowSplitIndex > 0 && CascadeSettings.ShadowSplitIndex < 16)
		{
			Result += CascadeSettings.ShadowSplitIndex * GShadowLODDistanceFactorCascadeScale;
		}
	}
	return Result;
}

void FProjectedShadowInfo::ModifyViewForShadow(FViewInfo* FoundView) const
{
	FIntRect OriginalViewRect = FoundView->ViewRect;
	FoundView->ViewRect = GetOuterViewRect();

	FoundView->ViewMatrices.HackRemoveTemporalAAProjectionJitter();

	if (CascadeSettings.bFarShadowCascade)
	{
		(int32&)FoundView->DrawDynamicFlags |= (int32)EDrawDynamicFlags::FarShadowCascade;
	}

	// must match logic in FProjectedShadowInfo::CalcAndUpdateLODToRender(...)
	FoundView->LODDistanceFactor *= GetLODDistanceFactor();

	// Don't do material texture mip biasing in shadow maps.
	FoundView->MaterialTextureMipBias = 0;

	FoundView->CachedViewUniformShaderParameters = MakeUnique<FViewUniformShaderParameters>();

	// Override the view matrix so that billboarding primitives will be aligned to the light
	FoundView->ViewMatrices.HackOverrideViewMatrixForShadows(TranslatedWorldToView);
	FBox VolumeBounds[TVC_MAX];
	FoundView->SetupUniformBufferParameters(
		VolumeBounds,
		TVC_MAX,
		*FoundView->CachedViewUniformShaderParameters);

	FoundView->ViewUniformBuffer = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(*FoundView->CachedViewUniformShaderParameters, UniformBuffer_SingleFrame);

	// we are going to set this back now because we only want the correct view rect for the uniform buffer. For LOD calculations, we want the rendering viewrect and proj matrix.
	FoundView->ViewRect = OriginalViewRect;

	extern int32 GPreshadowsForceLowestLOD;

	if (bPreShadow && GPreshadowsForceLowestLOD)
	{
		(int32&)FoundView->DrawDynamicFlags |= EDrawDynamicFlags::ForceLowestLOD;
	}
}

const FViewInfo* FProjectedShadowInfo::FindViewForShadow(TConstArrayView<FViewInfo> SceneRendererViews) const
{
	// If the shadow view was pre-selected during setup, return that.
	if (SelectedViewIndex != INDEX_NONE)
	{
		check(!bDirectionalLight);
		check(!bPerObjectOpaqueShadow);
		check(bWholeSceneShadow);

		return &SceneRendererViews[SelectedViewIndex];
	}

	// Choose an arbitrary view where this shadow's subject is relevant.
	const FViewInfo* FoundView = nullptr;
	for (int32 ViewIndex = 0; ViewIndex < SceneRendererViews.Num(); ViewIndex++)
	{
		const FViewInfo* CheckView = &SceneRendererViews[ViewIndex];
		const FVisibleLightViewInfo& VisibleLightViewInfo = CheckView->VisibleLightInfos[LightSceneInfo->Id];
		FPrimitiveViewRelevance ViewRel = VisibleLightViewInfo.ProjectedShadowViewRelevanceMap[ShadowId];
		if (ViewRel.bShadowRelevance)
		{
			FoundView = CheckView;
			break;
		}
	}
	check(FoundView);
	return FoundView;
}

void FProjectedShadowInfo::SetupShadowDepthView(FSceneRenderer* SceneRenderer)
{
	const FViewInfo* FoundView = FindViewForShadow(SceneRenderer->Views);
	check(FoundView && IsInParallelRenderingThread());
	FViewInfo* DepthPassView = FoundView->CreateSnapshot(EViewSnapshotType::Shadow);
	// We are starting a new collection of dynamic primitives for the shadow views.
	DepthPassView->DynamicPrimitiveCollector = FGPUScenePrimitiveCollector(&SceneRenderer->GetGPUSceneDynamicContext());

	ModifyViewForShadow(DepthPassView);
	ShadowDepthView = DepthPassView;
}

void FProjectedShadowInfo::GetShadowTypeNameForDrawEvent(FString& TypeName) const
{
	const FName ParentName = ParentSceneInfo ? ParentSceneInfo->Proxy->GetOwnerName() : NAME_None;

	if (bWholeSceneShadow)
	{
		if (CascadeSettings.ShadowSplitIndex >= 0)
		{
			TypeName = FString("WholeScene split") + FString::FromInt(CascadeSettings.ShadowSplitIndex);
		}
		else
		{
			if (CacheMode == SDCM_MovablePrimitivesOnly)
			{
				TypeName = FString("WholeScene MovablePrimitives");
			}
			else if (CacheMode == SDCM_StaticPrimitivesOnly)
			{
				TypeName = FString("WholeScene StaticPrimitives");
			}
			else
			{
				TypeName = FString("WholeScene");
			}
		}
	}
	else if (bPreShadow)
	{
		TypeName = FString("PreShadow ") + ParentName.ToString();
	}
	else
	{
		TypeName = FString("PerObject ") + ParentName.ToString();
	}
}

#if WITH_MGPU
// Shadows that are cached need to be copied to other GPUs after they render
bool FSceneRenderer::IsShadowCached(FProjectedShadowInfo* ProjectedShadowInfo) const
{
	// Preshadows that are going to be cached this frame should be copied to other GPUs.
	if (ProjectedShadowInfo->bPreShadow)
	{
		return !ProjectedShadowInfo->bDepthsCached && ProjectedShadowInfo->bAllocatedInPreshadowCache;
	}
	// SDCM_StaticPrimitivesOnly shadows don't update every frame so we need to copy
	// their depths to all possible GPUs.
	else if (!ProjectedShadowInfo->IsWholeSceneDirectionalShadow() && ProjectedShadowInfo->CacheMode == SDCM_StaticPrimitivesOnly)
	{
		// Cached whole scene shadows shouldn't be view dependent.
		checkSlow(ProjectedShadowInfo->DependentView == nullptr);

		return true;
	}
	return false;
}

FRHIGPUMask FSceneRenderer::GetGPUMaskForShadow(FProjectedShadowInfo* ProjectedShadowInfo) const
{
	// View dependent shadows only need to render depths on their view's GPUs.
	if (ProjectedShadowInfo->DependentView != nullptr)
	{
		return ProjectedShadowInfo->DependentView->GPUMask;
	}
	else
	{
		return AllViewsGPUMask;
	}
}
#endif // WITH_MGPU

static void UpdatePackedViewParamsFromPrevShadowState(Nanite::FPackedViewParams& Params, const FPersistentShadowState* PrevShadowState)
{
	if(PrevShadowState)
	{
		Params.PrevViewMatrices = PrevShadowState->ViewMatrices;
		Params.HZBTestViewRect = PrevShadowState->HZBTestViewRect;
		Params.Flags |= NANITE_VIEW_FLAG_HZBTEST;
	}
}

static void UpdateCurrentFrameHZB(FLightSceneInfo& LightSceneInfo, const FPersistentShadowStateKey& ShadowKey, const FProjectedShadowInfo* ProjectedShadowInfo, const TRefCountPtr<IPooledRenderTarget>& HZB, int32 CubeFaceIndex = -1)
{
	FPersistentShadowState State;
	State.ViewMatrices = ProjectedShadowInfo->GetShadowDepthRenderingViewMatrices(CubeFaceIndex);
	State.HZBTestViewRect = ProjectedShadowInfo->GetInnerViewRect();
	State.HZB = HZB;
	LightSceneInfo.PersistentShadows.Add(ShadowKey, State);
}

// Serialize tasks that touch the persistent shadow map.
static UE::Tasks::FPipe GPersistentShadowsPipe(UE_SOURCE_LOCATION);

// Returns nanite views setup task
static void RenderShadowDepthAtlasNanite(
	FRDGBuilder& GraphBuilder,
	ERHIFeatureLevel::Type FeatureLevel,
	FScene& Scene,
	const FViewFamilyInfo& ViewFamily,
	const FViewInfo& SceneView,
	FSceneUniformBuffer &SceneUniformBuffer,
	const FSortedShadowMapAtlas& ShadowMapAtlas,
	const int32 AtlasIndex)
{
	const FIntPoint AtlasSize = ShadowMapAtlas.RenderTargets.DepthTarget->GetDesc().Extent;

	const bool bUseHZB = (CVarNaniteShadowsUseHZB.GetValueOnRenderThread() != 0);
	TArray<TRefCountPtr<IPooledRenderTarget>>&	PrevAtlasHZBs = Scene.PrevAtlasHZBs;

	auto& StaticShadowsToEmit = *GraphBuilder.AllocObject<TArray<FProjectedShadowInfo*, SceneRenderingAllocator>>();
	auto& MovableShadowsToEmit = *GraphBuilder.AllocObject<TArray<FProjectedShadowInfo*, SceneRenderingAllocator>>();
	auto& OtherShadowsToEmit = *GraphBuilder.AllocObject<TArray<FProjectedShadowInfo*, SceneRenderingAllocator>>();

	for (int32 ShadowIndex = 0; ShadowIndex < ShadowMapAtlas.Shadows.Num(); ShadowIndex++)
	{
		FProjectedShadowInfo* ProjectedShadowInfo = ShadowMapAtlas.Shadows[ShadowIndex];

		if (!ProjectedShadowInfo->bNaniteGeometry)
		{
			continue;
		}

		if (ProjectedShadowInfo->CacheMode == SDCM_StaticPrimitivesOnly)
		{
			StaticShadowsToEmit.Add(ProjectedShadowInfo);
		}
		else if (ProjectedShadowInfo->CacheMode == SDCM_MovablePrimitivesOnly)
		{
			MovableShadowsToEmit.Add(ProjectedShadowInfo);
		}
		else
		{
			OtherShadowsToEmit.Add(ProjectedShadowInfo);
		}
	}

	auto DrawShadows = [&](const TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ShadowsToEmit, Nanite::EFilterFlags HiddenFilterFlags)
	{
		if (ShadowsToEmit.Num() > 0)
		{	
			Nanite::FPackedViewArray* PackedViews = Nanite::FPackedViewArray::CreateWithSetupTask(
				GraphBuilder,
				ShadowsToEmit.Num(),
				[&ShadowsToEmit, AtlasSize, AtlasIndex] (Nanite::FPackedViewArray::ArrayType& PackedViews)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(CreateShadowDepthAtlasNanitePackedViews);
	
					for (FProjectedShadowInfo* ProjectedShadowInfo : ShadowsToEmit)
					{
						Nanite::FPackedViewParams Initializer;
						Initializer.ViewMatrices = ProjectedShadowInfo->GetShadowDepthRenderingViewMatrices();
						Initializer.ViewRect = ProjectedShadowInfo->GetOuterViewRect();
						Initializer.RasterContextSize = AtlasSize;
						Initializer.PrevViewMatrices = Initializer.ViewMatrices;
						Initializer.HZBTestViewRect = ProjectedShadowInfo->GetInnerViewRect();
						Initializer.MaxPixelsPerEdgeMultipler = 1.0f / FShadowSceneRenderer::ComputeNaniteShadowsLODScaleFactor();
						Nanite::SetViewFrameIndex(ProjectedShadowInfo->ShadowDepthView, Initializer);

						// Orthographic shadow projections want depth clamping rather than clipping
						Initializer.Flags = ProjectedShadowInfo->ShouldClampToNearPlane() ? 0u : NANITE_VIEW_FLAG_NEAR_CLIP;
	
						FLightSceneInfo& LightSceneInfo = ProjectedShadowInfo->GetLightSceneInfo();

						if (LightSceneInfo.Proxy)
						{
							Initializer.bUseLightingChannelMask = true;
							Initializer.LightingChannelMask = LightSceneInfo.Proxy->GetLightingChannelMask();
						}
	
						FPersistentShadowStateKey ShadowKey;
						ShadowKey.AtlasIndex = AtlasIndex;
						ShadowKey.ProjectionId = ProjectedShadowInfo->ProjectionIndex;
						ShadowKey.SubjectPrimitiveComponentIndex = ProjectedShadowInfo->SubjectPrimitiveComponentIndex;
	
						FPersistentShadowState* PrevShadowState = LightSceneInfo.PrevPersistentShadows.Find(ShadowKey);
	
						UpdatePackedViewParamsFromPrevShadowState(Initializer, PrevShadowState);
						UpdateCurrentFrameHZB(LightSceneInfo, ShadowKey, ProjectedShadowInfo, nullptr);
	
						Nanite::SetCullingViewOverrides(ProjectedShadowInfo->ShadowDepthView, Initializer);
						PackedViews.Add(Nanite::CreatePackedView(Initializer));
					}
	
				}, &GPersistentShadowsPipe);
	
			RDG_EVENT_SCOPE(GraphBuilder, "Nanite Shadows");
	
			Nanite::FSharedContext SharedContext{};
			SharedContext.FeatureLevel = Scene.GetFeatureLevel();
			SharedContext.ShaderMap = GetGlobalShaderMap(SharedContext.FeatureLevel);
			SharedContext.Pipeline = Nanite::ERasterPipeline::Shadows;
	
			// NOTE: Rendering into an atlas like this is not going to work properly with HZB, but we are not currently using HZB here.
			// It might be worthwhile going through the virtual SM rendering path even for "dense" cases even just for proper handling of all the details.
			FIntRect FullAtlasViewRect(FIntPoint(0, 0), AtlasSize);
			FRDGTextureRef PrevAtlasHZB = (bUseHZB && PrevAtlasHZBs[AtlasIndex]) ? GraphBuilder.RegisterExternalTexture(PrevAtlasHZBs[AtlasIndex]) : nullptr;

			Nanite::FConfiguration CullingConfig = { 0 };
			CullingConfig.bIsShadowPass = true;
			CullingConfig.bTwoPassOcclusion = true;
			CullingConfig.bUpdateStreaming = CVarNaniteShadowsUpdateStreaming.GetValueOnRenderThread() != 0;
			CullingConfig.HiddenFilterFlags = HiddenFilterFlags;
			CullingConfig.SetViewFlags(SceneView);
	
			if (GNaniteShowStats != 0)
			{
				FString AtlasFilterName = FString::Printf(TEXT("ShadowAtlas%d"), AtlasIndex);
				CullingConfig.bExtractStats = Nanite::IsStatFilterActive(AtlasFilterName);
			}
	
			const Nanite::FRasterContextInitParams RasterInitParams =
			{
				.TextureSize = AtlasSize,
				.TextureRect = FullAtlasViewRect,
				.RasterMode = Nanite::EOutputBufferMode::DepthOnly
			};
			Nanite::FRasterContext RasterContext = Nanite::InitRasterContext(GraphBuilder, SharedContext, ViewFamily, RasterInitParams);
	
			TUniquePtr<Nanite::IRenderer> NaniteRenderer = Nanite::IRenderer::Create(
				GraphBuilder,
				Scene,
				SceneView,
				SceneUniformBuffer,
				SharedContext,
				RasterContext,
				CullingConfig,
				FullAtlasViewRect,
				PrevAtlasHZB
			);
	
			{
				NaniteRenderer->DrawGeometry(
					Scene.NaniteRasterPipelines[ENaniteMeshPass::BasePass],
					nullptr,
					*PackedViews
				);
			}
	
			if (bUseHZB)
			{
				FRDGTextureRef FurthestHZBTexture;
				BuildHZBFurthest(
					GraphBuilder,
					GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy),
					RasterContext.DepthBuffer,
					FullAtlasViewRect,
					FeatureLevel,
					Scene.GetShaderPlatform(),
					TEXT("Shadow.AtlasHZB"),
					/* OutFurthestHZBTexture = */ &FurthestHZBTexture,
					PF_R32_FLOAT);
				PrevAtlasHZBs[AtlasIndex] = GraphBuilder.ConvertToExternalTexture(FurthestHZBTexture);
			}
			else
			{
				PrevAtlasHZBs[AtlasIndex] = nullptr;
			}
	
			FRDGTextureRef ShadowMap = GraphBuilder.RegisterExternalTexture(ShadowMapAtlas.RenderTargets.DepthTarget);
	
			for (FProjectedShadowInfo* ProjectedShadowInfo : ShadowsToEmit)
			{
				const FIntRect AtlasViewRect = ProjectedShadowInfo->GetOuterViewRect();
	
				Nanite::EmitShadowMap(
					GraphBuilder,
					SharedContext,
					RasterContext,
					ShadowMap,
					AtlasViewRect,
					AtlasViewRect.Min,
					ProjectedShadowInfo->GetShadowDepthRenderingViewMatrices().GetViewToClip(),
					ProjectedShadowInfo->GetShaderDepthBias(),
					ProjectedShadowInfo->bDirectionalLight
				);
			}
		}
	};
	DrawShadows(StaticShadowsToEmit, Nanite::EFilterFlags::NonStaticMobility);
	DrawShadows(MovableShadowsToEmit, Nanite::EFilterFlags::StaticMobility);
	DrawShadows(OtherShadowsToEmit, Nanite::EFilterFlags::None);
}

bool IsParallelDispatchEnabled(const FProjectedShadowInfo* ProjectedShadowInfo, EShaderPlatform ShaderPlatform)
{
	return GRHICommandList.UseParallelAlgorithms() && CVarParallelShadows.GetValueOnRenderThread()
		&& (ProjectedShadowInfo->IsWholeSceneDirectionalShadow() || CVarParallelShadowsNonWholeScene.GetValueOnRenderThread())
		// Parallel dispatch is not supported on mobile platform
		&& !IsMobilePlatform(ShaderPlatform);
}

void FSceneRenderer::RenderShadowDepthMapAtlases(FRDGBuilder& GraphBuilder)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSceneRenderer::RenderShadowDepthMapAtlases);

	const bool bNaniteEnabled = 
		UseNanite(ShaderPlatform) &&
		ViewFamily.EngineShowFlags.NaniteMeshes &&
		CVarNaniteShadows.GetValueOnRenderThread() != 0 &&
		Nanite::GStreamingManager.HasResourceEntries();

	Scene->PrevAtlasHZBs.SetNum(SortedShadowsForShadowDepthPass.ShadowMapAtlases.Num());

	FRDGExternalAccessQueue ExternalAccessQueue;

	for (int32 AtlasIndex = 0; AtlasIndex < SortedShadowsForShadowDepthPass.ShadowMapAtlases.Num(); AtlasIndex++)
	{
		FSortedShadowMapAtlas& ShadowMapAtlas = SortedShadowsForShadowDepthPass.ShadowMapAtlases[AtlasIndex];
		FRDGTextureRef AtlasDepthTexture = GraphBuilder.RegisterExternalTexture(ShadowMapAtlas.RenderTargets.DepthTarget);
		const FIntPoint AtlasSize = AtlasDepthTexture->Desc.Extent;

		// On mobile CSM atlas sampled only in pixel shaders
		ERHIAccess AtlasDepthTextureAccessFinal = (IsMobilePlatform(ShaderPlatform) && !ShouldRenderVolumetricFog() ? ERHIAccess::SRVGraphics : ERHIAccess::SRVMask);

		// Check if we are using timesliced shadows for a runtime reflection capture
		if (Views[0].IsRuntimeReflectionCapture())
		{
			if (ShadowMapAtlas.Shadows.Num() && ShadowMapAtlas.Shadows[0]->GetLightSceneInfo().Proxy->TimesliceAtlasFrame == GFrameNumberRenderThread)
			{
				// We've started timeslicing on the current frame for the shadows in this atlas.  Mark the frame we are rendering those shadows.
				Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame = GFrameNumberRenderThread;
			}
			else if (Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame)
			{
				// Timeslicing is active from a previous frame, and we don't need to render.  Make readable because AtlasDepthTexture is not tracked via RDG yet.
				ShadowMapAtlas.RenderTargets.DepthTarget = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, AtlasDepthTexture, AtlasDepthTextureAccessFinal);
				continue;
			}
		}

		RDG_EVENT_SCOPE(GraphBuilder, "Atlas%u %ux%u", AtlasIndex, AtlasSize.X, AtlasSize.Y);

		TArray<FProjectedShadowInfo*, SceneRenderingAllocator> ParallelShadowPasses;
		TArray<FProjectedShadowInfo*, SceneRenderingAllocator> SerialShadowPasses;

		// Gather our passes here to minimize switching render passes
		for (FProjectedShadowInfo* ProjectedShadowInfo : ShadowMapAtlas.Shadows)
		{
			if (IsParallelDispatchEnabled(ProjectedShadowInfo, ShaderPlatform))
			{
				ParallelShadowPasses.Add(ProjectedShadowInfo);
			}
			else
			{
				SerialShadowPasses.Add(ProjectedShadowInfo);
			}
		}

	#if WANTS_DRAW_MESH_EVENTS && RDG_EVENTS
		FLightSceneProxy* CurrentLightForDrawEvent = nullptr;
		TOptional<TRDGEventScopeGuard<FRDGScope_RHI>> RDGScope;
	#endif

		const auto SetLightEventForShadow = [&](FProjectedShadowInfo* ProjectedShadowInfo)
		{
		#if WANTS_DRAW_MESH_EVENTS && RDG_EVENTS
			if (!CurrentLightForDrawEvent || ProjectedShadowInfo->GetLightSceneInfo().Proxy != CurrentLightForDrawEvent)
			{
				if (CurrentLightForDrawEvent)
				{
					RDGScope.Reset();
				}

				CurrentLightForDrawEvent = ProjectedShadowInfo->GetLightSceneInfo().Proxy;
				FString LightNameWithLevel;
				GetLightNameForDrawEvent(CurrentLightForDrawEvent, LightNameWithLevel);

				RDG_EVENT_SCOPE_CONSTRUCT(RDGScope, GraphBuilder, true, ERDGScopeFlags::None, RHI_GPU_STAT_ARGS_NONE, TEXT("RenderDepthForLight"), TEXT("%s"), *LightNameWithLevel);
			}
		#endif
		};

		const auto EndLightEvent = [&]()
		{
		#if WANTS_DRAW_MESH_EVENTS && RDG_EVENTS
			if (CurrentLightForDrawEvent)
			{
				RDGScope.Reset();
				CurrentLightForDrawEvent = nullptr;
			}
		#endif
		};

		AddClearShadowDepthPass(GraphBuilder, AtlasDepthTexture);

		if (ParallelShadowPasses.Num() > 0)
		{
			for (FProjectedShadowInfo* ProjectedShadowInfo : ParallelShadowPasses)
			{
				RDG_GPU_MASK_SCOPE(GraphBuilder, GetGPUMaskForShadow(ProjectedShadowInfo));
				SetLightEventForShadow(ProjectedShadowInfo);

				const bool bParallelDispatch = true;
				bool bDoCrossGPUCopy = false;
#if WITH_MGPU
				bDoCrossGPUCopy = IsShadowCached(ProjectedShadowInfo);
#endif
				ProjectedShadowInfo->RenderDepth(GraphBuilder, this, AtlasDepthTexture, bParallelDispatch, bDoCrossGPUCopy);
			}
		}

		EndLightEvent();

		if (SerialShadowPasses.Num() > 0)
		{
			for (FProjectedShadowInfo* ProjectedShadowInfo : SerialShadowPasses)
			{
				RDG_GPU_MASK_SCOPE(GraphBuilder, GetGPUMaskForShadow(ProjectedShadowInfo));
				SetLightEventForShadow(ProjectedShadowInfo);

				const bool bParallelDispatch = false;
				bool bDoCrossGPUCopy = false;
#if WITH_MGPU
				bDoCrossGPUCopy = IsShadowCached(ProjectedShadowInfo);
#endif
				ProjectedShadowInfo->RenderDepth(GraphBuilder, this, AtlasDepthTexture, bParallelDispatch, bDoCrossGPUCopy);
			}
		}

		EndLightEvent();

		if (bNaniteEnabled)
		{
			const FViewInfo& SceneView = Views[0];
			RenderShadowDepthAtlasNanite(GraphBuilder, FeatureLevel, *Scene, ViewFamily, SceneView, GetSceneUniforms(), ShadowMapAtlas, AtlasIndex);
		}

		// Make readable because AtlasDepthTexture is not tracked via RDG yet
		ShadowMapAtlas.RenderTargets.DepthTarget = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, AtlasDepthTexture, AtlasDepthTextureAccessFinal);
	}

	ExternalAccessQueue.Submit(GraphBuilder);
}

void FSceneRenderer::RenderShadowDepthMaps(FRDGBuilder& GraphBuilder, FDynamicShadowsTaskData* DynamicShadowsTaskData, FInstanceCullingManager &InstanceCullingManager, FRDGExternalAccessQueue& ExternalAccessQueue)
{
	ensureMsgf(!bShadowDepthRenderCompleted, TEXT("RenderShadowDepthMaps called twice in the same frame"));

	FShadowSceneRenderer* ShadowSceneRenderer = GetSceneExtensionsRenderers().GetRendererPtr<FShadowSceneRenderer>();

	// Exit early if dynamic shadows are disabled (avoids the unnecessary BeginDeferredCulling call, and prevents RenderShadowDepthMapAtlases from overwriting Scene->PrevAtlasHZBs)
	if (!ViewFamily.EngineShowFlags.DynamicShadows)
	{
		// Earlier code should not have added shadows to be processed, or created dynamic shadow related tasks or queries, if DynamicShadows are disabled
		check(SortedShadowsForShadowDepthPass.IsEmpty() && DynamicShadowsTaskData == nullptr && (ShadowSceneRenderer == nullptr || !ShadowSceneRenderer->HasNaniteVisibilityQuery()));

		bShadowDepthRenderCompleted = true;
		return;
	}

	{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderShadows);

	TRACE_CPUPROFILER_EVENT_SCOPE(FSceneRenderer::RenderShadowDepthMaps);
	SCOPED_NAMED_EVENT(FSceneRenderer_RenderShadowDepthMaps, FColor::Emerald);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, ShadowDepths, "ShadowDepths");

	if (DynamicShadowsTaskData)
	{
		FinishDynamicShadowMeshPassSetup(GraphBuilder, DynamicShadowsTaskData);
	}

	// Begin new deferred culling batching scope to catch shadow render passes, as there can use dynamic primitives that have not been uploaded before 
	// the previous batching scope. Also flushes the culling views registered during the setup (in InitViewsAfterPrepass) that are referenced in the shadow view
	// culling.
	InstanceCullingManager.BeginDeferredCulling(GraphBuilder);

	const bool bNaniteEnabled = 
		UseNanite(ShaderPlatform) &&
		ViewFamily.EngineShowFlags.NaniteMeshes &&
		Nanite::GStreamingManager.HasResourceEntries();

	if (ShadowSceneRenderer)
	{
		ShadowSceneRenderer->RenderVirtualShadowMaps(GraphBuilder, bNaniteEnabled);
	}

	// Render non-VSM shadows
	RenderShadowDepthMapAtlases(GraphBuilder);

	const bool bUseGeometryShader = !GRHISupportsArrayIndexFromAnyShader;

	for (int32 CubemapIndex = 0; CubemapIndex < SortedShadowsForShadowDepthPass.ShadowMapCubemaps.Num(); CubemapIndex++)
	{
		FSortedShadowMapAtlas& ShadowMap = SortedShadowsForShadowDepthPass.ShadowMapCubemaps[CubemapIndex];
		FRDGTextureRef ShadowDepthTexture = GraphBuilder.RegisterExternalTexture(ShadowMap.RenderTargets.DepthTarget);
		const FIntPoint TargetSize = ShadowDepthTexture->Desc.Extent;

		check(ShadowMap.Shadows.Num() == 1);
		FProjectedShadowInfo* ProjectedShadowInfo = ShadowMap.Shadows[0];
		RDG_GPU_MASK_SCOPE(GraphBuilder, GetGPUMaskForShadow(ProjectedShadowInfo));

		FString LightNameWithLevel;
		GetLightNameForDrawEvent(ProjectedShadowInfo->GetLightSceneInfo().Proxy, LightNameWithLevel);
		RDG_EVENT_SCOPE(GraphBuilder, "Cubemap %s %u^2", *LightNameWithLevel, TargetSize.X, TargetSize.Y);

		// Only clear when we're not copying from a cached shadow map.
		if (ProjectedShadowInfo->CacheMode != SDCM_MovablePrimitivesOnly || !Scene->GetExtension<FCSMSceneExtension>().GetCachedShadowMapDataRef(ProjectedShadowInfo->GetLightSceneInfo().Id, FMath::Max(ProjectedShadowInfo->CascadeSettings.ShadowSplitIndex, 0)).bCachedShadowMapHasPrimitives)
		{
			AddClearShadowDepthPass(GraphBuilder, ShadowDepthTexture);
		}

		{
			const bool bDoParallelDispatch = IsParallelDispatchEnabled(ProjectedShadowInfo, ShaderPlatform);
			const bool bDoCrossGPUCopy = false;
			ProjectedShadowInfo->RenderDepth(GraphBuilder, this, ShadowDepthTexture, bDoParallelDispatch, bDoCrossGPUCopy);
		}

		if (bNaniteEnabled && CVarNaniteShadows.GetValueOnRenderThread())
		{
			const bool bUseHZB = (CVarNaniteShadowsUseHZB.GetValueOnRenderThread() != 0);

			FString LightName;
			GetLightNameForDrawEvent(ProjectedShadowInfo->GetLightSceneInfo().Proxy, LightName);

			{
				RDG_EVENT_SCOPE( GraphBuilder, "Nanite Cubemap %s %ux%u", *LightName, ProjectedShadowInfo->ResolutionX, ProjectedShadowInfo->ResolutionY );
				
				FRDGTextureRef RDGShadowMap = GraphBuilder.RegisterExternalTexture( ShadowMap.RenderTargets.DepthTarget, TEXT("ShadowDepthBuffer") );

				const bool bUpdateStreaming = CVarNaniteShadowsUpdateStreaming.GetValueOnRenderThread() != 0;

				FLightSceneInfo& LightSceneInfo = ProjectedShadowInfo->GetLightSceneInfo();

				FString CubeFilterName;
				if (GNaniteShowStats != 0)
				{
					// Get the base light filter name.
					CubeFilterName = Nanite::GetFilterNameForLight(LightSceneInfo.Proxy);
					CubeFilterName.Append(TEXT("_Face_"));
				}

				for (int32 CubemapFaceIndex = 0; CubemapFaceIndex < 6; CubemapFaceIndex++)
				{
					RDG_EVENT_SCOPE( GraphBuilder, "Face %u", CubemapFaceIndex );
					
					// We always render to a whole face at once
					const FIntRect ShadowViewRect = FIntRect(0, 0, TargetSize.X, TargetSize.Y);
					check(ProjectedShadowInfo->X == ShadowViewRect.Min.X);
					check(ProjectedShadowInfo->Y == ShadowViewRect.Min.Y);
					check(ProjectedShadowInfo->ResolutionX == ShadowViewRect.Max.X);
					check(ProjectedShadowInfo->ResolutionY == ShadowViewRect.Max.Y);
					check(ProjectedShadowInfo->BorderSize == 0);

					FPersistentShadowStateKey ShadowKey;
					ShadowKey.ProjectionId = CubemapFaceIndex;
					ShadowKey.SubjectPrimitiveComponentIndex = 0;

					FPersistentShadowState* PrevShadowState = LightSceneInfo.PrevPersistentShadows.Find(ShadowKey);

					const FViewInfo& SceneView = Views[0];

					Nanite::FSharedContext SharedContext{};
					SharedContext.FeatureLevel = Scene->GetFeatureLevel();
					SharedContext.ShaderMap = GetGlobalShaderMap(SharedContext.FeatureLevel);
					SharedContext.Pipeline = Nanite::ERasterPipeline::Shadows;
					
					FRDGTextureRef PrevHZB = (bUseHZB && PrevShadowState && PrevShadowState->HZB) ? GraphBuilder.RegisterExternalTexture(PrevShadowState->HZB) : nullptr;

					Nanite::FConfiguration CullingConfig = { 0 };
					CullingConfig.bIsShadowPass	= true;
					CullingConfig.bTwoPassOcclusion	= true;
					CullingConfig.bUpdateStreaming	= bUpdateStreaming;
					if (ProjectedShadowInfo->CacheMode == SDCM_StaticPrimitivesOnly)
					{
						CullingConfig.HiddenFilterFlags = Nanite::EFilterFlags::NonStaticMobility;
					}
					else if (ProjectedShadowInfo->CacheMode == SDCM_MovablePrimitivesOnly)
					{
						CullingConfig.HiddenFilterFlags = Nanite::EFilterFlags::StaticMobility;
					}
					CullingConfig.SetViewFlags(SceneView);

					FString CubeFaceFilterName;
					if (GNaniteShowStats != 0)
					{
						CubeFaceFilterName = CubeFilterName;
						CubeFaceFilterName.AppendInt(CubemapFaceIndex);

						CullingConfig.bExtractStats = Nanite::IsStatFilterActive(CubeFaceFilterName);
					}

					const Nanite::FRasterContextInitParams RasterInitParams =
					{
						.TextureSize = TargetSize,
						.TextureRect = ShadowViewRect,
						.RasterMode = Nanite::EOutputBufferMode::DepthOnly
					};
					Nanite::FRasterContext RasterContext = Nanite::InitRasterContext(GraphBuilder, SharedContext, ViewFamily, RasterInitParams);

					auto NaniteRenderer = Nanite::IRenderer::Create(
						GraphBuilder,
						*Scene,
						SceneView,
						GetSceneUniforms(),
						SharedContext,
						RasterContext,
						CullingConfig,
						ShadowViewRect,
						PrevHZB
					);

					// Setup packed view
					Nanite::FPackedViewArray* PackedViews;

					{
						Nanite::FPackedViewParams Params;
						Params.ViewMatrices = ProjectedShadowInfo->GetShadowDepthRenderingViewMatrices(CubemapFaceIndex);
						Params.ViewRect = ShadowViewRect;
						Params.RasterContextSize = TargetSize;
						Params.PrevViewMatrices = Params.ViewMatrices;
						Params.HZBTestViewRect = ShadowViewRect;
						Params.MaxPixelsPerEdgeMultipler = 1.0f / FShadowSceneRenderer::ComputeNaniteShadowsLODScaleFactor();
						Nanite::SetViewFrameIndex(&SceneView, Params);

						// Cubemap shadows reverse the cull mode due to the face matrices (see FShadowDepthPassMeshProcessor::AddMeshBatch)
						Params.Flags |= NANITE_VIEW_FLAG_REVERSE_CULLING;

						UpdatePackedViewParamsFromPrevShadowState(Params, PrevShadowState);

						PackedViews = Nanite::FPackedViewArray::Create(GraphBuilder, Nanite::CreatePackedView(Params));
					}

					NaniteRenderer->DrawGeometry(
						Scene->NaniteRasterPipelines[ENaniteMeshPass::BasePass],
						nullptr,
						*PackedViews
					);

					Nanite::EmitCubemapShadow(
						GraphBuilder,
						SharedContext,
						RasterContext,
						RDGShadowMap,
						ShadowViewRect,
						CubemapFaceIndex,
						bUseGeometryShader);
										
					TRefCountPtr<IPooledRenderTarget> HZB;
					if (bUseHZB)
					{
						FRDGTextureRef FurthestHZBTexture;
						BuildHZBFurthest(
							GraphBuilder,
							GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy),
							RasterContext.DepthBuffer,
							ShadowViewRect,
							FeatureLevel,
							ShaderPlatform,
							TEXT("Shadow.CubemapHZB"),
							/* OutFurthestHZBTexture = */ &FurthestHZBTexture);

						HZB = GraphBuilder.ConvertToExternalTexture(FurthestHZBTexture);
					}
					UpdateCurrentFrameHZB(LightSceneInfo, ShadowKey, ProjectedShadowInfo, HZB, CubemapFaceIndex);
				}
			}
		}

		// Make readable because ShadowDepthTexture is not tracked via RDG yet
		ShadowMap.RenderTargets.DepthTarget = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, ShadowDepthTexture);
	}

	if (SortedShadowsForShadowDepthPass.PreshadowCache.Shadows.Num() > 0)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "PreshadowCache");

		FRDGTextureRef PreshadowCacheTexture = GraphBuilder.RegisterExternalTexture(SortedShadowsForShadowDepthPass.PreshadowCache.RenderTargets.DepthTarget);

		for (FProjectedShadowInfo* ProjectedShadowInfo : SortedShadowsForShadowDepthPass.PreshadowCache.Shadows)
		{
			if (!ProjectedShadowInfo->bDepthsCached)
			{
				RDG_GPU_MASK_SCOPE(GraphBuilder, GetGPUMaskForShadow(ProjectedShadowInfo));
				AddClearShadowDepthPass(GraphBuilder, PreshadowCacheTexture, ProjectedShadowInfo);

				const bool bParallelDispatch = IsParallelDispatchEnabled(ProjectedShadowInfo, ShaderPlatform);
				const bool bDoCrossGPUCopy = true;
				ProjectedShadowInfo->RenderDepth(GraphBuilder, this, PreshadowCacheTexture, bParallelDispatch, bDoCrossGPUCopy);
				ProjectedShadowInfo->bDepthsCached = true;
			}
		}
	}

	for (int32 AtlasIndex = 0; AtlasIndex < SortedShadowsForShadowDepthPass.TranslucencyShadowMapAtlases.Num(); AtlasIndex++)
	{
		const FSortedShadowMapAtlas& ShadowMapAtlas = SortedShadowsForShadowDepthPass.TranslucencyShadowMapAtlases[AtlasIndex];

		FRDGTextureRef ColorTarget0 = GraphBuilder.RegisterExternalTexture(ShadowMapAtlas.RenderTargets.ColorTargets[0]);
		FRDGTextureRef ColorTarget1 = GraphBuilder.RegisterExternalTexture(ShadowMapAtlas.RenderTargets.ColorTargets[1]);
		const FIntPoint TargetSize  = ColorTarget0->Desc.Extent;

		FRenderTargetBindingSlots RenderTargets;
		RenderTargets[0] = FRenderTargetBinding(ColorTarget0, ERenderTargetLoadAction::ELoad);
		RenderTargets[1] = FRenderTargetBinding(ColorTarget1, ERenderTargetLoadAction::ELoad);

		RDG_EVENT_SCOPE(GraphBuilder, "TranslucencyAtlas%u %u^2", AtlasIndex, TargetSize.X, TargetSize.Y);

		for (FProjectedShadowInfo* ProjectedShadowInfo : ShadowMapAtlas.Shadows)
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, GetGPUMaskForShadow(ProjectedShadowInfo));
			ProjectedShadowInfo->RenderTranslucencyDepths(GraphBuilder, this, RenderTargets, InstanceCullingManager);
		}
	}

	// Move current persistent shadow state to previous and clear current.
	GraphBuilder.AddSetupTask([Scene = Scene]
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CopyPersistentLightState);
		// TODO: This could be very slow.
		for (const FLightSceneInfoCompact& Light : Scene->Lights)
		{
			Light.LightSceneInfo->PrevPersistentShadows = Light.LightSceneInfo->PersistentShadows;
			Light.LightSceneInfo->PersistentShadows.Empty();
		}

	}, &GPersistentShadowsPipe);
	
	if (ShouldRenderHeterogeneousVolumes(Scene) && ShouldHeterogeneousVolumesCastShadows() &&
		(HeterogeneousVolumes::GetShadowPipeline() == HeterogeneousVolumes::EShadowPipeline::LiveShading))
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			FViewInfo& View = Views[ViewIndex];
			RenderAdaptiveVolumetricShadowMapWithLiveShading(
				GraphBuilder,
				View.GetSceneTextures(),
				Scene,
				View,
				VisibleLightInfos
			);
		}
	}
	}

	// Begin another deferred batch for any instance culling after ShadowDepths. Orininally introduced to work around a bug (now fixed) 
	// that would cause subsequent passes enabling HZB for all passes. Left in place because it can under some circumstances be a small 
	// performance win, possibly by changing the batching allowing a different permutation to be picked.
	if (FInstanceCullingContext::IsOcclusionCullingEnabled() && CVarShouldBeginDeferredCullingAfterShadowRendering.GetValueOnRenderThread() != 0)
	{
		InstanceCullingManager.BeginDeferredCulling(GraphBuilder);
	}
	bShadowDepthRenderCompleted = true;
}

ERasterizerCullMode SetupShadowCullMode(
	ERHIFeatureLevel::Type FeatureLevel,
	EShadowDepthType ShadowDepthType,
	const FMaterial& Material,
	const ERasterizerCullMode MeshCullMode,
	bool bCastShadowAsTwoSided)
{
	const bool bTwoSided = Material.IsTwoSided() || bCastShadowAsTwoSided;

	const bool bRenderSceneTwoSided = bTwoSided;
	const bool bShadowReversesCulling = EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::VSM) ? false : EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::OnePassPoint);
	const bool bReverseCullMode = bShadowReversesCulling;

	return bRenderSceneTwoSided ? CM_None : bReverseCullMode ? FMeshPassProcessor::InverseCullMode(MeshCullMode) : MeshCullMode;
}

void FShadowDepthPassMeshProcessor::CollectPSOInitializersInternal(
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FMaterial& RESTRICT MaterialResource,
	EShadowDepthType InShadowDepthType,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	bool bSupportsPositionAndNormalOnlyStream,
	bool bRequired,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	TMeshProcessorShaders<
		FShadowDepthVS,
		FShadowDepthBasePS> ShadowDepthPassShaders;
	if (!GetShadowDepthPassShaders(
		MaterialResource,
		VertexFactoryData.VertexFactoryType,
		FeatureLevel,
		InShadowDepthType,
		bSupportsPositionAndNormalOnlyStream,
		MaterialResource.MaterialModifiesMeshPosition_GameThread(),
		ShadowDepthPassShaders.VertexShader,
		ShadowDepthPassShaders.PixelShader))
	{
		return;
	}

	const bool bVFTypeSupportsNullPixelShader = VertexFactoryData.VertexFactoryType->SupportsNullPixelShader();
	const bool bUsePositionOnlyVS =
		bSupportsPositionAndNormalOnlyStream
		&& MaterialResource.WritesEveryPixel(true, bVFTypeSupportsNullPixelShader)
		&& !MaterialResource.MaterialModifiesMeshPosition_GameThread();
	
	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;	
	if (MeshPassType != EMeshPass::VSMShadowDepth)
	{
		ETextureCreateFlags ShadowMapCreateFlags = TexCreate_DepthStencilTargetable | TexCreate_ShaderResource;
		ShadowMapCreateFlags |= GFastVRamConfig.ShadowPointLight;
		if (EnumHasAnyFlags(InShadowDepthType, EShadowDepthType::OnePassPoint))
		{
			ShadowMapCreateFlags |= TexCreate_NoFastClear;
		}

		RenderTargetsInfo.NumSamples = 1;
		SetupDepthStencilInfo(PF_ShadowDepth, ShadowMapCreateFlags, ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop, RenderTargetsInfo);
	}

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		MaterialResource,
		PassDrawRenderState,
		RenderTargetsInfo,
		ShadowDepthPassShaders,
		MeshFillMode,
		MeshCullMode,
		PT_TriangleList,
		bUsePositionOnlyVS ? EMeshPassFeatures::PositionAndNormalOnly : EMeshPassFeatures::Default,
		bRequired,
		PSOInitializers);
}

bool FShadowDepthPassMeshProcessor::Process(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	int32 StaticMeshId,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		FShadowDepthVS,
		FShadowDepthBasePS> ShadowDepthPassShaders;
	if (!GetShadowDepthPassShaders(
		MaterialResource,
		VertexFactory->GetType(),
		FeatureLevel,
		ShadowDepthType,
		VertexFactory->SupportsPositionAndNormalOnlyStream(),
		MaterialResource.MaterialModifiesMeshPosition_RenderThread(),
		ShadowDepthPassShaders.VertexShader,
		ShadowDepthPassShaders.PixelShader))
	{
		return false;
	}

	FShadowDepthShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(ShadowDepthPassShaders.VertexShader, ShadowDepthPassShaders.PixelShader);

	const bool bUseGpuSceneInstancing = UseGPUScene(GShaderPlatformForFeatureLevel[FeatureLevel]) && VertexFactory->SupportsGPUScene(FeatureLevel);

	const bool bUsePositionOnlyVS =
		VertexFactory->SupportsPositionAndNormalOnlyStream()
		&& MaterialResource.WritesEveryPixel(true, VertexFactory->SupportsNullPixelShader())
		&& !MaterialResource.MaterialModifiesMeshPosition_RenderThread();

	// Need to replicate for cube faces on host if GPU-scene is not available (for this draw).
	const bool bPerformHostCubeFaceReplication = EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::OnePassPoint) && !bUseGpuSceneInstancing;
	const uint32 InstanceFactor = bPerformHostCubeFaceReplication ? 6 : 1;

	for (uint32 i = 0; i < InstanceFactor; i++)
	{
		ShaderElementData.LayerId = i;
		ShaderElementData.bUseGpuSceneInstancing = bUseGpuSceneInstancing;

		BuildMeshDrawCommands(
			MeshBatch,
			BatchElementMask,
			PrimitiveSceneProxy,
			MaterialRenderProxy,
			MaterialResource,
			PassDrawRenderState,
			ShadowDepthPassShaders,
			MeshFillMode,
			MeshCullMode,
			SortKey,
			bUsePositionOnlyVS ? EMeshPassFeatures::PositionAndNormalOnly : EMeshPassFeatures::Default,
			ShaderElementData);
	}

	return true;
}

bool FShadowDepthPassMeshProcessor::TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	const bool bShouldCastShadow = Material.ShouldCastDynamicShadows();

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);

	const ERasterizerCullMode MeshCullMode = FMeshPassProcessor::ComputeMeshCullMode(Material, OverrideSettings);
	ERasterizerCullMode FinalCullMode = SetupShadowCullMode(FeatureLevel, ShadowDepthType, Material, MeshCullMode, PrimitiveSceneProxy->CastsShadowAsTwoSided());

	bool bResult = true;
	if (bShouldCastShadow
		&& GetShadowQuality() != 0
		&& ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain())
		&& ShouldIncludeMaterialInDefaultOpaquePass(Material)
		&& EnumHasAnyFlags(MeshSelectionMask, MeshBatch.VertexFactory->SupportsGPUScene(FeatureLevel) ? EShadowMeshSelection::VSM : EShadowMeshSelection::SM))
	{
		const FMaterialRenderProxy* EffectiveMaterialRenderProxy = &MaterialRenderProxy;
		const FMaterial* EffectiveMaterial = &Material;
		const bool bVFTypeSupportsNullPixelShader = MeshBatch.VertexFactory->SupportsNullPixelShader();
		const bool bModifiesMeshPosition = DoMaterialAndPrimitiveModifyMeshPosition(Material, PrimitiveSceneProxy);

		if (UseDefaultMaterialForShadowDepth(Material, bVFTypeSupportsNullPixelShader, bModifiesMeshPosition))
		{
			const FMaterialRenderProxy* DefaultProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			const FMaterial* DefaultMaterialResource = DefaultProxy->GetMaterialNoFallback(FeatureLevel);
			check(DefaultMaterialResource);

			// Override with the default material for opaque materials that don't modify mesh position.
			EffectiveMaterialRenderProxy = DefaultProxy;
			EffectiveMaterial = DefaultMaterialResource;
		}

		bResult = Process(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, *EffectiveMaterialRenderProxy, *EffectiveMaterial, MeshFillMode, FinalCullMode);
	}

	return bResult;
}

void FShadowDepthPassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (MeshBatch.CastShadow)
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material)
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}
			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

void FShadowDepthPassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	// Early out if possible
	if (!PreCacheParams.bDefaultMaterial && 
		(GetShadowQuality() == 0 ||
		!Material.ShouldCastDynamicShadows() ||
		Material.GetMaterialDomain() != MD_Surface ||
		!ShouldIncludeMaterialInDefaultOpaquePass(Material) ||
		!EnumHasAnyFlags(MeshSelectionMask, VertexFactoryData.VertexFactoryType->SupportsPrimitiveIdStream() ? EShadowMeshSelection::VSM : EShadowMeshSelection::SM)))
	{
		return;
	}

	// Are we currently collecting PSO's for the default material
	if (PreCacheParams.bDefaultMaterial)
	{
		ERasterizerFillMode MeshFillMode = FM_Solid;
		bool bCastShadowsAsTwoSided = false;

		// Collect for each possible mesh cull mode	
		CollectPSOInitializersForEachShadowDepthType(VertexFactoryData, Material, MeshFillMode, CM_None, bCastShadowsAsTwoSided, PSOInitializers);
		CollectPSOInitializersForEachShadowDepthType(VertexFactoryData, Material, MeshFillMode, CM_CW, bCastShadowsAsTwoSided, PSOInitializers);
		CollectPSOInitializersForEachShadowDepthType(VertexFactoryData, Material, MeshFillMode, CM_CCW, bCastShadowsAsTwoSided, PSOInitializers);
	}
	else if (PreCacheParams.bCastShadow)
	{
		bool bCollectPSOs = true;
		const bool bVFTypeSupportsNullPixelShader = VertexFactoryData.VertexFactoryType->SupportsNullPixelShader();
		const FMaterial* EffectiveMaterial = &Material;
		if (UseDefaultMaterialForShadowDepth(Material, bVFTypeSupportsNullPixelShader, Material.MaterialModifiesMeshPosition_GameThread()))
		{
			if (VertexFactoryData.CustomDefaultVertexDeclaration)
			{
				EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
				EffectiveMaterial = UMaterial::GetDefaultMaterial(MD_Surface)->GetMaterialResource(SceneTexturesConfig.ShaderPlatform, ActiveQualityLevel);
			}
			else
			{
				bCollectPSOs = false;
			}
		}

		if (bCollectPSOs)
		{
			const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
			const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
			const ERasterizerCullMode MeshCullMode = FMeshPassProcessor::ComputeMeshCullMode(Material, OverrideSettings);
			CollectPSOInitializersForEachShadowDepthType(VertexFactoryData, *EffectiveMaterial, MeshFillMode, MeshCullMode, PreCacheParams.bCastShadowAsTwoSided, PSOInitializers);
		}
	}
}

void FShadowDepthPassMeshProcessor::CollectPSOInitializersForEachShadowDepthType(
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FMaterial& RESTRICT Material,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	bool bCastShadowAsTwoSided,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	const EShadowDepthType ShadowDepthVSMBit = ShadowDepthType & EShadowDepthType::VSM;
	bool bRequired = true;

	// Collect for directional shadows
	{
		const EShadowDepthType LocalShadowDepthType = ShadowDepthVSMBit | EShadowDepthType::Directional;
		ERasterizerCullMode FinalCullMode = SetupShadowCullMode(FeatureLevel, LocalShadowDepthType, Material, MeshCullMode, bCastShadowAsTwoSided);
		CollectPSOInitializersForEachStreamSetup(VertexFactoryData, Material, LocalShadowDepthType, MeshFillMode, FinalCullMode, bRequired, PSOInitializers);
	}

	// Collect for non-directional one pass point light shadows
	{
		const EShadowDepthType LocalShadowDepthType = ShadowDepthVSMBit | EShadowDepthType::OnePassPoint;
		ERasterizerCullMode FinalCullMode = SetupShadowCullMode(FeatureLevel, LocalShadowDepthType, Material, MeshCullMode, bCastShadowAsTwoSided);
		CollectPSOInitializersForEachStreamSetup(VertexFactoryData, Material, LocalShadowDepthType, MeshFillMode, FinalCullMode, bRequired, PSOInitializers);
	}

	// Collect for non-directional non-one pass point light shadows
	{
		const EShadowDepthType LocalShadowDepthType = ShadowDepthVSMBit | EShadowDepthType::Spot;
		ERasterizerCullMode FinalCullMode = SetupShadowCullMode(FeatureLevel, LocalShadowDepthType, Material, MeshCullMode, bCastShadowAsTwoSided);
		CollectPSOInitializersForEachStreamSetup(VertexFactoryData, Material, LocalShadowDepthType, MeshFillMode, FinalCullMode, bRequired, PSOInitializers);
	}
}

void FShadowDepthPassMeshProcessor::CollectPSOInitializersForEachStreamSetup(
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FMaterial& RESTRICT MaterialResource,
	EShadowDepthType InShadowDepthType,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	bool bRequired,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	SetStateForShadowDepth(InShadowDepthType, PassDrawRenderState);

	// Collect for when both use cases to be complete
	// Ideally position and normal stream is always available or can be checked on VF to be always available to reduce PSO precache count
	bool SupportsPositionAndNormalOnlyStream = true;
	if (VertexFactoryData.VertexFactoryType->SupportsPositionOnly())
	{
		CollectPSOInitializersInternal(VertexFactoryData, MaterialResource, InShadowDepthType, MeshFillMode, MeshCullMode, SupportsPositionAndNormalOnlyStream, bRequired, PSOInitializers);
	}
	
	// Always precache with the default streams (non PositionOnly or PositionAndNormalOnly stream)
	SupportsPositionAndNormalOnlyStream = false;
	CollectPSOInitializersInternal(VertexFactoryData, MaterialResource, InShadowDepthType, MeshFillMode, MeshCullMode, SupportsPositionAndNormalOnlyStream, bRequired, PSOInitializers);
}

FShadowDepthPassMeshProcessor::FShadowDepthPassMeshProcessor(
	const FScene* Scene,
	const ERHIFeatureLevel::Type InFeatureLevel,
	const FSceneView* InViewIfDynamicMeshCommand,
	EShadowDepthType InShadowDepthType,
	FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(GetShadowMeshPassType(InShadowDepthType), Scene, InFeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, ShadowDepthType(InShadowDepthType)
{
	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InFeatureLevel);
	if (UseNonNaniteVirtualShadowMaps(ShaderPlatform, FeatureLevel))
	{
		// set up mesh filtering.
		MeshSelectionMask = EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::VSM) ? EShadowMeshSelection::VSM : EShadowMeshSelection::SM;
	}
	else
	{
		// If VSMs are disabled, pipe all kinds of draws into the regular SMs
		MeshSelectionMask = EShadowMeshSelection::All;
	}
	SetStateForShadowDepth(ShadowDepthType, PassDrawRenderState);
}

EMeshPass::Type GetShadowMeshPassType(EShadowDepthType ShadowDepthType)
{
	if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::VSM))
	{
		return EMeshPass::VSMShadowDepth;
	}

	if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::OnePassPoint))
	{
		return EMeshPass::OnePassPointLightShadowDepth;
	}

	return EMeshPass::CSMShadowDepth;
}

bool UseCachedMeshDrawCommands(EShadowDepthType ShadowDepthType)
{
	if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::VSM | EShadowDepthType::Directional))
	{
		return true;
	}

	if (EnumHasAnyFlags(ShadowDepthType, EShadowDepthType::OnePassPoint) && GShadowOnePassPointLightUseCachedMDCs)
	{
		return true;
	}

	return false;
}

FMeshPassProcessor* CreateCSMShadowDepthPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FShadowDepthPassMeshProcessor(
		Scene,
		FeatureLevel,
		InViewIfDynamicMeshCommand,
		EShadowDepthType::Directional,
		InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(CSMShadowDepthPass, CreateCSMShadowDepthPassProcessor, EShadingPath::Deferred, EMeshPass::CSMShadowDepth, EMeshPassFlags::CachedMeshCommands);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(CSMMobileShadowDepthPass, CreateCSMShadowDepthPassProcessor, EShadingPath::Mobile, EMeshPass::CSMShadowDepth, EMeshPassFlags::CachedMeshCommands);

FMeshPassProcessor* CreateOnePassPointLightShadowDepthPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);

	if (GShadowOnePassPointLightUseCachedMDCs && DoesRuntimeSupportOnePassPointLightShadows(ShaderPlatform))
	{
		return new FShadowDepthPassMeshProcessor(
			Scene,
			FeatureLevel,
			InViewIfDynamicMeshCommand,
			EShadowDepthType::OnePassPoint,
			InDrawListContext);
	}
	return nullptr;
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(OnePassPointLightShadowDepthPass, CreateOnePassPointLightShadowDepthPassProcessor, EShadingPath::Deferred, EMeshPass::OnePassPointLightShadowDepth, EMeshPassFlags::CachedMeshCommands);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(OnePassPointLightMobileShadowDepthPass, CreateOnePassPointLightShadowDepthPassProcessor, EShadingPath::Mobile, EMeshPass::OnePassPointLightShadowDepth, EMeshPassFlags::CachedMeshCommands);

FMeshPassProcessor* CreateVSMShadowDepthPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);

	if (UseNonNaniteVirtualShadowMaps(ShaderPlatform, FeatureLevel))
	{
		return new FShadowDepthPassMeshProcessor(
			Scene,
			FeatureLevel,
			InViewIfDynamicMeshCommand,
			EShadowDepthType::VSM | EShadowDepthType::Directional,
			InDrawListContext);
	}

	return nullptr;
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(VSMShadowDepthPass, CreateVSMShadowDepthPassProcessor, EShadingPath::Deferred, EMeshPass::VSMShadowDepth, EMeshPassFlags::CachedMeshCommands);