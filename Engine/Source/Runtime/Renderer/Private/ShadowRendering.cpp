// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowRendering.cpp: Shadow rendering implementation
=============================================================================*/

#include "ShadowRendering.h"
#include "PrimitiveViewRelevance.h"
#include "DepthRendering.h"
#include "SceneRendering.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "HairStrands/HairStrandsRendering.h"
#include "VirtualShadowMaps/VirtualShadowMapProjection.h"
#include "Shadows/ShadowSceneRenderer.h"
#include "Shadows/ScreenSpaceShadows.h"
#include "Shadows/FirstPersonSelfShadow.h"
#include "RenderCore.h"
#include "TranslucentLighting.h"
#include "MobileBasePassRendering.h"
#include "RHIResourceUtils.h"
#include "Substrate/Substrate.h"
#include "SimpleMeshDrawCommandPass.h"

using namespace LightFunctionAtlas;
using namespace ShadowRendering;

///////////////////////////////////////////////////////////////////////////////////////////////////
// Directional light
static TAutoConsoleVariable<float> CVarCSMShadowDepthBias(
	TEXT("r.Shadow.CSMDepthBias"),
	10.0f,
	TEXT("Constant depth bias used by CSM"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarCSMShadowSlopeScaleDepthBias(
	TEXT("r.Shadow.CSMSlopeScaleDepthBias"),
	3.0f,
	TEXT("Slope scale depth bias used by CSM"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarPerObjectDirectionalShadowDepthBias(
	TEXT("r.Shadow.PerObjectDirectionalDepthBias"),
	10.0f,
	TEXT("Constant depth bias used by per-object shadows from directional lights\n")
	TEXT("Lower values give better shadow contact, but increase self-shadowing artifacts"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarPerObjectDirectionalShadowSlopeScaleDepthBias(
	TEXT("r.Shadow.PerObjectDirectionalSlopeDepthBias"),
	3.0f,
	TEXT("Slope scale depth bias used by per-object shadows from directional lights\n")
	TEXT("Lower values give better shadow contact, but increase self-shadowing artifacts"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarCSMSplitPenumbraScale(
	TEXT("r.Shadow.CSMSplitPenumbraScale"),
	0.5f,
	TEXT("Scale applied to the penumbra size of Cascaded Shadow Map splits, useful for minimizing the transition between splits"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarCSMDepthBoundsTest(
	TEXT("r.Shadow.CSMDepthBoundsTest"),
	1,
	TEXT("Whether to use depth bounds tests rather than stencil tests for the CSM bounds"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarShadowTransitionScale(
	TEXT("r.Shadow.TransitionScale"),
	60.0f,
	TEXT("This controls the 'fade in' region between a caster and where its shadow shows up.  Larger values make a smaller region which will have more self shadowing artifacts"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarCSMShadowReceiverBias(
	TEXT("r.Shadow.CSMReceiverBias"),
	0.9f,
	TEXT("Receiver bias used by CSM. Value between 0 and 1."),
	ECVF_RenderThreadSafe);


///////////////////////////////////////////////////////////////////////////////////////////////////
// Point light
static TAutoConsoleVariable<float> CVarPointLightShadowDepthBias(
	TEXT("r.Shadow.PointLightDepthBias"),
	0.02f,
	TEXT("Depth bias that is applied in the depth pass for shadows from point lights. (0.03 avoids peter paning but has some shadow acne)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarPointLightShadowSlopeScaleDepthBias(
	TEXT("r.Shadow.PointLightSlopeScaleDepthBias"),
	3.0f,
	TEXT("Slope scale depth bias that is applied in the depth pass for shadows from point lights"),
	ECVF_RenderThreadSafe);


///////////////////////////////////////////////////////////////////////////////////////////////////
// Rect light
static TAutoConsoleVariable<float> CVarRectLightShadowDepthBias(
	TEXT("r.Shadow.RectLightDepthBias"),
	0.025f,
	TEXT("Depth bias that is applied in the depth pass for shadows from rect lights. (0.03 avoids peter paning but has some shadow acne)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarRectLightShadowSlopeScaleDepthBias(
	TEXT("r.Shadow.RectLightSlopeScaleDepthBias"),
	2.5f,
	TEXT("Slope scale depth bias that is applied in the depth pass for shadows from rect lights"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarRectLightShadowReceiverBias(
	TEXT("r.Shadow.RectLightReceiverBias"),
	0.3f,
	TEXT("Receiver bias used by rect light. Value between 0 and 1."),
	ECVF_RenderThreadSafe);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Spot light
static TAutoConsoleVariable<float> CVarSpotLightShadowDepthBias(
	TEXT("r.Shadow.SpotLightDepthBias"),
	3.0f,
	TEXT("Depth bias that is applied in the depth pass for whole-scene projected shadows from spot lights"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSpotLightShadowSlopeScaleDepthBias(
	TEXT("r.Shadow.SpotLightSlopeDepthBias"),
	3.0f,
	TEXT("Slope scale depth bias that is applied in the depth pass for whole-scene projected shadows from spot lights"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarPerObjectSpotLightShadowDepthBias(
	TEXT("r.Shadow.PerObjectSpotLightDepthBias"),
	3.0f,
	TEXT("Depth bias that is applied in the depth pass for per-object projected shadows from spot lights"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarPerObjectSpotLightShadowSlopeScaleDepthBias(
	TEXT("r.Shadow.PerObjectSpotLightSlopeDepthBias"),
	3.0f,
	TEXT("Slope scale depth bias that is applied in the depth pass for per-object projected shadows from spot lights"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSpotLightShadowTransitionScale(
	TEXT("r.Shadow.SpotLightTransitionScale"),
	60.0f,
	TEXT("Transition scale for spotlights"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSpotLightShadowReceiverBias(
	TEXT("r.Shadow.SpotLightReceiverBias"),
	0.5f,
	TEXT("Receiver bias used by spotlights. Value between 0 and 1."),
	ECVF_RenderThreadSafe);


///////////////////////////////////////////////////////////////////////////////////////////////////
// General
static TAutoConsoleVariable<int32> CVarEnableModulatedSelfShadow(
	TEXT("r.Shadow.EnableModulatedSelfShadow"),
	0,
	TEXT("Allows modulated shadows to affect the shadow caster. (mobile only)"),
	ECVF_RenderThreadSafe);

static int GStencilOptimization = 1;
static FAutoConsoleVariableRef CVarStencilOptimization(
	TEXT("r.Shadow.StencilOptimization"),
	GStencilOptimization,
	TEXT("Removes stencil clears between shadow projections by zeroing the stencil during testing"),
	ECVF_RenderThreadSafe
	);


static int GShadowStencilCulling = 1;
static FAutoConsoleVariableRef CVarGShadowStencilCulling(
	TEXT("r.Shadow.StencilCulling"),
	GShadowStencilCulling,
	TEXT("Whether to use stencil light culling during shadow projection (default) or only depth."),
	ECVF_RenderThreadSafe
);

// Workaround for UE-300339
static TAutoConsoleVariable<int32> CVarShadowStencilCullingArtifactWorkaround(
	TEXT("r.Shadow.StencilCullingArtifactWorkaround"),
	1,
	TEXT("Workaround for a seemingly platform specific artifact (tile shaped artifacts) when both Substrate and r.Shadow.StencilCulling are enabled. The workaround forces a dummy pixel shader with late-Z in the stencil masking pass, which seems to fix the bug."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFilterMethod(
	TEXT("r.Shadow.FilterMethod"),
	0,
	TEXT("Chooses the shadow filtering method.\n")
	TEXT(" 0: Uniform PCF (default)\n")
	TEXT(" 1: PCSS (experimental)\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMaxSoftKernelSize(
	TEXT("r.Shadow.MaxSoftKernelSize"),
	40,
	TEXT("Mazimum size of the softening kernels in pixels."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarShadowMaxSlopeScaleDepthBias(
	TEXT("r.Shadow.ShadowMaxSlopeScaleDepthBias"),
	1.0f,
	TEXT("Max Slope depth bias used for shadows for all lights\n")
	TEXT("Higher values give better self-shadowing, but increase self-shadowing artifacts"),
	ECVF_RenderThreadSafe);

DECLARE_GPU_DRAWCALL_STAT_NAMED(DistanceFieldShadows, TEXT("DistanceField Shadows"));

const FVector4f GDummyWholeSceneDirectionalShadowStencilVertexBufferData[] =
{
	// Far Plane
	FVector4f( 1,  1,  1 /* StencilFar */),
	FVector4f(-1,  1,  1),
	FVector4f( 1, -1,  1),
	FVector4f( 1, -1,  1),
	FVector4f(-1,  1,  1),
	FVector4f(-1, -1,  1),

	// Near Plane
	FVector4f(-1,  1, -1 /* StencilNear */),
	FVector4f( 1,  1, -1),
	FVector4f(-1, -1, -1),
	FVector4f(-1, -1, -1),
	FVector4f( 1,  1, -1),
	FVector4f( 1, -1, -1),
};

/**
 * A dummy vertex buffer to bind when rendering whole scene shadows. This
 * prevents some D3D debug warnings about zero-element input layouts, but is not
 * strictly required.  It is, however, required for Metal.
 */
class FDummyWholeSceneDirectionalShadowStencilVertexBuffer : public FVertexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FDummyWholeSceneDirectionalShadowStencilVertexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(GDummyWholeSceneDirectionalShadowStencilVertexBufferData));
	}
};

TGlobalResource<FDummyWholeSceneDirectionalShadowStencilVertexBuffer> GDummyWholeSceneDirectionalShadowStencilVertexBuffer;

///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair
static TAutoConsoleVariable<int32> CVarHairStrandsCullPerObjectShadowCaster(
	TEXT("r.HairStrands.Shadow.CullPerObjectShadowCaster"),
	1,
	TEXT("Enable CPU culling of object casting per-object shadow (stationnary object)"),
	ECVF_RenderThreadSafe);

DEFINE_GPU_STAT(ShadowProjection);

// 0:off, 1:low, 2:med, 3:high, 4:very high, 5:max
uint32 GetShadowQuality()
{
	static const auto ICVarQuality = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ShadowQuality"));

	int Ret = ICVarQuality->GetValueOnAnyThread();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	static const auto ICVarLimit = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.LimitRenderingFeatures"));
	if(ICVarLimit)
	{
		int32 Limit = ICVarLimit->GetValueOnAnyThread();

		if(Limit > 2)
		{
			Ret = 0;
		}
	}
#endif

	return FMath::Clamp(Ret, 0, 5);
}

void GetOnePassPointShadowProjectionParameters(FRDGBuilder& GraphBuilder, const FProjectedShadowInfo* ShadowInfo, FOnePassPointShadowProjection& OutParameters)
{
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	//@todo DynamicGI: remove duplication with FOnePassPointShadowProjectionShaderParameters
	FRDGTexture* ShadowDepthTextureValue = ShadowInfo
		? GraphBuilder.RegisterExternalTexture(ShadowInfo->RenderTargets.DepthTarget)
		: SystemTextures.BlackDepthCube;

	OutParameters.ShadowDepthCubeTexture = ShadowDepthTextureValue;
	OutParameters.ShadowDepthCubeTexture2 = ShadowDepthTextureValue;
	// Use a comparison sampler to do hardware PCF
	OutParameters.ShadowDepthCubeTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, 0, 0, SCF_Less>::GetRHI();

	if (ShadowInfo)
	{
		for (int32 i = 0; i < ShadowInfo->OnePassShadowViewProjectionMatrices.Num(); i++)
		{
			OutParameters.ShadowViewProjectionMatrices[i] = FMatrix44f(ShadowInfo->OnePassShadowViewProjectionMatrices[i]);	// LWC_TODO: Precision loss
		}

		OutParameters.InvShadowmapResolution = 1.0f / ShadowInfo->ResolutionX;
	}
	else
	{
		FPlatformMemory::Memzero(&OutParameters.ShadowViewProjectionMatrices[0], sizeof(OutParameters.ShadowViewProjectionMatrices));
		OutParameters.InvShadowmapResolution = 0;
	}
}

/*-----------------------------------------------------------------------------
	FShadowVolumeBoundProjectionVS
-----------------------------------------------------------------------------*/

void FShadowVolumeBoundProjectionVS::SetParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FSceneView& View,
	const FProjectedShadowInfo* ShadowInfo,
	EShadowProjectionVertexShaderFlags Flags)
{
	if(ShadowInfo->IsWholeScenePointLightShadow())
	{
		// Handle stenciling sphere for point light.
		StencilingGeometryParameters.Set(BatchedParameters, View, ShadowInfo->LightSceneInfo);
	}
	else
	{
		StencilingGeometryParameters.Set(BatchedParameters, FVector4f(0,0,0,1));
	}

	if ((Flags & EShadowProjectionVertexShaderFlags::DrawingFrustum) != EShadowProjectionVertexShaderFlags::None)
	{
		const FVector PreShadowToPreView(View.ViewMatrices.GetPreViewTranslation() - ShadowInfo->PreShadowTranslation);
		SetShaderValue(BatchedParameters, InvReceiverInnerMatrix, ShadowInfo->InvReceiverInnerMatrix);
		SetShaderValue(BatchedParameters, PreShadowToPreViewTranslation, FVector4f((FVector3f)PreShadowToPreView, 0));
	}
	else
	{
		SetShaderValue(BatchedParameters, InvReceiverInnerMatrix, FMatrix44f::Identity);
		SetShaderValue(BatchedParameters, PreShadowToPreViewTranslation, FVector4f(0, 0, 0, 0));
	}
}

IMPLEMENT_TYPE_LAYOUT(FShadowProjectionPixelShaderInterface);

IMPLEMENT_SHADER_TYPE(,FShadowProjectionNoTransformVS,TEXT("/Engine/Private/ShadowProjectionVertexShader.usf"),TEXT("ShadowProjectionNoTransformVS"),SF_Vertex);

IMPLEMENT_SHADER_TYPE(,FShadowVolumeBoundProjectionVS,TEXT("/Engine/Private/ShadowProjectionVertexShader.usf"),TEXT("ShadowVolumeBoundProjectionVS"),SF_Vertex);

template<uint32 T>
TModulatedShadowProjection<T>::TModulatedShadowProjection(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
TShadowProjectionPS<T, false, true>(Initializer)
{
	ModulatedShadowColorParameter.Bind(Initializer.ParameterMap, TEXT("ModulatedShadowColor"));
	MobileBasePassUniformBuffer.Bind(Initializer.ParameterMap, FMobileBasePassUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
} 

/**
 * Implementations for TShadowProjectionPS.  
 */
#if !UE_BUILD_DOCS
#define IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(Quality,UseFadePlane,UseTransmission, SupportSubPixel) \
	typedef TShadowProjectionPS<Quality, UseFadePlane, false, UseTransmission, SupportSubPixel> FShadowProjectionPS##Quality##UseFadePlane##UseTransmission##SupportSubPixel; \
	IMPLEMENT_SHADER_TYPE(template<>,FShadowProjectionPS##Quality##UseFadePlane##UseTransmission##SupportSubPixel,TEXT("/Engine/Private/ShadowProjectionPixelShader.usf"),TEXT("Main"),SF_Pixel);

// Projection shaders without the distance fade, with different quality levels.
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(1,false,false,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(2,false,false,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(3,false,false,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(4,false,false,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(5,false,false,false);

IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(1,false,true,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(2,false,true,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(3,false,true,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(4,false,true,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(5,false,true,false);

// Projection shaders with the distance fade, with different quality levels.
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(1,true,false,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(2,true,false,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(3,true,false,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(4,true,false,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(5,true,false,false);

IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(1,true,true,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(2,true,true,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(3,true,true,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(4,true,true,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(5,true,true,false);

// Projection shaders without the distance fade, without transmission, with Sub-PixelSupport with different quality levels
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(1, false, false, true);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(2, false, false, true);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(3, false, false, true);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(4, false, false, true);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(5, false, false, true);

#undef IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER

#define IMPLEMENT_MODULATED_SHADOW_PROJECTION_PIXEL_SHADER(Quality) \
	template class TModulatedShadowProjection<Quality>; \
	using FShadowModulatedProjectionPS##Quality = TShadowProjectionPS<Quality, false, true>; \
	IMPLEMENT_TEMPLATE_TYPE_LAYOUT(template<>, FShadowModulatedProjectionPS##Quality); \
	IMPLEMENT_SHADER_TYPE(template<>, TModulatedShadowProjection<Quality>, TEXT("/Engine/Private/ShadowProjectionPixelShader.usf"), TEXT("Main"), SF_Pixel);

// Implement a pixel shader for rendering modulated shadow projections.
IMPLEMENT_MODULATED_SHADOW_PROJECTION_PIXEL_SHADER(1);
IMPLEMENT_MODULATED_SHADOW_PROJECTION_PIXEL_SHADER(2);
IMPLEMENT_MODULATED_SHADOW_PROJECTION_PIXEL_SHADER(3);
IMPLEMENT_MODULATED_SHADOW_PROJECTION_PIXEL_SHADER(4);
IMPLEMENT_MODULATED_SHADOW_PROJECTION_PIXEL_SHADER(5);

#undef IMPLEMENT_MODULATED_SHADOW_PROJECTION_PIXEL_SHADER

#endif

// with different quality levels
IMPLEMENT_SHADER_TYPE(template<>,TShadowProjectionFromTranslucencyPS<1>,TEXT("/Engine/Private/ShadowProjectionPixelShader.usf"),TEXT("Main"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>,TShadowProjectionFromTranslucencyPS<2>,TEXT("/Engine/Private/ShadowProjectionPixelShader.usf"),TEXT("Main"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>,TShadowProjectionFromTranslucencyPS<3>,TEXT("/Engine/Private/ShadowProjectionPixelShader.usf"),TEXT("Main"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>,TShadowProjectionFromTranslucencyPS<4>,TEXT("/Engine/Private/ShadowProjectionPixelShader.usf"),TEXT("Main"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>,TShadowProjectionFromTranslucencyPS<5>,TEXT("/Engine/Private/ShadowProjectionPixelShader.usf"),TEXT("Main"),SF_Pixel);

// Implement a pixel shader for rendering one pass point light shadows with different quality levels
/**
 * Pixel shader used to project one pass point light shadows.
 */
// Quality = 0 / 1
template <uint32 Quality, bool bUseTransmission, bool bUseSubPixel>
class TOnePassPointShadowProjectionPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(TOnePassPointShadowProjectionPS,Global);
public:

	TOnePassPointShadowProjectionPS() {}

	TOnePassPointShadowProjectionPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FGlobalShader(Initializer)
	{
		HairStrandsParameters.Bind(Initializer.ParameterMap, FHairStrandsViewUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		SubstrateGlobalParameters.Bind(Initializer.ParameterMap, FSubstrateGlobalUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		OnePassShadowParameters.Bind(Initializer.ParameterMap);
		ShadowDepthTextureSampler.Bind(Initializer.ParameterMap,TEXT("ShadowDepthTextureSampler"));
		LightPosition.Bind(Initializer.ParameterMap,TEXT("LightPositionAndInvRadius"));
		ShadowFadeFraction.Bind(Initializer.ParameterMap,TEXT("ShadowFadeFraction"));
		ShadowSharpen.Bind(Initializer.ParameterMap,TEXT("ShadowSharpen"));
		PointLightDepthBias.Bind(Initializer.ParameterMap,TEXT("PointLightDepthBias"));
		PointLightProjParameters.Bind(Initializer.ParameterMap, TEXT("PointLightProjParameters"));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters,OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADOW_QUALITY"), Quality);
		if (IsMobilePlatform(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("USE_TRANSMISSION"), 0);
		}
		else
		{
			OutEnvironment.SetDefine(TEXT("USE_TRANSMISSION"), (uint32)(bUseTransmission ? 1 : 0));
		}
		OutEnvironment.SetDefine(TEXT("SUBPIXEL_SHADOW"), (uint32)(bUseSubPixel ? 1 : 0));
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (IsMobilePlatform(Parameters.Platform))
		{
			return !bUseTransmission;
		}

		return true;
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		int32 ViewIndex,
		const FViewInfo& View,
		const FProjectedShadowInfo* ShadowInfo,
		FRHIUniformBuffer* HairStrandsUniformBuffer)
	{
		const FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();

		OnePassShadowParameters.Set(BatchedParameters, View, ShadowInfo);

		const FLightSceneProxy& LightProxy = *(ShadowInfo->GetLightSceneInfo().Proxy);

		SetShaderValue(BatchedParameters, LightPosition, FVector4f(FVector3f(FVector(LightProxy.GetPosition()) + PreViewTranslation), 1.0f / LightProxy.GetRadius()));

		SetShaderValue(BatchedParameters, ShadowFadeFraction, ShadowInfo->FadeAlphas[ViewIndex]);
		SetShaderValue(BatchedParameters, ShadowSharpen, LightProxy.GetShadowSharpen() * 7.0f + 1.0f);

		FVector2d ProjectionParams = FVector2d(ShadowInfo->OnePassShadowFaceProjectionMatrix.M[2][2], ShadowInfo->OnePassShadowFaceProjectionMatrix.M[3][2]);
		FVector2f InverseProjParams = FVector2f(1.0 / ProjectionParams.Y, ProjectionParams.X / ProjectionParams.Y);
		SetShaderValue(BatchedParameters, PointLightDepthBias, FVector3f(ShadowInfo->GetShaderDepthBias(), ShadowInfo->GetShaderSlopeDepthBias(), ShadowInfo->GetShaderMaxSlopeDepthBias()));
		SetShaderValue(BatchedParameters, PointLightProjParameters, InverseProjParams);

		if (HairStrandsParameters.IsBound())
		{
			SetUniformBufferParameter(BatchedParameters, HairStrandsParameters, HairStrandsUniformBuffer);
		}

		if (SubstrateGlobalParameters.IsBound())
		{
			TRDGUniformBufferRef<FSubstrateGlobalUniformParameters> SubstrateUniformBuffer = Substrate::BindSubstrateGlobalUniformParameters(View);
			SetUniformBufferParameter(BatchedParameters, SubstrateGlobalParameters, SubstrateUniformBuffer->GetRHIRef());
		}

		FScene* Scene = nullptr;

		if (View.Family->Scene)
		{
			Scene = View.Family->Scene->GetRenderScene();
		}

		SetSamplerParameter(BatchedParameters, ShadowDepthTextureSampler, TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI());

		auto DeferredLightParameter = GetUniformBufferParameter<FDeferredLightUniformStruct>();

		if (DeferredLightParameter.IsBound())
		{
			SetDeferredLightParameters(BatchedParameters, DeferredLightParameter, &ShadowInfo->GetLightSceneInfo(), View, LightFunctionAtlas::IsEnabled(View, LightFunctionAtlas::ELightFunctionAtlasSystem::DeferredLighting));
		}
	}

private:
	LAYOUT_FIELD(FOnePassPointShadowProjectionShaderParameters, OnePassShadowParameters);
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthTextureSampler);
	LAYOUT_FIELD(FShaderParameter, LightPosition);
	LAYOUT_FIELD(FShaderParameter, ShadowFadeFraction);
	LAYOUT_FIELD(FShaderParameter, ShadowSharpen);
	LAYOUT_FIELD(FShaderParameter, PointLightDepthBias);
	LAYOUT_FIELD(FShaderParameter, PointLightProjParameters);
	LAYOUT_FIELD(FShaderUniformBufferParameter, HairStrandsParameters);
	LAYOUT_FIELD(FShaderUniformBufferParameter, SubstrateGlobalParameters);
};
#define IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(Quality,UseTransmission,UseSubPixel) \
	typedef TOnePassPointShadowProjectionPS<Quality,  UseTransmission, UseSubPixel> FOnePassPointShadowProjectionPS##Quality##UseTransmission##UseSubPixel; \
	IMPLEMENT_SHADER_TYPE(template<>,FOnePassPointShadowProjectionPS##Quality##UseTransmission##UseSubPixel,TEXT("/Engine/Private/ShadowProjectionPixelShader.usf"),TEXT("MainOnePassPointLightPS"),SF_Pixel);

IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(1, false, true);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(2, false, true);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(3, false, true);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(4, false, true);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(5, false, true);

IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(1, false, false);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(2, false, false);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(3, false, false);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(4, false, false);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(5, false, false);

IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(1, true, false);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(2, true, false);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(3, true, false);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(4, true, false);
IMPLEMENT_ONEPASS_POINT_SHADOW_PROJECTION_PIXEL_SHADER(5, true, false);

// Implements a pixel shader for directional light PCSS.
#define IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(Quality,UseFadePlane) \
	typedef TDirectionalPercentageCloserShadowProjectionPS<Quality, UseFadePlane> TDirectionalPercentageCloserShadowProjectionPS##Quality##UseFadePlane; \
	IMPLEMENT_SHADER_TYPE(template<>,TDirectionalPercentageCloserShadowProjectionPS##Quality##UseFadePlane,TEXT("/Engine/Private/ShadowProjectionPixelShader.usf"),TEXT("Main"),SF_Pixel);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(5,false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(5,true);
#undef IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER

// Implements a pixel shader for spot light PCSS.
#define IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(Quality,UseFadePlane) \
	typedef TSpotPercentageCloserShadowProjectionPS<Quality, UseFadePlane> TSpotPercentageCloserShadowProjectionPS##Quality##UseFadePlane; \
	IMPLEMENT_SHADER_TYPE(template<>,TSpotPercentageCloserShadowProjectionPS##Quality##UseFadePlane,TEXT("/Engine/Private/ShadowProjectionPixelShader.usf"),TEXT("Main"),SF_Pixel);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(5, false);
IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER(5, true);
#undef IMPLEMENT_SHADOW_PROJECTION_PIXEL_SHADER

template<typename VertexShaderType, typename PixelShaderType>
static void BindShaderShaders(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	int32 ViewIndex, const FViewInfo& View, const FProjectedShadowInfo* ShadowInfo, uint32 StencilRef, FRHIUniformBuffer* HairStrandsUniformBuffer = nullptr)
{
	TShaderRef<VertexShaderType> VertexShader = View.ShaderMap->GetShader<VertexShaderType>();
	TShaderRef<PixelShaderType> PixelShader = View.ShaderMap->GetShader<PixelShaderType>();

	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, View, ShadowInfo, EShadowProjectionVertexShaderFlags::DrawingFrustum);

	FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

	const bool bUseLightFunctionAtlas = LightFunctionAtlas::IsEnabled(View, ELightFunctionAtlasSystem::DeferredLighting);
	PixelShader->SetParameters(BatchedParameters, ViewIndex, View, ShadowInfo, bUseLightFunctionAtlas);

	if (Substrate::IsSubstrateEnabled())
	{
		TRDGUniformBufferRef<FSubstrateGlobalUniformParameters> SubstrateUniformBuffer = Substrate::BindSubstrateGlobalUniformParameters(View);
		PixelShader->FGlobalShader::template SetParameters<FSubstrateGlobalUniformParameters>(BatchedParameters, SubstrateUniformBuffer->GetRHIRef());
	}

	if (HairStrandsUniformBuffer)
	{
		PixelShader->FGlobalShader::template SetParameters<FHairStrandsViewUniformParameters>(BatchedParameters, HairStrandsUniformBuffer);
	}

	RHICmdList.SetBatchedShaderParameters(PixelShader.GetPixelShader(), BatchedParameters);
}


static void BindShadowProjectionShaders(int32 Quality, FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer GraphicsPSOInit, int32 ViewIndex, const FViewInfo& View,
	const FProjectedShadowInfo* ShadowInfo, uint32 StencilRef, bool bMobileModulatedProjections, FRHIUniformBuffer* HairStrandsUniformBuffer)
{
	const bool bSubPixelShadow = HairStrandsUniformBuffer != nullptr;
	if (bSubPixelShadow)
	{
		check(!bMobileModulatedProjections);

		if (ShadowInfo->IsWholeSceneDirectionalShadow())
		{
			switch (Quality)
			{
			case 1: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<1, false, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef, HairStrandsUniformBuffer); break;
			case 2: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<2, false, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef, HairStrandsUniformBuffer); break;
			case 3: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<3, false, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef, HairStrandsUniformBuffer); break;
			case 4: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<4, false, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef, HairStrandsUniformBuffer); break;
			case 5: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<5, false, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef, HairStrandsUniformBuffer); break;
			default:
				check(0);
			}
		}
		else
		{
			switch (Quality)
			{
			case 1: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<1, false, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef, HairStrandsUniformBuffer); break;
			case 2: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<2, false, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef, HairStrandsUniformBuffer); break;
			case 3: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<3, false, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef, HairStrandsUniformBuffer); break;
			case 4: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<4, false, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef, HairStrandsUniformBuffer); break;
			case 5: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<5, false, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef, HairStrandsUniformBuffer); break;
			default:
				check(0);
			}
		}
		return;
	}

	if (ShadowInfo->bTranslucentShadow)
	{
		switch (Quality)
		{
		case 1: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionFromTranslucencyPS<1> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
		case 2: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionFromTranslucencyPS<2> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
		case 3: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionFromTranslucencyPS<3> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
		case 4: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionFromTranslucencyPS<4> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
		case 5: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionFromTranslucencyPS<5> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
		default:
			check(0);
		}
	}
	else if (ShadowInfo->IsWholeSceneDirectionalShadow())
	{
		if (CVarFilterMethod.GetValueOnRenderThread() == 1)
		{
			if (ShadowInfo->CascadeSettings.FadePlaneLength > 0)
				BindShaderShaders<FShadowProjectionNoTransformVS, TDirectionalPercentageCloserShadowProjectionPS<5, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef);
			else
				BindShaderShaders<FShadowProjectionNoTransformVS, TDirectionalPercentageCloserShadowProjectionPS<5, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef);
		}
		else if (ShadowInfo->CascadeSettings.FadePlaneLength > 0)
		{
			if (ShadowInfo->bTransmission)
			{
				switch (Quality)
				{
				case 1: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<1, true, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 2: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<2, true, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 3: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<3, true, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 4: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<4, true, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 5: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<5, true, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				default:
					check(0);
				}
			}
			else
			{
				switch (Quality)
				{
				case 1: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<1, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 2: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<2, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 3: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<3, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 4: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<4, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 5: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<5, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				default:
					check(0);
				}
			}
		}
		else
		{
			if (ShadowInfo->bTransmission)
			{
				switch (Quality)
				{
				case 1: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<1, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 2: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<2, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 3: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<3, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 4: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<4, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 5: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<5, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				default:
					check(0);
				}
			}
			else
			{ 
				switch (Quality)
				{
				case 1: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<1, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 2: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<2, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 3: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<3, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 4: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<4, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 5: BindShaderShaders<FShadowProjectionNoTransformVS, TShadowProjectionPS<5, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				default:
					check(0);
				}
			}
		}
	}
	else
	{
		if(bMobileModulatedProjections)
		{
			switch (Quality)
			{
			case 1: BindShaderShaders<FShadowVolumeBoundProjectionVS, TModulatedShadowProjection<1> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
			case 2: BindShaderShaders<FShadowVolumeBoundProjectionVS, TModulatedShadowProjection<2> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
			case 3: BindShaderShaders<FShadowVolumeBoundProjectionVS, TModulatedShadowProjection<3> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
			case 4: BindShaderShaders<FShadowVolumeBoundProjectionVS, TModulatedShadowProjection<4> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
			case 5: BindShaderShaders<FShadowVolumeBoundProjectionVS, TModulatedShadowProjection<5> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
			default:
				check(0);
			}
		}
		else if (ShadowInfo->bTransmission)
		{
			switch (Quality)
			{
			case 1: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<1, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
			case 2: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<2, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
			case 3: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<3, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
			case 4: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<4, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
			case 5: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<5, false, false, true> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
			default:
				check(0);
			}
		}
		else
		{
			if (CVarFilterMethod.GetValueOnRenderThread() == 1 && ShadowInfo->GetLightSceneInfo().Proxy->GetLightType() == LightType_Spot)
			{
				BindShaderShaders<FShadowVolumeBoundProjectionVS, TSpotPercentageCloserShadowProjectionPS<5, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef);
			}
			else
			{
				switch (Quality)
				{
				case 1: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<1, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 2: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<2, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 3: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<3, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 4: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<4, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				case 5: BindShaderShaders<FShadowVolumeBoundProjectionVS, TShadowProjectionPS<5, false> >(RHICmdList, GraphicsPSOInit, ViewIndex, View, ShadowInfo, StencilRef); break;
				default:
					check(0);
				}
			}
		}
	}

	check(GraphicsPSOInit.BoundShaderState.VertexShaderRHI);
	check(GraphicsPSOInit.BoundShaderState.PixelShaderRHI);
}

bool FProjectedShadowInfo::SupportsShadowStencilCulling(FStaticShaderPlatform ShaderPlatform)
{
	if (IsMobilePlatform(ShaderPlatform) && IsMobileDeferredShadingEnabled(ShaderPlatform))
	{
		// ShadowStencilMask now clashes with a mobile deferred ShadingModels mask
		return false;
	}
	return GShadowStencilCulling != 0;
}

bool FProjectedShadowInfo::UseShadowStencilCulling(const FViewInfo& View) const
{
	if (!SupportsShadowStencilCulling(View.GetShaderPlatform()))
	{
		return false;
	}

	ensureMsgf(DependentView == nullptr || DependentView == &View, TEXT("Shadow projection rendered to unexpected view. Need to check assumptions in UseShadowStencilCulling(...)"));

	return true;
}

FRHIBlendState* FProjectedShadowInfo::GetBlendStateForProjection(
	int32 ShadowMapChannel, 
	bool bIsWholeSceneDirectionalShadow,
	bool bUseFadePlane,
	bool bProjectingForForwardShading, 
	bool bMobileModulatedProjections)
{
	// With forward shading we are packing shadowing for all 4 possible stationary lights affecting each pixel into channels of the same texture, based on assigned shadowmap channels.
	// With deferred shading we have 4 channels for each light.  
	//	* CSM and per-object shadows are kept in separate channels to allow fading CSM out to precomputed shadowing while keeping per-object shadows past the fade distance.
	//	* Subsurface shadowing requires an extra channel for each

	FRHIBlendState* BlendState = nullptr;

	if (bProjectingForForwardShading)
	{
		if (bUseFadePlane)
		{
			if (ShadowMapChannel == 0)
			{
				// alpha is used to fade between cascades
				BlendState = TStaticBlendState<CW_RED, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI();
			}
			else if (ShadowMapChannel == 1)
			{
				BlendState = TStaticBlendState<CW_GREEN, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI();
			}
			else if (ShadowMapChannel == 2)
			{
				BlendState = TStaticBlendState<CW_BLUE, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI();
			}
			else if (ShadowMapChannel == 3)
			{
				BlendState = TStaticBlendState<CW_ALPHA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI();
			}
		}
		else
		{
			if (ShadowMapChannel == 0)
			{
				BlendState = TStaticBlendState<CW_RED, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI();
			}
			else if (ShadowMapChannel == 1)
			{
				BlendState = TStaticBlendState<CW_GREEN, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI();
			}
			else if (ShadowMapChannel == 2)
			{
				BlendState = TStaticBlendState<CW_BLUE, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI();
			}
			else if (ShadowMapChannel == 3)
			{
				BlendState = TStaticBlendState<CW_ALPHA, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI();
			}
		}

		checkf(BlendState, TEXT("Only shadows whose stationary lights have a valid ShadowMapChannel can be projected with forward shading"));
	}
	else
	{
		// See also GetShadowTermsBase from DeferredLightingCommon.ush.
		// Light Attenuation channel assignment:
		//  R:     WholeSceneShadows, non SSS
		//  G:     WholeSceneShadows,     SSS
		//  B: non WholeSceneShadows, non SSS
		//  A: non WholeSceneShadows,     SSS
		//
		// SSS: SubsurfaceScattering materials
		// non SSS: shadow for opaque materials
		// WholeSceneShadows: directional light CSM
		// non WholeSceneShadows: spotlight, per object shadows, translucency lighting, omni-directional lights

		if (bIsWholeSceneDirectionalShadow)
		{
			// Note: blend logic has to match ordering in FCompareFProjectedShadowInfoBySplitIndex.  For example the fade plane blend mode requires that shadow to be rendered first.
			// use R and G in Light Attenuation
			if (bUseFadePlane)
			{
				// alpha is used to fade between cascades, we don't don't need to do BO_Min as we leave B and A untouched which has translucency shadow
				BlendState = TStaticBlendState<CW_RG, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI();
			}
			else
			{
				// first cascade rendered doesn't require fading (CO_Min is needed to combine multiple shadow passes)
				// RTDF shadows: CO_Min is needed to combine with far shadows which overlap the same depth range
				BlendState = TStaticBlendState<CW_RG, BO_Min, BF_One, BF_One>::GetRHI();
			}
		}
		else
		{
			if (bMobileModulatedProjections)
			{
				// Color modulate shadows, ignore alpha.
				BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_Zero, BF_SourceColor, BO_Add, BF_Zero, BF_One, CW_NONE>::GetRHI();
			}
			else
			{
				// use B and A in Light Attenuation
				// CO_Min is needed to combine multiple shadow passes
				BlendState = TStaticBlendState<CW_BA, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI();
			}
		}
	}

	return BlendState;
}

FRHIBlendState* FProjectedShadowInfo::GetBlendStateForProjection(bool bProjectingForForwardShading, bool bMobileModulatedProjections) const
{
	return GetBlendStateForProjection(
		GetLightSceneInfo().GetDynamicShadowMapChannel(),
		IsWholeSceneDirectionalShadow(),
		CascadeSettings.FadePlaneLength > 0 && !bRayTracedDistanceField,
		bProjectingForForwardShading,
		bMobileModulatedProjections);
}

class FFrustumVertexBuffer : public FVertexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector4f>(TEXT("FProjectedShadowInfoStencilFrustum"), 8)
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitActionInitializer()
			.DetermineInitialState();

		TRHIBufferInitializer<FVector4f> OutFrustumVertices = RHICmdList.CreateBufferInitializer(CreateDesc);

		for(uint32 vZ = 0;vZ < 2;vZ++)
		{
			for(uint32 vY = 0;vY < 2;vY++)
			{
				for(uint32 vX = 0;vX < 2;vX++)
				{
					OutFrustumVertices[GetCubeVertexIndex(vX,vY,vZ)] = FVector4f(
						(vX ? -1.0f : 1.0f),
						(vY ? -1.0f : 1.0f),
						(vZ ?  1.0f : 0.0f),
						1.0f);
				}
			}
		}

		VertexBufferRHI = OutFrustumVertices.Finalize();
	}
};
TGlobalResource<FFrustumVertexBuffer> GFrustumVertexBuffer;

void FProjectedShadowInfo::SetupFrustumForProjection(const FViewInfo* View, TArray<FVector4f, TInlineAllocator<8>>& OutFrustumVertices, bool& bOutCameraInsideShadowFrustum, FPlane* OutPlanes) const
{
	bOutCameraInsideShadowFrustum = true;

	// Calculate whether the camera is inside the shadow frustum, or the near plane is potentially intersecting the frustum.
	if (!IsWholeSceneDirectionalShadow())
	{
		OutFrustumVertices.AddUninitialized(8);

		// The shadow transforms and view transforms are relative to different origins, so the world coordinates need to be translated.
		const FVector3f PreShadowToPreViewTranslation(View->ViewMatrices.GetPreViewTranslation() - PreShadowTranslation);

		// fill out the frustum vertices (this is only needed in the non-whole scene case)
		for(uint32 vZ = 0;vZ < 2;vZ++)
		{
			for(uint32 vY = 0;vY < 2;vY++)
			{
				for(uint32 vX = 0;vX < 2;vX++)
				{
					const FVector4f UnprojectedVertex = InvReceiverInnerMatrix.TransformFVector4(
						FVector4f(
							(vX ? -1.0f : 1.0f),
							(vY ? -1.0f : 1.0f),
							(vZ ?  1.0f : 0.0f),
							1.0f
						)
					);
					const FVector3f ProjectedVertex = UnprojectedVertex / UnprojectedVertex.W + PreShadowToPreViewTranslation;
					OutFrustumVertices[GetCubeVertexIndex(vX,vY,vZ)] = FVector4f(ProjectedVertex, 0);
				}
			}
		}

		const FVector ShadowViewOrigin = View->ViewMatrices.GetViewOrigin();
		const FVector ShadowPreViewTranslation = View->ViewMatrices.GetPreViewTranslation();

		const FVector FrontTopRight		= (FVector4)OutFrustumVertices[GetCubeVertexIndex(0,0,1)] - ShadowPreViewTranslation;
		const FVector FrontTopLeft		= (FVector4)OutFrustumVertices[GetCubeVertexIndex(1,0,1)] - ShadowPreViewTranslation;
		const FVector FrontBottomLeft	= (FVector4)OutFrustumVertices[GetCubeVertexIndex(1,1,1)] - ShadowPreViewTranslation;
		const FVector FrontBottomRight	= (FVector4)OutFrustumVertices[GetCubeVertexIndex(0,1,1)] - ShadowPreViewTranslation;
		const FVector BackTopRight		= (FVector4)OutFrustumVertices[GetCubeVertexIndex(0,0,0)] - ShadowPreViewTranslation;
		const FVector BackTopLeft		= (FVector4)OutFrustumVertices[GetCubeVertexIndex(1,0,0)] - ShadowPreViewTranslation;
		const FVector BackBottomLeft	= (FVector4)OutFrustumVertices[GetCubeVertexIndex(1,1,0)] - ShadowPreViewTranslation;
		const FVector BackBottomRight	= (FVector4)OutFrustumVertices[GetCubeVertexIndex(0,1,0)] - ShadowPreViewTranslation;

		const FPlane Front(FrontTopRight, FrontTopLeft, FrontBottomLeft);
		const float FrontDistance = Front.PlaneDot(ShadowViewOrigin);

		const FPlane Right(BackBottomRight, BackTopRight, FrontTopRight);
		const float RightDistance = Right.PlaneDot(ShadowViewOrigin);

		const FPlane Back(BackTopLeft, BackTopRight, BackBottomRight);
		const float BackDistance = Back.PlaneDot(ShadowViewOrigin);

		const FPlane Left(FrontTopLeft, BackTopLeft, BackBottomLeft);
		const float LeftDistance = Left.PlaneDot(ShadowViewOrigin);

		const FPlane Top(BackTopRight, BackTopLeft, FrontTopLeft);
		const float TopDistance = Top.PlaneDot(ShadowViewOrigin);

		const FPlane Bottom(BackBottomLeft, BackBottomRight, FrontBottomLeft);
		const float BottomDistance = Bottom.PlaneDot(ShadowViewOrigin);

		OutPlanes[0] = Front;
		OutPlanes[1] = Right;
		OutPlanes[2] = Back;
		OutPlanes[3] = Left;
		OutPlanes[4] = Top;
		OutPlanes[5] = Bottom;

		// Use a distance threshold to treat the case where the near plane is intersecting the frustum as the camera being inside
		// The near plane handling is not exact since it just needs to be conservative about saying the camera is outside the frustum
		const float DistanceThreshold = -View->NearClippingDistance * 3.0f;

		bOutCameraInsideShadowFrustum = 
			FrontDistance > DistanceThreshold && 
			RightDistance > DistanceThreshold && 
			BackDistance > DistanceThreshold && 
			LeftDistance > DistanceThreshold && 
			TopDistance > DistanceThreshold && 
			BottomDistance > DistanceThreshold;
	}
}

class FWholeSceneDirectionalShadowStencilVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FWholeSceneDirectionalShadowStencilVS);
	SHADER_USE_PARAMETER_STRUCT(FWholeSceneDirectionalShadowStencilVS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, ClipZValues)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FWholeSceneDirectionalShadowStencilVS, "/Engine/Private/ShadowProjectionVertexShader.usf", "WholeSceneDirectionalShadowStencilVS", SF_Vertex);

class FLateZDummyPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLateZDummyPS);
	SHADER_USE_PARAMETER_STRUCT(FLateZDummyPS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FLateZDummyPS, "/Engine/Private/LateZDummyPS.usf", "MainPS", SF_Pixel);

void FProjectedShadowInfo::SetupProjectionStencilMask(
	FRHICommandList& RHICmdList, 
	const FViewInfo* View, 
	int32 ViewIndex, 
	const FSceneRenderer* SceneRender,
	const TArray<FVector4f, TInlineAllocator<8>>& FrustumVertices,
	bool bMobileModulatedProjections, 
	bool bCameraInsideShadowFrustum,
	const FInstanceCullingDrawParams& InstanceCullingDrawParams) const
{
	FMeshPassProcessorRenderState DrawRenderState;

	// Depth test wo/ writes, no color writing.
	DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
	DrawRenderState.SetBlendState(TStaticBlendState<CW_NONE, BO_Add, BF_Zero, BF_SourceColor, BO_Add, BF_Zero, BF_One, CW_NONE>::GetRHI());

	// If this is a preshadow, mask the projection by the receiver primitives.
	if (bPreShadow || bSelfShadowOnly)
	{
		SCOPED_DRAW_EVENTF(RHICmdList, EventMaskSubjects, TEXT("Stencil Mask Subjects"));

		// NOTE: If instanced stereo is enabled, we need to render each view of the stereo pair using the instanced stereo transform to avoid bias issues.
		//       This means doing 2x renders, but letting the scissor rect kill the undesired half. Drawing the full mask once is easy, but since the outer
		//       loop is over each view, the stencil mask is not retained when the right view comes around.
		// TODO: Support instanced stereo properly in the projection stenciling pass.
		const bool bIsInstancedStereo = View->IsInstancedStereo();
		if (bIsInstancedStereo && ProjectionStencilingPasses.IsValidIndex(View->PrimaryViewIndex))
		{
			ensure(ProjectionStencilingPasses[View->PrimaryViewIndex]->GetInstanceCullingMode() == EInstanceCullingMode::Stereo);

			const FViewInfo* PrimaryView = View->GetPrimaryView();
			const FViewInfo* InstancedView = View->GetInstancedView();
			if (View->IsInstancedStereo())
			{
				float LeftMinX = PrimaryView->ViewRect.Min.X;
				float LeftMaxX = PrimaryView->ViewRect.Max.X;
				float LeftMinY = PrimaryView->ViewRect.Min.Y;
				float LeftMaxY = PrimaryView->ViewRect.Max.Y;

				float RightMinX = InstancedView->ViewRect.Min.X;
				float RightMaxX = InstancedView->ViewRect.Max.X;
				float RightMinY = InstancedView->ViewRect.Min.Y;
				float RightMaxY = InstancedView->ViewRect.Max.Y;

				// multi-viewport - collapse the other view in pair to be 0-width 0-height, effectively disabling it
				if (IStereoRendering::IsAPrimaryView(*View))
				{
					RightMaxX = RightMinX;
					RightMaxY = RightMinY;
				}
				else
				{
					LeftMinX = LeftMaxX;	// not a typo, just to have viewports adjacent to each other still
					LeftMaxY = LeftMinY;
				}

				RHICmdList.SetStereoViewport(LeftMinX, RightMinX, LeftMinY, RightMinY, /*MinZ*/ 0.0f,
					LeftMaxX, RightMaxX, LeftMaxY, RightMaxY, /*MaxZ*/ 1.0f);
			}
			else
			{
				// clip planes
				RHICmdList.SetViewport(PrimaryView->ViewRect.Min.X, PrimaryView->ViewRect.Min.Y, 0.0f, InstancedView->ViewRect.Max.X, InstancedView->ViewRect.Max.Y, 1.0f);
				RHICmdList.SetScissorRect(true, View->ViewRect.Min.X, View->ViewRect.Min.Y, View->ViewRect.Max.X, View->ViewRect.Max.Y);
			}
			// Submit the first (and only pass - we share that at least) as the pass is set up for stereo.
			ProjectionStencilingPasses[View->PrimaryViewIndex]->SubmitDraw(RHICmdList, InstanceCullingDrawParams);

			RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
			RHICmdList.SetViewport(View->ViewRect.Min.X, View->ViewRect.Min.Y, 0.0f, View->ViewRect.Max.X, View->ViewRect.Max.Y, 1.0f);
		}
		else if (ViewIndex < ProjectionStencilingPasses.Num())
		{
			check(ProjectionStencilingPasses[ViewIndex] != nullptr);
			ProjectionStencilingPasses[ViewIndex]->SubmitDraw(RHICmdList, InstanceCullingDrawParams);
		}

		
	}
	else if (IsWholeSceneDirectionalShadow())
	{
		// Increment stencil on front-facing zfail, decrement on back-facing zfail.
		DrawRenderState.SetDepthStencilState(
			TStaticDepthStencilState<
			false, CF_DepthNearOrEqual,
			true, CF_Always, SO_Keep, SO_Increment, SO_Keep,
			true, CF_Always, SO_Keep, SO_Decrement, SO_Keep,
			ShadowStencilMask, ShadowStencilMask
			>::GetRHI());

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		DrawRenderState.ApplyToPSO(GraphicsPSOInit);

		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();

		checkSlow(CascadeSettings.ShadowSplitIndex >= 0);
		checkSlow(bDirectionalLight);

		// Draw 2 fullscreen planes, front facing one at the near subfrustum plane, and back facing one at the far.
		const FVector4 Near = View->ViewMatrices.GetViewToClip().TransformFVector4(FVector4(0, 0, CascadeSettings.SplitNear));
		const FVector4 Far = View->ViewMatrices.GetViewToClip().TransformFVector4(FVector4(0, 0, CascadeSettings.SplitFar));
		const FVector4::FReal StencilNear = (Near.Z / Near.W);
		const FVector4::FReal StencilFar = (Far.Z / Far.W);

		TShaderMapRef<FWholeSceneDirectionalShadowStencilVS> VertexShader(View->ShaderMap);
			
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		FWholeSceneDirectionalShadowStencilVS::FParameters Parameters;
		Parameters.ClipZValues = FVector4f((float)StencilFar, (float)StencilNear, 0, 0); // LWC_TODO: precision loss

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), Parameters);

		RHICmdList.SetStreamSource(0, GDummyWholeSceneDirectionalShadowStencilVertexBuffer.VertexBufferRHI, 0);
		RHICmdList.DrawPrimitive(0, (CascadeSettings.ShadowSplitIndex > 0) ? 2 : 1, 1);
	}
	// Not a preshadow, mask the projection to any pixels inside the frustum.
	else
	{
		if (bCameraInsideShadowFrustum)
		{
			// Use zfail stenciling when the camera is inside the frustum or the near plane is potentially clipping, 
			// Because zfail handles these cases while zpass does not.
			// zfail stenciling is somewhat slower than zpass because on modern GPUs HiZ will be disabled when setting up stencil.
			// Increment stencil on front-facing zfail, decrement on back-facing zfail.
			DrawRenderState.SetDepthStencilState(
				TStaticDepthStencilState<
				false, CF_DepthNearOrEqual,
				true, CF_Always, SO_Keep, SO_Increment, SO_Keep,
				true, CF_Always, SO_Keep, SO_Decrement, SO_Keep,
				ShadowStencilMask, ShadowStencilMask
				>::GetRHI());
		}
		else
		{
			// Increment stencil on front-facing zpass, decrement on back-facing zpass.
			// HiZ will be enabled on modern GPUs which will save a little GPU time.
			DrawRenderState.SetDepthStencilState(
				TStaticDepthStencilState<
				false, CF_DepthNearOrEqual,
				true, CF_Always, SO_Keep, SO_Keep, SO_Increment,
				true, CF_Always, SO_Keep, SO_Keep, SO_Decrement,
				ShadowStencilMask, ShadowStencilMask
				>::GetRHI());
		}
		
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		DrawRenderState.ApplyToPSO(GraphicsPSOInit);
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();

		// Find the projection shaders.
		TShaderMapRef<FShadowVolumeBoundProjectionVS> VertexShader(View->ShaderMap);
		TShaderRef<FLateZDummyPS> PixelShader;

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		if (CVarShadowStencilCullingArtifactWorkaround.GetValueOnRenderThread() != 0 && IsRHIDeviceNVIDIA())
		{
			PixelShader = View->ShaderMap->GetShader<FLateZDummyPS>();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		}
		else
		{
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = nullptr;
		}
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		// Set the projection vertex shader parameters
		SetShaderParametersLegacyVS(RHICmdList, VertexShader, *View, this, EShadowProjectionVertexShaderFlags::DrawingFrustum);

		RHICmdList.SetStreamSource(0, GFrustumVertexBuffer.VertexBufferRHI, 0);

		// Shadow projection stenciling is special-cased to run per-view for instanced stereo views.
		// TODO: Support instanced stereo properly in the projection stenciling pass.
		const bool bIsInstancedStereo = View->IsInstancedStereo();
		const uint32 NumberOfInstances = bIsInstancedStereo ? 1 : View->GetStereoPassInstanceFactor();

		// Draw the frustum using the stencil buffer to mask just the pixels which are inside the shadow frustum.
		RHICmdList.DrawIndexedPrimitive(GCubeIndexBuffer.IndexBufferRHI, 0, 0, 8, 0, 12, NumberOfInstances);

		// if rendering modulated shadows mask out subject mesh elements to prevent self shadowing.
		if (bMobileModulatedProjections && !CVarEnableModulatedSelfShadow.GetValueOnRenderThread())
		{
			if (ViewIndex < ProjectionStencilingPasses.Num())
			{
				ProjectionStencilingPasses[ViewIndex]->SubmitDraw(RHICmdList, InstanceCullingDrawParams);
			}
		}
	}
}

// Mark hair-strands pixels encompassed into the rasterized volumes. 
// * Mark 1: for hair pixels encompassed into the volume
// * Mark 2: for pixels touching the volume
// This affects only the hair-only depth stencil texture.
void FProjectedShadowInfo::SetupProjectionStencilMaskForHair(FRHICommandList& RHICmdList, const FViewInfo* View) const
{
	check(!IsWholeSceneDirectionalShadow());

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

	TShaderRef<FShadowVolumeBoundProjectionVS> VertexShader = View->ShaderMap->GetShader<FShadowVolumeBoundProjectionVS>();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.bDepthBounds = false;
	GraphicsPSOInit.DepthStencilState =
		TStaticDepthStencilState<
		false, CF_GreaterEqual,
		true, CF_Always, SO_Keep, SO_Keep, SO_Increment,
		true, CF_Always, SO_Keep, SO_Keep, SO_Increment,
		0xff, 0xff>::GetRHI();
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_RED, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = nullptr;
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
	SetShaderParametersLegacyVS(RHICmdList, VertexShader, *View, this, EShadowProjectionVertexShaderFlags::DrawingFrustum);

	RHICmdList.SetStreamSource(0, GFrustumVertexBuffer.VertexBufferRHI, 0);
	RHICmdList.DrawIndexedPrimitive(GCubeIndexBuffer.IndexBufferRHI, 0, 0, 8, 0, 12, 1);
}

BEGIN_SHADER_PARAMETER_STRUCT(FShadowProjectionPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RDG_TEXTURE_ACCESS(ShadowTexture0, ERHIAccess::SRVGraphics)
	RDG_TEXTURE_ACCESS(ShadowTexture1, ERHIAccess::SRVGraphics)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FProjectedShadowInfo::RenderProjection(
	FRDGBuilder& GraphBuilder,
	const FShadowProjectionPassParameters& CommonPassParameters,
	int32 ViewIndex,
	const FViewInfo* View,
	const FLightSceneProxy* LightSceneProxy,
	const FSceneRenderer* SceneRender,
	bool bProjectingForForwardShading,
	bool bSubPixelShadow) const
{
	// Find the shadow's view relevance.
	const FVisibleLightViewInfo& VisibleLightViewInfo = View->VisibleLightInfos[LightSceneInfo->Id];
	{
		FPrimitiveViewRelevance ViewRelevance = VisibleLightViewInfo.ProjectedShadowViewRelevanceMap[ShadowId];

		// Don't render shadows for subjects which aren't view relevant.
		if (ViewRelevance.bShadowRelevance == false)
		{
			return;
		}
	}

	bool bCameraInsideShadowFrustum;
	TArray<FVector4f, TInlineAllocator<8>> FrustumVertices;
	FPlane OutPlanes[6];
	SetupFrustumForProjection(View, FrustumVertices, bCameraInsideShadowFrustum, OutPlanes);

	const bool bDepthBoundsTestEnabled = IsWholeSceneDirectionalShadow() && GSupportsDepthBoundsTest && CVarCSMDepthBoundsTest.GetValueOnRenderThread() != 0;// && !bSubPixelSupport;

	if (bSubPixelShadow)
	{
		// Do not apply pre-shadow on opaque geometry during sub-pixel pass as we only care about opaque geometry 'casting' shadow (not receiving shadow)
		// However, applied pre-shadow onto hair primitive (which are the only one able to cast deep shadow)
		if (bPreShadow)
		{
			const bool bIsValid = ReceiverPrimitives.Num() > 0 && ReceiverPrimitives[0]->Proxy->CastsDeepShadow();
			if (!bIsValid)
			{
				return;
			}
		}

		const bool bValidPlanes = FrustumVertices.Num() > 0;
		if (bValidPlanes && CVarHairStrandsCullPerObjectShadowCaster.GetValueOnRenderThread() > 0)
		{
			// Skip volume which does not intersect hair clusters
			bool bIntersect = bValidPlanes;
			for (const FHairStrandsMacroGroupData& Data : View->HairStrandsViewData.MacroGroupDatas)
			{
				const FSphere BoundSphere = Data.Bounds.GetSphere();
				// Return the signed distance to the plane. The planes are pointing inward
				const float D0 = -OutPlanes[0].PlaneDot(BoundSphere.Center);
				const float D1 = -OutPlanes[1].PlaneDot(BoundSphere.Center);
				const float D2 = -OutPlanes[2].PlaneDot(BoundSphere.Center);
				const float D3 = -OutPlanes[3].PlaneDot(BoundSphere.Center);
				const float D4 = -OutPlanes[4].PlaneDot(BoundSphere.Center);
				const float D5 = -OutPlanes[5].PlaneDot(BoundSphere.Center);

				const bool bOutside =
					D0 - BoundSphere.W > 0 ||
					D1 - BoundSphere.W > 0 ||
					D2 - BoundSphere.W > 0 ||
					D3 - BoundSphere.W > 0 ||
					D4 - BoundSphere.W > 0 ||
					D5 - BoundSphere.W > 0;

				bIntersect = !bOutside;
				if (bIntersect)
				{
					break;
				}
			}

			// The light frustum does not intersect the hair cluster, and thus doesn't have any interacction with it, and the shadow mask computation is not needed in this case
			if (!bIntersect)
			{
				return;
			}
		}
	}

	FString EventName;

#if WANTS_DRAW_MESH_EVENTS
	if (GetEmitDrawEvents())
	{
		GetShadowTypeNameForDrawEvent(EventName);
	}
#endif

	// Interpret null render targets as skipping the render pass; this is currently used by mobile.
	FRDGTextureRef OutputTexture0 = CommonPassParameters.RenderTargets[0].GetTexture();
	const ERDGPassFlags PassFlags = OutputTexture0 == nullptr ? ERDGPassFlags::SkipRenderPass : ERDGPassFlags::None;

	auto* PassParameters = GraphBuilder.AllocParameters<FShadowProjectionPassParameters>();
	*PassParameters = CommonPassParameters;
	PassParameters->View = View->GetShaderParameters();
	PassParameters->RenderTargets.MultiViewCount = View->GetMultiViewCount();
	// LoadAction from CommonPassParameters have to be adjusted depending on whether this is a first write to a target
	PassParameters->RenderTargets[0].SetLoadAction(GetLoadActionIfProduced(OutputTexture0, ERenderTargetLoadAction::EClear));

	if (RenderTargets.DepthTarget)
	{
		PassParameters->ShadowTexture0 = GraphBuilder.RegisterExternalTexture(RenderTargets.DepthTarget);
	}
	else
	{
		PassParameters->ShadowTexture0 = GraphBuilder.RegisterExternalTexture(RenderTargets.ColorTargets[0]);
		PassParameters->ShadowTexture1 = GraphBuilder.RegisterExternalTexture(RenderTargets.ColorTargets[1]);
	}

	const bool bIsInstancedStereo = View->IsInstancedStereo();
	if (ViewIndex < ProjectionStencilingPasses.Num() && ProjectionStencilingPasses[ViewIndex] != nullptr)
	{
		// GPUCULL_TODO: get rid of const cast
		FSimpleMeshDrawCommandPass& ProjectionStencilingPass = *const_cast<FSimpleMeshDrawCommandPass*>(ProjectionStencilingPasses[ViewIndex]);
		ProjectionStencilingPass.BuildRenderingCommands(GraphBuilder, *View, *SceneRender->Scene, PassParameters->InstanceCullingDrawParams);
	}
	else if (bIsInstancedStereo && (bPreShadow || bSelfShadowOnly))
	{
		// NOTE: This here is a hack that must match up to the use inside SetupProjectionStencilMask, where we use the Stereo setup but draw each eye independently
		//       by scissoring the undersired half (while setting the full viewport to get the scaling to match the stereo pathfor base/pre-pass 1:1).
		ensure(View->StereoPass == EStereoscopicPass::eSSP_SECONDARY);

		// GPUCULL_TODO: get rid of const cast
		FSimpleMeshDrawCommandPass& ProjectionStencilingPass = *const_cast<FSimpleMeshDrawCommandPass*>(ProjectionStencilingPasses[0]);
		ensure(ProjectionStencilingPass.GetInstanceCullingMode() == EInstanceCullingMode::Stereo);
		ProjectionStencilingPass.BuildRenderingCommands(GraphBuilder, *View->GetPrimaryView(), *SceneRender->Scene, PassParameters->InstanceCullingDrawParams);
	}

	const FInstanceCullingDrawParams& InstanceCullingDrawParams = PassParameters->InstanceCullingDrawParams;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ShadowProjection %s", *EventName),
		PassParameters,
		ERDGPassFlags::Raster | PassFlags,
		[this, SceneRender, View, ViewIndex, LightSceneProxy, bProjectingForForwardShading, &InstanceCullingDrawParams, bSubPixelShadow, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		RenderProjectionInternal(RHICmdList, ViewIndex, View, LightSceneProxy, SceneRender, bProjectingForForwardShading, false, InstanceCullingDrawParams, bSubPixelShadow && PassParameters->HairStrands ? PassParameters->HairStrands.GetUniformBuffer()->GetRHI() : nullptr);
	});
}

void FProjectedShadowInfo::RenderProjectionInternal(
	FRHICommandList& RHICmdList,
	int32 ViewIndex,
	const FViewInfo* View,
	const FLightSceneProxy* LightSceneProxy,
	const FSceneRenderer* SceneRender,
	bool bProjectingForForwardShading,
	bool bMobileModulatedProjections,
	const FInstanceCullingDrawParams& InstanceCullingDrawParams, 
	FRHIUniformBuffer* HairStrandsUniformBuffer) const
{
	RHICmdList.SetViewport(View->ViewRect.Min.X, View->ViewRect.Min.Y, 0.0f, View->ViewRect.Max.X, View->ViewRect.Max.Y, 1.0f);
	LightSceneProxy->SetScissorRect(RHICmdList, *View, View->ViewRect);

	FScopeCycleCounter Scope(bWholeSceneShadow ? GET_STATID(STAT_RenderWholeSceneShadowProjectionsTime) : GET_STATID(STAT_RenderPerObjectShadowProjectionsTime));

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

	FPlane OutFrustmPlanes[6];
	bool bCameraInsideShadowFrustum;
	TArray<FVector4f, TInlineAllocator<8>> FrustumVertices;
	SetupFrustumForProjection(View, FrustumVertices, bCameraInsideShadowFrustum, OutFrustmPlanes);

	const bool bDepthBoundsTestEnabled = IsWholeSceneDirectionalShadow() && GSupportsDepthBoundsTest && CVarCSMDepthBoundsTest.GetValueOnRenderThread() != 0;// && !bSubPixelSupport;
	const bool bSubPixelSupport = !bDepthBoundsTestEnabled && HairStrandsUniformBuffer != nullptr;// HairStrands::HasViewHairStrandsData(*View);
	const bool bStencilTestEnabled = !bDepthBoundsTestEnabled && !bSubPixelSupport && UseShadowStencilCulling(*View);
	const uint32 StencilRef = bSubPixelSupport && !IsWholeSceneDirectionalShadow() && !bCameraInsideShadowFrustum ? 1u : 0u;

	if (bStencilTestEnabled)
	{
		SetupProjectionStencilMask(RHICmdList, View, ViewIndex, SceneRender, FrustumVertices, bMobileModulatedProjections, bCameraInsideShadowFrustum, InstanceCullingDrawParams);
	}

	// Mark stencil so that only hair pixel within volume bound will be affected by the pre-shadow mask
	if (bSubPixelSupport)
	{
		DrawClearQuad(RHICmdList, false, FLinearColor::Transparent, false, 0, true, 0);
		if (!IsWholeSceneDirectionalShadow())
		{
			SetupProjectionStencilMaskForHair(RHICmdList, View);
		}
	}

	// solid rasterization w/ back-face culling.
	GraphicsPSOInit.RasterizerState = (View->bReverseCulling || IsWholeSceneDirectionalShadow()) ? TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI();

	GraphicsPSOInit.bDepthBounds = bDepthBoundsTestEnabled;
	if (bDepthBoundsTestEnabled)
	{
		// no depth test or writes
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	}
	else if (bStencilTestEnabled)
	{
		if (GStencilOptimization)
		{
			// No depth test or writes, zero the stencil
			// Note: this will disable hi-stencil on many GPUs, but still seems 
			// to be faster. However, early stencil still works 
			GraphicsPSOInit.DepthStencilState =
				TStaticDepthStencilState<
				false, CF_Always,
				true, CF_NotEqual, SO_Zero, SO_Zero, SO_Zero,
				false, CF_Always, SO_Zero, SO_Zero, SO_Zero,
				ShadowStencilMask, ShadowStencilMask
				>::GetRHI();
		}
		else
		{
			// no depth test or writes, Test stencil for non-zero.
			GraphicsPSOInit.DepthStencilState =
				TStaticDepthStencilState<
				false, CF_Always,
				true, CF_NotEqual, SO_Keep, SO_Keep, SO_Keep,
				false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
				ShadowStencilMask, ShadowStencilMask
				>::GetRHI();
		}
	}
	else
	{
		if (bSubPixelSupport)
		{
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthFartherOrEqual, true, CF_Equal, SO_Keep, SO_Keep, SO_Keep, true, CF_Equal, SO_Keep, SO_Keep, SO_Keep, 0xFF, 0xFF>::GetRHI();
		}
		else
		{
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthFartherOrEqual>::GetRHI();
		}
	}

	GraphicsPSOInit.BlendState = GetBlendStateForProjection(bProjectingForForwardShading, bMobileModulatedProjections);

	GraphicsPSOInit.PrimitiveType = IsWholeSceneDirectionalShadow() ? PT_TriangleStrip : PT_TriangleList;

	{
		uint32 LocalQuality = GetShadowQuality();

		if (LocalQuality > 1)
		{
			if (IsWholeSceneDirectionalShadow() && CascadeSettings.ShadowSplitIndex > 0)
			{
				// adjust kernel size so that the penumbra size of distant splits will better match up with the closer ones
				const float SizeScale = CascadeSettings.ShadowSplitIndex / FMath::Max(0.001f, CVarCSMSplitPenumbraScale.GetValueOnRenderThread());
			}
			else if (LocalQuality > 2 && !bWholeSceneShadow)
			{
				static auto CVarPreShadowResolutionFactor = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.Shadow.PreShadowResolutionFactor"));
				const int32 TargetResolution = bPreShadow ? FMath::TruncToInt(512 * CVarPreShadowResolutionFactor->GetValueOnRenderThread()) : 512;

				int32 Reduce = 0;

				{
					int32 Res = ResolutionX;

					while (Res < TargetResolution)
					{
						Res *= 2;
						++Reduce;
					}
				}

				// Never drop to quality 1 due to low resolution, aliasing is too bad
				LocalQuality = FMath::Clamp((int32)LocalQuality - Reduce, 3, 5);
			}
		}

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
		BindShadowProjectionShaders(LocalQuality, RHICmdList, GraphicsPSOInit, ViewIndex, *View, this, StencilRef, bMobileModulatedProjections, HairStrandsUniformBuffer);

		if (bDepthBoundsTestEnabled)
		{
			SetDepthBoundsTest(RHICmdList, CascadeSettings.SplitNear, CascadeSettings.SplitFar, View->ViewMatrices.GetViewToClip());
		}
	}

	const bool bIsInstancedStereo = View->IsInstancedStereo();
	const uint32 NumberOfInstances = bIsInstancedStereo ? 1 : View->GetStereoPassInstanceFactor();
	if (IsWholeSceneDirectionalShadow())
	{
		RHICmdList.SetStreamSource(0, GClearVertexBuffer.VertexBufferRHI, 0);
		RHICmdList.DrawPrimitive(0, 2, NumberOfInstances);
	}
	else
	{
		RHICmdList.SetStreamSource(0, GFrustumVertexBuffer.VertexBufferRHI, 0);
		// Draw the frustum using the projection shader..
		RHICmdList.DrawIndexedPrimitive(GCubeIndexBuffer.IndexBufferRHI, 0, 0, 8, 0, 12, NumberOfInstances);
	}

	if (bStencilTestEnabled)
	{
		// Clear the stencil buffer to 0.
		if (!GStencilOptimization)
		{
			DrawClearQuad(RHICmdList, false, FLinearColor::Transparent, false, 0, true, 0);
		}
	}

	RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
}

void FProjectedShadowInfo::RenderMobileModulatedShadowProjection(
	FRHICommandList& RHICmdList,
	int32 ViewIndex,
	const FViewInfo* View,
	const FLightSceneProxy* LightSceneProxy,
	const FSceneRenderer* SceneRender) const
{
	// Find the shadow's view relevance.
	const FVisibleLightViewInfo& VisibleLightViewInfo = View->VisibleLightInfos[LightSceneInfo->Id];
	{
		FPrimitiveViewRelevance ViewRelevance = VisibleLightViewInfo.ProjectedShadowViewRelevanceMap[ShadowId];

		// Don't render shadows for subjects which aren't view relevant.
		if (ViewRelevance.bShadowRelevance == false)
		{
			return;
		}
	}

	const bool bProjectingForForwardShading = false;
	const bool bMobileModulatedProjections = true;

	// GPUCULL_TODO, mobile modulated shadow is inside the mobile render pass, probably couldn't work with the GPUCull.
	FInstanceCullingDrawParams InstanceCullingDrawParams;
	RenderProjectionInternal(RHICmdList, ViewIndex, View, LightSceneProxy, SceneRender, bProjectingForForwardShading, bMobileModulatedProjections, InstanceCullingDrawParams, nullptr);
}

template <uint32 Quality, bool bUseTransmission, bool bUseSubPixel>
static void SetPointLightShaderTempl(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, int32 ViewIndex, const FViewInfo& View, const FProjectedShadowInfo* ShadowInfo, FRHIUniformBuffer* HairStrandsUniformBuffer = nullptr)
{
	TShaderMapRef<FShadowVolumeBoundProjectionVS> VertexShader(View.ShaderMap);
	TShaderMapRef<TOnePassPointShadowProjectionPS<Quality,bUseTransmission,bUseSubPixel> > PixelShader(View.ShaderMap);

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, View, ShadowInfo, EShadowProjectionVertexShaderFlags::None);
	SetShaderParametersLegacyPS(RHICmdList, PixelShader, ViewIndex, View, ShadowInfo, HairStrandsUniformBuffer);
}

void FProjectedShadowInfo::RenderOnePassPointLightProjection(
	FRDGBuilder& GraphBuilder,
	const FShadowProjectionPassParameters& CommonPassParameters,
	int32 ViewIndex,
	const FViewInfo& View,
	const FLightSceneProxy* LightSceneProxy,
	bool bProjectingForForwardShading,
	bool bSubPixelShadow) const
{
	SCOPE_CYCLE_COUNTER(STAT_RenderWholeSceneShadowProjectionsTime);

	checkSlow(bOnePassPointLightShadow);
	
	const FSphere LightBounds = LightSceneInfo->Proxy->GetBoundingSphere();
	const bool bUseTransmission = LightSceneInfo->Proxy->Transmission();
	const bool bCameraInsideLightGeometry = ((FVector)View.ViewMatrices.GetViewOrigin() - LightBounds.Center).SizeSquared() < FMath::Square(LightBounds.W * 1.05f + View.NearClippingDistance * 2.0f);

	// Interpret null render targets as skipping the render pass; this is currently used by mobile.
	FRDGTextureRef OutputTexture0 = CommonPassParameters.RenderTargets[0].GetTexture();
	const ERDGPassFlags PassFlags = OutputTexture0 == nullptr ? ERDGPassFlags::SkipRenderPass : ERDGPassFlags::None;

	auto* PassParameters = GraphBuilder.AllocParameters<FShadowProjectionPassParameters>();
	*PassParameters = CommonPassParameters;
	PassParameters->View = View.GetShaderParameters();
	// LoadAction from CommonPassParameters have to be adjusted depending on whether this is a first write to a target
	PassParameters->RenderTargets[0].SetLoadAction(GetLoadActionIfProduced(OutputTexture0, ERenderTargetLoadAction::EClear));

	if (RenderTargets.DepthTarget)
	{
		PassParameters->ShadowTexture0 = GraphBuilder.RegisterExternalTexture(RenderTargets.DepthTarget);
	}
	else
	{
		PassParameters->ShadowTexture0 = GraphBuilder.RegisterExternalTexture(RenderTargets.ColorTargets[0]);
		PassParameters->ShadowTexture1 = GraphBuilder.RegisterExternalTexture(RenderTargets.ColorTargets[1]);
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("OnePassPointLightProjection"),
		PassParameters,
		ERDGPassFlags::Raster | PassFlags,
		[this, &View, LightBounds, bProjectingForForwardShading, bCameraInsideLightGeometry, bUseTransmission, ViewIndex, LightSceneProxy, bSubPixelShadow, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
		LightSceneProxy->SetScissorRect(RHICmdList, View, View.ViewRect);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = GetBlendStateForProjection(bProjectingForForwardShading, false);
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		if (bCameraInsideLightGeometry)
		{
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			// Render backfaces with depth tests disabled since the camera is inside (or close to inside) the light geometry
			GraphicsPSOInit.RasterizerState = View.bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
		}
		else
		{
			// Render frontfaces with depth tests on to get the speedup from HiZ since the camera is outside the light geometry
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
			GraphicsPSOInit.RasterizerState = View.bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI();
		}

		const uint32 LocalQuality = GetShadowQuality();

		if (LocalQuality > 1)
		{
			// adjust kernel size so that the penumbra size of distant splits will better match up with the closer ones
			//const float SizeScale = ShadowInfo->ResolutionX;
			int32 Reduce = 0;

			{
				int32 Res = ResolutionX;

				while (Res < 512)
				{
					Res *= 2;
					++Reduce;
				}
			}
		}

		if (bSubPixelShadow)
		{
			// Do not apply pre-shadow on opaque geometry during sub-pixel pass as we only care about opaque geometry 'casting' shadow (not receiving shadow)
			// However, applied pre-shadow onto hair primitive (which are the only one able to cast deep shadow)
			if (bPreShadow)
			{
				const bool bIsValid = ReceiverPrimitives.Num() > 0 && ReceiverPrimitives[0]->Proxy->CastsDeepShadow();
				if (!bIsValid)
				{
					return;
				}
			}

			// Skip volume which does not intersect hair clusters
			bool bIntersect = false;
			if (CVarHairStrandsCullPerObjectShadowCaster.GetValueOnRenderThread() > 0)
			{
				for (const FHairStrandsMacroGroupData& Data : View.HairStrandsViewData.MacroGroupDatas)
				{
					const FSphere BoundSphere = Data.Bounds.GetSphere();
					if (BoundSphere.Intersects(LightBounds))
					{
						bIntersect = true;
						break;
					}
				}

				// The light frustum does not intersect the hair cluster, and thus doesn't have any interacction with it, and the shadow mask computation is not needed in this case
				if (!bIntersect)
				{
					return;
				}
			}

			FRHIUniformBuffer* HairStrandsUniformBuffer = PassParameters->HairStrands.GetUniformBuffer()->GetRHI();
			switch (LocalQuality)
			{
			case 1: SetPointLightShaderTempl<1, false, true>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this, HairStrandsUniformBuffer); break;
			case 2: SetPointLightShaderTempl<2, false, true>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this, HairStrandsUniformBuffer); break;
			case 3: SetPointLightShaderTempl<3, false, true>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this, HairStrandsUniformBuffer); break;
			case 4: SetPointLightShaderTempl<4, false, true>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this, HairStrandsUniformBuffer); break;
			case 5: SetPointLightShaderTempl<5, false, true>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this, HairStrandsUniformBuffer); break;
			default:
				check(0);
			}
		}
		else if (bUseTransmission)
		{
			switch (LocalQuality)
			{
			case 1: SetPointLightShaderTempl<1, true, false>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this); break;
			case 2: SetPointLightShaderTempl<2, true, false>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this); break;
			case 3: SetPointLightShaderTempl<3, true, false>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this); break;
			case 4: SetPointLightShaderTempl<4, true, false>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this); break;
			case 5: SetPointLightShaderTempl<5, true, false>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this); break;
			default:
				check(0);
			}
		}
		else
		{
			switch (LocalQuality)
			{
			case 1: SetPointLightShaderTempl<1, false, false>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this); break;
			case 2: SetPointLightShaderTempl<2, false, false>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this); break;
			case 3: SetPointLightShaderTempl<3, false, false>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this); break;
			case 4: SetPointLightShaderTempl<4, false, false>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this); break;
			case 5: SetPointLightShaderTempl<5, false, false>(RHICmdList, GraphicsPSOInit, ViewIndex, View, this); break;
			default:
				check(0);
			}
		}

		// Project the point light shadow with some approximately bounding geometry, 
		// So we can get speedups from depth testing and not processing pixels outside of the light's influence.
		StencilingGeometry::DrawSphere(RHICmdList, View.GetStereoPassInstanceFactor());
		RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
	});
}

void FProjectedShadowInfo::RenderFrustumWireframe(FPrimitiveDrawInterface* PDI) const
{
	// Find the ID of an arbitrary subject primitive to use to color the shadow frustum.
	int32 SubjectPrimitiveId = 0;
	if(DynamicSubjectPrimitives.Num())
	{
		SubjectPrimitiveId = DynamicSubjectPrimitives[0]->GetIndex();
	}

	const FMatrix44f InvShadowTransform = (bWholeSceneShadow || bPreShadow) ? TranslatedWorldToClipInnerMatrix.InverseFast() : InvReceiverInnerMatrix;

	FColor Color;

	if (IsWholeSceneDirectionalShadow())
	{
		Color = FColor::White;
		switch(CascadeSettings.ShadowSplitIndex)
		{
			case 0: Color = FColor::Red; break;
			case 1: Color = FColor::Yellow; break;
			case 2: Color = FColor::Green; break;
			case 3: Color = FColor::Blue; break;
		}
	}
	else
	{
		Color = FLinearColor::MakeFromHSV8(( ( SubjectPrimitiveId + LightSceneInfo->Id.GetIndex() ) * 31 ) & 255, 0, 255).ToFColor(true);
	}

	// Render the wireframe for the frustum derived from ReceiverMatrix.
	DrawFrustumWireframe(
		PDI,
		FMatrix(InvShadowTransform) * FTranslationMatrix(-PreShadowTranslation),
		Color,
		SDPG_World
		);
}

FVector4f FProjectedShadowInfo::GetClipToShadowBufferUvScaleBias() const
{
	const FIntPoint ShadowBufferResolution = GetShadowBufferResolution();
	const float InvBufferResolutionX = 1.0f / (float)ShadowBufferResolution.X;
	const float ShadowResolutionFractionX = 0.5f * (float)ResolutionX * InvBufferResolutionX;
	const float InvBufferResolutionY = 1.0f / (float)ShadowBufferResolution.Y;
	const float ShadowResolutionFractionY = 0.5f * (float)ResolutionY * InvBufferResolutionY;
	
	return FVector4f(ShadowResolutionFractionX,
		-ShadowResolutionFractionY,
		(X + BorderSize) * InvBufferResolutionX + ShadowResolutionFractionX,
		(Y + BorderSize) * InvBufferResolutionY + ShadowResolutionFractionY);
}


FMatrix FProjectedShadowInfo::GetScreenToShadowMatrix(const FSceneView& View, uint32 TileOffsetX, uint32 TileOffsetY, uint32 TileResolutionX, uint32 TileResolutionY) const
{
	const FIntPoint ShadowBufferResolution = GetShadowBufferResolution();
	const float InvBufferResolutionX = 1.0f / (float)ShadowBufferResolution.X;
	const float ShadowResolutionFractionX = 0.5f * (float)TileResolutionX * InvBufferResolutionX;
	const float InvBufferResolutionY = 1.0f / (float)ShadowBufferResolution.Y;
	const float ShadowResolutionFractionY = 0.5f * (float)TileResolutionY * InvBufferResolutionY;
	// Calculate the matrix to transform a screenspace position into shadow map space

	FMatrix ScreenToShadow;
	FMatrix ViewDependentTransform =
		// Z of the position being transformed is actually view space Z, 
			// Transform it into post projection space by applying the projection matrix,
			// Which is the required space before applying View.InvTranslatedViewProjectionMatrix
		View.ViewMatrices.GetScreenToClipMatrix() *
		// Transform the post projection space position into translated world space
		// Translated world space is normal world space translated to the view's origin, 
		// Which prevents floating point imprecision far from the world origin.
		View.ViewMatrices.GetClipToTranslatedWorld() *
		FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation());

	FMatrix ShadowMapDependentTransform =
		// Translate to the origin of the shadow's translated world space
		FTranslationMatrix(PreShadowTranslation) *
		// Transform into the shadow's post projection space
		// This has to be the same transform used to render the shadow depths
		FMatrix(TranslatedWorldToClipInnerMatrix) *
		// Scale and translate x and y to be texture coordinates into the ShadowInfo's rectangle in the shadow depth buffer
		// Normalize z by MaxSubjectDepth, as was done when writing shadow depths
		FMatrix(
			FPlane(ShadowResolutionFractionX,0,							0,									0),
			FPlane(0,						 -ShadowResolutionFractionY,0,									0),
			FPlane(0,						0,							InvMaxSubjectDepth,	0),
			FPlane(
				(TileOffsetX + BorderSize) * InvBufferResolutionX + ShadowResolutionFractionX,
				(TileOffsetY + BorderSize) * InvBufferResolutionY + ShadowResolutionFractionY,
				0,
				1
				)
			);
	
	if (View.UsesMobileMultiViewShaders())
	{
		// In Multiview, we split ViewDependentTransform out into ViewUniformShaderParameters.MobileMultiviewShadowTransform
		// So we can multiply it later in shader.
		ScreenToShadow = ShadowMapDependentTransform;
	}
	else
	{
		ScreenToShadow = ViewDependentTransform * ShadowMapDependentTransform;
	}
	return ScreenToShadow;
}

FMatrix FProjectedShadowInfo::GetWorldToShadowMatrix(FVector4f& ShadowmapMinMax, const FIntPoint* ShadowBufferResolutionOverride) const
{
	FIntPoint ShadowBufferResolution = ( ShadowBufferResolutionOverride ) ? *ShadowBufferResolutionOverride : GetShadowBufferResolution();

	const float InvBufferResolutionX = 1.0f / (float)ShadowBufferResolution.X;
	const float ShadowResolutionFractionX = 0.5f * (float)ResolutionX * InvBufferResolutionX;
	const float InvBufferResolutionY = 1.0f / (float)ShadowBufferResolution.Y;
	const float ShadowResolutionFractionY = 0.5f * (float)ResolutionY * InvBufferResolutionY;

	const FMatrix WorldToShadowMatrix =
		// Translate to the origin of the shadow's translated world space
		FTranslationMatrix(PreShadowTranslation) *
		// Transform into the shadow's post projection space
		// This has to be the same transform used to render the shadow depths
		FMatrix(TranslatedWorldToClipInnerMatrix) *
		// Scale and translate x and y to be texture coordinates into the ShadowInfo's rectangle in the shadow depth buffer
		// Normalize z by MaxSubjectDepth, as was done when writing shadow depths
		FMatrix(
			FPlane(ShadowResolutionFractionX,0,							0,									0),
			FPlane(0,						 -ShadowResolutionFractionY,0,									0),
			FPlane(0,						0,							InvMaxSubjectDepth,	0),
			FPlane(
				(X + BorderSize) * InvBufferResolutionX + ShadowResolutionFractionX,
				(Y + BorderSize) * InvBufferResolutionY + ShadowResolutionFractionY,
				0,
				1
			)
		);

	ShadowmapMinMax = FVector4f(
		(X + BorderSize) * InvBufferResolutionX, 
		(Y + BorderSize) * InvBufferResolutionY,
		(X + BorderSize * 2 + ResolutionX) * InvBufferResolutionX, 
		(Y + BorderSize * 2 + ResolutionY) * InvBufferResolutionY);

	return WorldToShadowMatrix;
}

void FProjectedShadowInfo::UpdateShaderDepthBias()
{
	float DepthBias = 0;
	float SlopeScaleDepthBias = 1;

	if (IsWholeScenePointLightShadow())
	{
		const bool bIsRectLight = LightSceneInfo->Proxy->GetLightType() == LightType_Rect;
		float DeptBiasConstant = 0;
		float SlopeDepthBiasConstant = 0;
		if (bIsRectLight)
		{
			DeptBiasConstant = CVarRectLightShadowDepthBias.GetValueOnRenderThread();
			SlopeDepthBiasConstant = CVarRectLightShadowSlopeScaleDepthBias.GetValueOnRenderThread();
		}
		else
		{
			DeptBiasConstant = CVarPointLightShadowDepthBias.GetValueOnRenderThread();
			SlopeDepthBiasConstant = CVarPointLightShadowSlopeScaleDepthBias.GetValueOnRenderThread();
		}

		DepthBias = DeptBiasConstant * 512.0f / FMath::Max(ResolutionX, ResolutionY);
		// * 2.0f to be compatible with the system we had before ShadowBias
		DepthBias *= 2.0f * LightSceneInfo->Proxy->GetUserShadowBias();

		SlopeScaleDepthBias = SlopeDepthBiasConstant;
		SlopeScaleDepthBias *= LightSceneInfo->Proxy->GetUserShadowSlopeBias();
	}
	else if (IsWholeSceneDirectionalShadow())
	{
		check(CascadeSettings.ShadowSplitIndex >= 0);

		// the z range is adjusted to we need to adjust here as well
		DepthBias = CVarCSMShadowDepthBias.GetValueOnRenderThread() / (MaxSubjectZ - MinSubjectZ);
		const float WorldSpaceTexelScale = ShadowBounds.W / ResolutionX;
		DepthBias = FMath::Lerp(DepthBias, DepthBias * WorldSpaceTexelScale, CascadeSettings.CascadeBiasDistribution);
		DepthBias *= LightSceneInfo->Proxy->GetUserShadowBias();

		SlopeScaleDepthBias = CVarCSMShadowSlopeScaleDepthBias.GetValueOnRenderThread();
		SlopeScaleDepthBias *= LightSceneInfo->Proxy->GetUserShadowSlopeBias();
	}
	else if (bPreShadow)
	{
		// Preshadows don't need a depth bias since there is no self shadowing
		DepthBias = 0;
		SlopeScaleDepthBias = 0;
	}
	else
	{
		// per object shadows (the whole-scene are taken care of above)
		if(bDirectionalLight)
		{
			// we use CSMShadowDepthBias cvar but this is per object shadows, maybe we want to use different settings

			// the z range is adjusted to we need to adjust here as well
			DepthBias = CVarPerObjectDirectionalShadowDepthBias.GetValueOnRenderThread() / (MaxSubjectZ - MinSubjectZ);

			float WorldSpaceTexelScale = ShadowBounds.W / FMath::Max(ResolutionX, ResolutionY);
		
			DepthBias *= WorldSpaceTexelScale;
			DepthBias *= 0.5f;	// avg GetUserShadowBias, in that case we don't want this adjustable

			SlopeScaleDepthBias = CVarPerObjectDirectionalShadowSlopeScaleDepthBias.GetValueOnRenderThread();
			SlopeScaleDepthBias *= LightSceneInfo->Proxy->GetUserShadowSlopeBias();
		}
		else // Only spot-lights left (both whole-scene and per-object), as point-lights dont have per-object shadowing
		{
			ensure(!bDirectionalLight);
			ensure(!bOnePassPointLightShadow);
			if (bPerObjectOpaqueShadow)
			{
				// spot lights (old code, might need to be improved)
				const float LightTypeDepthBias = CVarPerObjectSpotLightShadowDepthBias.GetValueOnRenderThread();
				DepthBias = LightTypeDepthBias * 512.0f / ((MaxSubjectZ - MinSubjectZ) * FMath::Max(ResolutionX, ResolutionY));
				// * 2.0f to be compatible with the system we had before ShadowBias
				DepthBias *= 2.0f * LightSceneInfo->Proxy->GetUserShadowBias();

				SlopeScaleDepthBias = CVarPerObjectSpotLightShadowSlopeScaleDepthBias.GetValueOnRenderThread();
				SlopeScaleDepthBias *= LightSceneInfo->Proxy->GetUserShadowSlopeBias();
			}
		else
		{
			// spot lights (old code, might need to be improved)
			const float LightTypeDepthBias = CVarSpotLightShadowDepthBias.GetValueOnRenderThread();
			DepthBias = LightTypeDepthBias * 512.0f / ((MaxSubjectZ - MinSubjectZ) * FMath::Max(ResolutionX, ResolutionY));
			// * 2.0f to be compatible with the system we had before ShadowBias
			DepthBias *= 2.0f * LightSceneInfo->Proxy->GetUserShadowBias();

			SlopeScaleDepthBias = CVarSpotLightShadowSlopeScaleDepthBias.GetValueOnRenderThread();
			SlopeScaleDepthBias *= LightSceneInfo->Proxy->GetUserShadowSlopeBias();
		}
		}

		// Prevent a large depth bias due to low resolution from causing near plane clipping
		DepthBias = FMath::Min(DepthBias, .1f);
	}

	ShaderDepthBias = FMath::Max(DepthBias, 0.0f);
	ShaderSlopeDepthBias = FMath::Max(DepthBias * SlopeScaleDepthBias, 0.0f);
	ShaderMaxSlopeDepthBias = CVarShadowMaxSlopeScaleDepthBias.GetValueOnRenderThread();
}

float FProjectedShadowInfo::ComputeTransitionSize() const
{
	float TransitionSize = 1.0f;

	if (IsWholeScenePointLightShadow())
	{
		// todo: optimize
		TransitionSize = bDirectionalLight ? (1.0f / CVarShadowTransitionScale.GetValueOnRenderThread()) : (1.0f / CVarSpotLightShadowTransitionScale.GetValueOnRenderThread());
		// * 2.0f to be compatible with the system we had before ShadowBias
		TransitionSize *= 2.0f * LightSceneInfo->Proxy->GetUserShadowBias();
	}
	else if (IsWholeSceneDirectionalShadow())
	{
		check(CascadeSettings.ShadowSplitIndex >= 0);

		// todo: remove GetShadowTransitionScale()
		// make 1/ ShadowTransitionScale, SpotLightShadowTransitionScale

		// the z range is adjusted to we need to adjust here as well
		TransitionSize = CVarCSMShadowDepthBias.GetValueOnRenderThread() / (MaxSubjectZ - MinSubjectZ);

		float WorldSpaceTexelScale = ShadowBounds.W / ResolutionX;

		TransitionSize *= WorldSpaceTexelScale;
		TransitionSize *= LightSceneInfo->Proxy->GetUserShadowBias();
	}
	else if (bPreShadow)
	{
		// Preshadows don't have self shadowing, so make sure the shadow starts as close to the caster as possible
		TransitionSize = 0.0f;
	}
	else
	{
		// todo: optimize
		TransitionSize = bDirectionalLight ? (1.0f / CVarShadowTransitionScale.GetValueOnRenderThread()) : (1.0f / CVarSpotLightShadowTransitionScale.GetValueOnRenderThread());
		// * 2.0f to be compatible with the system we had before ShadowBias
		TransitionSize *= 2.0f * LightSceneInfo->Proxy->GetUserShadowBias();
	}

	// Make sure that shadow soft transition size is greater than zero so 1/TransitionSize shader parameter won't be INF.
	const float MinTransitionSize = 0.00001f;
	return FMath::Max(TransitionSize, MinTransitionSize);
}

bool FProjectedShadowInfo::IsWholeSceneDirectionalShadow() const
{
	return bWholeSceneShadow && CascadeSettings.ShadowSplitIndex >= 0 && bDirectionalLight;
}

bool FProjectedShadowInfo::IsWholeScenePointLightShadow() const
{
	return bWholeSceneShadow && (LightSceneInfo->Proxy->GetLightType() == LightType_Point || LightSceneInfo->Proxy->GetLightType() == LightType_Rect);
}

float FProjectedShadowInfo::GetShaderReceiverDepthBias() const
{
	float ShadowReceiverBias = 1;
	{
		switch (GetLightSceneInfo().Proxy->GetLightType())
		{
		case LightType_Directional	: ShadowReceiverBias = CVarCSMShadowReceiverBias.GetValueOnRenderThread(); break;
		case LightType_Rect			: ShadowReceiverBias = CVarRectLightShadowReceiverBias.GetValueOnRenderThread(); break;
		case LightType_Spot			: ShadowReceiverBias = CVarSpotLightShadowReceiverBias.GetValueOnRenderThread(); break;
		case LightType_Point		: ShadowReceiverBias = GetShaderSlopeDepthBias(); break;
		}
	}

	// Return the min lerp value for depth biasing
	// 0 : max bias when NoL == 0
	// 1 : no bias
	return 1.0f - FMath::Clamp(ShadowReceiverBias, 0.0f, 1.0f);
}
/*-----------------------------------------------------------------------------
FDeferredShadingSceneRenderer
-----------------------------------------------------------------------------*/

/**
 * Used by RenderLights to figure out if projected shadows need to be rendered to the attenuation buffer.
 *
 * @param LightSceneInfo Represents the current light
 * @return true if anything needs to be rendered
 */
bool FSceneRenderer::CheckForProjectedShadows( const FLightSceneInfo* LightSceneInfo ) const
{
	// First person self shadow is currently implemented as screen space shadow but serves as a standalone shadowing solution
	// for first person primitives as they are not supported by any other shadowing method.
	if (ShouldRenderFirstPersonSelfShadowForLight(*this, ViewFamily, Views, *LightSceneInfo))
	{
		return true;
	}

	// If light has ray-traced occlusion enabled, then it will project some shadows. No need 
	// for doing a lookup through shadow maps data
	const ELightOcclusionType LightOcclusionType = GetLightOcclusionType(*LightSceneInfo->Proxy, ViewFamily);
	if (LightOcclusionType == ELightOcclusionType::Raytraced)
		return true;

	// Find the projected shadows cast by this light.
	const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];
	for( int32 ShadowIndex=0; ShadowIndex < VisibleLightInfo.AllProjectedShadows.Num(); ShadowIndex++ )
	{
		const FProjectedShadowInfo* ProjectedShadowInfo = VisibleLightInfo.AllProjectedShadows[ShadowIndex];

		// Check that the shadow is visible in at least one view before rendering it.
		bool bShadowIsVisible = false;
		for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			if (ProjectedShadowInfo->DependentView && ProjectedShadowInfo->DependentView != &View && ProjectedShadowInfo->DependentView != View.GetPrimaryView())
			{
				continue;
			}
			const FVisibleLightViewInfo& VisibleLightViewInfo = View.VisibleLightInfos[LightSceneInfo->Id];
			bShadowIsVisible |= VisibleLightViewInfo.ProjectedShadowVisibilityMap[ShadowIndex];
		}

		if(bShadowIsVisible)
		{
			return true;
		}
	}
	return false;
}


void FSceneRenderer::RenderShadowProjections(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef OutputTexture,
	const FMinimalSceneTextures& SceneTextures,
	const FLightSceneProxy* LightSceneProxy,
	TArrayView<const FProjectedShadowInfo* const> Shadows,
	bool bSubPixelShadow,
	bool bProjectingForForwardShading)
{
	CheckShadowDepthRenderCompleted();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		const bool bIsInstancedStereo = View.IsInstancedStereo();
		if ((!bIsInstancedStereo && !View.ShouldRenderView()) || (bSubPixelShadow && !HairStrands::HasViewHairStrandsData(View)))
		{
			continue;
		}
		const FExclusiveDepthStencil ExclusiveDepthStencil = FExclusiveDepthStencil::DepthRead_StencilWrite;
		View.BeginRenderView();

		// Sanity check
		if (bSubPixelShadow)
		{
			check(View.HairStrandsViewData.VisibilityData.HairOnlyDepthTexture);
		}

		FShadowProjectionPassParameters CommonPassParameters;
		CommonPassParameters.SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);
		CommonPassParameters.HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);

		if (Substrate::IsSubstrateEnabled())
		{
			CommonPassParameters.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		}

		FRDGTextureRef DepthStencilTexture = nullptr;
		// When using MSAA we need to bind the texture with the correct number of samples
		if (!(OutputTexture->Desc.NumSamples > 1) && SceneTextures.Depth.IsSeparate() && HasBeenProduced(SceneTextures.Depth.Resolve))
		{
			DepthStencilTexture = SceneTextures.Depth.Resolve;
		}
		else
		{
			DepthStencilTexture = SceneTextures.Depth.Target;
		}
		CommonPassParameters.RenderTargets[0] = FRenderTargetBinding(OutputTexture, GetLoadActionIfProduced(OutputTexture, ERenderTargetLoadAction::EClear));
		CommonPassParameters.RenderTargets.DepthStencil =
			bSubPixelShadow ?
			FDepthStencilBinding(View.HairStrandsViewData.VisibilityData.HairOnlyDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, ExclusiveDepthStencil) :
			FDepthStencilBinding(DepthStencilTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, ExclusiveDepthStencil);

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		// Project the shadow depth buffers onto the scene.
		for (const FProjectedShadowInfo* ProjectedShadowInfo : Shadows)
		{
			if (ProjectedShadowInfo->bAllocated) 
			{
				// Only project the shadow if it's large enough in this particular view (split screen, etc... may have shadows that are large in one view but irrelevantly small in others)
				if (ProjectedShadowInfo->FadeAlphas[ViewIndex] > 1.0f / 256.0f)
				{
					if (ProjectedShadowInfo->bOnePassPointLightShadow)
					{
						ProjectedShadowInfo->RenderOnePassPointLightProjection(GraphBuilder, CommonPassParameters, ViewIndex, View, LightSceneProxy, bProjectingForForwardShading, bSubPixelShadow);
					}
					else
					{
						ProjectedShadowInfo->RenderProjection(GraphBuilder, CommonPassParameters, ViewIndex, &View, LightSceneProxy, this, bProjectingForForwardShading, bSubPixelShadow);
					}
				}
			}
		}
	}
}

void FSceneRenderer::RenderShadowProjections(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef ScreenShadowMaskTexture,
	FRDGTextureRef ScreenShadowMaskSubPixelTexture,
	const FLightSceneInfo* LightSceneInfo,
	bool bProjectingForForwardShading)
{
	CheckShadowDepthRenderCompleted();

	const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];
	const FLightSceneProxy* LightSceneProxy = LightSceneInfo->Proxy;

	// Allocate arrays using the graph allocator so we can safely reference them in passes.
	using FProjectedShadowInfoArray = TArray<FProjectedShadowInfo*, SceneRenderingAllocator>;
	auto& DistanceFieldShadows = *GraphBuilder.AllocObject<FProjectedShadowInfoArray>();
	auto& NormalShadows = *GraphBuilder.AllocObject<FProjectedShadowInfoArray>();

	for (int32 ShadowIndex = 0; ShadowIndex < VisibleLightInfo.ShadowsToProject.Num(); ShadowIndex++)
	{
		FProjectedShadowInfo* ProjectedShadowInfo = VisibleLightInfo.ShadowsToProject[ShadowIndex];
		if (ProjectedShadowInfo->bRayTracedDistanceField)
		{
			DistanceFieldShadows.Add(ProjectedShadowInfo);
		}
		else if (ProjectedShadowInfo->HasVirtualShadowMap())
		{
			// Skip - handled elsewhere
		}
		else
		{
			NormalShadows.Add(ProjectedShadowInfo);
		}
	}

	if (NormalShadows.Num() > 0)
	{
		const auto RenderNormalShadows = [&](FRDGTextureRef OutputTexture, bool bSubPixel)
		{
			FString LightNameWithLevel;
			GetLightNameForDrawEvent(LightSceneProxy, LightNameWithLevel);
			RDG_EVENT_SCOPE(GraphBuilder, "%s", *LightNameWithLevel);

			FSceneRenderer::RenderShadowProjections(
				GraphBuilder,
				OutputTexture,
				SceneTextures,
				LightSceneProxy,
				NormalShadows,
				bSubPixel,
				bProjectingForForwardShading);
		};

		if (ScreenShadowMaskTexture)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "Shadows");
			RenderNormalShadows(ScreenShadowMaskTexture, false);
		}

		if (ScreenShadowMaskSubPixelTexture)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "SubPixelShadows");
			RenderNormalShadows(ScreenShadowMaskSubPixelTexture, true);
		}
	}

	if (ScreenShadowMaskTexture && DistanceFieldShadows.Num() > 0)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, DistanceFieldShadows, "DistanceFieldShadows");

		// Distance field shadows need to be renderer last as they blend over far shadow cascades.
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

			FIntRect ScissorRect;
			if (!LightSceneProxy->GetScissorRect(ScissorRect, View, View.ViewRect))
			{
				ScissorRect = View.ViewRect;
			}

			if (ScissorRect.Area() > 0)
			{
				for (int32 ShadowIndex = 0; ShadowIndex < DistanceFieldShadows.Num(); ShadowIndex++)
				{
					FProjectedShadowInfo* ProjectedShadowInfo = DistanceFieldShadows[ShadowIndex];

					if (Views.Num() == 1 || !ProjectedShadowInfo->DependentView || ProjectedShadowInfo->DependentView == &View || ProjectedShadowInfo->DependentView == View.GetPrimaryView())
					{
						ProjectedShadowInfo->RenderRayTracedDistanceFieldProjection(
							GraphBuilder,
							SceneTextures,
							ScreenShadowMaskTexture,
							View,
							ViewIndex,
							ScissorRect,
							bProjectingForForwardShading);
					}
				}
			}
		}
	}
}

void FSceneRenderer::BeginAsyncDistanceFieldShadowProjections(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures, const FDynamicShadowsTaskData* TaskData) const
{
	extern int32 GDFShadowAsyncCompute;

	TConstArrayView<FProjectedShadowInfo*> ProjectedDistanceFieldShadows = GetProjectedDistanceFieldShadows(TaskData);

	if (!!GDFShadowAsyncCompute && ViewFamily.EngineShowFlags.DynamicShadows && GetShadowQuality() > 0 && ProjectedDistanceFieldShadows.Num() > 0)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, DistanceFieldShadows, "DistanceFieldShadows");

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

			for (int32 DFShadowIndex = 0; DFShadowIndex < ProjectedDistanceFieldShadows.Num(); ++DFShadowIndex)
			{
				FProjectedShadowInfo* ProjectedShadowInfo = ProjectedDistanceFieldShadows[DFShadowIndex];

				FIntRect ScissorRect;
				if (ProjectedShadowInfo->bDirectionalLight || !ProjectedShadowInfo->GetLightSceneInfo().Proxy->GetScissorRect(ScissorRect, View, View.ViewRect))
				{
					ScissorRect = View.ViewRect;
				}

				if (ScissorRect.Area() > 0 && (Views.Num() == 1 || ProjectedShadowInfo->DependentView == &View || ProjectedShadowInfo->DependentView == View.GetPrimaryView() || !ProjectedShadowInfo->DependentView))
				{
					// Kick off distance field shadow calculation in async compute
					// Don't need store result reference because it is internally cached by FProjectedShadowInfo
					ProjectedShadowInfo->RenderRayTracedDistanceFieldProjection(GraphBuilder, true, SceneTextures, View, ScissorRect);
				}
			}
		}
	}
}


void FDeferredShadingSceneRenderer::CollectLightForTranslucencyLightingVolumeInjection(
	const FLightSceneInfo* LightSceneInfo,
	bool bSupportShadowMaps,
	FTranslucentLightInjectionCollector& Collector)
{
	FLightGridSceneExtension& LightGridExtension = Scene->GetExtension<FLightGridSceneExtension>();

	bool bInjectedShadowedLight = false;
	if (bSupportShadowMaps)
	{
		const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];
		const TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ShadowMaps = VisibleLightInfo.ShadowsToProject;
		for (int32 ShadowIndex = 0; ShadowIndex < ShadowMaps.Num(); ShadowIndex++)
		{
			const FProjectedShadowInfo* ProjectedShadowInfo = ShadowMaps[ShadowIndex];

			if (ProjectedShadowInfo->bAllocated
				&& ProjectedShadowInfo->bWholeSceneShadow
				// Not supported on translucency yet
				&& !ProjectedShadowInfo->bRayTracedDistanceField
				// Don't inject shadowed lighting with whole scene shadows used for previewing a light with static shadows,
				// Since that would cause a mismatch with the built lighting
				// However, stationary directional lights allow whole scene shadows that blend with precomputed shadowing
				&& (!LightSceneInfo->Proxy->HasStaticShadowing() || ProjectedShadowInfo->IsWholeSceneDirectionalShadow()))
			{
				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
				{
					const FViewInfo& View = Views[ViewIndex];
					// The translucency volume is shared between eyes when doing stereo so only need to inject once for primary
					if (View.IsPrimarySceneView() && LightSceneInfo->ShouldRenderLight(View))
					{
						const FViewInfo* DependentView = ProjectedShadowInfo->DependentView;
						if (DependentView == nullptr || DependentView == &View)
						{
							FLightGridSceneExtension::FViewData* LightGridViewData = LightGridExtension.GetViewData(View);

							Collector.AddLightForInjection(View, ViewIndex, LightGridViewData, VisibleLightInfos, *LightSceneInfo, ProjectedShadowInfo);
							bInjectedShadowedLight = true;
						}
					}
				}
			}
		}
	}

	if (!bInjectedShadowedLight)
	{
		// Add unshadowed (or shadowed via VSM) lighting contribution
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			FViewInfo& View = Views[ViewIndex];
			if (View.IsPrimarySceneView() && LightSceneInfo->ShouldRenderLight(View))
			{
				FLightGridSceneExtension::FViewData* LightGridViewData = LightGridExtension.GetViewData(View);

				Collector.AddLightForInjection(View, ViewIndex, LightGridViewData, VisibleLightInfos, *LightSceneInfo);
			}
		}
	}
}


void FDeferredShadingSceneRenderer::RenderDeferredShadowProjections(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef ScreenShadowMaskTexture,
	FRDGTextureRef ScreenShadowMaskSubPixelTexture)
{
	CheckShadowDepthRenderCompleted();

	SCOPED_NAMED_EVENT(FDeferredShadingSceneRenderer_RenderShadowProjections, FColor::Emerald);
	SCOPE_CYCLE_COUNTER(STAT_ProjectedShadowDrawTime);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, ShadowProjection, "ShadowProjectionOnOpaque");

	const bool bProjectingForForwardShading = false;
	const bool bIsGBufferInput = ScreenShadowMaskTexture != nullptr;
	const bool bIsHairInput = ScreenShadowMaskSubPixelTexture != nullptr;

	if (bIsGBufferInput)
	{
		RenderShadowProjections(GraphBuilder, SceneTextures, ScreenShadowMaskTexture, nullptr, LightSceneInfo, bProjectingForForwardShading);
		GetSceneExtensionsRenderers().GetRenderer<FShadowSceneRenderer>().ApplyVirtualShadowMapProjectionForLight(GraphBuilder, SceneTextures, LightSceneInfo, EVirtualShadowMapProjectionInputType::GBuffer, ScreenShadowMaskTexture);

		const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];
		RenderCapsuleDirectShadows(GraphBuilder, *LightSceneInfo, ScreenShadowMaskTexture, VisibleLightInfo.CapsuleShadowsToProject, bProjectingForForwardShading);

		// Inject deep shadow mask for regular shadow map. When using virtual shadow map, it is directly handled in the shadow kernel.
		if (HairStrands::HasViewHairStrandsData(Views))
		{
			bool bNeedHairShadowMaskPass = false;
			const bool bVirtualShadowOnePass = VisibleLightInfo.VirtualShadowMapClipmaps.Num() > 0;
			if (!bVirtualShadowOnePass && VisibleLightInfo.ShadowsToProject.Num() > 0)
			{
				bNeedHairShadowMaskPass = !VisibleLightInfo.ShadowsToProject[0]->HasVirtualShadowMap();
			}
			if (bNeedHairShadowMaskPass)
			{
				RenderHairStrandsShadowMask(GraphBuilder, Views, LightSceneInfo, VisibleLightInfos, false /*bForward*/, ScreenShadowMaskTexture);
			}
		}
	}
	else if (bIsHairInput)
	{
		RenderShadowProjections(GraphBuilder, SceneTextures, nullptr, ScreenShadowMaskSubPixelTexture, LightSceneInfo, bProjectingForForwardShading);
		GetSceneExtensionsRenderers().GetRenderer<FShadowSceneRenderer>().ApplyVirtualShadowMapProjectionForLight(GraphBuilder, SceneTextures, LightSceneInfo, EVirtualShadowMapProjectionInputType::HairStrands, ScreenShadowMaskSubPixelTexture);
	}
}

void FMobileSceneRenderer::RenderModulatedShadowProjections(FRHICommandList& RHICmdList, int32 ViewIndex, const FViewInfo& View)
{
	if (!ViewFamily.EngineShowFlags.DynamicShadows || View.bIsPlanarReflection || bRequiresShadowProjections)
	{
		return;
	}

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RenderShadowProjections);
	SCOPED_NAMED_EVENT(FMobileSceneRenderer_RenderModulatedShadowProjections, FColor::Emerald);
	SCOPE_CYCLE_COUNTER(STAT_ProjectedShadowDrawTime);

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, ShadowProjection, "ShadowProjectionOnOpaque");

	// render shadowmaps for relevant lights.
	for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
	{
		const FLightSceneInfo* LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;
		const FLightSceneProxy* LightSceneProxy = LightSceneInfo->Proxy;

		if(LightSceneInfo->ShouldRenderLightViewIndependent() && LightSceneProxy && LightSceneProxy->CastsModulatedShadows())
		{
			const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];

			if (VisibleLightInfo.ShadowsToProject.Num() > 0)
			{			
				LightSceneProxy->SetScissorRect(RHICmdList, View, View.ViewRect);
				TArrayView<const FProjectedShadowInfo* const> Shadows = VisibleLightInfo.ShadowsToProject;

				// Project the shadow depth buffers onto the scene.
				for (const FProjectedShadowInfo* ProjectedShadowInfo : Shadows)
				{
					if (ProjectedShadowInfo->bAllocated)
					{
						// Only project the shadow if it's large enough in this particular view (split screen, etc... may have shadows that are large in one view but irrelevantly small in others)
						if (ProjectedShadowInfo->FadeAlphas[ViewIndex] > 1.0f / 256.0f 
							// Skip, if it is a whole scene directional shadow
							&& !ProjectedShadowInfo->IsWholeSceneDirectionalShadow())
						{
							checkSlow(!ProjectedShadowInfo->bOnePassPointLightShadow);
							ProjectedShadowInfo->RenderMobileModulatedShadowProjection(RHICmdList, ViewIndex, &View, LightSceneProxy, this);
						}
					}
				}

				RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
			}
		}
	}
}

void FMobileSceneRenderer::ClearMobileScreenSpaceShadows(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FLightSceneProxy* LightSceneProxy, FRDGTextureRef ScreenShadowMaskTexture)
{
	FRenderTargetParameters* ClearPassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
	ClearPassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenShadowMaskTexture, ERenderTargetLoadAction::ENoAction);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ScreenSpaceShadowMaskClearQuad"),
		ClearPassParameters,
		ERDGPassFlags::Raster,
		[&View, LightSceneProxy](FRHICommandList& RHICmdList)
		{
			FIntRect ScissorRect;
			if (!LightSceneProxy->GetScissorRect(ScissorRect, View, View.ViewRect))
			{
				ScissorRect = View.ViewRect;
			}

			if (ScissorRect.Min.X < ScissorRect.Max.X && ScissorRect.Min.Y < ScissorRect.Max.Y)
			{
				RHICmdList.SetViewport(ScissorRect.Min.X, ScissorRect.Min.Y, 0.0f, ScissorRect.Max.X, ScissorRect.Max.Y, 1.0f);
				DrawClearQuad(RHICmdList, true, FLinearColor(1, 1, 1, 1), false, 0, false, 0);
			}
		});
}

void FMobileSceneRenderer::RenderMobileScreenSpaceShadows(FRDGBuilder& GraphBuilder, FIntPoint Extent, FRDGTextureRef ScreenShadowMaskTexture)
{
	static auto* ContactShadowsCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ContactShadows"));
	if (ScreenShadowMaskTexture == nullptr
		|| IsOpenGLPlatform(ShaderPlatform) // FIXME: GL does not support writing to rg32f images
		|| (ContactShadowsCVar && ContactShadowsCVar->GetValueOnRenderThread() == 0))
	{
		return;
	}
		
	RDG_RHI_EVENT_SCOPE(GraphBuilder, RenderMobileScreenSpaceShadows);

	const FMinimalSceneTextures& SceneTextures = GetActiveSceneTextures();
	for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
	{
		const FLightSceneInfo* LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;

		const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];
		const FLightSceneProxy* LightSceneProxy = LightSceneInfo->Proxy;

		const bool bProjectingForForwardShading = true;

		if (LightSceneProxy->GetLightType() == LightType_Directional &&
			LightSceneInfo->GetDynamicShadowMapChannel() != -1)
		{
			// Dynamic shadows are projected into channels of the light attenuation texture based on their assigned DynamicShadowMapChannel
			// Only render screen space shadows if light is assigned to a valid DynamicShadowMapChannel
			RenderScreenSpaceShadows(GraphBuilder, SceneTextures, Views, LightSceneInfo, bProjectingForForwardShading, ScreenShadowMaskTexture);
		}
	}
}

FRDGTextureRef FMobileSceneRenderer::RenderMobileShadowProjections(FRDGBuilder& GraphBuilder, FIntPoint Extent, bool bIsMobileMultiView)
{
	RDG_RHI_EVENT_SCOPE(GraphBuilder, RenderMobileShadowProjections);
	
	FRDGTextureDesc TargetDesc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Extent, PF_B8G8R8A8, FClearValueBinding::White, TexCreate_RenderTargetable | TexCreate_ShaderResource, bIsMobileMultiView);
	FRDGTextureRef ScreenShadowMaskTexture = GraphBuilder.CreateTexture(TargetDesc, TEXT("ScreenSpaceShadowMaskTextureMobile"));

	ScreenShadowMaskTexture->EncloseVisualizeExtent(FamilySize);
	
	const FMinimalSceneTextures& SceneTextures = GetActiveSceneTextures();
	for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
	{
		const FLightSceneInfo* LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;

		const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];
		const FLightSceneProxy* LightSceneProxy = LightSceneInfo->Proxy;

		const bool bProjectingForForwardShading = !(IsMobileDeferredShadingEnabled(ShaderPlatform) && LightSceneProxy->GetLightType() == LightType_Point);

		// Other than point light, local light shadows don't render to shadow mask texture on mobile deferred
		if (LightSceneProxy->GetLightType() == LightType_Directional || LightSceneProxy->GetLightType() == LightType_Point || !IsMobileDeferredShadingEnabled(ShaderPlatform))
		{
			RenderShadowProjections(GraphBuilder, SceneTextures,
				ScreenShadowMaskTexture,
				nullptr,
				LightSceneInfo,
				bProjectingForForwardShading);
		}
		RenderCapsuleDirectShadows(GraphBuilder, *LightSceneInfo, ScreenShadowMaskTexture, VisibleLightInfo.CapsuleShadowsToProject, bProjectingForForwardShading);
	}

	return ScreenShadowMaskTexture;
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FTranslucentSelfShadowUniformParameters, "TranslucentSelfShadow");

void SetupTranslucentSelfShadowUniformParameters(const FProjectedShadowInfo* ShadowInfo, FTranslucentSelfShadowUniformParameters& OutParameters)
{
	if (ShadowInfo)
	{
		FVector4f ShadowmapMinMax;
		FMatrix WorldToShadowMatrixValue = ShadowInfo->GetWorldToShadowMatrix(ShadowmapMinMax);

		OutParameters.WorldToShadowMatrix = FMatrix44f(WorldToShadowMatrixValue);		// LWC_TODO: Precision loss?
		OutParameters.ShadowUVMinMax = ShadowmapMinMax;

		const FLightSceneProxy* const LightProxy = ShadowInfo->GetLightSceneInfo().Proxy;
		OutParameters.DirectionalLightDirection = FVector3f(LightProxy->GetDirection());

		//@todo - support fading from both views
		const float FadeAlpha = ShadowInfo->FadeAlphas[0];
		FLinearColor LightColor;
		if (LightProxy->IsUsedAsAtmosphereSunLight())
		{
			LightColor = LightProxy->GetSunIlluminanceAccountingForSkyAtmospherePerPixelTransmittance();
		}
		else
		{
			LightColor = LightProxy->GetColor();
		}
		// Incorporate the diffuse scale of 1 / PI into the light color
		OutParameters.DirectionalLightColor = FVector4f(FVector3f(LightColor * FadeAlpha / PI), FadeAlpha);

		OutParameters.Transmission0 = ShadowInfo->RenderTargets.ColorTargets[0]->GetRHI();
		OutParameters.Transmission1 = ShadowInfo->RenderTargets.ColorTargets[1]->GetRHI();
		OutParameters.Transmission0Sampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		OutParameters.Transmission1Sampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
	else
	{
		OutParameters.Transmission0 = GBlackTexture->TextureRHI;
		OutParameters.Transmission1 = GBlackTexture->TextureRHI;
		OutParameters.Transmission0Sampler = GBlackTexture->SamplerStateRHI;
		OutParameters.Transmission1Sampler = GBlackTexture->SamplerStateRHI;
		
		OutParameters.DirectionalLightColor = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
	}
}

void FEmptyTranslucentSelfShadowUniformBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	FTranslucentSelfShadowUniformParameters Parameters;
	SetupTranslucentSelfShadowUniformParameters(nullptr, Parameters);
	SetContentsNoUpdate(Parameters);

	Super::InitRHI(RHICmdList);
}


FViewMatrices FProjectedShadowInfo::GetShadowDepthRenderingViewMatrices(int32 CubeFaceIndex, bool bUseForVSMCubeFaceWorkaround) const
{
	FViewMatrices::FMinimalInitializer MatricesInitializer;
	MatricesInitializer.ViewOrigin = -PreShadowTranslation;
	MatricesInitializer.ConstrainedViewRect = GetOuterViewRect();

	ensure(!bOnePassPointLightShadow || CubeFaceIndex >= 0 && CubeFaceIndex < 6);
	if (bOnePassPointLightShadow && CubeFaceIndex >= 0 && CubeFaceIndex < 6)
	{
		if (bUseForVSMCubeFaceWorkaround)
		{
			MatricesInitializer.ViewRotationMatrix = OnePassShadowViewMatrices[CubeFaceIndex] * FScaleMatrix(FVector(1, 1, -1));
		}
		else
		{
			MatricesInitializer.ViewRotationMatrix = OnePassShadowViewMatrices[CubeFaceIndex];
		}
		MatricesInitializer.ProjectionMatrix = OnePassShadowFaceProjectionMatrix;
	}
	else
	{
		MatricesInitializer.ViewRotationMatrix = TranslatedWorldToView;
		MatricesInitializer.ProjectionMatrix = ViewToClipOuter;
	}

	return FViewMatrices(MatricesInitializer);
}


/** */
TGlobalResource< FEmptyTranslucentSelfShadowUniformBuffer > GEmptyTranslucentSelfShadowUniformBuffer;
