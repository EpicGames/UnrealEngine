// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowRendering.h: Shadow rendering definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "Templates/RefCounting.h"
#include "ConvexVolume.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "LightRendering.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "Math/MathFwd.h"
#include "Math/Plane.h"
#include "Math/Sphere.h"
#include "MeshDrawCommands.h"
#include "MeshPassProcessor.h"
#include "PSOPrecache.h"
#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "RHIFeatureLevel.h"
#include "RHIFwd.h"
#include "RHIResources.h"
#include "RHIShaderParameters.h"
#include "RHIShaderPlatform.h"
#include "RHIStaticStates.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphFwd.h"
#include "RenderGraphResources.h"
#include "RenderResource.h"
#include "Runtime/Engine/Public/SceneInterface.h"
#include "Runtime/Engine/Public/SceneManagement.h"
#include "Runtime/RenderCore/Public/Shader.h"
#include "Runtime/RenderCore/Public/ShaderParameterMacros.h"
#include "Runtime/RenderCore/Public/ShaderParameterUtils.h"
#include "Runtime/Renderer/Internal/SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "Runtime/Renderer/Private/SceneRendering.h"
#include "VirtualShadowMapDefinitions.h"
#include "Runtime/Renderer/Public/SystemTextures.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "RHI.h"
#include "UniformBuffer.h"
#include "SceneInterface.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "HitProxies.h"
#include "RendererInterface.h"
#include "SceneManagement.h"
#include "ScenePrivateBase.h"
#include "SceneCore.h"
#include "SystemTextures.h"
#include "PostProcess/SceneRenderTargets.h"
#include "SceneRenderTargetParameters.h"
#include "ShaderParameterUtils.h"
#include "HairStrands/HairStrandsRendering.h"
#include "Substrate/Substrate.h"
#include "SimpleMeshDrawCommandPass.h"
#include "Engine/SubsurfaceProfile.h"
#include "SceneCulling/SceneCulling.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

class FPrimitiveSceneInfo;
class FPrimitiveSceneProxy;
class FProjectedShadowInfo;
class FScene;
class FSceneRenderer;
class FViewInfo;
class FVirtualShadowMapArrayCacheManager;
class FVirtualShadowMapPerLightCacheEntry;
class FLightTileIntersectionParameters;
class FDistanceFieldCulledObjectBufferParameters;
class FSimpleMeshDrawCommandPass;
class FInstanceCullingDrawParams;
struct FCullingVolume;
class FSceneCullingRendererHierarchicalCullingQuery;
struct FSceneTexturesConfig;
class FPrimitiveDrawInterface;
class FShadowProjectionPassParameters;
class FPrimitiveSceneInfoCompact;
struct FDynamicShadowsTaskData;
class FInstanceCullingManager;
class FShadowMapRenderTargetsRefCounted;
class FVirtualShadowMapClipmap;
class FPerObjectProjectedShadowInitializer;
class FWholeSceneProjectedShadowInitializer;
class FMeshElementCollector;
class FStaticMeshBatch;
class FStaticMeshBatchRelevance;
namespace Nanite
{
	struct FPackedView;
}

namespace ShadowRendering
{
	// Use Shadow stencil mask is set to 0x07u instead of 0xFF so that that highest bit can be used for Substrate classification without clearing the stencil bit for pre-shadow/per-object static shadow mask
	constexpr uint32 ShadowStencilMask = 0x07u;
};

DECLARE_GPU_DRAWCALL_STAT_NAMED_EXTERN(ShadowDepths, TEXT("Shadow Depths"));

/** Renders a cone with a spherical cap, used for rendering spot lights in deferred passes. */
extern void DrawStencilingCone(const FMatrix& ConeToWorld, float ConeAngle, float SphereRadius, const FVector& PreViewTranslation);

class FShadowDepthBasePS;

/** 
 * Overrides a material used for shadow depth rendering with the default material when appropriate.
 * Overriding in this manner can reduce state switches and the number of shaders that have to be compiled.
 * This logic needs to stay in sync with shadow depth shader ShouldCache logic.
 */
void OverrideWithDefaultMaterialForShadowDepth(
	const FMaterialRenderProxy*& InOutMaterialRenderProxy, 
	const FMaterial*& InOutMaterialResource,
	ERHIFeatureLevel::Type InFeatureLevel
	);

enum EShadowDepthRenderMode
{
	/** The render mode used by regular shadows */
	ShadowDepthRenderMode_Normal,

	/** The render mode used when injecting emissive-only objects into the RSM. */
	ShadowDepthRenderMode_EmissiveOnly,

	/** The render mode used when rendering volumes which block global illumination. */
	ShadowDepthRenderMode_GIBlockingVolumes,
};

enum class EShadowDepthType
{
	None = 0,
	VSM = 1 << 0,
	Directional = 1 << 1,
	Spot = 1 << 2,
	OnePassPoint = 1 << 3
};

ENUM_CLASS_FLAGS(EShadowDepthType);

extern EMeshPass::Type GetShadowMeshPassType(EShadowDepthType ShadowDepthType);

extern bool UseCachedMeshDrawCommands(EShadowDepthType ShadowDepthType);

/**
 * Used to select what meshes to draw into what shadow infos. Each mesh selects the support it has, and
 * the shadow info stores a mask of what types it collects. This makes it easy to allow regular SMs to
 * collect both types if VSMs are disabled.
 */
enum class EShadowMeshSelection : uint8
{
	SM = 1U << 0U,
	VSM = 1U << 1U, // declared if GPU-Scene instace culling is supported.
	All = SM | VSM,
};

ENUM_CLASS_FLAGS(EShadowMeshSelection)


class FShadowDepthPassMeshProcessor : public FSceneRenderingAllocatorObject<FShadowDepthPassMeshProcessor>, public FMeshPassProcessor
{
public:

	FShadowDepthPassMeshProcessor(
		const FScene* Scene, 
		const ERHIFeatureLevel::Type InFeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand,
		EShadowDepthType ShadowDepthType,
		FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

	FMeshPassProcessorRenderState PassDrawRenderState;

private:
	bool TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	bool Process(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	void CollectPSOInitializersForEachShadowDepthType(
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		bool bCastShadowAsTwoSided,
		TArray<FPSOPrecacheData>& PSOInitializers);

	void CollectPSOInitializersForEachStreamSetup(
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		EShadowDepthType InShadowDepthType,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		bool bRequired,
		TArray<FPSOPrecacheData>& PSOInitializers);

	void CollectPSOInitializersInternal(
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		EShadowDepthType InShadowDepthType,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		bool bSupportsPositionAndNormalOnlyStream,
		bool bRequired,
		TArray<FPSOPrecacheData>& PSOInitializers);

	EShadowDepthType ShadowDepthType;
	EShadowMeshSelection MeshSelectionMask = EShadowMeshSelection::All;
};

enum EShadowDepthCacheMode
{
	SDCM_MovablePrimitivesOnly,
	SDCM_StaticPrimitivesOnly,
	SDCM_CSMScrolling,
	SDCM_Uncached
};

inline bool IsShadowCacheModeOcclusionQueryable(EShadowDepthCacheMode CacheMode)
{
	// SDCM_StaticPrimitivesOnly shadowmaps are emitted randomly as the cache needs to be updated,
	// And therefore not appropriate for occlusion queries which are latent and therefore need to be stable.
	// Only one the cache modes from ComputeWholeSceneShadowCacheModes should be queryable
	return CacheMode != SDCM_StaticPrimitivesOnly;
}

struct FTiledShadowRendering
{
	enum class ETileType : uint8
	{
		Tile16bits,
		Tile12bits
	};
	FRDGBufferRef		DrawIndirectParametersBuffer;
	FRDGBufferSRVRef	TileListDataBufferSRV;
	uint32				TileSize;
	ETileType			TileType = ETileType::Tile16bits;
};

class FShadowMapRenderTargets
{
public:
	TArray<IPooledRenderTarget*, SceneRenderingAllocator> ColorTargets;
	IPooledRenderTarget* DepthTarget;

	FShadowMapRenderTargets() :
		DepthTarget(nullptr)
	{}

	FIntPoint GetSize() const
	{
		if (DepthTarget)
		{
			return DepthTarget->GetDesc().Extent;
		}
		else 
		{
			check(ColorTargets.Num() > 0);
			return ColorTargets[0]->GetDesc().Extent;
		}
	}

	void CopyReferencesFromRenderTargets(const FShadowMapRenderTargetsRefCounted& SourceTargets)
	{
		int32 ColorTargetsCount = SourceTargets.ColorTargets.Num();
		ColorTargets.Empty(ColorTargetsCount);
		ColorTargets.AddDefaulted(ColorTargetsCount);
		for (int32 TargetIndex = 0; TargetIndex < ColorTargetsCount; TargetIndex++)
		{
			ColorTargets[TargetIndex] = SourceTargets.ColorTargets[TargetIndex].GetReference();
		}

		DepthTarget = SourceTargets.DepthTarget.GetReference();
	}
};

typedef TFunctionRef<void(FRHICommandList& RHICmdList, bool bFirst)> FBeginShadowRenderPassFunction;
using FPackedNaniteView = Nanite::FPackedView;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FShadowDepthPassUniformParameters,)
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER(FMatrix44f, ProjectionMatrix)
	SHADER_PARAMETER(FMatrix44f, ViewMatrix)
	SHADER_PARAMETER(FVector4f, ShadowParams)
	SHADER_PARAMETER(float, bClampToNearPlane)
	SHADER_PARAMETER_ARRAY(FMatrix44f, ShadowViewProjectionMatrices, [6])
	SHADER_PARAMETER_ARRAY(FMatrix44f, ShadowViewMatrices, [6])
	SHADER_PARAMETER(uint32, ShadowDepthType)

	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, VirtualSmPageTable)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPackedNaniteView >, PackedNaniteViews)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint4 >, AllocatedPageRectBounds)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint4 >, UncachedPageRectBounds)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray< uint >, OutDepthBufferArray)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FMobileShadowDepthPassUniformParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT(FMobileSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER(FMatrix44f, ProjectionMatrix)
	SHADER_PARAMETER(FMatrix44f, ViewMatrix)
	SHADER_PARAMETER(FVector4f, ShadowParams)
	SHADER_PARAMETER(float, bClampToNearPlane)
	SHADER_PARAMETER_ARRAY(FMatrix44f, ShadowViewProjectionMatrices, [6])
	SHADER_PARAMETER(uint32, ShadowDepthType)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

/**
 * Information about a projected shadow.
 */
class FProjectedShadowInfo : public FRefCountedObject
{
public:
	typedef TArray<const FPrimitiveSceneInfo*,SceneRenderingAllocator> PrimitiveArrayType;

	/** 
	 * The view to be used when rendering this shadow's depths. 
	 * WARNING: This view is a bastardization of the 'main view' - i.e., the view used to render the visible geometry (or one of them in the case of multi-view, e.g., split screen)
	 * the only substantial differences are that the _view_ matrix is overridden (using the 'HackOverrideViewMatrixForShadows')
	 * and the 'HackRemoveTemporalAAProjectionJitter' has been called, to presumably remove the AA Jitter.
	 * IT DOES NOT contain the correct shadow projection matrix, PreViewTranslation, or much else that you would expect.
	 * Instead use the GetShadowDepthRenderingViewMatrices() function to get these for rendering shadows.
	 * Main reason for this state of affairs is to facilitate code that depends on the 'main' view matrices for LOD calculations in various places.
	 */
	FViewInfo* ShadowDepthView;

	/** The depth or color targets this shadow was rendered to. */
	FShadowMapRenderTargets RenderTargets;

	EShadowDepthCacheMode CacheMode;

	/** The main view this shadow must be rendered in, or NULL for a view independent shadow. */
	FViewInfo* DependentView;

	/** Index of the shadow into FVisibleLightInfo::AllProjectedShadows. */
	int32 ShadowId;

	/** A translation that is applied to world-space before transforming by one of the shadow matrices. */
	FVector PreShadowTranslation;

	/** 
	 * The view matrix of the shadow, ALSO used as an override to the main view's view matrix when rendering the shadow depth pass. 
	 */
	FMatrix TranslatedWorldToView;

	/** View space to clip space. Excluding border area. */
	FMatrix ViewToClipInner;
	
	/** View space to clip space. Including border area. */
	FMatrix ViewToClipOuter;
	
	/** 
	 * Matrix used for rendering the shadow depth buffer.  
	 * Note that this does not necessarily contain all of the shadow casters with CSM, since the vertex shader flattens them onto the near plane of the projection.
	 */
	FMatrix44f TranslatedWorldToClipInnerMatrix;
	FMatrix44f TranslatedWorldToClipOuterMatrix;

	FMatrix44f InvReceiverInnerMatrix;

	float InvMaxSubjectDepth;

	/** 
	 * Subject depth extents, in world space units. 
	 * These can be used to convert shadow depth buffer values back into world space units.
	 */
	float MaxSubjectZ;
	float MinSubjectZ;

	/** Frustum containing all potential shadow casters. Including border area. */
	FConvexVolume CasterOuterFrustum;

	/** Frustum containing all shadow receivers. Excluding border area. */
	FConvexVolume ReceiverInnerFrustum;

	float MinPreSubjectZ;

	FSphere ShadowBounds;

	FShadowCascadeSettings CascadeSettings;

	/** 
	 * X and Y position of the shadow in the appropriate depth buffer.  These are only initialized after the shadow has been allocated. 
	 * The actual contents of the shadowmap are at X + BorderSize, Y + BorderSize.
	 */
	uint32 X;
	uint32 Y;

	/** 
	 * Resolution of the shadow, excluding the border. 
	 * The full size of the region allocated to this shadow is therefore ResolutionX + 2 * BorderSize, ResolutionY + 2 * BorderSize.
	 */
	uint32 ResolutionX;
	uint32 ResolutionY;

	/** Size of the border, if any, used to allow filtering without clamping for shadows stored in an atlas. */
	uint32 BorderSize;

	/** Scissor rect size to exclude portion of the CSM slices outside the view frustum */
	FIntRect ScissorRectOptim;

	/** Compute the scissors coords to exclude portion of the CSM slices outside the view frustum*/
	void ComputeScissorRectOptim();

	/** The largest percent of either the width or height of any view. */
	float MaxScreenPercent;

	/** Fade Alpha per view. */
	TArray<float, TInlineAllocator<2> > FadeAlphas;

	/** Whether the shadow has been allocated in the shadow depth buffer, and its X and Y properties have been initialized. */
	uint32 bAllocated : 1;

	/** Whether the shadow's projection has been rendered. */
	uint32 bRendered : 1;

	/** Whether the shadow has been allocated in the preshadow cache, so its X and Y properties offset into the preshadow cache depth buffer. */
	uint32 bAllocatedInPreshadowCache : 1;

	/** Whether the shadow is in the preshadow cache and its depths are up to date. */
	uint32 bDepthsCached : 1;

	// redundant to LightSceneInfo->Proxy->GetLightType() == LightType_Directional, could be made ELightComponentType LightType
	uint32 bDirectionalLight : 1;

	/** Whether the shadow is a point light shadow that renders all faces of a cubemap in one pass. */
	uint32 bOnePassPointLightShadow : 1;

	/** Whether the shadow is a virtual shadow map. */
	uint32 bVSM : 1;

	/** Whether this shadow affects the whole scene or only a group of objects. */
	uint32 bWholeSceneShadow : 1;

	/** Whether this shadow should support casting shadows from translucent surfaces. */
	uint32 bTranslucentShadow : 1;

	/** Whether the shadow will be computed by ray tracing the distance field. */
	uint32 bRayTracedDistanceField : 1;

	/** Whether this is a per-object shadow that should use capsule shapes to shadow instead of the mesh's triangles. */
	uint32 bCapsuleShadow : 1;

	/** Whether the shadow is a preshadow or not.  A preshadow is a per object shadow that handles the static environment casting on a dynamic receiver. */
	uint32 bPreShadow : 1;

	/** To not cast a shadow on the ground outside the object and having higher quality (useful for first person weapon). */
	uint32 bSelfShadowOnly : 1;

	/** Whether the shadow is a per object shadow or not. */
	uint32 bPerObjectOpaqueShadow : 1;

	/** Whether turn on back-lighting transmission. */
	uint32 bTransmission : 1;

	/** Whether turn on hair strands deep shadow. */
	uint32 bHairStrandsDeepShadow : 1;

	/** Whether to render Nanite geometry into this shadow map. */
	uint32 bNaniteGeometry : 1;

	/** Whether the the shadow overlaps any nanite primitives */
	uint32 bContainsNaniteSubjects : 1;
	uint32 bShouldRenderVSM : 1;

	/** Whether this shadow should support casting shadows from volumetric surfaces. */
	uint32 bVolumetricShadow : 1;

	EShadowMeshSelection MeshSelectionMask = EShadowMeshSelection::All;

	TSharedPtr<FVirtualShadowMapPerLightCacheEntry> VirtualShadowMapPerLightCacheEntry;
	TSharedPtr<FVirtualShadowMapClipmap> VirtualShadowMapClipmap;

	/** View projection matrices for each cubemap face, used by one pass point light shadows. */
	TArray<FMatrix> OnePassShadowViewProjectionMatrices;

	/** View matrices for each cubemap face, used by one pass point light shadows. */
	TArray<FMatrix> OnePassShadowViewMatrices;
	
	/** Face projection matrix for cube map shadows. */
	FMatrix OnePassShadowFaceProjectionMatrix;

	/** Controls fading out of per-object shadows in the distance to avoid casting super-sharp shadows far away. */
	float PerObjectShadowFadeStart;
	float InvPerObjectShadowFadeLength;

	/** The Z offset in shadow light coordinate of the cached shadow bound center and current shadow bound center when scrolling the cached shadow map. */
	float CSMScrollingZOffset;

	FVector4f OverlappedUVOnCachedShadowMap;
	FVector4f OverlappedUVOnCurrentShadowMap;

	/** The extra culling planes to cull the static meshes which already in the overlapped area when scrolling the cached shadow map. */
	TArray<FPlane, TInlineAllocator<4>> CSMScrollingExtraCullingPlanes;

	/** Projection index for light types that use multiple projections for shadows. */
	int32 ProjectionIndex = 0;

	/** Index of the subject primitive for per object shadows. */
	int32 SubjectPrimitiveComponentIndex = -1;

	int32 ExplicitNaniteInstanceChunkStartOffset = INDEX_NONE;
	int32 ExplicitNaniteInstanceChunkEndOffset = INDEX_NONE;
	int32 ExplicitNaniteInstanceLastChunkSize = 0;

	TArray<int32, TInlineAllocator<6> > ViewIds;
	
public:

	// default constructor
	FProjectedShadowInfo();

	~FProjectedShadowInfo();

	/**
	 * for a per-object shadow. e.g. translucent particle system or a dynamic object in a precomputed shadow situation
	 * @param InParentSceneInfo must not be 0
	 * @return success, if false the shadow project is invalid and the projection should nto be created
	 */
	bool SetupPerObjectProjection(
		FLightSceneInfo* InLightSceneInfo,
		const FPrimitiveSceneInfo* InParentSceneInfo,
		const FPerObjectProjectedShadowInitializer& Initializer,
		bool bInPreShadow,
		uint32 InResolutionX,
		uint32 MaxShadowResolutionY,
		uint32 InBorderSize,
		float InMaxScreenPercent,
		bool bInTranslucentShadow
		);

	/** for a whole-scene shadow. */
	void SetupWholeSceneProjection(
		FLightSceneInfo* InLightSceneInfo,
		FViewInfo* InDependentView,
		const FWholeSceneProjectedShadowInitializer& Initializer,
		uint32 InResolutionX,
		uint32 InResolutionY,
		uint32 InSnapResolutionX,
		uint32 InSnapResolutionY,
		uint32 InBorderSize,
		int32 InSelectedViewIndex = INDEX_NONE
		);

	/** for a clipmap shadow. */
	void SetupClipmapProjection(
		FLightSceneInfo* InLightSceneInfo,
		FViewInfo* InDependentView,
		const TSharedPtr<FVirtualShadowMapClipmap> &VirtualShadowMapClipmap,
		float InMaxNonFarCascadeDistance
		);

	/**
	 * Get `FViewMatrices` instance set up for drawing geometry to the SM, using the outer projection,
	 * @parameter bUseForVSMCubeFaceWorkaround - If true, when get a cube face the OnePassShadowViewMatrices is flipped in Y to undo the 
	 *                                           flip used when creating OnePassShadowViewMatrices. This is also done when sampling the VSM.
	 */
	FViewMatrices GetShadowDepthRenderingViewMatrices(int32 CubeFaceIndex = -1, bool bUseForVSMCubeFaceWorkaround = false) const;

	float GetShaderDepthBias() const { return ShaderDepthBias; }
	float GetShaderSlopeDepthBias() const { return ShaderSlopeDepthBias; }
	float GetShaderMaxSlopeDepthBias() const { return ShaderMaxSlopeDepthBias; }
	float GetShaderReceiverDepthBias() const;

	void SetStateForView(FRHICommandList& RHICmdList, bool bRequiresBorderClamp = false) const;

	/** Get view rect excluding border area */
	FIntRect GetInnerViewRect() const
	{
		return {
			int32(X + BorderSize),
			int32(Y + BorderSize),
			int32(X + BorderSize + ResolutionX),
			int32(Y + BorderSize + ResolutionY),
			};
	}

	/** Get view rect including border area */
	FIntRect GetOuterViewRect() const	
	{
		return {
			int32(X),
			int32(Y),
			int32(X + 2 * BorderSize + ResolutionX),
			int32(Y + 2 * BorderSize + ResolutionY),
		};
	}

	/** Set state for depth rendering */
	void SetStateForDepth(FMeshPassProcessorRenderState& DrawRenderState) const;

	void ClearDepth(FRHICommandList& RHICmdList) const;

	/** Renders shadow maps for translucent primitives. */
	void RenderTranslucencyDepths(FRDGBuilder& GraphBuilder, class FSceneRenderer* SceneRenderer, const FRenderTargetBindingSlots& RenderTargets, FInstanceCullingManager& InstanceCullingManager);

	static bool SupportsShadowStencilCulling(FStaticShaderPlatform ShaderPlatform);

	bool UseShadowStencilCulling(const FViewInfo& View) const;

	static FRHIBlendState* GetBlendStateForProjection(
		int32 ShadowMapChannel,
		bool bIsWholeSceneDirectionalShadow,
		bool bUseFadePlane,
		bool bProjectingForForwardShading,
		bool bMobileModulatedProjections);

	FRHIBlendState* GetBlendStateForProjection(bool bProjectingForForwardShading, bool bMobileModulatedProjections) const;

	/**
	 * Projects the shadow onto the scene for a particular view.
	 */
	void RenderProjection(
		FRDGBuilder& GraphBuilder,
		const FShadowProjectionPassParameters& CommonPassParameters,
		int32 ViewIndex,
		const FViewInfo* View,
		const FLightSceneProxy* LightSceneProxy,
		const FSceneRenderer* SceneRender,
		bool bProjectingForForwardShading,
		bool bSubPixelShadow) const;

	void RenderProjectionInternal(
		FRHICommandList& RHICmdList,
		int32 ViewIndex,
		const FViewInfo* View,
		const FLightSceneProxy* LightSceneProxy,
		const FSceneRenderer* SceneRender,
		bool bProjectingForForwardShading,
		bool bMobileModulatedProjections,
		const FInstanceCullingDrawParams& InstanceCullingDrawParams, 
		FRHIUniformBuffer* HairStrandsUniformBuffer) const;

	/**
	 * Projects the mobile modulated shadow onto the scene for a particular view.
	 */
	void RenderMobileModulatedShadowProjection(
		FRHICommandList& RHICmdList,
		int32 ViewIndex,
		const FViewInfo* View,
		const FLightSceneProxy* LightSceneProxy,
		const FSceneRenderer* SceneRender) const;

	/**
	* Renders the shadow subject depth, to a particular hacked view
	*/
	void RenderDepth(
		FRDGBuilder& GraphBuilder,
		const FSceneRenderer* SceneRenderer,
		FRDGTextureRef ShadowDepthTexture,
		bool bDoParallelDispatch,
		bool bDoCrossGPUCopy);

	/** Reset cached ray traced distance field shadows texture. */
	void ResetRayTracedDistanceFieldShadow(const FViewInfo* View)
	{
		DistanceFieldShadowViewGPUData& SDFShadowViewGPUData = CachedDistanceFieldShadowViewGPUData.FindOrAdd(View);
		SDFShadowViewGPUData.RayTracedShadowsTexture = nullptr;
	}

	/**
	* Render ray traced distance field shadows into a texture.
	* Output texture is cached per view. This is useful to support async compute.
	* (ie: kick off distance field shadows early in the frame using async compute, and then combine result into shadow mask texture when necessary)
	*/
	FScreenPassTexture RenderRayTracedDistanceFieldProjection(
		FRDGBuilder& GraphBuilder,
		bool bAsyncCompute,
		const FMinimalSceneTextures& SceneTextures,
		const FViewInfo& View,
		const FIntRect& ScissorRect);

	/** 
	* Renders ray traced distance field shadows into an existing texture.
	* Will use cached results if already calculated earlier in the frame (ie: async compute)
	*/
	void RenderRayTracedDistanceFieldProjection(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		FRDGTextureRef ScreenShadowMaskTexture,
		const FViewInfo& View,
		const int32 ViewIndex,
		FIntRect ScissorRect,
		bool bProjectingForForwardShading,
		bool bForceRGBModulation = false,
		FTiledShadowRendering* TiledShadowRendering = nullptr);

	/** Render one pass point light shadow projections. */
	void RenderOnePassPointLightProjection(
		FRDGBuilder& GraphBuilder,
		const FShadowProjectionPassParameters& CommonPassParameters,
		int32 ViewIndex,
		const FViewInfo& View,
		const FLightSceneProxy* LightSceneProxy,
		bool bProjectingForForwardShading,
		bool bSubPixelShadow) const;

	/**
	 * Renders the projected shadow's frustum wireframe with the given FPrimitiveDrawInterface.
	 */
	void RenderFrustumWireframe(FPrimitiveDrawInterface* PDI) const;

	/**
	 * Adds a primitive to the shadow's subject list.
	 */
	bool AddSubjectPrimitive(FPrimitiveSceneInfo* PrimitiveSceneInfo, TArrayView<FViewInfo> ViewArray, bool bRecordShadowSubjectForMobileShading, bool bInvertViewShadowRelevance = false);

	uint64 AddSubjectPrimitive_AnyThread(
		const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact,
		TArrayView<FViewInfo> ViewArray,
		ERHIFeatureLevel::Type FeatureLevel,
		struct FAddSubjectPrimitiveStats& OutStats,
		struct FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer) const;

	void PresizeSubjectPrimitiveArrays(struct FAddSubjectPrimitiveStats const& Stats);

	void FinalizeAddSubjectPrimitive(
		FDynamicShadowsTaskData& TaskData,
		struct FAddSubjectPrimitiveOp const& Op,
		TArrayView<FViewInfo> ViewArray,
		struct FFinalizeAddSubjectPrimitiveContext& Context);

	/**
	* @return TRUE if this shadow info has any casting subject prims to render
	*/
	bool HasSubjectPrims() const;

	/**
	 * Adds a primitive to the shadow's receiver list.
	 */
	void AddReceiverPrimitive(FPrimitiveSceneInfo* PrimitiveSceneInfo);

	enum class EGatherDynamicMeshElementsPass : uint8
	{
		// Processes all operations in a single pass.
		All,

		// Parallel pass runs first and processes elements in parallel with other shadows.
		Parallel,

		// Serial pass runs second and processes elements serially other shadows.
		Serial,
	};

	/** Gathers dynamic mesh elements for all the shadow's primitives arrays. */
	bool GatherDynamicMeshElements(FMeshElementCollector& MeshCollector, FSceneRenderer& Renderer, class FVisibleLightInfo& VisibleLightInfo, TArray<const FSceneView*>& ReusedViewsArray, EGatherDynamicMeshElementsPass Pass);

	void SetupMeshDrawCommandsForShadowDepth(FSceneRenderer& Renderer, FInstanceCullingManager& InstanceCullingManager);

	void SetupMeshDrawCommandsForProjectionStenciling(FSceneRenderer& Renderer, FInstanceCullingManager& InstanceCullingManager);

	/** 
	 * @param View view to check visibility in
	 * @return true if this shadow info has any subject prims visible in the view
	 */
	bool SubjectsVisible(const FViewInfo& View) const;

	/** Clears arrays allocated with the scene rendering allocator. */
	void ClearTransientArrays();
	
	/** Hash function. */
	friend uint32 GetTypeHash(const FProjectedShadowInfo* ProjectedShadowInfo)
	{
		return PointerHash(ProjectedShadowInfo);
	}

	/** Returns a matrix that transforms a screen space position into shadow space. */
	FMatrix GetScreenToShadowMatrix(const FSceneView& View) const
	{
		return GetScreenToShadowMatrix(View, X, Y, ResolutionX, ResolutionY);
	}

	FVector4f GetClipToShadowBufferUvScaleBias() const;

	/** Returns a matrix that transforms a screen space position into shadow space. 
		Additional parameters allow overriding of shadow's tile location.
		Used with modulated shadows to reduce precision problems when calculating ScreenToShadow in pixel shader.
	*/
	FMatrix GetScreenToShadowMatrix(const FSceneView& View, uint32 TileOffsetX, uint32 TileOffsetY, uint32 TileResolutionX, uint32 TileResolutionY) const;

	/** Returns a matrix that transforms a world space position into shadow space. */
	FMatrix GetWorldToShadowMatrix(FVector4f& ShadowmapMinMax, const FIntPoint* ShadowBufferResolutionOverride = nullptr) const;

	/** Returns the resolution of the shadow buffer used for this shadow, based on the shadow's type. */
	FIntPoint GetShadowBufferResolution() const
	{
		if (HasVirtualShadowMap())
		{
			return FIntPoint(VSM_VIRTUAL_MAX_RESOLUTION_XY, VSM_VIRTUAL_MAX_RESOLUTION_XY);
		}
		return RenderTargets.GetSize();
	}

	/** Computes and updates ShaderDepthBias and ShaderSlopeDepthBias */
	void UpdateShaderDepthBias();

	/** How large the soft PCF comparison should be, similar to DepthBias, before this was called TransitionScale and 1/Size */
	float ComputeTransitionSize() const;

	bool IsWholeSceneDirectionalShadow() const;
	bool IsWholeScenePointLightShadow() const;

	bool ShouldClampToNearPlane() const
	{
		return IsWholeSceneDirectionalShadow() || (bPreShadow && bDirectionalLight);
	}

	// 0 if Setup...() wasn't called yet
	FLightSceneInfo& GetLightSceneInfo() const { return *LightSceneInfo; }
	const FLightSceneInfoCompact& GetLightSceneInfoCompact() const { return LightSceneInfoCompact; }
	/**
	 * Parent primitive of the shadow group that created this shadow, if not a bWholeSceneShadow.
	 * 0 if Setup...() wasn't called yet
	 */	
	const FPrimitiveSceneInfo* GetParentSceneInfo() const { return ParentSceneInfo; }

	/** Creates a new view from the pool and caches it in ShadowDepthView for depth rendering. */
	void SetupShadowDepthView(FSceneRenderer* SceneRenderer);

	EShadowDepthType GetShadowDepthType() const 
	{
		EShadowDepthType ShadowDepthType = bVSM ? EShadowDepthType::VSM : EShadowDepthType::None;

		if (bDirectionalLight)
		{
			ShadowDepthType |= EShadowDepthType::Directional;
		}
		else if (bOnePassPointLightShadow)
		{
			ShadowDepthType |= EShadowDepthType::OnePassPoint;
		}
		else
		{
			ShadowDepthType |= EShadowDepthType::Spot;
		}

		return ShadowDepthType;
	}

	EMeshPass::Type GetTargetMeshPassType() const
	{
		return ::GetShadowMeshPassType(GetShadowDepthType());
	}

	bool HasVirtualShadowMap() const { return VirtualShadowMapPerLightCacheEntry.IsValid(); }

	FParallelMeshDrawCommandPass& GetShadowDepthPass() { return ShadowDepthPass; }

	float GetMaxNonFarCascadeDistance() const { return MaxNonFarCascadeDistance; }

	void BuildRenderingCommands(
		FRDGBuilder& GraphBuilder,
		FGPUScene& GPUScene,
		FInstanceCullingDrawParams& InstanceCullingDrawParams)
	{
		ShadowDepthView->DynamicPrimitiveCollector.Commit();
		return ShadowDepthPass.BuildRenderingCommands(GraphBuilder, GPUScene, InstanceCullingDrawParams);
	}

	/** Check if we need to set the scissor rect to exclude portion of the CSM slices outside the view frustum */
	bool ShouldUseCSMScissorOptim() const;

	const TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>& GetDynamicSubjectHeterogeneousVolumeMeshElements() const { return DynamicSubjectHeterogeneousVolumeMeshElements; }

	FCullingVolume GetCullingVolume() const;

	/**
	 * Create a FSceneCullingRendererHierarchicalCullingQuery and initialize the SceneCullingRendererCullingQuery pointer to drive hierarchical CPU culling for the projected shadow.
	 * The SceneCullingRendererCullingQuery may be null if the system is not enabled (or there are no primitives with hierarchical instance culling set up).
	 */
	void SetupCullingQuery(FSceneRenderer& SceneRenderer);

	/**
	 * Finds a relevant view for a shadow
	 */
	const FViewInfo* FindViewForShadow(TConstArrayView<FViewInfo> SceneRendererViews) const;

private:
	// 0 if Setup...() wasn't called yet
	FLightSceneInfo* LightSceneInfo;
	FLightSceneInfoCompact LightSceneInfoCompact;

	/**
	 * Parent primitive of the shadow group that created this shadow, if not a bWholeSceneShadow.
	 * 0 if Setup...() wasn't called yet or for whole scene shadows
	 */	
	const FPrimitiveSceneInfo* ParentSceneInfo;

	/** dynamic shadow casting elements */
	PrimitiveArrayType DynamicSubjectPrimitives;
	/** For preshadows, this contains the receiver primitives to mask the projection to. */
	PrimitiveArrayType ReceiverPrimitives;
	/** Subject primitives with translucent relevance. */
	PrimitiveArrayType SubjectTranslucentPrimitives;
	/** Subject primitives for heterogeneous volume shadows. */
	PrimitiveArrayType SubjectHeterogeneousVolumePrimitives;

	/** Dynamic mesh elements for subject primitives. */
	TArray<FMeshBatchAndRelevance,SceneRenderingAllocator> DynamicSubjectMeshElements;
	/** Dynamic mesh elements for translucent subject primitives. */
	TArray<FMeshBatchAndRelevance,SceneRenderingAllocator> DynamicSubjectTranslucentMeshElements;
	/** Dynamic mesh elements for heterogeneous volume primitives. */
	TArray<FMeshBatchAndRelevance, SceneRenderingAllocator> DynamicSubjectHeterogeneousVolumeMeshElements;

	TArray<const FStaticMeshBatch*, SceneRenderingAllocator> SubjectMeshCommandBuildRequests;
	TArray<EMeshDrawCommandCullingPayloadFlags, SceneRenderingAllocator> SubjectMeshCommandBuildFlags;

	/** Number of elements of DynamicSubjectMeshElements meshes. */
	int32 NumDynamicSubjectMeshElements;

	/** Number of elements of SubjectMeshCommandBuildRequests meshes. */
	int32 NumSubjectMeshCommandBuildRequestElements;

	FMeshCommandOneFrameArray ShadowDepthPassVisibleCommands;
	FParallelMeshDrawCommandPass ShadowDepthPass;
	FSceneCullingRendererHierarchicalCullingQuery* SceneCullingRendererCullingQuery = nullptr;

	TArray<FSimpleMeshDrawCommandPass *, TInlineAllocator<8>> ProjectionStencilingPasses;

	FDynamicMeshDrawCommandStorage DynamicMeshDrawCommandStorage;
	FGraphicsMinimalPipelineStateSet GraphicsMinimalPipelineStateSet;
	bool NeedsShaderInitialisation;
	// If >= 0.0f then this records the cascade distance to the farthest non-'far' shadow cascade. Only used for clipmaps at the moment.
	float MaxNonFarCascadeDistance = -1.0f;

	/**
	 * Bias during in shadowmap rendering, stored redundantly for better performance 
	 * Set by UpdateShaderDepthBias(), get with GetShaderDepthBias(), -1 if not set
	 */
	float ShaderDepthBias;
	float ShaderSlopeDepthBias;
	float ShaderMaxSlopeDepthBias;

	/** The scene renderer primary view selected for this whole-scene shadow (i.e, local light shadow), not used for directional or per object shadows.*/
	int32 SelectedViewIndex = INDEX_NONE;

	/**  Cached light tile intersection parameters in case we needed to trace a distance field multiple times for a view but for different depth buffer*/
	struct DistanceFieldShadowViewGPUData 
	{
		/** Ray traced DF shadow intermediate output. Populated by BeginRenderRayTracedDistanceFieldProjection and consumed by RenderRayTracedDistanceFieldProjection. */
		FRDGTextureRef RayTracedShadowsTexture = nullptr;

		FLightTileIntersectionParameters*			SDFLightTileIntersectionParameters = nullptr;
		FDistanceFieldCulledObjectBufferParameters*	SDFCulledObjectBufferParameters = nullptr;
		FLightTileIntersectionParameters*			HeightFieldLightTileIntersectionParameters = nullptr;
		FDistanceFieldCulledObjectBufferParameters*	HeightFieldCulledObjectBufferParameters = nullptr;
	};
	TMap<const FViewInfo*, DistanceFieldShadowViewGPUData> CachedDistanceFieldShadowViewGPUData;

	void CopyCachedShadowMap(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		const FSceneRenderer* SceneRenderer,
		const FRenderTargetBindingSlots& RenderTargets,
		const FMeshPassProcessorRenderState& DrawRenderState);

	float GetLODDistanceFactor() const;

	/**
	* Modifies the passed in view for this shadow
	*/
	void ModifyViewForShadow(FViewInfo* FoundView) const;

	friend class FVirtualShadowMapArray;
	void BeginRenderView(FRDGBuilder& GraphBuilder, FScene* Scene);

	void AddCachedMeshDrawCommandsForPass(
		const FMeshDrawCommandPrimitiveIdInfo &PrimitiveIdInfo,
		const FPrimitiveSceneInfo* InPrimitiveSceneInfo,
		const FStaticMeshBatchRelevance& RESTRICT StaticMeshRelevance,
		const FStaticMeshBatch& StaticMesh,
		EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags,
		const FScene* Scene,
		EMeshPass::Type PassType,
		FMeshCommandOneFrameArray& VisibleMeshCommands,
		TArray<const FStaticMeshBatch*, SceneRenderingAllocator>& MeshCommandBuildRequests,
		TArray<EMeshDrawCommandCullingPayloadFlags, SceneRenderingAllocator> MeshCommandBuildFlags,
		int32& NumMeshCommandBuildRequestElements);

	void AddCachedMeshDrawCommands_AnyThread(
		const FScene* Scene,
		const FStaticMeshBatchRelevance& RESTRICT StaticMeshRelevance,
		int32 StaticMeshIdx,
		int32& NumAcceptedStaticMeshes,
		struct FAddSubjectPrimitiveResult& OutResult,
		struct FAddSubjectPrimitiveStats& OutStats,
		struct FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer) const;


	/*
	 * Helper to calculate the LOD such that we do not repeat this in threaded & non threaded versions
	 * NOTE: Reads and modifies the CurrentView.PrimitivesLODMask for the given primitive. 
	 * Storing it is an optimization for the case where a primitive is not visible in the main view but in several shadow views (e.g., cascades).
	 * This also creates a potential data race if we ever process multiple shadows infos for the same view and primitive in parallel (e.g., cascades).
	 */
	FORCEINLINE_DEBUGGABLE FLODMask CalcAndUpdateLODToRender(FViewInfo& CurrentView, const FBoxSphereBounds& Bounds, const FPrimitiveSceneInfo* PrimitiveSceneInfo, int32 ForcedLOD) const;

	/**
	 * Check relevance flags and shadow mesh pass filter, returns true if the mesh is to be added.
	 * @param bOutDrawingStaticMeshes - set to true if the relevancy checks pass, even if the mesh was discarded using the pass filter check. Otherwise the mesh is treated as dynamic.
	 */
	FORCEINLINE bool ShouldDrawStaticMesh(const FStaticMeshBatchRelevance& StaticMeshRelevance, const FLODMask &ShadowLODToRender, bool& bOutDrawingStaticMeshes) const;

	/** Will return if we should draw the static mesh for the shadow, and will perform lazy init of primitive if it wasn't visible */
	bool ShouldDrawStaticMeshes(FViewInfo& InCurrentView, FPrimitiveSceneInfo* InPrimitiveSceneInfo);

	bool ShouldDrawStaticMeshes_AnyThread(
		FViewInfo& InCurrentView,
		const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact,
		bool bMayBeFading,
		struct FAddSubjectPrimitiveResult& OutResult,
		struct FAddSubjectPrimitiveStats& OutStats,
		struct FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer) const;

	void GetShadowTypeNameForDrawEvent(FString& TypeName) const;

	/** Updates object buffers needed by ray traced distance field shadows. */
	int32 UpdateShadowCastingObjectBuffers() const;

	/** Gathers dynamic mesh elements for the given primitive array. */
	bool GatherDynamicMeshElementsArray(
		FMeshElementCollector& Collector,
		const PrimitiveArrayType& PrimitiveArray, 
		const TArray<const FSceneView*>& Views,
		const FSceneViewFamily& ViewFamily,
		TArray<FMeshBatchAndRelevance,SceneRenderingAllocator>& OutDynamicMeshElements,
		int32& OutNumDynamicSubjectMeshElements,
		EGatherDynamicMeshElementsPass Pass);

	bool GatherDynamicHeterogeneousVolumeMeshElementsArray(
		FSceneRenderer& Renderer,
		FMeshElementCollector& Collector,
		const PrimitiveArrayType& PrimitiveArray,
		const TArray<const FSceneView*>& Views,
		const FSceneViewFamily& ViewFamily,
		TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>& OutDynamicMeshElements,
		int32& OutNumDynamicSubjectMeshElements,
		EGatherDynamicMeshElementsPass Pass);

	void SetupFrustumForProjection(const FViewInfo* View, TArray<FVector4f, TInlineAllocator<8>>& OutFrustumVertices, bool& bOutCameraInsideShadowFrustum, FPlane* OutPlanes) const;

	void SetupProjectionStencilMask(
		FRHICommandList& RHICmdList,
		const FViewInfo* View, 
		int32 ViewIndex,
		const class FSceneRenderer* SceneRender,
		const TArray<FVector4f, TInlineAllocator<8>>& FrustumVertices,
		bool bMobileModulatedProjections,
		bool bCameraInsideShadowFrustum,
		const FInstanceCullingDrawParams& InstanceCullingDrawParams) const;

	void SetupProjectionStencilMaskForHair(FRHICommandList& RHICmdList, const FViewInfo* View) const;

	FORCEINLINE bool TestPrimitiveFarCascadeConditions(bool bPrimitiveCastsFarShadow, const FBoxSphereBounds& Bounds) const;

	friend class FShadowDepthVS;
	friend class FShadowDepthBasePS;
	friend class FShadowVolumeBoundProjectionVS;
	friend class FShadowProjectionPS;
};

enum class EShadowProjectionVertexShaderFlags
{
	None,
	DrawingFrustum  = 0x01
};
ENUM_CLASS_FLAGS(EShadowProjectionVertexShaderFlags)

/**
* A vertex shader for projecting a shadow depth buffer onto the scene.
*/
class FShadowVolumeBoundProjectionVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FShadowVolumeBoundProjectionVS,Global);
public:

	FShadowVolumeBoundProjectionVS() {}
	FShadowVolumeBoundProjectionVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer) 
	{
		const FShaderParameterMap& ParameterMap = Initializer.ParameterMap;

		StencilingGeometryParameters.Bind(ParameterMap);
		InvReceiverInnerMatrix.Bind(ParameterMap, TEXT("InvReceiverInnerMatrix"));
		PreShadowToPreViewTranslation.Bind(ParameterMap, TEXT("PreShadowToPreViewTranslation"));
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View,	const FProjectedShadowInfo* ShadowInfo, EShadowProjectionVertexShaderFlags Flags);

private:
	LAYOUT_FIELD(FStencilingGeometryShaderParameters, StencilingGeometryParameters);
	LAYOUT_FIELD(FShaderParameter, InvReceiverInnerMatrix);
	LAYOUT_FIELD(FShaderParameter, PreShadowToPreViewTranslation);
};

class FShadowProjectionNoTransformVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FShadowProjectionNoTransformVS,Global);
public:
	FShadowProjectionNoTransformVS() {}
	FShadowProjectionNoTransformVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer) 
	{
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FProjectedShadowInfo* ShadowInfo, EShadowProjectionVertexShaderFlags Flags) {}
};

/**
 * FShadowProjectionPixelShaderInterface - used to handle templated versions
 */

class FShadowProjectionPixelShaderInterface : public FGlobalShader
{
	DECLARE_TYPE_LAYOUT(FShadowProjectionPixelShaderInterface, NonVirtual);
public:

	FShadowProjectionPixelShaderInterface() 
		:	FGlobalShader()
	{}

	/**
	 * Constructor - binds all shader params and initializes the sample offsets
	 * @param Initializer - init data from shader compiler
	 */
	FShadowProjectionPixelShaderInterface(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		:	FGlobalShader(Initializer)
	{}
};

/** Shadow projection parameters used by multiple shaders. */
class FShadowProjectionShaderParameters
{
	DECLARE_TYPE_LAYOUT(FShadowProjectionShaderParameters, NonVirtual);
public:
	void Bind(const FShader::CompiledShaderInitializerType& Initializer)
	{
		const FShaderParameterMap& ParameterMap = Initializer.ParameterMap;
		ScreenToShadowMatrix.Bind(ParameterMap,TEXT("ScreenToShadowMatrix"));
		SoftTransitionScale.Bind(ParameterMap,TEXT("SoftTransitionScale"));
		ShadowBufferSize.Bind(ParameterMap,TEXT("ShadowBufferSize"));
		ShadowDepthTexture.Bind(ParameterMap,TEXT("ShadowDepthTexture"));
		ShadowDepthTextureSampler.Bind(ParameterMap,TEXT("ShadowDepthTextureSampler"));
		ProjectionDepthBias.Bind(ParameterMap,TEXT("ProjectionDepthBiasParameters"));
		FadePlaneOffset.Bind(ParameterMap,TEXT("FadePlaneOffset"));
		InvFadePlaneLength.Bind(ParameterMap,TEXT("InvFadePlaneLength"));
		ShadowTileOffsetAndSizeParam.Bind(ParameterMap, TEXT("ShadowTileOffsetAndSize"));
		LightPositionOrDirection.Bind(ParameterMap, TEXT("LightPositionOrDirection"));
		PerObjectShadowFadeStart.Bind(ParameterMap, TEXT("PerObjectShadowFadeStart"));
		InvPerObjectShadowFadeLength.Bind(ParameterMap, TEXT("InvPerObjectShadowFadeLength"));
		ShadowNearAndFarDepth.Bind(ParameterMap, TEXT("ShadowNearAndFarDepth"));
		bCascadeUseFadePlane.Bind(ParameterMap, TEXT("bCascadeUseFadePlane"));
	}

	void Set(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FProjectedShadowInfo* ShadowInfo, bool bModulatedShadows, bool bUseFadePlane, bool SubPixelShadow)
	{
		const FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
		const FIntPoint ShadowBufferResolution = ShadowInfo->GetShadowBufferResolution();

		if (ShadowTileOffsetAndSizeParam.IsBound())
		{
			FVector2D InverseShadowBufferResolution(1.0f / ShadowBufferResolution.X, 1.0f / ShadowBufferResolution.Y);
			FVector4f ShadowTileOffsetAndSize(
				(ShadowInfo->BorderSize + ShadowInfo->X) * InverseShadowBufferResolution.X,
				(ShadowInfo->BorderSize + ShadowInfo->Y) * InverseShadowBufferResolution.Y,
				ShadowInfo->ResolutionX * InverseShadowBufferResolution.X,
				ShadowInfo->ResolutionY * InverseShadowBufferResolution.Y);
			SetShaderValue(BatchedParameters, ShadowTileOffsetAndSizeParam, ShadowTileOffsetAndSize);
		}

		// Set the transform from screen coordinates to shadow depth texture coordinates.
		if (bModulatedShadows)
		{
			// UE-29083 : work around precision issues with ScreenToShadowMatrix on low end devices.
			const FMatrix44f ScreenToShadow = FMatrix44f(ShadowInfo->GetScreenToShadowMatrix(View, 0, 0, ShadowBufferResolution.X, ShadowBufferResolution.Y));	// LWC_TODO: Precision loss
			SetShaderValue(BatchedParameters, ScreenToShadowMatrix, ScreenToShadow);
		}
		else
		{
			const FMatrix44f ScreenToShadow = FMatrix44f(ShadowInfo->GetScreenToShadowMatrix(View));		// LWC_TODO: Precision loss
			SetShaderValue(BatchedParameters, ScreenToShadowMatrix, ScreenToShadow);
		}

		if (SoftTransitionScale.IsBound())
		{
			const float TransitionSize = ShadowInfo->ComputeTransitionSize();

			SetShaderValue(BatchedParameters, SoftTransitionScale, FVector3f(0, 0, 1.0f / TransitionSize));
		}

		if (ShadowBufferSize.IsBound())
		{
			FVector2D ShadowBufferSizeValue(ShadowBufferResolution.X, ShadowBufferResolution.Y);

			SetShaderValue(BatchedParameters, ShadowBufferSize,
				FVector4f(ShadowBufferSizeValue.X, ShadowBufferSizeValue.Y, 1.0f / ShadowBufferSizeValue.X, 1.0f / ShadowBufferSizeValue.Y));
		}

		FRHITexture* ShadowDepthTextureValue;

		// Translucency shadow projection has no depth target
		if (ShadowInfo->RenderTargets.DepthTarget)
		{
			ShadowDepthTextureValue = ShadowInfo->RenderTargets.DepthTarget->GetRHI();
		}
		else
		{
			ShadowDepthTextureValue = GSystemTextures.BlackDummy->GetRHI();
		}
			
		FRHISamplerState* DepthSamplerState = TStaticSamplerState<SF_Point,AM_Clamp,AM_Clamp,AM_Clamp>::GetRHI();

		SetTextureParameter(BatchedParameters, ShadowDepthTexture, ShadowDepthTextureSampler, DepthSamplerState, ShadowDepthTextureValue);

		if (ShadowDepthTextureSampler.IsBound())
		{
			SetSamplerParameter(BatchedParameters, ShadowDepthTextureSampler, DepthSamplerState);
		}

		SetShaderValue(BatchedParameters, ProjectionDepthBias, FVector4f(ShadowInfo->GetShaderDepthBias(), ShadowInfo->GetShaderSlopeDepthBias(), ShadowInfo->GetShaderReceiverDepthBias(), ShadowInfo->MaxSubjectZ - ShadowInfo->MinSubjectZ));
		SetShaderValue(BatchedParameters, FadePlaneOffset, ShadowInfo->CascadeSettings.FadePlaneOffset);

		if(InvFadePlaneLength.IsBound() && bUseFadePlane)
		{
			check(ShadowInfo->CascadeSettings.FadePlaneLength > 0);
			SetShaderValue(BatchedParameters, InvFadePlaneLength, 1.0f / ShadowInfo->CascadeSettings.FadePlaneLength);
		}

		if (LightPositionOrDirection.IsBound())
		{
			const FVector LightDirection = ShadowInfo->GetLightSceneInfo().Proxy->GetDirection();
			const FVector LightPosition = ShadowInfo->GetLightSceneInfo().Proxy->GetPosition() + PreViewTranslation;
			const bool bIsDirectional = ShadowInfo->GetLightSceneInfo().Proxy->GetLightType() == LightType_Directional;
			SetShaderValue(BatchedParameters, LightPositionOrDirection, bIsDirectional ? FVector4f((FVector3f)LightDirection,0) : FVector4f((FVector3f)LightPosition,1));
		}

		if (SubPixelShadow)
		{
			float DeviceZNear = 1;
			float DeviceZFar = 0;
			const bool bIsCascadedShadow = ShadowInfo->bDirectionalLight && !(ShadowInfo->bPerObjectOpaqueShadow || ShadowInfo->bPreShadow);
			if (bIsCascadedShadow)
			{
				FVector4 Near = View.ViewMatrices.GetViewToClip().TransformFVector4(FVector4(0, 0, ShadowInfo->CascadeSettings.SplitNear));
				FVector4 Far = View.ViewMatrices.GetViewToClip().TransformFVector4(FVector4(0, 0, ShadowInfo->CascadeSettings.SplitFar));
				// LWC_TODO: precision loss?
				DeviceZNear = (float)(Near.Z / Near.W);
				DeviceZFar = (float)(Far.Z / Far.W);
			}

			FVector2f SliceNearAndFarDepth;
			SliceNearAndFarDepth.X = DeviceZNear;
			SliceNearAndFarDepth.Y = DeviceZFar;
			SetShaderValue(BatchedParameters, ShadowNearAndFarDepth, SliceNearAndFarDepth);
			SetShaderValue(BatchedParameters, bCascadeUseFadePlane, bUseFadePlane ? 1 : 0);
		}

		SetShaderValue(BatchedParameters, PerObjectShadowFadeStart, ShadowInfo->PerObjectShadowFadeStart);
		SetShaderValue(BatchedParameters, InvPerObjectShadowFadeLength, ShadowInfo->InvPerObjectShadowFadeLength);
	}

private:
	LAYOUT_FIELD(FShaderParameter, ScreenToShadowMatrix);
	LAYOUT_FIELD(FShaderParameter, SoftTransitionScale);
	LAYOUT_FIELD(FShaderParameter, ShadowBufferSize);
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthTexture);
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthTextureSampler);
	LAYOUT_FIELD(FShaderParameter, ProjectionDepthBias);
	LAYOUT_FIELD(FShaderParameter, FadePlaneOffset);
	LAYOUT_FIELD(FShaderParameter, InvFadePlaneLength);
	LAYOUT_FIELD(FShaderParameter, ShadowTileOffsetAndSizeParam);
	LAYOUT_FIELD(FShaderParameter, LightPositionOrDirection);
	LAYOUT_FIELD(FShaderParameter, PerObjectShadowFadeStart);
	LAYOUT_FIELD(FShaderParameter, InvPerObjectShadowFadeLength);
	LAYOUT_FIELD(FShaderParameter, ShadowNearAndFarDepth);
	LAYOUT_FIELD(FShaderParameter, bCascadeUseFadePlane);
};

/**
 * TShadowProjectionPS
 * A pixel shader for projecting a shadow depth buffer onto the scene.  Used with any light type casting normal shadows.
 */
template<uint32 Quality, bool bUseFadePlane = false, bool bModulatedShadows = false, bool bUseTransmission = false, bool SubPixelShadow = false>
class TShadowProjectionPS : public FShadowProjectionPixelShaderInterface
{
	DECLARE_SHADER_TYPE(TShadowProjectionPS,Global);
public:

	TShadowProjectionPS()
		: FShadowProjectionPixelShaderInterface()
	{ 
	}

	/**
	 * Constructor - binds all shader params and initializes the sample offsets
	 * @param Initializer - init data from shader compiler
	 */
	TShadowProjectionPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FShadowProjectionPixelShaderInterface(Initializer)
	{
		ProjectionParameters.Bind(Initializer);
		ShadowFadeFraction.Bind(Initializer.ParameterMap,TEXT("ShadowFadeFraction"));
		ShadowSharpen.Bind(Initializer.ParameterMap,TEXT("ShadowSharpen"));
		LightPosition.Bind(Initializer.ParameterMap, TEXT("LightPositionAndInvRadius"));

	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
			|| (bUseTransmission == 0 && SubPixelShadow == 0); // Mobile uses only those
	}

	/**
	 * Add any defines required by the shader
	 * @param OutEnvironment - shader environment to modify
	 */
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FShadowProjectionPixelShaderInterface::ModifyCompilationEnvironment(Parameters,OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADOW_QUALITY"), Quality);
		OutEnvironment.SetDefine(TEXT("SUBPIXEL_SHADOW"), (uint32)(SubPixelShadow ? 1 : 0));
		OutEnvironment.SetDefine(TEXT("USE_FADE_PLANE"), (uint32)(bUseFadePlane ? 1 : 0));
		OutEnvironment.SetDefine(TEXT("USE_TRANSMISSION"), (uint32)(bUseTransmission ? 1 : 0));

		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), (uint32)(bMobileForceDepthRead ? 1 : 0));
	}

	/**
	 * Sets the pixel shader's parameters
	 * @param View - current view
	 * @param ShadowInfo - projected shadow info for a single light
	 */
	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		int32 ViewIndex,
		const FSceneView& View,
		const FProjectedShadowInfo* ShadowInfo,
		bool bUseLightFunctionAtlas)
	{
		const FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
		const bool bUseFadePlaneEnable = ShadowInfo->CascadeSettings.FadePlaneLength > 0;

		ProjectionParameters.Set(BatchedParameters, View, ShadowInfo, bModulatedShadows, bUseFadePlaneEnable, SubPixelShadow);
		const FLightSceneProxy& LightProxy = *(ShadowInfo->GetLightSceneInfo().Proxy);

		SetShaderValue(BatchedParameters, ShadowFadeFraction, ShadowInfo->FadeAlphas[ViewIndex] );
		SetShaderValue(BatchedParameters, ShadowSharpen, LightProxy.GetShadowSharpen() * 7.0f + 1.0f );
		SetShaderValue(BatchedParameters, LightPosition, FVector4f(FVector3f((FVector)LightProxy.GetPosition() + PreViewTranslation), 1.0f / LightProxy.GetRadius()));

		auto DeferredLightParameter = GetUniformBufferParameter<FDeferredLightUniformStruct>();

		if (DeferredLightParameter.IsBound())
		{
			SetDeferredLightParameters(BatchedParameters, DeferredLightParameter, &ShadowInfo->GetLightSceneInfo(), View, bUseLightFunctionAtlas);
		}


		FScene* Scene = nullptr;

		if (View.Family->Scene)
		{
			Scene = View.Family->Scene->GetRenderScene();
		}
	}

protected:
	LAYOUT_FIELD(FShadowProjectionShaderParameters, ProjectionParameters);
	LAYOUT_FIELD(FShaderParameter, ShadowFadeFraction);
	LAYOUT_FIELD(FShaderParameter, ShadowSharpen);
	LAYOUT_FIELD(FShaderParameter, LightPosition);
};

/** Pixel shader to project modulated shadows onto the scene. */
template<uint32 Quality>
class TModulatedShadowProjection : public TShadowProjectionPS<Quality, false, true>
{
	DECLARE_SHADER_TYPE(TModulatedShadowProjection, Global);
public:

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		TShadowProjectionPS<Quality, false, true>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MODULATED_SHADOWS"), 1);
		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEPTHREAD_SUBPASS"), bMobileForceDepthRead ? 0u : 1u);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	TModulatedShadowProjection() {}

	TModulatedShadowProjection(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		int32 ViewIndex,
		const FSceneView& View,
		const FProjectedShadowInfo* ShadowInfo,
		bool bUseLightFunctionAtlas)
	{
		TShadowProjectionPS<Quality, false, true>::SetParameters(BatchedParameters, ViewIndex, View, ShadowInfo, bUseLightFunctionAtlas);
		SetShaderValue(BatchedParameters, ModulatedShadowColorParameter, ShadowInfo->GetLightSceneInfo().Proxy->GetModulatedShadowColor());
	}

protected:
	LAYOUT_FIELD(FShaderParameter, ModulatedShadowColorParameter);
	LAYOUT_FIELD(FShaderUniformBufferParameter, MobileBasePassUniformBuffer);
};

/** Translucency shadow projection uniform buffer containing data needed for Fourier opacity maps. */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FTranslucentSelfShadowUniformParameters, )
	SHADER_PARAMETER(FMatrix44f, WorldToShadowMatrix)
	SHADER_PARAMETER(FVector4f, ShadowUVMinMax)
	SHADER_PARAMETER(FVector4f, DirectionalLightDirection)
	SHADER_PARAMETER(FVector4f, DirectionalLightColor)
	SHADER_PARAMETER_TEXTURE(Texture2D, Transmission0)
	SHADER_PARAMETER_TEXTURE(Texture2D, Transmission1)
	SHADER_PARAMETER_SAMPLER(SamplerState, Transmission0Sampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, Transmission1Sampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

extern void SetupTranslucentSelfShadowUniformParameters(const FProjectedShadowInfo* ShadowInfo, FTranslucentSelfShadowUniformParameters& OutParameters);

/**
* Default translucent self shadow data.
*/
class FEmptyTranslucentSelfShadowUniformBuffer : public TUniformBuffer< FTranslucentSelfShadowUniformParameters >
{
	typedef TUniformBuffer< FTranslucentSelfShadowUniformParameters > Super;
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

/** Global uniform buffer containing the default precomputed lighting data. */
extern TGlobalResource< FEmptyTranslucentSelfShadowUniformBuffer > GEmptyTranslucentSelfShadowUniformBuffer;

/** Pixel shader to project both opaque and translucent shadows onto opaque surfaces. */
template<uint32 Quality> 
class TShadowProjectionFromTranslucencyPS : public TShadowProjectionPS<Quality>
{
	DECLARE_SHADER_TYPE(TShadowProjectionFromTranslucencyPS,Global);
public:

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		TShadowProjectionPS<Quality>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("APPLY_TRANSLUCENCY_SHADOWS"), 1);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && TShadowProjectionPS<Quality>::ShouldCompilePermutation(Parameters);
	}

	TShadowProjectionFromTranslucencyPS() {}

	TShadowProjectionFromTranslucencyPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		TShadowProjectionPS<Quality>(Initializer)
	{
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		int32 ViewIndex,
		const FSceneView& View,
		const FProjectedShadowInfo* ShadowInfo,
		bool bUseLightFunctionAtlas)
	{
		TShadowProjectionPS<Quality>::SetParameters(BatchedParameters, ViewIndex, View, ShadowInfo, bUseLightFunctionAtlas);

		FTranslucentSelfShadowUniformParameters TranslucentSelfShadowUniformParameters;
		SetupTranslucentSelfShadowUniformParameters(ShadowInfo, TranslucentSelfShadowUniformParameters);
		SetUniformBufferParameterImmediate(BatchedParameters, this->template GetUniformBufferParameter<FTranslucentSelfShadowUniformParameters>(), TranslucentSelfShadowUniformParameters);
	}
};


BEGIN_SHADER_PARAMETER_STRUCT(FOnePassPointShadowProjection, )
	SHADER_PARAMETER_RDG_TEXTURE(TextureCube, ShadowDepthCubeTexture)
	SHADER_PARAMETER_RDG_TEXTURE(TextureCube, ShadowDepthCubeTexture2)
	SHADER_PARAMETER_SAMPLER(SamplerComparisonState, ShadowDepthCubeTextureSampler)
	SHADER_PARAMETER_ARRAY(FMatrix44f, ShadowViewProjectionMatrices, [6])
	SHADER_PARAMETER(float, InvShadowmapResolution)
END_SHADER_PARAMETER_STRUCT()

extern void GetOnePassPointShadowProjectionParameters(FRDGBuilder& GraphBuilder, const FProjectedShadowInfo* ShadowInfo, FOnePassPointShadowProjection& OutParameters);


/** One pass point light shadow projection parameters used by multiple shaders. */
class FOnePassPointShadowProjectionShaderParameters
{
	DECLARE_TYPE_LAYOUT(FOnePassPointShadowProjectionShaderParameters, NonVirtual);
public:

	void Bind(const FShaderParameterMap& ParameterMap)
	{
		ShadowDepthTexture.Bind(ParameterMap,TEXT("ShadowDepthCubeTexture"));
		ShadowDepthTexture2.Bind(ParameterMap, TEXT("ShadowDepthCubeTexture2"));
		ShadowDepthCubeComparisonSampler.Bind(ParameterMap,TEXT("ShadowDepthCubeTextureSampler"));
		ShadowViewProjectionMatrices.Bind(ParameterMap, TEXT("ShadowViewProjectionMatrices"));
		InvShadowmapResolution.Bind(ParameterMap, TEXT("InvShadowmapResolution"));
		LightPositionOrDirection.Bind(ParameterMap, TEXT("LightPositionOrDirection"));
	}

	inline void Set(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FProjectedShadowInfo* ShadowInfo) const
	{
		const FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();

		FRHITexture* ShadowDepthTextureValue = GBlackTextureDepthCube->TextureRHI;
		
		if (ShadowInfo)
		{
			if (FRHITexture* Texture = ShadowInfo->RenderTargets.DepthTarget->GetRHI())
			{
				ShadowDepthTextureValue = Texture;
			}
		}

		SetTextureParameter(BatchedParameters, ShadowDepthTexture, ShadowDepthTextureValue);
		SetTextureParameter(BatchedParameters, ShadowDepthTexture2, ShadowDepthTextureValue);

		if (LightPositionOrDirection.IsBound())
		{
			const FVector LightPosition = ShadowInfo ? FVector(ShadowInfo->GetLightSceneInfo().Proxy->GetPosition()) : FVector::ZeroVector;
			SetShaderValue(BatchedParameters, LightPositionOrDirection, FVector4f(FVector3f(LightPosition + PreViewTranslation), 1));
		}
		
		if (ShadowDepthCubeComparisonSampler.IsBound())
		{
			// Use a comparison sampler to do hardware PCF
			SetSamplerParameter(BatchedParameters, ShadowDepthCubeComparisonSampler, TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, 0, 0, SCF_Less>::GetRHI());
		}

		const int32 NumShaderMatrices = FMath::DivideAndRoundUp<int32>(ShadowViewProjectionMatrices.GetNumBytes(), sizeof(FMatrix44f));

		if (ShadowInfo)
		{
			const int32 NumUsableMatrices = FMath::Min<int32>(NumShaderMatrices, ShadowInfo->OnePassShadowViewProjectionMatrices.Num());
			TArray<FMatrix44f, SceneRenderingAllocator> TypeCastedMatrices;
			TypeCastedMatrices.AddUninitialized(NumUsableMatrices);
			for (int32 i = 0; i < NumUsableMatrices; i++)
			{
				TypeCastedMatrices[i] = FMatrix44f(ShadowInfo->OnePassShadowViewProjectionMatrices[i]);		// LWC_TODO: Precision loss?
			}
			if (NumUsableMatrices < NumShaderMatrices)
			{
				TypeCastedMatrices.AddZeroed(NumShaderMatrices - NumUsableMatrices);
			}
			SetShaderValueArray<FMatrix44f>(
				BatchedParameters,
				ShadowViewProjectionMatrices,
				TypeCastedMatrices.GetData(),
				NumShaderMatrices
				);

			SetShaderValue(BatchedParameters,InvShadowmapResolution,1.0f / ShadowInfo->ResolutionX);
		}
		else
		{
			TArray<FMatrix44f, SceneRenderingAllocator> ZeroMatrices;
			ZeroMatrices.AddZeroed(NumShaderMatrices);

			SetShaderValueArray<FMatrix44f>(
				BatchedParameters,
				ShadowViewProjectionMatrices,
				ZeroMatrices.GetData(),
				NumShaderMatrices
				);

			SetShaderValue(BatchedParameters,InvShadowmapResolution,0);
		}
	}

	/** Serializer. */ 
	/*friend FArchive& operator<<(FArchive& Ar,FOnePassPointShadowProjectionShaderParameters& P)
	{
		Ar << P.ShadowDepthTexture;
		Ar << P.ShadowDepthTexture2;
		Ar << P.ShadowDepthCubeComparisonSampler;
		Ar << P.ShadowViewProjectionMatrices;
		Ar << P.InvShadowmapResolution;
		Ar << P.LightPositionOrDirection;
		return Ar;
	}*/

private:
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthTexture);
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthTexture2);
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthCubeComparisonSampler);
	LAYOUT_FIELD(FShaderParameter, ShadowViewProjectionMatrices);
	LAYOUT_FIELD(FShaderParameter, InvShadowmapResolution);
	LAYOUT_FIELD(FShaderParameter, LightPositionOrDirection);
};

// Reversed Z
struct FShadowProjectionMatrix : FMatrix
{
	FShadowProjectionMatrix(FVector::FReal MinZ, FVector::FReal MaxZ, const FVector4& WAxis) :
		FMatrix(
			FPlane(1,	0,	0,													WAxis.X),
			FPlane(0,	1,	0,													WAxis.Y),
			FPlane(0,	0,      -(WAxis.Z * MinZ + WAxis.W) / (MaxZ - MinZ),	WAxis.Z),
			FPlane(0,	0, MaxZ *(WAxis.Z * MinZ + WAxis.W) / (MaxZ - MinZ),	WAxis.W)
			)
	{}

	// Off center projection
	FShadowProjectionMatrix( const FVector2D& Min, const FVector2D& Max, const FVector4& WAxis )
		: FMatrix(
			FPlane( 2.0f / (Max.X - Min.X),				0.0f,								0.0f, WAxis.X),
			FPlane( 0.0f,								2.0f / (Max.Y - Min.Y),				0.0f, WAxis.Y),
			FPlane( -(Max.X + Min.X) / (Max.X - Min.X),	-(Max.Y + Min.Y) / (Max.Y - Min.Y),	0.0f, WAxis.Z),
			FPlane( 0.0f,								0.0f,								1.0f, WAxis.W)
			)
	{}

	// Change near and far plane
	FShadowProjectionMatrix( const FMatrix& InMatrix, float MinZ, float MaxZ )
		: FMatrix( InMatrix )
	{
		M[2][2] = -( M[2][3] * MinZ + M[3][3] ) / ( MaxZ - MinZ );
		M[3][2] = MaxZ * -M[2][2];
	}
};


/** Pixel shader to project directional PCSS onto the scene. */
template<uint32 Quality, bool bUseFadePlane>
class TDirectionalPercentageCloserShadowProjectionPS : public TShadowProjectionPS<Quality, bUseFadePlane>
{
	DECLARE_SHADER_TYPE(TDirectionalPercentageCloserShadowProjectionPS, Global);
public:

	TDirectionalPercentageCloserShadowProjectionPS() {}
	TDirectionalPercentageCloserShadowProjectionPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		TShadowProjectionPS<Quality, bUseFadePlane>(Initializer)
	{
		PCSSParameters.Bind(Initializer.ParameterMap, TEXT("PCSSParameters"));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		TShadowProjectionPS<Quality, bUseFadePlane>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_PCSS"), 1);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return TShadowProjectionPS<Quality, bUseFadePlane>::ShouldCompilePermutation(Parameters)
			&& FDataDrivenShaderPlatformInfo::GetSupportsPercentageCloserShadows(Parameters.Platform);
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		int32 ViewIndex,
		const FSceneView& View,
		const FProjectedShadowInfo* ShadowInfo,
		bool bUseLightFunctionAtlas)
	{
		TShadowProjectionPS<Quality, bUseFadePlane>::SetParameters(BatchedParameters, ViewIndex, View, ShadowInfo, bUseLightFunctionAtlas);

		// GetLightSourceAngle returns the full angle.
		float TanLightSourceAngle = FMath::Tan(0.5 * FMath::DegreesToRadians(ShadowInfo->GetLightSceneInfo().Proxy->GetLightSourceAngle()));

		static IConsoleVariable* CVarMaxSoftShadowKernelSize = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.MaxSoftKernelSize"));
		check(CVarMaxSoftShadowKernelSize);
		int32 MaxKernelSize = CVarMaxSoftShadowKernelSize->GetInt();

		float SW = 2.0 * ShadowInfo->ShadowBounds.W;
		float SZ = ShadowInfo->MaxSubjectZ - ShadowInfo->MinSubjectZ;

		FVector4f PCSSParameterValues = FVector4f(TanLightSourceAngle * SZ / SW, MaxKernelSize / float(ShadowInfo->ResolutionX), 0, 0);
		SetShaderValue(BatchedParameters, PCSSParameters, PCSSParameterValues);
	}

protected:
	LAYOUT_FIELD(FShaderParameter, PCSSParameters);
};


/** Pixel shader to project PCSS spot light onto the scene. */
template<uint32 Quality, bool bUseFadePlane>
class TSpotPercentageCloserShadowProjectionPS : public TShadowProjectionPS<Quality, bUseFadePlane>
{
	DECLARE_SHADER_TYPE(TSpotPercentageCloserShadowProjectionPS, Global);
public:

	TSpotPercentageCloserShadowProjectionPS() {}
	TSpotPercentageCloserShadowProjectionPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		TShadowProjectionPS<Quality, bUseFadePlane>(Initializer)
	{
		PCSSParameters.Bind(Initializer.ParameterMap, TEXT("PCSSParameters"));
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
			&& FDataDrivenShaderPlatformInfo::GetSupportsPercentageCloserShadows(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		TShadowProjectionPS<Quality, bUseFadePlane>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_PCSS"), 1);
		OutEnvironment.SetDefine(TEXT("SPOT_LIGHT_PCSS"), 1);
	}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		int32 ViewIndex,
		const FSceneView& View,
		const FProjectedShadowInfo* ShadowInfo,
		bool bUseLightFunctionAtlas)
	{
		check(ShadowInfo->GetLightSceneInfo().Proxy->GetLightType() == LightType_Spot);

		TShadowProjectionPS<Quality, bUseFadePlane>::SetParameters(BatchedParameters, ViewIndex, View, ShadowInfo, bUseLightFunctionAtlas);

		static IConsoleVariable* CVarMaxSoftShadowKernelSize = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Shadow.MaxSoftKernelSize"));
		check(CVarMaxSoftShadowKernelSize);
		int32 MaxKernelSize = CVarMaxSoftShadowKernelSize->GetInt();

		FVector4f PCSSParameterValues = FVector4f(0, MaxKernelSize / float(ShadowInfo->ResolutionX), 0, 0);
		SetShaderValue(BatchedParameters, PCSSParameters, PCSSParameterValues);
	}

protected:
	LAYOUT_FIELD(FShaderParameter, PCSSParameters);
};


// Sort by descending resolution
struct FCompareFProjectedShadowInfoByResolution
{
	FORCEINLINE bool operator() (const FProjectedShadowInfo& A, const FProjectedShadowInfo& B) const
	{
		return (B.ResolutionX * B.ResolutionY < A.ResolutionX * A.ResolutionY);
	}
};

// Sort by shadow type (CSMs first, then other types).
// Then sort CSMs by descending split index, and other shadows by resolution.
// Used to render shadow cascades in far to near order, whilst preserving the
// descending resolution sort behavior for other shadow types.
// Note: the ordering must match the requirements of blend modes set in SetBlendStateForProjection (blend modes that overwrite must come first)
struct FCompareFProjectedShadowInfoBySplitIndex
{
	FORCEINLINE bool operator()( const FProjectedShadowInfo& A, const FProjectedShadowInfo& B ) const
	{
		if (A.IsWholeSceneDirectionalShadow())
		{
			if (B.IsWholeSceneDirectionalShadow())
			{
				if (A.bRayTracedDistanceField != B.bRayTracedDistanceField)
				{
					// RTDF shadows need to be rendered after all CSM, because they overlap in depth range with Far Cascades, which will use an overwrite blend mode for the fade plane.
					if (!A.bRayTracedDistanceField && B.bRayTracedDistanceField)
					{
						return true;
					}

					if (A.bRayTracedDistanceField && !B.bRayTracedDistanceField)
					{
						return false;
					}
				}

				// Both A and B are CSMs
				// Compare Split Indexes, to order them far to near.
				return (B.CascadeSettings.ShadowSplitIndex < A.CascadeSettings.ShadowSplitIndex);
			}

			// A is a CSM, B is per-object shadow etc.
			// B should be rendered after A.
			return true;
		}
		else
		{
			if (B.IsWholeSceneDirectionalShadow())
			{
				// B should be rendered before A.
				return false;
			}
			
			// Neither shadow is a CSM
			// Sort by descending resolution.
			return FCompareFProjectedShadowInfoByResolution()(A, B);
		}
	}
};

bool IsSceneCaptureCubeShareCascadesEnabled(EShaderPlatform Platform);