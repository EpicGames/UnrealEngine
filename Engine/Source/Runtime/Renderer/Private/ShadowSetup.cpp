// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowSetup.cpp: Dynamic shadow setup implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "Async/Fundamental/Scheduler.h"
#include "Async/Fundamental/Task.h"
#include "Async/ParallelFor.h"
#include "Async/TaskGraphInterfaces.h"
#include "CanvasTypes.h"
#include "CapsuleShadowRendering.h"
#include "Containers/AllowShrinking.h"
#include "Containers/ArrayView.h"
#include "Containers/ChunkedArray.h"
#include "ConvexVolume.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DepthRendering.h"
#include "DynamicPrimitiveDrawing.h"
#include "Engine/EngineTypes.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "GameTime.h"
#include "GlobalRenderResources.h"
#include "GPUScene.h"
#include "HairStrands/HairStrandsData.h"
#include "HAL/IConsoleManager.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "LightComponentId.h"
#include "LightRendering.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "LineTypes.h"
#include "Logging/LogMacros.h"
#include "Materials/MaterialRelevance.h"
#include "Math/Color.h"
#include "Math/GenericOctree.h"
#include "Math/Plane.h"
#include "Math/Sphere.h"
#include "MeshDrawCommands.h"
#include "MeshElementCollector.h"
#include "MeshPassProcessor.h"
#include "Misc/App.h"
#include "Misc/Fork.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Misc/ScopeLock.h"
#include "MultiGPU.h"
#include "Nanite/NaniteShared.h"
#include "PrimitiveComponentId.h"
#include "PrimitiveDrawingUtils.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveViewRelevance.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/CsvProfilerTrace.h"
#include "ReadOnlyCVARCache.h"
#include "RenderCommandTag.h"
#include "RenderCore.h"
#include "RendererInterface.h"
#include "LogRenderer.h"
#include "RendererPrivateUtils.h"
#include "RendererUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderingThread.h"
#include "RenderTargetPool.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIPipeline.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneCore.h"
#include "SceneCulling/SceneCulling.h"
#include "SceneCulling/SceneCullingRenderer.h"
#include "SceneExtensions.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneRenderBuilder.h"
#include "SceneRendererInterface.h"
#include "SceneRendering.h"
#include "SceneTypes.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "SceneVisibility.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShadowRendering.h"
#include "Shadows/ShadowSceneRenderer.h"
#include "Shadows/CSMSceneExtension.h"
#include "ShowFlags.h"
#include "SimpleMeshDrawCommandPass.h"
#include "StaticMeshBatch.h"
#include "Stats/Stats.h"
#include "StereoRendering.h"
#include "Tasks/Pipe.h"
#include "Tasks/Task.h"
#include "TextureLayout.h"
#include "UnrealEngine.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VirtualShadowMaps/VirtualShadowMapCacheManager.h"
#include "VirtualShadowMaps/VirtualShadowMapClipmap.h"
#include "SceneViewState.h"

using namespace UE::Geometry;
using namespace ShadowRendering;

/** Number of cube map shadow depth surfaces that will be created and used for rendering one pass point light shadows. */
static const int32 NumCubeShadowDepthSurfaces = 5;

/** Number of surfaces used for translucent shadows. */
static const int32 NumTranslucencyShadowSurfaces = 2;

float GMinScreenRadiusForShadowCaster = 0.01f;
static FAutoConsoleVariableRef CVarMinScreenRadiusForShadowCaster(
	TEXT("r.Shadow.RadiusThreshold"),
	GMinScreenRadiusForShadowCaster,
	TEXT("Cull shadow casters if they are too small, value is the minimal screen space bounding sphere radius"),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GCacheWholeSceneShadows = 1;
FAutoConsoleVariableRef CVarCacheWholeSceneShadows(
	TEXT("r.Shadow.CacheWholeSceneShadows"),
	GCacheWholeSceneShadows,
	TEXT("When enabled, movable point and spot light whole scene shadow depths from static primitives will be cached as an optimization."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GMaxNumPointShadowCacheUpdatesPerFrame = -1;
FAutoConsoleVariableRef CVarMaxNumPointShadowCacheUpdatePerFrame(
	TEXT("r.Shadow.MaxNumPointShadowCacheUpdatesPerFrame"),
	GMaxNumPointShadowCacheUpdatesPerFrame,
	TEXT("Maximum number of point light shadow cache updates allowed per frame."
		"Only affect updates caused by resolution change. -1 means no limit."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GMaxNumSpotShadowCacheUpdatesPerFrame = -1;
FAutoConsoleVariableRef CVarMaxNumSpotShadowCacheUpdatePerFrame(
	TEXT("r.Shadow.MaxNumSpotShadowCacheUpdatesPerFrame"),
	GMaxNumSpotShadowCacheUpdatesPerFrame,
	TEXT("Maximum number of spot light shadow cache updates allowed per frame."
		"Only affect updates caused by resolution change. -1 means no limit."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GWholeSceneShadowCacheMb = 150;
FAutoConsoleVariableRef CVarWholeSceneShadowCacheMb(
	TEXT("r.Shadow.WholeSceneShadowCacheMb"),
	GWholeSceneShadowCacheMb,
	TEXT("Amount of memory that can be spent caching whole scene shadows.  ShadowMap allocations in a single frame can cause this to be exceeded."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GCachedShadowsCastFromMovablePrimitives = 1;
FAutoConsoleVariableRef CVarCachedWholeSceneShadowsCastFromMovablePrimitives(
	TEXT("r.Shadow.CachedShadowsCastFromMovablePrimitives"),
	GCachedShadowsCastFromMovablePrimitives,
	TEXT("Whether movable primitives should cast a shadow from cached whole scene shadows (movable point and spot lights).\n")
	TEXT("Disabling this can be used to remove the copy of the cached shadowmap."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

// Temporary chicken bit to back out optimization if there are any issues
int32 GSkipCullingNaniteMeshes = 1;
FAutoConsoleVariableRef CVarSkipCullingNaniteMeshes(
	TEXT("r.Shadow.SkipCullingNaniteMeshes"),
	GSkipCullingNaniteMeshes,
	TEXT("When enabled, CPU culling will ignore nanite meshes."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

/** Can be used to visualize preshadow frustums when the shadowfrustums show flag is enabled. */
static TAutoConsoleVariable<int32> CVarDrawPreshadowFrustum(
	TEXT("r.Shadow.DrawPreshadowFrustums"),
	0,
	TEXT("visualize preshadow frustums when the shadowfrustums show flag is enabled"),
	ECVF_RenderThreadSafe
	);

/** Whether to allow preshadows (static world casting on character), can be disabled for debugging. */
static TAutoConsoleVariable<int32> CVarAllowPreshadows(
	TEXT("r.Shadow.Preshadows"),
	1,
	TEXT("Whether to allow preshadows (static world casting on character)"),
	ECVF_RenderThreadSafe
	);

/** Whether to allow per object shadows (character casting on world), can be disabled for debugging. */
static TAutoConsoleVariable<int32> CVarAllowPerObjectShadows(
	TEXT("r.Shadow.PerObject"),
	1,
	TEXT("Whether to render per object shadows (character casting on world)\n")
	TEXT("0: off\n")
	TEXT("1: on (default)"),
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<float> CVarShadowFadeExponent(
	TEXT("r.Shadow.FadeExponent"),
	0.25f,
	TEXT("Controls the rate at which shadows are faded out"),
	ECVF_RenderThreadSafe);

static int32 GShadowLightViewConvexHullCull = 1;
static FAutoConsoleVariableRef CVarShadowLightViewConvexHullCull(
	TEXT("r.Shadow.LightViewConvexHullCull"),
	GShadowLightViewConvexHullCull,
	TEXT("Enables culling of shadow casters that do not intersect the convex hull of the light origin and view frustum."),
	ECVF_RenderThreadSafe);

/**
 * Whether preshadows can be cached as an optimization.  
 * Disabling the caching through this setting is useful when debugging.
 */
static TAutoConsoleVariable<int32> CVarCachePreshadows(
	TEXT("r.Shadow.CachePreshadow"),
	1,
	TEXT("Whether preshadows can be cached as an optimization"),
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarInsetDownscaleFactor(
	TEXT("r.Shadow.InsetDownscaleFactor"),
	1,
	TEXT("Use a lower resolution for inset shadows (OriginalResolution / DownscaleFactor)"),
	ECVF_RenderThreadSafe
);

/**
 * NOTE: This flag is intended to be kept only as long as deemed neccessary to be sure that no artifacts were introduced.
 *       This allows a quick hot-fix to disable the change if need be.
 */
static TAutoConsoleVariable<int32> CVarResolutionScaleZeroDisablesSm(
	TEXT("r.Shadow.ResolutionScaleZeroDisablesSm"),
	1,
	TEXT("DEPRECATED: If 1 (default) then setting Shadow Resolution Scale to zero disables shadow maps for the light."),
	ECVF_RenderThreadSafe
);


bool ShouldUseCachePreshadows()
{
	return CVarCachePreshadows.GetValueOnRenderThread() != 0;
}

int32 GPreshadowsForceLowestLOD = 0;
FAutoConsoleVariableRef CVarPreshadowsForceLowestLOD(
	TEXT("r.Shadow.PreshadowsForceLowestDetailLevel"),
	GPreshadowsForceLowestLOD,
	TEXT("When enabled, static meshes render their lowest detail level into preshadow depth maps.  Disabled by default as it causes artifacts with poor quality LODs (tree billboard)."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

/**
 * This value specifies how much bounds will be expanded when rendering a cached preshadow (0.15 = 15% larger).
 * Larger values result in more cache hits, but lower resolution and pull more objects into the depth pass.
 */
static TAutoConsoleVariable<float> CVarPreshadowExpandFraction(
	TEXT("r.Shadow.PreshadowExpand"),
	0.15f,
	TEXT("How much bounds will be expanded when rendering a cached preshadow (0.15 = 15% larger)"),
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<float> CVarPreShadowResolutionFactor(
	TEXT("r.Shadow.PreShadowResolutionFactor"),
	0.5f,
	TEXT("Mulitplier for preshadow resolution"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarShadowTexelsPerPixel(
	TEXT("r.Shadow.TexelsPerPixel"),
	1.27324f,
	TEXT("The ratio of subject pixels to shadow texels for per-object shadows"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarShadowTexelsPerPixelPointlight(
	TEXT("r.Shadow.TexelsPerPixelPointlight"),
	1.27324f,
	TEXT("The ratio of subject pixels to shadow texels for point lights"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarShadowTexelsPerPixelSpotlight(
	TEXT("r.Shadow.TexelsPerPixelSpotlight"),
	2.0f * 1.27324f,
	TEXT("The ratio of subject pixels to shadow texels for spotlights"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarShadowTexelsPerPixelRectlight(
	TEXT("r.Shadow.TexelsPerPixelRectlight"),
	1.27324f,
	TEXT("The ratio of subject pixels to shadow texels for rect lights"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarPreShadowFadeResolution(
	TEXT("r.Shadow.PreShadowFadeResolution"),
	16,
	TEXT("Resolution in texels below which preshadows are faded out"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowFadeResolution(
	TEXT("r.Shadow.FadeResolution"),
	64,
	TEXT("Resolution in texels below which shadows are faded out"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMinShadowResolution(
	TEXT("r.Shadow.MinResolution"),
	32,
	TEXT("Minimum dimensions (in texels) allowed for rendering shadow subject depths"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMinPreShadowResolution(
	TEXT("r.Shadow.MinPreShadowResolution"),
	8,
	TEXT("Minimum dimensions (in texels) allowed for rendering preshadow depths"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarParallelGatherShadowPrimitives(
	TEXT("r.ParallelGatherShadowPrimitives"),
	1,  
	TEXT("Toggles parallel Gather shadow primitives. 0 = off; 1 = on"),
	ECVF_RenderThreadSafe
	);

int32 GParallelInitDynamicShadows = 1;
static FAutoConsoleVariableRef CVarParallelInitDynamicShadows(
	TEXT("r.ParallelInitDynamicShadows"),
	GParallelInitDynamicShadows,
	TEXT("Toggles parallel dynamic shadow initialization. 0 = off; 1 = on"),
	ECVF_RenderThreadSafe
);

int32 GNumShadowDynamicMeshElementTasks = 4;
static FAutoConsoleVariableRef CVarNumShadowDynamicMeshElementTasks(
	TEXT("r.Visibility.DynamicMeshElements.NumShadowViewTasks"),
	GNumShadowDynamicMeshElementTasks,
	TEXT("Controls the number of gather dynamic mesh elements tasks to run asynchronously during shadow visibility."),
	ECVF_RenderThreadSafe
);

uint32 GetNumShadowDynamicMeshElementTasks()
{
	if (!IsParallelGatherDynamicMeshElementsEnabled())
	{
		return 0;
	}

	return FMath::Clamp<int32>(GNumShadowDynamicMeshElementTasks, 0, LowLevelTasks::FScheduler::Get().GetNumWorkers());
}

static TAutoConsoleVariable<int32> CVarParallelGatherNumPrimitivesPerPacket(
	TEXT("r.ParallelGatherNumPrimitivesPerPacket"),
	256,  
	TEXT("Number of primitives per packet.  Only used when r.Shadow.UseOctreeForCulling is disabled."),
	ECVF_RenderThreadSafe
	);

int32 GUseOctreeForShadowCulling = 1;
FAutoConsoleVariableRef CVarUseOctreeForShadowCulling(
	TEXT("r.Shadow.UseOctreeForCulling"),
	GUseOctreeForShadowCulling,
	TEXT("Whether to use the primitive octree for shadow subject culling.  The octree culls large groups of primitives at a time, but introduces cache misses walking the data structure."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static TAutoConsoleVariable<int32> CVarVisualizePrimitiveOctree(
	TEXT("r.Shadow.VisualizePrimitiveOctree"),
	0,
	TEXT("Render a wireframe outline of the nodes in the primitive octree for debugging purposes"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);
#endif

static TAutoConsoleVariable<int32> CVarAlwaysAllocateMaxResolutionAtlases(
	TEXT("r.Shadow.AlwaysAllocateMaxResolutionAtlases"),
	0,
	TEXT("If enabled, will always allocate shadow map atlases at the maximum resolution rather than trimming unused space. This will waste more memory but can possibly reduce render target pool fragmentation and thrash."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarVsmUseFarShadowRules(
	TEXT("r.Shadow.Virtual.UseFarShadowCulling"),
	1,
	TEXT("Switch between implementing the far shadow culling logic for VSMs."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarClipmapUseConservativeCulling(
	TEXT("r.Shadow.Virtual.Clipmap.UseConservativeCulling"),
	0,
	TEXT("Conservative culling removes the frustum-clipped culling volume for the non-nanite geometry for VSM rendering. This means a lot more geometry is submitted, and also marked as rendered.\n")
	TEXT("Useful to diagnose if there are culling artifacts in virtual shadow map clip maps due to errors in the tracking code."),
	ECVF_RenderThreadSafe
);

CSV_DECLARE_CATEGORY_EXTERN(LightCount);

#if !UE_BUILD_SHIPPING
// read and written on the render thread
bool GDumpShadowSetup = false;
void DumpShadowDumpSetup()
{
	ENQUEUE_RENDER_COMMAND(DumpShadowDumpSetup)(
		[](FRHICommandList& RHICmdList)
		{
			GDumpShadowSetup = true;
		});
}

FAutoConsoleCommand CmdDumpShadowDumpSetup(
	TEXT("r.DumpShadows"),
	TEXT("Dump shadow setup (for developer only, only for non shiping build)"),
	FConsoleCommandDelegate::CreateStatic(DumpShadowDumpSetup)
	);
#endif // !UE_BUILD_SHIPPING

static TAutoConsoleVariable<bool> CVarMobileInsetShadows(
	TEXT("r.Mobile.SupportInsetShadows"),
	false,
	TEXT("Enables inset shadows on mobile. Currently not supported on platforms not using ShadowMaskTexture. Default = false."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

/** Whether to round the shadow map up to power of two on mobile platform. */
static TAutoConsoleVariable<int32> CVarMobileShadowmapRoundUpToPowerOfTwo(
	TEXT("r.Mobile.ShadowmapRoundUpToPowerOfTwo"),
	0,
	TEXT("Round the shadow map up to power of two on mobile platform, in case there is any compatibility issue.\n")
	TEXT(" 0: Disable (Default)\n")
	TEXT(" 1: Enabled"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarCSMCaching(
	TEXT("r.Shadow.CSMCaching"),
	0,
	TEXT("0: Render CSM every frame.\n")
	TEXT("1: Enable CSM caching. (default)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarCSMScrollingOverlapAreaThrottle(
	TEXT("r.Shadow.CSMScrollingOverlapAreaThrottle"),
	0.75f,
	TEXT("The minimum ratio of the overlap area for CSM scrolling."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarCSMScrollingMaxExtraStaticShadowSubjects(
	TEXT("r.Shadow.MaxCSMScrollingStaticShadowSubjects"),
	5,
	TEXT("The maximum number of extra static shadow subjects need to be drawed when scrolling CSM."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarCSMScissorOptim(
	TEXT("r.Shadow.CSMScissorOptim"),
	false,
	TEXT("Compute optimized scissor rect size to exclude portions of the CSM slices outside the view frustum"),
	ECVF_RenderThreadSafe
);

static bool GDoesShadowFadeUseResolutionScale = 1;
FAutoConsoleVariableRef CVarComputeShadowFadeUsingResolutionScale(
	TEXT("r.Shadow.DoesFadeUseResolutionScale"),
	GDoesShadowFadeUseResolutionScale,
	TEXT("When enabled (default), the shadow resolution scale for the individual light feeds into the fade out calculation, increasing the control of shadow fade out.\n"
		 "  Setting the cvar to 0 restores the behavior to pre UE 5.6."),
	ECVF_RenderThreadSafe
	);

// For now, this feature is only enabled for the mobile renderer.  It could theoretically work for the deferred renderer as well, but the implementation
// currently isn't compatible with Virtual Shadow Maps, which are usually enabled by default for the deferred renderer.  In practice, it's only important
// for the mobile renderer, as it enables timesliced rendering of shadow cascades for mobile specific runtime reflection captures.
static TAutoConsoleVariable<bool> CVarSceneCaptureCubeMobileShareCascades(
	TEXT("r.SceneCapture.CubeMobileShareCascades"),
	true,
	TEXT("When set to true, mobile cube scene captures render cascade shadow depths once and share that single shadow map across all cube faces."),
	ECVF_RenderThreadSafe);

// Note that a mirror of this function is present in the Engine module's DirectionalLightComponent.cpp.  If this implementation changes, that also needs to be updated!
bool IsSceneCaptureCubeShareCascadesEnabled(EShaderPlatform Platform)
{
	return CVarSceneCaptureCubeMobileShareCascades.GetValueOnAnyThread() && IsMobilePlatform(Platform);
}

/**
 * Helper function to maintain the common login of caching the whole scene shadow and be able to handle the specific-purpose through the CacheLambda and UnCacheLambda
 */
template <typename CacheLambdaType, typename UnCacheLambdaType>
void TryToCacheShadowMap(const FScene* Scene, int64 CachedShadowMapsSize, int32& OutNumShadowMaps, EShadowDepthCacheMode* OutCacheModes, uint32& NumCachesUpdatedThisFrame, CacheLambdaType&& CacheLambda, UnCacheLambdaType&& UnCacheLambda)
{
	if (CachedShadowMapsSize < static_cast<int64>(GWholeSceneShadowCacheMb) * 1024 * 1024)
	{
		OutNumShadowMaps = 2;
		// Note: ShadowMap with static primitives rendered first so movable shadowmap can composite
		OutCacheModes[0] = SDCM_StaticPrimitivesOnly;
		OutCacheModes[1] = SDCM_MovablePrimitivesOnly;
		++NumCachesUpdatedThisFrame;

		CacheLambda();
	}
	else
	{
		OutNumShadowMaps = 1;
		OutCacheModes[0] = SDCM_Uncached;

		UnCacheLambda();
	}
}

/**
 * Helper function to determine fade alpha value for shadows based on resolution. In the below ASCII art (1) is
 * the MinShadowResolution and (2) is the ShadowFadeResolution. Alpha will be 0 below the min resolution and 1
 * above the fade resolution. In between it is going to be an exponential curve with the values between (1) and (2)
 * being normalized in the 0..1 range.
 *
 *  
 *  |    /-------
 *  |  /
 *  |/
 *  1-----2-------
 *
 * @param	MaxUnclampedResolution		Requested resolution, unclamped so it can be below min
 * @param	ShadowFadeResolution		Resolution at which fade begins
 * @param	MinShadowResolution			Minimum resolution of shadow
 *
 * @return	fade value between 0 and 1
 */
float CalculateShadowFadeAlpha(const float MaxUnclampedResolution, const uint32 ShadowFadeResolution, const uint32 MinShadowResolution)
{
	// NB: MaxUnclampedResolution < 0 will return FadeAlpha = 0.0f. 

	float FadeAlpha = 0.0f;
	// Shadow size is above fading resolution.
	if (MaxUnclampedResolution > ShadowFadeResolution)
	{
		FadeAlpha = 1.0f;
	}
	// Shadow size is below fading resolution but above min resolution.
	else if (MaxUnclampedResolution > MinShadowResolution)
	{
		const float Exponent = CVarShadowFadeExponent.GetValueOnRenderThread();
		
		// Use the limit case ShadowFadeResolution = MinShadowResolution
		// to gracefully handle this case.
		if (MinShadowResolution >= ShadowFadeResolution)
		{
			const float SizeRatio = (float)(MaxUnclampedResolution - MinShadowResolution);
			FadeAlpha = 1.0f - FMath::Pow(SizeRatio, Exponent);
		} 
		else
		{
			const float InverseRange = 1.0f / (ShadowFadeResolution - MinShadowResolution);
			const float FirstFadeValue = FMath::Pow(InverseRange, Exponent);
			const float SizeRatio = (float)(MaxUnclampedResolution - MinShadowResolution) * InverseRange;
			// Rescale the fade alpha to reduce the change between no fading and the first value, which reduces popping with small ShadowFadeExponent's
			FadeAlpha = (FMath::Pow(SizeRatio, Exponent) - FirstFadeValue) / (1.0f - FirstFadeValue);
		}
	}
	return FadeAlpha;
}

typedef TArray<FVector,TInlineAllocator<8> > FBoundingBoxVertexArray;

/** Stores the indices for an edge of a bounding volume. */
struct FBoxEdge
{
	uint16 FirstEdgeIndex;
	uint16 SecondEdgeIndex;
	FBoxEdge(uint16 InFirst, uint16 InSecond) :
		FirstEdgeIndex(InFirst),
		SecondEdgeIndex(InSecond)
	{}
};

typedef TArray<FBoxEdge,TInlineAllocator<12> > FBoundingBoxEdgeArray;

/**
 * Creates an array of vertices and edges for a bounding box.
 * @param Box - The bounding box
 * @param OutVertices - Upon return, the array will contain the vertices of the bounding box.
 * @param OutEdges - Upon return, will contain indices of the edges of the bounding box.
 */
static void GetBoundingBoxVertices(const FBox& Box,FBoundingBoxVertexArray& OutVertices, FBoundingBoxEdgeArray& OutEdges)
{
	OutVertices.Empty(8);
	OutVertices.AddUninitialized(8);
	for(int32 X = 0;X < 2;X++)
	{
		for(int32 Y = 0;Y < 2;Y++)
		{
			for(int32 Z = 0;Z < 2;Z++)
			{
				OutVertices[X * 4 + Y * 2 + Z] = FVector(
					X ? Box.Min.X : Box.Max.X,
					Y ? Box.Min.Y : Box.Max.Y,
					Z ? Box.Min.Z : Box.Max.Z
					);
			}
		}
	}

	OutEdges.Empty(12);
	OutEdges.AddUninitialized(12);
	for(uint16 X = 0;X < 2;X++)
	{
		uint16 BaseIndex = X * 4;
		OutEdges[X * 4 + 0] = FBoxEdge(BaseIndex, BaseIndex + 1);
		OutEdges[X * 4 + 1] = FBoxEdge(BaseIndex + 1, BaseIndex + 3);
		OutEdges[X * 4 + 2] = FBoxEdge(BaseIndex + 3, BaseIndex + 2);
		OutEdges[X * 4 + 3] = FBoxEdge(BaseIndex + 2, BaseIndex);
	}
	for(uint16 XEdge = 0;XEdge < 4;XEdge++)
	{
		OutEdges[8 + XEdge] = FBoxEdge(XEdge, XEdge + 4);
	}
}

/**
 * Computes the transform contains a set of bounding box vertices and minimizes the pre-transform volume inside the post-transform clip space.
 * @param ZAxis - The Z axis of the transform.
 * @param Points - The points that represent the bounding volume.
 * @param Edges - The edges of the bounding volume.
 * @return true if it successfully found a non-zero area projection of the bounding points.
 */
static bool GetBestShadowTransform(const FVector& ZAxis,const FBoundingBoxVertexArray& Points, const FBoundingBoxEdgeArray& Edges, FVector& OutXAxis, FVector2D& OutMinProjected, FVector2D& OutMaxProjected)
{
	// Find the axis parallel to the edge between any two boundary points with the smallest projection of the bounds onto the axis.

	float BestProjectedArea = FLT_MAX;
	bool bValidProjection = false;

	// Cache unaliased pointers to point and edge data
	const FVector* RESTRICT PointsPtr = Points.GetData();
	const FBoxEdge* RESTRICT EdgesPtr = Edges.GetData();

	const int32 NumPoints = Points.Num();
	const int32 NumEdges = Edges.Num();

	// We're always dealing with box geometry here, so we can hint the compiler
	UE_ASSUME( NumPoints == 8 );
	UE_ASSUME( NumEdges == 12 );

	for(int32 EdgeIndex = 0;EdgeIndex < NumEdges; ++EdgeIndex)
	{
		const FVector Point = PointsPtr[EdgesPtr[EdgeIndex].FirstEdgeIndex];
		const FVector OtherPoint = PointsPtr[EdgesPtr[EdgeIndex].SecondEdgeIndex];
		const FVector PointDelta = OtherPoint - Point;
		const FVector TrialXAxis = (PointDelta - ZAxis * (PointDelta | ZAxis)).GetSafeNormal();
		const FVector TrialYAxis = (ZAxis ^ TrialXAxis).GetSafeNormal();

		// Calculate the size of the projection of the bounds onto this axis and an axis orthogonal to it and the Z axis.
		FVector2D MinProjected = {  FLT_MAX,  FLT_MAX };
		FVector2D MaxProjected = { -FLT_MAX, -FLT_MAX };
		for(int32 ProjectedPointIndex = 0;ProjectedPointIndex < NumPoints; ++ProjectedPointIndex)
		{
			const float ProjectedX = PointsPtr[ProjectedPointIndex] | TrialXAxis;
			MinProjected.X = FMath::Min(MinProjected.X,ProjectedX);
			MaxProjected.X = FMath::Max(MaxProjected.X,ProjectedX);
			const float ProjectedY = PointsPtr[ProjectedPointIndex] | TrialYAxis;
			MinProjected.Y = FMath::Min(MinProjected.Y,ProjectedY);
			MaxProjected.Y = FMath::Max(MaxProjected.Y,ProjectedY);
		}

		const FVector2D ProjectedExtent = MaxProjected - MinProjected;
		const float ProjectedArea = ProjectedExtent.X * ProjectedExtent.Y;

		if(ProjectedArea < BestProjectedArea - .05f 
			// Only allow projections with non-zero area
			&& ProjectedArea > DELTA)
		{
			bValidProjection = true;
			BestProjectedArea = ProjectedArea;

			if(ProjectedExtent.Y > ProjectedExtent.X)
			{
				// Always make the X axis the largest one.
				OutXAxis = TrialYAxis;
				OutMinProjected = FVector2D( MinProjected.Y, -MaxProjected.X );
				OutMaxProjected = FVector2D( MaxProjected.Y, -MinProjected.X );
			}
			else
			{
				OutXAxis = TrialXAxis;
				OutMinProjected = MinProjected;
				OutMaxProjected = MaxProjected;
			}
		}
	}

	// Only create the shadow if the projected extent of the given points has a non-zero area.
	if(bValidProjection && BestProjectedArea > DELTA)
	{
		return true;
	}
	else
	{
		return false;
	}
}

int32 GetMaxShadowResolution(ERHIFeatureLevel::Type FeatureLevel)
{
	static const auto* MaxShadowResolutionCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Shadow.MaxResolution"));
	int32 MaxShadowResolution = MaxShadowResolutionCVar->GetValueOnRenderThread();

	if (FeatureLevel < ERHIFeatureLevel::SM5)
	{
		// ensure there is always enough space for mobile renderer's tiled shadow maps
		// by reducing the shadow map resolution.
		int32 MaxShadowDepthBufferDim = FMath::Max(GMaxShadowDepthBufferSizeX, GMaxShadowDepthBufferSizeY);
		if (MaxShadowResolution * 2 > MaxShadowDepthBufferDim)
		{
			MaxShadowResolution = MaxShadowDepthBufferDim / 2;
		}
	}

	return MaxShadowResolution;
}

/** Returns the size of the shadow depth buffer, taking into account platform limitations and game specific resolution limits. */
static FIntPoint GetShadowDepthTextureResolution(ERHIFeatureLevel::Type FeatureLevel)
{
	int32 MaxShadowRes = GetMaxShadowResolution(FeatureLevel);
	const FIntPoint ShadowBufferResolution(
		FMath::Clamp(MaxShadowRes, 1, (int32)GMaxShadowDepthBufferSizeX),
		FMath::Clamp(MaxShadowRes, 1, (int32)GMaxShadowDepthBufferSizeY));

	return ShadowBufferResolution;
}

static int32 GetTranslucentShadowDownsampleFactor()
{
	return 2;
}

static FIntPoint GetPreShadowCacheTextureResolution(ERHIFeatureLevel::Type FeatureLevel)
{
	const FIntPoint ShadowDepthResolution = GetShadowDepthTextureResolution(FeatureLevel);
	// Higher numbers increase cache hit rate but also memory usage
	const int32 ExpandFactor = GetTranslucentShadowDownsampleFactor();

	float Factor = CVarPreShadowResolutionFactor.GetValueOnRenderThread();

	FIntPoint Ret;

	Ret.X = FMath::Clamp(FMath::TruncToInt(ShadowDepthResolution.X * Factor) * ExpandFactor, 1, (int32)GMaxShadowDepthBufferSizeX);
	Ret.Y = FMath::Clamp(FMath::TruncToInt(ShadowDepthResolution.Y * Factor) * ExpandFactor, 1, (int32)GMaxShadowDepthBufferSizeY);

	return Ret;
}

static FIntPoint GetTranslucentShadowDepthTextureResolution()
{
	FIntPoint ShadowDepthResolution = GetShadowDepthTextureResolution(ERHIFeatureLevel::SM5);

	const int32 Factor = 2;

	ShadowDepthResolution.X = FMath::Clamp(ShadowDepthResolution.X / Factor, 1, (int32)GMaxShadowDepthBufferSizeX);
	ShadowDepthResolution.Y = FMath::Clamp(ShadowDepthResolution.Y / Factor, 1, (int32)GMaxShadowDepthBufferSizeY);

	return ShadowDepthResolution;
}

/** Returns an index in the range [0, NumCubeShadowDepthSurfaces) given an input resolution. */
static int32 GetCubeShadowDepthZIndex(ERHIFeatureLevel::Type FeatureLevel, int32 ShadowResolution)
{
	FIntPoint ObjectShadowBufferResolution = GetShadowDepthTextureResolution(FeatureLevel);

	// Use a lower resolution because cubemaps use a lot of memory
	ObjectShadowBufferResolution.X /= 2;
	ObjectShadowBufferResolution.Y /= 2;
	const int32 SurfaceSizes[NumCubeShadowDepthSurfaces] =
	{
		ObjectShadowBufferResolution.X,
		ObjectShadowBufferResolution.X / 2,
		ObjectShadowBufferResolution.X / 4,
		ObjectShadowBufferResolution.X / 8,
		CVarMinShadowResolution.GetValueOnRenderThread()
	};

	for (int32 SearchIndex = 0; SearchIndex < NumCubeShadowDepthSurfaces; SearchIndex++)
	{
		if (ShadowResolution >= SurfaceSizes[SearchIndex])
		{
			return SearchIndex;
		}
	}

	check(0);
	return 0;
}

/** Returns the appropriate resolution for a given cube shadow index. */
static int32 GetCubeShadowDepthZResolution(ERHIFeatureLevel::Type FeatureLevel, int32 ShadowIndex)
{
	checkSlow(ShadowIndex >= 0 && ShadowIndex < NumCubeShadowDepthSurfaces);
	FIntPoint ObjectShadowBufferResolution = GetShadowDepthTextureResolution(FeatureLevel);

	// Use a lower resolution because cubemaps use a lot of memory
	ObjectShadowBufferResolution.X = FMath::Max(ObjectShadowBufferResolution.X / 2, 1);
	ObjectShadowBufferResolution.Y = FMath::Max(ObjectShadowBufferResolution.Y / 2, 1);
	const int32 SurfaceSizes[NumCubeShadowDepthSurfaces] =
	{
		ObjectShadowBufferResolution.X,
		FMath::Max(ObjectShadowBufferResolution.X / 2, 1),
		FMath::Max(ObjectShadowBufferResolution.X / 4, 1),
		FMath::Max(ObjectShadowBufferResolution.X / 8, 1),
		CVarMinShadowResolution.GetValueOnRenderThread()
	};
	return SurfaceSizes[ShadowIndex];
}


FProjectedShadowInfo::FProjectedShadowInfo()
	: ShadowDepthView(NULL)
	, CacheMode(SDCM_Uncached)
	, DependentView(0)
	, ShadowId(INDEX_NONE)
	, PreShadowTranslation(0, 0, 0)
	, MaxSubjectZ(0)
	, MinSubjectZ(0)
	, ShadowBounds(0)
	, X(0)
	, Y(0)
	, ResolutionX(0)
	, ResolutionY(0)
	, BorderSize(0)
	, MaxScreenPercent(1.0f)
	, bAllocated(false)
	, bRendered(false)
	, bAllocatedInPreshadowCache(false)
	, bDepthsCached(false)
	, bDirectionalLight(false)
	, bOnePassPointLightShadow(false)
	, bVSM(false)
	, bWholeSceneShadow(false)
	, bTranslucentShadow(false)
	, bRayTracedDistanceField(false)
	, bCapsuleShadow(false)
	, bPreShadow(false)
	, bSelfShadowOnly(false)
	, bPerObjectOpaqueShadow(false)
	, bTransmission(false)
	, bHairStrandsDeepShadow(false)
	, bNaniteGeometry(true)
	, bContainsNaniteSubjects(false)
	, bShouldRenderVSM(true)
	, bVolumetricShadow(false)
	, PerObjectShadowFadeStart(UE_OLD_WORLD_MAX)		// LWC_TODO: Upgrade to double? Should have been HALF_WORLD_MAX to match precision.
	, InvPerObjectShadowFadeLength(0.0f)
	, OverlappedUVOnCachedShadowMap(-1.0f, -1.0f, -1.0f, -1.0f)
	, OverlappedUVOnCurrentShadowMap(-1.0f, -1.0f, -1.0f, -1.0f)
	, LightSceneInfo(0)
	, ParentSceneInfo(0)
	, NumDynamicSubjectMeshElements(0)
	, NumSubjectMeshCommandBuildRequestElements(0)
	, ShaderDepthBias(0.0f)
	, ShaderSlopeDepthBias(0.0f)
{
}

FProjectedShadowInfo::~FProjectedShadowInfo()
{
	for (auto ProjectionStencilingPass : ProjectionStencilingPasses)
	{
		delete ProjectionStencilingPass;
	}
}

/** Shadow border needs to be wide enough to prevent the shadow filtering from picking up content in other shadowmaps in the atlas. */
const static uint32 SHADOW_BORDER = 4; 

bool FProjectedShadowInfo::SetupPerObjectProjection(
	FLightSceneInfo* InLightSceneInfo,
	const FPrimitiveSceneInfo* InParentSceneInfo,
	const FPerObjectProjectedShadowInitializer& Initializer,
	bool bInPreShadow,
	uint32 InResolutionX,
	uint32 MaxShadowResolutionY,
	uint32 InBorderSize,
	float InMaxScreenPercent,
	bool bInTranslucentShadow)
{
	check(InParentSceneInfo);

	LightSceneInfo = InLightSceneInfo;
	LightSceneInfoCompact = InLightSceneInfo;
	ParentSceneInfo = InParentSceneInfo;
	PreShadowTranslation = Initializer.PreShadowTranslation;
	ShadowBounds = FSphere(Initializer.SubjectBounds.Origin - Initializer.PreShadowTranslation, Initializer.SubjectBounds.SphereRadius);
	ResolutionX = InResolutionX;
	BorderSize = InBorderSize;
	MaxScreenPercent = InMaxScreenPercent;
	bDirectionalLight = InLightSceneInfo->Proxy->GetLightType() == LightType_Directional;
	const EShaderPlatform ShaderPlatform = LightSceneInfo->Scene->GetShaderPlatform();
	bCapsuleShadow = InParentSceneInfo->Proxy->CastsCapsuleDirectShadow() && !bInPreShadow && IsCapsuleDirectShadowsEnabled(ShaderPlatform);
	bTranslucentShadow = bInTranslucentShadow;
	bPreShadow = bInPreShadow;
	bSelfShadowOnly = InParentSceneInfo->Proxy->CastsSelfShadowOnly();
	bTransmission = InLightSceneInfo->Proxy->Transmission();
	bHairStrandsDeepShadow = InLightSceneInfo->Proxy->CastsHairStrandsDeepShadow();
	bVolumetricShadow = InLightSceneInfo->Proxy->CastsVolumetricShadow();
	
	check(!bRayTracedDistanceField);
	
	// Create an array of the extreme vertices of the subject's bounds.
	FBoundingBoxVertexArray BoundsPoints;
	FBoundingBoxEdgeArray BoundsEdges;
	GetBoundingBoxVertices(Initializer.SubjectBounds.GetBox(),BoundsPoints,BoundsEdges);
	
	const FVector FaceDirection( 1, 0, 0 );

	// Project the bounding box vertices.
	FBoundingBoxVertexArray ProjectedBoundsPoints;
	for (int32 PointIndex = 0; PointIndex < BoundsPoints.Num(); PointIndex++)
	{
		const FVector TransformedBoundsPoint = Initializer.WorldToLight.TransformPosition(BoundsPoints[PointIndex]);
		const FVector4::FReal TransformedBoundsPointW = Dot4(FVector4(0, 0, TransformedBoundsPoint.X,1), Initializer.WAxis);
		if (TransformedBoundsPointW >= DELTA)
		{
			ProjectedBoundsPoints.Add(TransformedBoundsPoint / TransformedBoundsPointW);
		}
		else
		{
			//ProjectedBoundsPoints.Add(FVector(FLT_MAX, FLT_MAX, FLT_MAX));
			return false;
		}
	}
	
	FVector XAxis;
	FVector2D MinProjected;
	FVector2D MaxProjected;
	
	if (GetBestShadowTransform(FaceDirection, ProjectedBoundsPoints, BoundsEdges, XAxis, MinProjected, MaxProjected))
	{
		FVector ZAxis = FaceDirection;
		FVector YAxis = (ZAxis ^ XAxis).GetSafeNormal();

		// Compute the transform from light-space to shadow-space.
		const FMatrix LightToView = FBasisVectorMatrix(XAxis,YAxis,ZAxis,FVector(0,0,0));
		TranslatedWorldToView = Initializer.WorldToLight * LightToView;

		const FBox ShadowSubjectBounds = Initializer.SubjectBounds.GetBox().TransformBy( TranslatedWorldToView );

		MinSubjectZ = FMath::Max<float>(Initializer.MinLightW, ShadowSubjectBounds.Min.Z);
		float MaxReceiverZ = FMath::Min(MinSubjectZ + Initializer.MaxDistanceToCastInLightW, (float)HALF_WORLD_MAX);
		// Max can end up smaller than min due to the clamp to HALF_WORLD_MAX above
		MaxReceiverZ = FMath::Max(MaxReceiverZ, MinSubjectZ + 1);
		MaxSubjectZ = FMath::Max<float>(ShadowSubjectBounds.Max.Z, MinSubjectZ + 1);

		const FVector2D ProjectedExtent = MaxProjected - MinProjected;
		const float AspectRatio = ProjectedExtent.X / ProjectedExtent.Y;
		ResolutionY = FMath::Clamp<uint32>(FMath::TruncToInt(InResolutionX / AspectRatio), 1, MaxShadowResolutionY);

		if (ResolutionX == 0 || ResolutionY == 0)
		{
			return false;
		}

		// Scale matrix to also include the border
		FVector2D BorderScale = FVector2D(ResolutionX / float(ResolutionX + 2 * BorderSize), ResolutionY / float(ResolutionY + 2 * BorderSize));
		FMatrix BorderScaleMatrix = FScaleMatrix(FVector(BorderScale, 1.0f));

		const FMatrix ProjectionInner = FShadowProjectionMatrix( MinProjected, MaxProjected, Initializer.WAxis );
		const FMatrix ProjectionOuter = ProjectionInner * BorderScaleMatrix;

		FMatrix ReceiverInnerMatrix;
		float MaxSubjectDepth;
		if (bPreShadow)
		{
			// Preshadow frustum bounds go from the light to the furthest extent of the object in light space
			ViewToClipInner = FShadowProjectionMatrix( ProjectionInner, Initializer.MinLightW, MaxSubjectZ );
			ViewToClipOuter = FShadowProjectionMatrix( ProjectionOuter, Initializer.MinLightW, MaxSubjectZ );
			ReceiverInnerMatrix = TranslatedWorldToView * FShadowProjectionMatrix( ProjectionInner, MinSubjectZ, MaxSubjectZ );
			MaxSubjectDepth = bDirectionalLight ? 1.0f : Initializer.MinLightW;
		}
		else
		{
			ViewToClipInner = FShadowProjectionMatrix( ProjectionInner, MinSubjectZ, MaxSubjectZ );
			ViewToClipOuter = FShadowProjectionMatrix( ProjectionOuter, MinSubjectZ, MaxSubjectZ );
			ReceiverInnerMatrix = TranslatedWorldToView * FShadowProjectionMatrix( ProjectionInner, MinSubjectZ, MaxReceiverZ );
			MaxSubjectDepth = bDirectionalLight ? 1.0f : MinSubjectZ;

			if (bDirectionalLight)
			{
				// No room to fade out if the end of receiver range is inside the subject range, it will just clip.
				if (MaxSubjectZ < MaxReceiverZ)
				{
					float ShadowSubjectRange = MaxSubjectZ - MinSubjectZ;
					float FadeLength = FMath::Min(ShadowSubjectRange, MaxReceiverZ - MaxSubjectZ);

					PerObjectShadowFadeStart = (MaxReceiverZ - MinSubjectZ - FadeLength) / ShadowSubjectRange;
					InvPerObjectShadowFadeLength = ShadowSubjectRange / FMath::Max(0.000001f, FadeLength);
				}
			}
		}
		TranslatedWorldToClipInnerMatrix = FMatrix44f(TranslatedWorldToView * ViewToClipInner);		// LWC_TODO: Precision loss?
		TranslatedWorldToClipOuterMatrix = FMatrix44f(TranslatedWorldToView * ViewToClipOuter);

		InvMaxSubjectDepth = 1.0f / MaxSubjectDepth;

		MinPreSubjectZ = Initializer.MinLightW;

		GetViewFrustumBounds(CasterOuterFrustum, FMatrix(TranslatedWorldToClipOuterMatrix), true);
		GetViewFrustumBounds(ReceiverInnerFrustum, ReceiverInnerMatrix, true);
		
		InvReceiverInnerMatrix = FMatrix44f(ReceiverInnerMatrix.InverseFast());
		
		UpdateShaderDepthBias();

		return true;
	}

	return false;
}

void FProjectedShadowInfo::SetupWholeSceneProjection(
	FLightSceneInfo* InLightSceneInfo,
	FViewInfo* InDependentView,
	const FWholeSceneProjectedShadowInitializer& Initializer,
	uint32 InResolutionX,
	uint32 InResolutionY,
	uint32 InSnapResolutionX,
	uint32 InSnapResolutionY,
	uint32 InBorderSize, 
	int32 InSelectedViewIndex)
{	
	LightSceneInfo = InLightSceneInfo;
	LightSceneInfoCompact = InLightSceneInfo;
	DependentView = InDependentView;
	PreShadowTranslation = Initializer.PreShadowTranslation;
	CascadeSettings = Initializer.CascadeSettings;
	ResolutionX = InResolutionX;
	ResolutionY = InResolutionY;
	bDirectionalLight = InLightSceneInfo->Proxy->GetLightType() == LightType_Directional;
	bOnePassPointLightShadow = Initializer.bOnePassPointLightShadow;
	bRayTracedDistanceField = Initializer.bRayTracedDistanceField;
	bWholeSceneShadow = true;
	bTransmission = InLightSceneInfo->Proxy->Transmission();
	bHairStrandsDeepShadow = InLightSceneInfo->Proxy->CastsHairStrandsDeepShadow();
	bVolumetricShadow = InLightSceneInfo->Proxy->CastsVolumetricShadow();
	BorderSize = InBorderSize;
	SelectedViewIndex = InSelectedViewIndex;

	// Scale matrix to also include the border
	FMatrix BorderScaleMatrix = FScaleMatrix(FVector(ResolutionX / float(ResolutionX + 2 * BorderSize), ResolutionY / float(ResolutionY + 2 * BorderSize), 1.0f));

	FMatrix InnerScaleMatrix = FScaleMatrix(FVector(Initializer.Scales.X, Initializer.Scales.Y, 1.0f));
	FMatrix OuterScaleMatrix = BorderScaleMatrix * InnerScaleMatrix;

	checkf(
		Initializer.WorldToLight.M[3][0] == 0.0f &&
		Initializer.WorldToLight.M[3][1] == 0.0f &&
		Initializer.WorldToLight.M[3][2] == 0.0f,
		TEXT("WorldToLight has translation") );

	const FMatrix FaceMatrix(
		FPlane( 0, 0, 1, 0 ),
		FPlane( 0, 1, 0, 0 ),
		FPlane(-1, 0, 0, 0 ),
		FPlane( 0, 0, 0, 1 ) );
	
	TranslatedWorldToView = Initializer.WorldToLight * FaceMatrix;

	MaxSubjectZ = TranslatedWorldToView.TransformPosition(Initializer.SubjectBounds.Origin).Z + Initializer.SubjectBounds.SphereRadius;
	MinSubjectZ = FMath::Max(MaxSubjectZ - Initializer.SubjectBounds.SphereRadius * 2,Initializer.MinLightW);
	MaxSubjectZ = FMath::Min(MaxSubjectZ, Initializer.MaxDistanceToCastInLightW);

	FMatrix WorldToViewScaledInner = TranslatedWorldToView * InnerScaleMatrix;

	if(bDirectionalLight)
	{
		// Limit how small the depth range can be for smaller cascades
		// This is needed for shadow modes like subsurface shadows which need depth information outside of the smaller cascade depth range
		//@todo - expose this value to the ini
		const float DepthRangeClamp = 5000;
		MaxSubjectZ = FMath::Max(MaxSubjectZ, DepthRangeClamp);
		MinSubjectZ = FMath::Min(MinSubjectZ, -DepthRangeClamp);

		// Transform the shadow's position into shadowmap space
		const FVector TransformedPosition = WorldToViewScaledInner.TransformPosition(-PreShadowTranslation);

		// Largest amount that the shadowmap will be downsampled to during sampling
		// We need to take this into account when snapping to get a stable result
		// This corresponds to the maximum kernel filter size used by subsurface shadows in ShadowProjectionPixelShader.usf
		const int32 MaxDownsampleFactor = 4;
		// Determine the distance necessary to snap the shadow's position to the nearest texel
		const float SnapX = FMath::Fmod(TransformedPosition.X, (FVector::FReal)2.0f * MaxDownsampleFactor / InSnapResolutionX);
		const float SnapY = FMath::Fmod(TransformedPosition.Y, (FVector::FReal)2.0f * MaxDownsampleFactor / InSnapResolutionY);
		// Snap the shadow's position and transform it back into world space
		// This snapping prevents sub-texel camera movements which removes view dependent aliasing from the final shadow result
		// This only maintains stable shadows under camera translation and rotation
		const FVector SnappedWorldPosition = WorldToViewScaledInner.InverseFast().TransformPosition(TransformedPosition - FVector(SnapX, SnapY, 0.0f));
		PreShadowTranslation = -SnappedWorldPosition;
	}

	if (CascadeSettings.ShadowSplitIndex >= 0 && bDirectionalLight)
	{
		checkSlow(InDependentView);

		// TODO: Madness? After setting up the projection based on the bounds we go back and get it from the proxy again???
		ShadowBounds = InLightSceneInfo->Proxy->GetShadowSplitBounds(
			*InDependentView, 
			bRayTracedDistanceField ? INDEX_NONE : CascadeSettings.ShadowSplitIndex, 
			InLightSceneInfo->IsPrecomputedLightingValid(), 
			0);
	}
	else
	{
		ShadowBounds = FSphere(-PreShadowTranslation, Initializer.SubjectBounds.SphereRadius);
	}

	checkf(MaxSubjectZ > MinSubjectZ, TEXT("MaxSubjectZ %f MinSubjectZ %f SubjectBounds.SphereRadius %f"), MaxSubjectZ, MinSubjectZ, Initializer.SubjectBounds.SphereRadius);

	// TODO: That does this clamp achieve?
	const float ClampedMaxLightW = FMath::Min(MinSubjectZ + Initializer.MaxDistanceToCastInLightW, (float)HALF_WORLD_MAX);
	MinPreSubjectZ = Initializer.MinLightW;

	ViewToClipInner = InnerScaleMatrix * FShadowProjectionMatrix(MinSubjectZ, MaxSubjectZ, Initializer.WAxis);
	ViewToClipOuter = OuterScaleMatrix * FShadowProjectionMatrix(MinSubjectZ, MaxSubjectZ, Initializer.WAxis);
	TranslatedWorldToClipInnerMatrix = FMatrix44f(TranslatedWorldToView * ViewToClipInner);		// LWC_TODO: Precision loss
	TranslatedWorldToClipOuterMatrix = FMatrix44f(TranslatedWorldToView * ViewToClipOuter);
	
	// Nothing to do with subject depth. Just a scale factor to map z to 0-1.
	float MaxSubjectDepth = bDirectionalLight ? 1.0f : MinSubjectZ;

	InvMaxSubjectDepth = 1.0f / MaxSubjectDepth;

	// Any meshes between the light and the subject can cast shadows, also any meshes inside the subject region
	const FMatrix CasterOuterMatrix = WorldToViewScaledInner * BorderScaleMatrix * FShadowProjectionMatrix(Initializer.MinLightW, MaxSubjectZ, Initializer.WAxis);
	const FMatrix ReceiverInnerMatrix = WorldToViewScaledInner * FShadowProjectionMatrix(MinSubjectZ, ClampedMaxLightW, Initializer.WAxis);
	GetViewFrustumBounds(CasterOuterFrustum, CasterOuterMatrix, true);
	GetViewFrustumBounds(ReceiverInnerFrustum, ReceiverInnerMatrix, true);
	
	InvReceiverInnerMatrix = FMatrix44f(ReceiverInnerMatrix.Inverse());

	UpdateShaderDepthBias();

	if (Initializer.bOnePassPointLightShadow)
	{
		const static FVector CubeDirections[6] =
		{
			FVector(-1, 0, 0),
			FVector(1, 0, 0),
			FVector(0, -1, 0),
			FVector(0, 1, 0),
			FVector(0, 0, -1),
			FVector(0, 0, 1)
		};

		const static FVector UpVectors[6] =
		{
			FVector(0, 1, 0),
			FVector(0, 1, 0),
			FVector(0, 0, -1),
			FVector(0, 0, 1),
			FVector(0, 1, 0),
			FVector(0, 1, 0)
		};

		const FLightSceneProxy& LightProxy = *(GetLightSceneInfo().Proxy);

		OnePassShadowFaceProjectionMatrix = FReversedZPerspectiveMatrix(PI / 4.0f, 1, 1, 1, LightProxy.GetRadius());

		// Light projection and bounding volume is set up relative to the light position
		// the view pre-translation (relative to light) is added later, when rendering & sampling.
		const FVector LightPosition = Initializer.WorldToLight.GetOrigin();

		OnePassShadowViewMatrices.Empty(6);
		OnePassShadowViewProjectionMatrices.Empty(6);
		const FMatrix ScaleMatrix = FScaleMatrix(FVector(1, -1, 1));

		// fill in the caster frustum with the far plane from every face
		CasterOuterFrustum.Planes.Empty();
		for (int32 FaceIndex = 0; FaceIndex < 6; FaceIndex++)
		{
			// Create a view projection matrix for each cube face
			const FMatrix WorldToLightMatrix = FLookFromMatrix(LightPosition, CubeDirections[FaceIndex], UpVectors[FaceIndex]) * ScaleMatrix;
			OnePassShadowViewMatrices.Add(WorldToLightMatrix);
			const FMatrix ShadowViewProjectionMatrix = WorldToLightMatrix * OnePassShadowFaceProjectionMatrix;
			OnePassShadowViewProjectionMatrices.Add(ShadowViewProjectionMatrix);
			// Add plane representing cube face to bounding volume
			CasterOuterFrustum.Planes.Add(FPlane(CubeDirections[FaceIndex], LightProxy.GetRadius()));
		}
		CasterOuterFrustum.Init();
	}

	if (ShouldUseCSMScissorOptim())
	{
		ComputeScissorRectOptim();
	}
}


void FProjectedShadowInfo::SetupClipmapProjection(FLightSceneInfo* InLightSceneInfo, FViewInfo* InDependentView, const TSharedPtr<FVirtualShadowMapClipmap>& InVirtualShadowMapClipmap, float InMaxNonFarCascadeDistance)
{
	VirtualShadowMapClipmap = InVirtualShadowMapClipmap;
	LightSceneInfo = InLightSceneInfo;
	LightSceneInfoCompact = InLightSceneInfo;
	DependentView = InDependentView;
	ResolutionX = FVirtualShadowMap::VirtualMaxResolutionXY;
	ResolutionY = FVirtualShadowMap::VirtualMaxResolutionXY;
	bDirectionalLight = true;
	bWholeSceneShadow = true;
	bVolumetricShadow = InLightSceneInfo->Proxy->CastsVolumetricShadow();
	BorderSize = 0;
	MaxNonFarCascadeDistance = InMaxNonFarCascadeDistance;
	bVSM = true;
	MeshSelectionMask = EShadowMeshSelection::VSM;

	const int32 ClipmapIndex = VirtualShadowMapClipmap->GetLevelCount() - 1;
	const FMatrix WorldToLightViewRotationMatrix = VirtualShadowMapClipmap->GetWorldToLightViewRotationMatrix();
	const FMatrix ViewToClipMatrix = VirtualShadowMapClipmap->GetViewToClipMatrix(ClipmapIndex);
		
	PreShadowTranslation = VirtualShadowMapClipmap->GetPreViewTranslation(ClipmapIndex);
	const FMatrix CasterMatrix = WorldToLightViewRotationMatrix * ViewToClipMatrix;
	GetViewFrustumBounds(CasterOuterFrustum, CasterMatrix, true);
	ReceiverInnerFrustum = CasterOuterFrustum;

	if (CVarClipmapUseConservativeCulling.GetValueOnRenderThread() != 0)
	{
		ShadowBounds = VirtualShadowMapClipmap->GetBoundingSphere();
		CascadeSettings.ShadowBoundsAccurate = VirtualShadowMapClipmap->GetViewFrustumBounds();
	}
	else
	{
		// NOTE: Only here for debug! This version causes issues with VSM caching as it may cull objects
		// that are outside the current camera frustum but overlap a page that will then be cached. We need
		// all cached pages to be complete.
		ShadowBounds = InLightSceneInfo->Proxy->GetShadowSplitBoundsDepthRange(*DependentView, DependentView->ViewMatrices.GetViewOrigin(), 10.0f, VirtualShadowMapClipmap->GetMaxRadius(), &CascadeSettings);
	}
	
	// Um... it's checked in IsWholeSceneDirectionalShadow()
	CascadeSettings.ShadowSplitIndex = 1000;

	ViewToClipInner = ViewToClipMatrix;
	ViewToClipOuter = ViewToClipMatrix;
	TranslatedWorldToView = WorldToLightViewRotationMatrix;
	TranslatedWorldToClipInnerMatrix = FMatrix44f(TranslatedWorldToView * ViewToClipInner);
	TranslatedWorldToClipOuterMatrix = FMatrix44f(TranslatedWorldToView * ViewToClipOuter);
}

static EMeshDrawCommandCullingPayloadFlags GetCullingPayloadFlags(bool bIsLODRange, bool bIsMinLODInRange, bool bIsMaxLODInRange)
{
	// When we apply LOD selection on GPU we submit a range of LODs and then cull each one according to screen size.
	// At both ends of a LOD range we only want to cull by screen size in one direction. This ensures that all possible screen sizes map to one LOD in the range.
	EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags = EMeshDrawCommandCullingPayloadFlags::Default;
	CullingPayloadFlags |= bIsLODRange && !bIsMaxLODInRange ? EMeshDrawCommandCullingPayloadFlags::MinScreenSizeCull : (EMeshDrawCommandCullingPayloadFlags)0;
	CullingPayloadFlags |= bIsLODRange && !bIsMinLODInRange ? EMeshDrawCommandCullingPayloadFlags::MaxScreenSizeCull : (EMeshDrawCommandCullingPayloadFlags)0;
	return CullingPayloadFlags;
}

static EMeshDrawCommandCullingPayloadFlags GetCullingPayloadFlags(FLODMask LODMask, int8 LODIndex)
{
	return GetCullingPayloadFlags(LODMask.IsLODRange(), LODMask.IsMinLODInRange(LODIndex), LODMask.IsMaxLODInRange(LODIndex));
}

void FProjectedShadowInfo::AddCachedMeshDrawCommandsForPass(
	const FMeshDrawCommandPrimitiveIdInfo& PrimitiveIdInfo, 
	const FPrimitiveSceneInfo* InPrimitiveSceneInfo,
	const FStaticMeshBatchRelevance& RESTRICT StaticMeshRelevance,
	const FStaticMeshBatch& StaticMesh,
	EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags,
	const FScene* Scene,
	EMeshPass::Type PassType,
	FMeshCommandOneFrameArray& VisibleMeshCommands,
	TArray<const FStaticMeshBatch*, SceneRenderingAllocator>& MeshCommandBuildRequests,
	TArray<EMeshDrawCommandCullingPayloadFlags, SceneRenderingAllocator> MeshCommandBuildFlags,
	int32& NumMeshCommandBuildRequestElements)
{
	const EShadingPath ShadingPath = GetFeatureLevelShadingPath(Scene->GetFeatureLevel());
	const bool bUseCachedMeshCommand = UseCachedMeshDrawCommands()
		&& !!(FPassProcessorManager::GetPassFlags(ShadingPath, PassType) & EMeshPassFlags::CachedMeshCommands)
		&& StaticMeshRelevance.bSupportsCachingMeshDrawCommands;

	if (bUseCachedMeshCommand)
	{
		const int32 StaticMeshCommandInfoIndex = StaticMeshRelevance.GetStaticMeshCommandInfoIndex(PassType);
		if (StaticMeshCommandInfoIndex >= 0)
		{
			const FCachedMeshDrawCommandInfo& CachedMeshDrawCommand = InPrimitiveSceneInfo->StaticMeshCommandInfos[StaticMeshCommandInfoIndex];
			const FCachedPassMeshDrawList& SceneDrawList = Scene->CachedDrawLists[PassType];
			const FMeshDrawCommand* MeshDrawCommand = CachedMeshDrawCommand.StateBucketId >= 0
					? &Scene->CachedMeshDrawCommandStateBuckets[PassType].GetByElementId(CachedMeshDrawCommand.StateBucketId).Key
					: &SceneDrawList.MeshDrawCommands[CachedMeshDrawCommand.CommandIndex];

			FVisibleMeshDrawCommand NewVisibleMeshDrawCommand;

			NewVisibleMeshDrawCommand.Setup(
				MeshDrawCommand,
				PrimitiveIdInfo,
				CachedMeshDrawCommand.StateBucketId,
				CachedMeshDrawCommand.MeshFillMode,
				CachedMeshDrawCommand.MeshCullMode,
				CachedMeshDrawCommand.Flags,
				CachedMeshDrawCommand.SortKey,
				CachedMeshDrawCommand.CullingPayload, 
				CullingPayloadFlags);

			VisibleMeshCommands.Add(NewVisibleMeshDrawCommand);
		}
	}
	else
	{
		NumMeshCommandBuildRequestElements += StaticMeshRelevance.NumElements;
		MeshCommandBuildRequests.Add(&StaticMesh);
		MeshCommandBuildFlags.Add(CullingPayloadFlags);
	}
}

inline bool AreAnyViewsStaticSceneOnly(TArrayView<FViewInfo> Views)
{
	for (const FViewInfo& View : Views)
	{
		if (View.bStaticSceneOnly)
		{
			return true;
		}
	}
	return false;
}

typedef TArray<FAddSubjectPrimitiveOp> FShadowSubjectPrimitives;
typedef TArray<FAddSubjectPrimitiveStats> FPerShadowGatherStats;

struct FDrawDebugShadowFrustumOp
{
	FDrawDebugShadowFrustumOp() = default;
	FDrawDebugShadowFrustumOp(FViewInfo& InView, FProjectedShadowInfo& InProjectedShadowInfo)
		: View(&InView)
		, ProjectedShadowInfo(&InProjectedShadowInfo)
	{}

	FViewInfo* View = nullptr;
	FProjectedShadowInfo* ProjectedShadowInfo = nullptr;
};

using FProjectedShadowInfoList = TArray<FProjectedShadowInfo*, SceneRenderingAllocator>;

struct FFilteredShadowArrays
{
	FFilteredShadowArrays() = default;

	// Sort visible shadows based on their allocation needs
	// 2d shadowmaps for this frame only that can be atlased across lights
	FProjectedShadowInfoList Shadows;
	// 2d shadowmaps that will persist across frames, can't be atlased
	FProjectedShadowInfoList CachedSpotlightShadows;
	FProjectedShadowInfoList TranslucentShadows;
	// 2d shadowmaps that persist across frames
	FProjectedShadowInfoList CachedPreShadows;
	// Cubemaps, can't be atlased
	FProjectedShadowInfoList WholeScenePointShadows;

	FProjectedShadowInfoList WholeSceneDirectionalShadows;
	FProjectedShadowInfoList CachedWholeSceneDirectionalShadows;

	/** Distance field shadows to project. Used to avoid iterating through the scene lights array. */
	TArray<FProjectedShadowInfo*, TInlineAllocator<2, SceneRenderingAllocator>> ProjectedDistanceFieldShadows;

	FProjectedShadowInfoList ShadowsToSetupViews;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FSceneRendererLightBitArray OnePassShadowUnsupportedLights;
#endif
};

class FShadowMeshCollector
{
public:
	static FShadowMeshCollector* Create(FRHICommandList& RHICmdList, FSceneRenderer& SceneRenderer)
	{
		return SceneRenderer.Allocator.Create<FShadowMeshCollector>(RHICmdList, SceneRenderer);
	}

	FMeshElementCollector& GetCollector()
	{
		return Collector;
	}

	void Finish()
	{
		Collector.Finish();
		DynamicVertexBuffer.Commit();
		DynamicIndexBuffer.Commit();
		Pipe.WaitUntilEmpty();
	}

	UE::Tasks::FPipe& GetPipe()
	{
		return Pipe;
	}

private:
	FShadowMeshCollector(FRHICommandList& RHICmdList, FSceneRenderer& SceneRenderer)
		: Collector(SceneRenderer.FeatureLevel, SceneRenderer.Allocator)
	{
		DynamicVertexBuffer.Init(RHICmdList);
		DynamicIndexBuffer.Init(RHICmdList);
		Collector.Start(RHICmdList, DynamicVertexBuffer, DynamicIndexBuffer, SceneRenderer.DynamicReadBufferForShadows);
	}

	FMeshElementCollector Collector;
	FGlobalDynamicVertexBuffer DynamicVertexBuffer;
	FGlobalDynamicIndexBuffer DynamicIndexBuffer;
	UE::Tasks::FPipe Pipe{ UE_SOURCE_LOCATION };

	template <typename T>
	friend class TConcurrentLinearBulkObjectAllocator;
};

// Common setup and working data for all GatherShadowPrimitives Tasks
struct FDynamicShadowsTaskData
{
	// Common data read from all points
	FSceneRenderer* SceneRenderer;
	const FScene* Scene;
	TArrayView<FViewInfo> Views;
	FSceneRenderingBulkObjectAllocator& Allocator;
	FInstanceCullingManager& InstanceCullingManager;
	const ERHIFeatureLevel::Type FeatureLevel;
	const EShaderPlatform ShaderPlatform;
	const bool bStaticSceneOnly;
	const bool bRunningEarly;
	const bool bMultithreaded;
	const bool bMultithreadedCreateAndFilterShadows;
	const bool bMultithreadedGDME;
	float MinScreenBoundsRadiusSquared;			// For screen size culling

	// Whether any light in the scene has a ray traced distance field shadow.
	bool bHasRayTracedDistanceFieldShadows = false;
	bool bFinishedMeshPassSetup = false;

	// Whether static actors with a static shadowing should be rendered into CSM on mobile
	bool bMobileAllowStaticCSM = false;

	// Generated from prepare task
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> PreShadows;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> ViewDependentWholeSceneShadows;
	TArray<FDrawDebugShadowFrustumOp, SceneRenderingAllocator> DrawDebugShadowFrustumOps;

	// Written from task
	TArray<struct FGatherShadowPrimitivesPacket*, SceneRenderingAllocator> Packets;
	FPerShadowGatherStats GatherStats;
	FFilteredShadowArrays ShadowArrays;
	UE::Tasks::FTaskEvent BeginGatherAndSortLightsTask{ UE_SOURCE_LOCATION };
	UE::Tasks::FTaskEvent FilterDynamicShadowsTask{ UE_SOURCE_LOCATION };

	// Gather Dynamic Mesh Elements state
	TArray<FShadowMeshCollector*, TInlineAllocator<1, SceneRenderingAllocator>> MeshCollectors;
	TArray<FRHICommandListImmediate::FQueuedCommandList, FConcurrentLinearArrayAllocator> CommandLists;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> ShadowsToGather;
	TBitArray<SceneRenderingBitArrayAllocator> ShadowsToGatherInSerialPass;
	UE::Tasks::FTaskEvent BeginGatherDynamicMeshElementsTask{ UE_SOURCE_LOCATION };
	UE::Tasks::FTaskEvent SetupMeshPassTask{ UE_SOURCE_LOCATION };
	std::atomic_int32_t ShadowsToGatherNextIndex = { 0 };

	// Command list used to allocate shadow depth render targets.
	FRHICommandList* RHICmdListForAllocateTargets = nullptr;

	// Used by RenderThread
	FGraphEventRef TaskEvent;

	FShadowMeshCollector& GetSerialMeshCollector() const
	{
		return *MeshCollectors.Last();
	}

	void PresizeViewDependentWholeSceneShadows()
	{
		for (int32 ShadowIdx = 0, Num = ViewDependentWholeSceneShadows.Num(); ShadowIdx < Num; ++ShadowIdx)
		{
			ViewDependentWholeSceneShadows[ShadowIdx]->PresizeSubjectPrimitiveArrays(GatherStats[ShadowIdx]);
		}
	}
	
	FDynamicShadowsTaskData(const FDynamicShadowsTaskData&) = delete;

	FDynamicShadowsTaskData(FRHICommandListImmediate& InRHICmdList, FSceneRenderer* InSceneRenderer, FInstanceCullingManager& InInstanceCullingManager, bool bInRunningEarly)
		: SceneRenderer(InSceneRenderer)
		, Scene(InSceneRenderer->Scene)
		, Views(InSceneRenderer->Views)
		, Allocator(SceneRenderer->Allocator)
		, InstanceCullingManager(InInstanceCullingManager)
		, FeatureLevel(Scene->GetFeatureLevel())
		, ShaderPlatform(GShaderPlatformForFeatureLevel[FeatureLevel])
		, bStaticSceneOnly(AreAnyViewsStaticSceneOnly(Views))
		, bRunningEarly(bInRunningEarly)
		, bMultithreaded((FApp::ShouldUseThreadingForPerformance() || FForkProcessHelper::IsForkedMultithreadInstance()) && CVarParallelGatherShadowPrimitives.GetValueOnRenderThread() > 0)
		, bMultithreadedCreateAndFilterShadows(bRunningEarly && bMultithreaded && GRHISupportsAsyncGetRenderQueryResult && GParallelInitDynamicShadows > 0)
		, bMultithreadedGDME(bMultithreadedCreateAndFilterShadows && GetNumShadowDynamicMeshElementTasks() > 0)
	{
		if (bMultithreadedCreateAndFilterShadows)
		{
			RHICmdListForAllocateTargets = new FRHICommandList(FRHIGPUMask::All());
			RHICmdListForAllocateTargets->SwitchPipeline(ERHIPipeline::Graphics);
			CommandLists.Emplace(RHICmdListForAllocateTargets);
		}
		else
		{
			RHICmdListForAllocateTargets = &InRHICmdList;
		}

		if (bMultithreadedGDME)
		{
			MeshCollectors.Reserve(GetNumShadowDynamicMeshElementTasks());
		}
		else
		{
			MeshCollectors.Emplace(Allocator.Create<FShadowMeshCollector>(InRHICmdList, *SceneRenderer));
			BeginGatherDynamicMeshElementsTask.Trigger();
		}

		bMobileAllowStaticCSM = (FeatureLevel == ERHIFeatureLevel::ES3_1 && FReadOnlyCVARCache::AllowStaticLighting() && FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers());

		// We can just pass the first view to this function, as the view is only used to check flags to see if the current scene renderer is
		// a runtime reflection capture, which use a different more aggressive screen culling size.  All views in such a scene renderer will
		// have the same flags set.
		MinScreenBoundsRadiusSquared = ScreenVisibilityGetMinScreenBoundsRadiusSquared(Views[0]);
	}

	~FDynamicShadowsTaskData()
	{
		check(bFinishedMeshPassSetup);
		check(CommandLists.IsEmpty());
		check(MeshCollectors.IsEmpty());
	}
};

UE::Tasks::FTask GetGatherAndSortLightsPrerequisiteTask(const FDynamicShadowsTaskData* TaskData)
{
	if (TaskData)
	{
		return TaskData->BeginGatherAndSortLightsTask;
	}
	return {};
}

void BeginShadowGatherDynamicMeshElements(FDynamicShadowsTaskData* TaskData)
{
	if (TaskData)
	{
		TaskData->BeginGatherDynamicMeshElementsTask.Trigger();
	}
}

TConstArrayView<FProjectedShadowInfo*> GetProjectedDistanceFieldShadows(const FDynamicShadowsTaskData* TaskData)
{
	if (TaskData)
	{
		check(!TaskData->TaskEvent);
		return TaskData->ShadowArrays.ProjectedDistanceFieldShadows;
	}
	return TConstArrayView<FProjectedShadowInfo*>();
}

struct FAddSubjectPrimitiveOverflowedIndices
{
	TArray<uint16> MDCIndices;
	TArray<uint16> MeshIndices;
};

struct FFinalizeAddSubjectPrimitiveContext
{
	const uint16* OverflowedMDCIndices;
	const uint16* OverflowedMeshIndices;
};

struct FAddSubjectPrimitiveResult
{
	union
	{
		uint64 Qword;
		struct
		{
			uint32 bCopyCachedMeshDrawCommand : 1;
			uint32 bRequestMeshCommandBuild : 1;
			uint32 bOverflowed : 1;
			uint32 bDynamicSubjectPrimitive : 1;
			uint32 bTranslucentSubjectPrimitive : 1;
			uint32 bHeterogeneousVolumeSubjectPrimitive : 1;
			uint32 bNeedUniformBufferUpdate : 1;
			uint32 bNeedPrimitiveFadingStateUpdate : 1;
			uint32 bFadingIn : 1;
			uint32 bAddOnRenderThread : 1;
			uint32 bShouldRecordShadowSubjectsForMobile : 1;
			uint32 bIsLodRange : 1;
			uint32 LodRangeMin : 7;
			uint32 LodRangeMax : 7;

			union
			{
				uint16 MDCOrMeshIndices[2];
				struct 
				{
					uint16 NumMDCIndices;
					uint16 NumMeshIndices;
				};
			};
		};
	};

	void AcceptMDC(int32 NumAcceptedStaticMeshes, int32 MDCIdx, FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer)
	{
		check(NumAcceptedStaticMeshes >= 0 && MDCIdx < MAX_uint16);
		if (NumAcceptedStaticMeshes < 2)
		{
			MDCOrMeshIndices[NumAcceptedStaticMeshes] = uint16(MDCIdx + 1);
			if (bRequestMeshCommandBuild)
			{
				const uint16 Tmp = MDCOrMeshIndices[1];
				MDCOrMeshIndices[1] = MDCOrMeshIndices[0];
				MDCOrMeshIndices[0] = Tmp;
			}
		}
		else
		{
			if (NumAcceptedStaticMeshes == 2)
			{
				HandleOverflow(OverflowBuffer);
			}
			check(bOverflowed);
			OverflowBuffer.MDCIndices.Add(MDCIdx);
			++NumMDCIndices;
		}
		bCopyCachedMeshDrawCommand = true;
	}

	void AcceptMesh(int32 NumAcceptedStaticMeshes, int32 MeshIdx, FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer)
	{
		check(NumAcceptedStaticMeshes >= 0 && MeshIdx < MAX_uint16);
		if (NumAcceptedStaticMeshes < 2)
		{
			MDCOrMeshIndices[NumAcceptedStaticMeshes] = uint16(MeshIdx + 1);
		}
		else
		{
			if (NumAcceptedStaticMeshes == 2)
			{
				HandleOverflow(OverflowBuffer);
			}
			check(bOverflowed);
			OverflowBuffer.MeshIndices.Add(MeshIdx);
			++NumMeshIndices;
		}
		bRequestMeshCommandBuild = true;
	}

	int32 GetMDCIndices(FFinalizeAddSubjectPrimitiveContext& Context, const uint16*& OutMDCIndices, int32& OutIdxBias) const
	{
		int32 NumMDCs;
		OutIdxBias = -1;
		if (bOverflowed)
		{
			OutMDCIndices = Context.OverflowedMDCIndices;
			NumMDCs = NumMDCIndices;
			check(NumMDCs > 0);
			Context.OverflowedMDCIndices += NumMDCs;
			OutIdxBias = 0;
		}
		else
		{
			OutMDCIndices = MDCOrMeshIndices;
			NumMDCs = !MDCOrMeshIndices[1] ? 1 : (!bRequestMeshCommandBuild ? 2 : 1);
		}
		return NumMDCs;
	}

	int32 GetMeshIndices(FFinalizeAddSubjectPrimitiveContext& Context, const uint16*& OutMeshIndices, int32& OutIdxBias) const
	{
		int32 NumMeshes;
		OutIdxBias = -1;
		if (bOverflowed)
		{
			OutMeshIndices = Context.OverflowedMeshIndices;
			NumMeshes = NumMeshIndices;
			check(NumMeshes > 0);
			Context.OverflowedMeshIndices += NumMeshes;
			OutIdxBias = 0;
		}
		else if (!bCopyCachedMeshDrawCommand)
		{
			OutMeshIndices = MDCOrMeshIndices;
			NumMeshes = !MDCOrMeshIndices[1] ? 1 : 2;
		}
		else
		{
			OutMeshIndices = &MDCOrMeshIndices[1];
			NumMeshes = 1;
		}
		return NumMeshes;
	}

	void SetLodRange(FLODMask InLodMask)
	{
		bIsLodRange = InLodMask.IsLODRange();
		LodRangeMin = bIsLodRange ? InLodMask.LODIndex0 : 0;
		LodRangeMax = bIsLodRange ? InLodMask.LODIndex1 : 0;
	}

private:
	void HandleOverflow(FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer)
	{
		if (bCopyCachedMeshDrawCommand && !bRequestMeshCommandBuild)
		{
			OverflowBuffer.MDCIndices.Add(MDCOrMeshIndices[0] - 1);
			OverflowBuffer.MDCIndices.Add(MDCOrMeshIndices[1] - 1);
			NumMDCIndices = 2;
			NumMeshIndices = 0;
		}
		else if (bCopyCachedMeshDrawCommand)
		{
			OverflowBuffer.MDCIndices.Add(MDCOrMeshIndices[0] - 1);
			OverflowBuffer.MeshIndices.Add(MDCOrMeshIndices[1] - 1);
			NumMDCIndices = 1;
			NumMeshIndices = 1;
		}
		else
		{
			check(bRequestMeshCommandBuild);
			OverflowBuffer.MeshIndices.Add(MDCOrMeshIndices[0] - 1);
			OverflowBuffer.MeshIndices.Add(MDCOrMeshIndices[1] - 1);
			NumMDCIndices = 0;
			NumMeshIndices = 2;
		}
		bOverflowed = true;
	}
};

static_assert(sizeof(FAddSubjectPrimitiveResult) == 8, "Unexpected size for FAddSubjectPrimitiveResult");

struct FAddSubjectPrimitiveOp
{
	FPrimitiveSceneInfo* PrimitiveSceneInfo;
	FAddSubjectPrimitiveResult Result;
};

struct FAddSubjectPrimitiveStats
{
	int32 NumCachedMDCCopies;
	int32 NumMDCBuildRequests;
	int32 NumDynamicSubs;
	int32 NumTranslucentSubs;
	int32 NumHeterogeneousVolumeSubs;
	int32 NumDeferredPrimitives;

	FAddSubjectPrimitiveStats()
		: NumCachedMDCCopies(0)
		, NumMDCBuildRequests(0)
		, NumDynamicSubs(0)
		, NumTranslucentSubs(0)
		, NumHeterogeneousVolumeSubs(0)
		, NumDeferredPrimitives(0)
	{}

	void InterlockedAdd(const FAddSubjectPrimitiveStats& Other)
	{
		if (Other.NumCachedMDCCopies > 0)
		{
			FPlatformAtomics::InterlockedAdd(&NumCachedMDCCopies, Other.NumCachedMDCCopies);
		}
		if (Other.NumMDCBuildRequests > 0)
		{
			FPlatformAtomics::InterlockedAdd(&NumMDCBuildRequests, Other.NumMDCBuildRequests);
		}
		if (Other.NumDynamicSubs > 0)
		{
			FPlatformAtomics::InterlockedAdd(&NumDynamicSubs, Other.NumDynamicSubs);
		}
		if (Other.NumTranslucentSubs > 0)
		{
			FPlatformAtomics::InterlockedAdd(&NumTranslucentSubs, Other.NumTranslucentSubs);
		}
		if (Other.NumHeterogeneousVolumeSubs > 0)
		{
			FPlatformAtomics::InterlockedAdd(&NumHeterogeneousVolumeSubs, Other.NumHeterogeneousVolumeSubs);
		}
		if (Other.NumDeferredPrimitives > 0)
		{
			FPlatformAtomics::InterlockedAdd(&NumDeferredPrimitives, Other.NumDeferredPrimitives);
		}
	}
};

void FProjectedShadowInfo::AddCachedMeshDrawCommands_AnyThread(
	const FScene* Scene,
	const FStaticMeshBatchRelevance& RESTRICT StaticMeshRelevance,
	int32 StaticMeshIdx,
	int32& NumAcceptedStaticMeshes,
	FAddSubjectPrimitiveResult& OutResult,
	FAddSubjectPrimitiveStats& OutStats,
	FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer) const
{
	const EMeshPass::Type PassType = GetTargetMeshPassType();
	const EShadingPath ShadingPath = GetFeatureLevelShadingPath(Scene->GetFeatureLevel());
	const bool bUseCachedMeshCommand = UseCachedMeshDrawCommands_AnyThread()
		&& !!(FPassProcessorManager::GetPassFlags(ShadingPath, PassType) & EMeshPassFlags::CachedMeshCommands)
		&& StaticMeshRelevance.bSupportsCachingMeshDrawCommands;

	if (bUseCachedMeshCommand)
	{
		const int32 StaticMeshCommandInfoIndex = StaticMeshRelevance.GetStaticMeshCommandInfoIndex(PassType);
		if (StaticMeshCommandInfoIndex >= 0)
		{
			++OutStats.NumCachedMDCCopies;
			OutResult.AcceptMDC(NumAcceptedStaticMeshes++, StaticMeshCommandInfoIndex, OverflowBuffer);
		}
	}
	else
	{
		++OutStats.NumMDCBuildRequests;
		OutResult.AcceptMesh(NumAcceptedStaticMeshes++, StaticMeshIdx, OverflowBuffer);
	}
}

FLODMask FProjectedShadowInfo::CalcAndUpdateLODToRender(FViewInfo& CurrentView, const FBoxSphereBounds& Bounds, const FPrimitiveSceneInfo* PrimitiveSceneInfo, int32 ForcedLOD) const
{
	const int32 PrimitiveId = PrimitiveSceneInfo->GetIndex();

	// must match logic in FProjectedShadowInfo::ModifyViewForShadow(...)
	const float ShadowLODDistanceFactor = GetLODDistanceFactor();
	const bool bNeedsIndependentLODMask = ShadowLODDistanceFactor != 1.0f
		// Must always trigger the culling calculations for this case as they compute both the LOD mask and culling result.
		|| (SceneCullingRendererCullingQuery && SceneCullingRendererCullingQuery->HasCullingInfo(PrimitiveId));

	FLODMask ShadowLODToRender = CurrentView.PrimitivesLODMask[PrimitiveId];
	// calculate it it's not set OR if LOD is overridden
	if (ForcedLOD > -1 || !ShadowLODToRender.IsValid() || bNeedsIndependentLODMask)
	{
		const float LODScale = ShadowLODDistanceFactor * CurrentView.LODDistanceFactor * GetCachedScalabilityCVars().StaticMeshLODDistanceScale;
		ShadowLODToRender = ComputePrimitiveMeshLODs(CurrentView, ForcedLOD, LODScale, PrimitiveSceneInfo->Proxy, PrimitiveSceneInfo, PrimitiveId, Bounds, SceneCullingRendererCullingQuery);

		// TODO: support caching when ShadowLODDistanceFactorEnabled (cascades of the same type (regular/far) could reuse results)
		if (!bNeedsIndependentLODMask)
		{
			CurrentView.PrimitivesLODMask[PrimitiveId] = ShadowLODToRender;
		}
	}

	// Use lowest LOD for PreShadow
	if (bPreShadow && GPreshadowsForceLowestLOD)
	{
		int8 LODToRenderScan = -MAX_int8;

		for (int32 Index = 0; Index < PrimitiveSceneInfo->StaticMeshRelevances.Num(); Index++)
		{
			LODToRenderScan = FMath::Max<int8>(PrimitiveSceneInfo->StaticMeshRelevances[Index].GetLODIndex(), LODToRenderScan);
		}
		if (LODToRenderScan != -MAX_int8)
		{
			ShadowLODToRender.SetLOD(LODToRenderScan);
		}
	}

	if (CascadeSettings.bFarShadowCascade)
	{
		if (!ShadowLODToRender.IsLODRange()) // todo: GPU LOD doesn't support this bias.
		{
			extern ENGINE_API int32 GFarShadowStaticMeshLODBias;
			int8 LODToRenderScan = ShadowLODToRender.LODIndex0 + GFarShadowStaticMeshLODBias;

			for (int32 Index = PrimitiveSceneInfo->StaticMeshRelevances.Num() - 1; Index >= 0; Index--)
			{
				if (LODToRenderScan == PrimitiveSceneInfo->StaticMeshRelevances[Index].GetLODIndex())
				{
					ShadowLODToRender.SetLOD(LODToRenderScan);
					break;
				}
			}
		}
	}
	return ShadowLODToRender;
}

bool FProjectedShadowInfo::ShouldUseCSMScissorOptim() const
{
	return CVarCSMScissorOptim.GetValueOnRenderThread() != 0 && bWholeSceneShadow && bDirectionalLight && !bOnePassPointLightShadow && !bRayTracedDistanceField;
}

FORCEINLINE bool FProjectedShadowInfo::ShouldDrawStaticMesh(const FStaticMeshBatchRelevance& StaticMeshRelevance, const FLODMask& ShadowLODToRender, bool& bOutDrawingStaticMeshes) const
{
	if ((StaticMeshRelevance.CastShadow || (bSelfShadowOnly && StaticMeshRelevance.bUseForDepthPass)) && ShadowLODToRender.ContainsLOD(StaticMeshRelevance.GetLODIndex()))
	{
		bOutDrawingStaticMeshes = true;

		if (EnumHasAnyFlags(MeshSelectionMask, StaticMeshRelevance.bSupportsGPUScene ? EShadowMeshSelection::VSM : EShadowMeshSelection::SM))
		{
			return true;
		}
	}
	return false;
}


bool FProjectedShadowInfo::ShouldDrawStaticMeshes(FViewInfo& InCurrentView, FPrimitiveSceneInfo* InPrimitiveSceneInfo)
{
	bool bDrawingStaticMeshes = false;
	const int32 PrimitiveId = InPrimitiveSceneInfo->GetIndex();
	const FMeshDrawCommandPrimitiveIdInfo PrimitiveIdInfo = InPrimitiveSceneInfo->GetMDCIdInfo();
	const int32 ForcedLOD = (InCurrentView.Family->EngineShowFlags.LOD) ? (GetCVarForceLODShadow() != -1 ? GetCVarForceLODShadow() : GetCVarForceLOD()) : -1;
	FLODMask ShadowLODToRender = CalcAndUpdateLODToRender(InCurrentView, InPrimitiveSceneInfo->Proxy->GetBounds(), InPrimitiveSceneInfo, ForcedLOD);
	const EShadowDepthType ShadowDepthType = GetShadowDepthType();

	// Don't cache if it requires per view per mesh state for distance cull fade.
	const bool bIsPrimitiveDistanceCullFading = InCurrentView.PotentiallyFadingPrimitiveMap[PrimitiveId];
	const bool bCanCache = !bIsPrimitiveDistanceCullFading;

	if (bCanCache && UseCachedMeshDrawCommands(ShadowDepthType))
	{
		const EMeshPass::Type MeshPassType = GetShadowMeshPassType(ShadowDepthType);

		for (int32 MeshIndex = 0; MeshIndex < InPrimitiveSceneInfo->StaticMeshRelevances.Num(); MeshIndex++)
		{
			const FStaticMeshBatchRelevance& StaticMeshRelevance = InPrimitiveSceneInfo->StaticMeshRelevances[MeshIndex];
			const FStaticMeshBatch& StaticMesh = InPrimitiveSceneInfo->StaticMeshes[MeshIndex];

			if (ShouldDrawStaticMesh(StaticMeshRelevance, ShadowLODToRender, bDrawingStaticMeshes))
			{
				const EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags = GetCullingPayloadFlags(ShadowLODToRender, StaticMeshRelevance.GetLODIndex());

				AddCachedMeshDrawCommandsForPass(
					PrimitiveIdInfo,
					InPrimitiveSceneInfo,
					StaticMeshRelevance,
					StaticMesh,
					CullingPayloadFlags,
					InPrimitiveSceneInfo->Scene,
					MeshPassType,
					ShadowDepthPassVisibleCommands,
					SubjectMeshCommandBuildRequests,
					SubjectMeshCommandBuildFlags,
					NumSubjectMeshCommandBuildRequestElements);
			}
		}
	}
	else
	{
		for (int32 MeshIndex = 0; MeshIndex < InPrimitiveSceneInfo->StaticMeshRelevances.Num(); MeshIndex++)
		{
			const FStaticMeshBatchRelevance& StaticMeshRelevance = InPrimitiveSceneInfo->StaticMeshRelevances[MeshIndex];
			const FStaticMeshBatch& StaticMesh = InPrimitiveSceneInfo->StaticMeshes[MeshIndex];

			if (ShouldDrawStaticMesh(StaticMeshRelevance, ShadowLODToRender, bDrawingStaticMeshes))
			{
				NumSubjectMeshCommandBuildRequestElements += StaticMeshRelevance.NumElements;
				SubjectMeshCommandBuildRequests.Add(&StaticMesh);
				SubjectMeshCommandBuildFlags.Add(GetCullingPayloadFlags(ShadowLODToRender, StaticMeshRelevance.GetLODIndex()));
			}
		}
	}

	return bDrawingStaticMeshes;
}

bool FProjectedShadowInfo::ShouldDrawStaticMeshes_AnyThread(
	FViewInfo& CurrentView,
	const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact,
	bool bMayBeFading,
	FAddSubjectPrimitiveResult& OutResult,
	FAddSubjectPrimitiveStats& OutStats,
	FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer) const
{
	bool bDrawingStaticMeshes = false;
	const bool WholeSceneDirectionalShadow = IsWholeSceneDirectionalShadow();
	const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneInfoCompact.PrimitiveSceneInfo;
	const FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfoCompact.Proxy;
	const EShadowDepthType ShadowDepthType = GetShadowDepthType();

	{
		const int32 ForcedLOD = CurrentView.Family->EngineShowFlags.LOD ? (GetCVarForceLODShadow_AnyThread() != -1 ? GetCVarForceLODShadow_AnyThread() : GetCVarForceLOD_AnyThread()) : -1;

		FLODMask ShadowLODToRender = CalcAndUpdateLODToRender(CurrentView, FBoxSphereBounds(PrimitiveSceneInfoCompact.Bounds), PrimitiveSceneInfo, ForcedLOD);
		OutResult.SetLodRange(ShadowLODToRender);

		// Don't cache if it requires per view per mesh state for distance cull fade.
		const bool bCanCache = !bMayBeFading;

		if (bCanCache && UseCachedMeshDrawCommands(ShadowDepthType))
		{
			int32 NumAcceptedStaticMeshes = 0;

			for (int32 MeshIndex = 0; MeshIndex < PrimitiveSceneInfo->StaticMeshRelevances.Num(); MeshIndex++)
			{
				const FStaticMeshBatchRelevance& StaticMeshRelevance = PrimitiveSceneInfo->StaticMeshRelevances[MeshIndex];
				const FStaticMeshBatch& StaticMesh = PrimitiveSceneInfo->StaticMeshes[MeshIndex];

				if (ShouldDrawStaticMesh(StaticMeshRelevance, ShadowLODToRender, bDrawingStaticMeshes))
				{
					AddCachedMeshDrawCommands_AnyThread(PrimitiveSceneInfo->Scene, StaticMeshRelevance, MeshIndex, NumAcceptedStaticMeshes, OutResult, OutStats, OverflowBuffer);
				}
			}
		}
		else
		{
			int32 NumAcceptedStaticMeshes = 0;

			for (int32 MeshIndex = 0; MeshIndex < PrimitiveSceneInfo->StaticMeshRelevances.Num(); MeshIndex++)
			{
				const FStaticMeshBatchRelevance& StaticMeshRelevance = PrimitiveSceneInfo->StaticMeshRelevances[MeshIndex];
				const FStaticMeshBatch& StaticMesh = PrimitiveSceneInfo->StaticMeshes[MeshIndex];

				if (ShouldDrawStaticMesh(StaticMeshRelevance, ShadowLODToRender, bDrawingStaticMeshes))
				{
					check(MeshIndex < MAX_uint16);
					++OutStats.NumMDCBuildRequests;
					OutResult.AcceptMesh(NumAcceptedStaticMeshes++, MeshIndex, OverflowBuffer);
				}
			}
		}
	}

	return bDrawingStaticMeshes;
}

bool FProjectedShadowInfo::AddSubjectPrimitive(FPrimitiveSceneInfo* PrimitiveSceneInfo, TArrayView<FViewInfo> ViewArray, bool bShouldRecordShadowSubjectsForMobileShading, bool bInvertViewShadowRelevance)
{
	// Ray traced shadows use the GPU managed distance field object buffers, no CPU culling should be used
	check(!bRayTracedDistanceField);
	bool bWasAdded = false;
	int32 PrimitiveId = INDEX_NONE;
	if (!ReceiverPrimitives.Contains(PrimitiveSceneInfo)
		// Far cascade only casts from primitives marked for it
		&& TestPrimitiveFarCascadeConditions(PrimitiveSceneInfo->Proxy->CastsFarShadow(), PrimitiveSceneInfo->Proxy->GetBounds()))
	{
		const FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy;

		TArrayView<FViewInfo> Views;
		const bool bWholeSceneDirectionalShadow = IsWholeSceneDirectionalShadow();

		if (bWholeSceneDirectionalShadow)
		{
			Views = TArrayView<FViewInfo>(DependentView, 1);
		}
		else if (SelectedViewIndex != INDEX_NONE)
		{
			Views = TArrayView<FViewInfo>(&ViewArray[SelectedViewIndex], 1);
		}
		else
		{
			checkf(ViewArray.Num(),
				TEXT("bWholeSceneShadow=%s, CascadeSettings.ShadowSplitIndex=%d, bDirectionalLight=%s"),
				bWholeSceneShadow ? TEXT("true") : TEXT("false"),
				CascadeSettings.ShadowSplitIndex,
				bDirectionalLight ? TEXT("true") : TEXT("false"));

			Views = ViewArray;
		}

		bool bOpaque = false;
		bool bTranslucentRelevance = false;
		bool bShadowRelevance = false;
		bool bDynamicRelevance = false;
		bool bHeterogeneousVolumeRelevance = false;

		PrimitiveId = PrimitiveSceneInfo->GetIndex();

		for (FViewInfo& CurrentView : Views)
		{
			FPrimitiveViewRelevance& ViewRelevance = CurrentView.PrimitiveViewRelevanceMap[PrimitiveId];

			if (!ViewRelevance.bInitializedThisFrame)
			{
				if( CurrentView.IsPerspectiveProjection() )
				{
					// Compute the distance between the view and the primitive.
					float DistanceSquared = (Proxy->GetBounds().Origin - CurrentView.GetShadowViewMatrices().GetViewOrigin()).SizeSquared();

					bool bIsDistanceCulled = CurrentView.IsDistanceCulled(
						DistanceSquared,
						Proxy->GetMinDrawDistance(),
						Proxy->GetMaxDrawDistance(),
						PrimitiveSceneInfo
						);
					if( bIsDistanceCulled )
					{
						continue;
					}
				}

				// Respect HLOD visibility which can hide child LOD primitives
				if (CurrentView.ViewState &&
					CurrentView.ViewState->HLODVisibilityState.IsValidPrimitiveIndex(PrimitiveId) &&
					CurrentView.ViewState->HLODVisibilityState.IsNodeForcedHidden(PrimitiveId))
				{
					continue;
				}

				if ((CurrentView.ShowOnlyPrimitives.IsSet() &&
					!CurrentView.ShowOnlyPrimitives->Contains(PrimitiveSceneInfo->Proxy->GetPrimitiveComponentId())) ||
					CurrentView.HiddenPrimitives.Contains(PrimitiveSceneInfo->Proxy->GetPrimitiveComponentId()))
				{
					continue;
				}

				// Compute the subject primitive's view relevance since it wasn't cached
				// Update the main view's PrimitiveViewRelevanceMap
				ViewRelevance = PrimitiveSceneInfo->Proxy->GetViewRelevance(&CurrentView);
			}

			bOpaque |= ViewRelevance.bOpaque || ViewRelevance.bMasked;
			bTranslucentRelevance |= ViewRelevance.HasTranslucency();
			bShadowRelevance |= bInvertViewShadowRelevance ? bool(!ViewRelevance.bShadowRelevance) : bool(ViewRelevance.bShadowRelevance);
			bDynamicRelevance = bDynamicRelevance || ViewRelevance.bDynamicRelevance;
			bHeterogeneousVolumeRelevance = ViewRelevance.bHasVolumeMaterialDomain && PrimitiveSceneInfo->Proxy->IsHeterogeneousVolume();
		}

		if (bShadowRelevance)
		{
			// Update the primitive component's last render time. Allows the component to update when using bCastWhenHidden.
			const float CurrentWorldTime = Views[0].Family->Time.GetWorldTimeSeconds();
			PrimitiveSceneInfo->UpdateComponentLastRenderTime(CurrentWorldTime, /*bUpdateLastRenderTimeOnScreen=*/false);
		}

		if (bOpaque && bShadowRelevance)
		{
			const FBoxSphereBounds& Bounds = Proxy->GetBounds();
			bool bDrawingStaticMeshes = false;

			if (PrimitiveSceneInfo->StaticMeshes.Num() > 0)
			{
				for (FViewInfo& CurrentView : Views)
				{
					if (!CurrentView.ShouldRenderView())
					{
						continue;
					}
					// Note: skip small-mesh culling for VSM since it needs it drawn for GPU-side caching.
					if (bWholeSceneShadow && CacheMode != SDCM_StaticPrimitivesOnly && !bVSM)
					{
						const float DistanceSquared = ( Bounds.Origin - CurrentView.GetShadowViewMatrices().GetViewOrigin() ).SizeSquared();
						const float LODScaleSquared = FMath::Square(CurrentView.LODDistanceFactor);
						const bool bDrawShadowDepth = FMath::Square(Bounds.SphereRadius) > FMath::Square(GMinScreenRadiusForShadowCaster) * DistanceSquared * LODScaleSquared;
						if( !bDrawShadowDepth )
						{
							// cull object if it's too small to be considered as shadow caster
							continue;
						}
					}

					if (CurrentView.PrimitiveViewRelevanceMap[PrimitiveId].bStaticRelevance)
					{
						bDrawingStaticMeshes |= ShouldDrawStaticMeshes(CurrentView, PrimitiveSceneInfo);
					}
				}
			}

			if (bDrawingStaticMeshes)
			{
				if (bShouldRecordShadowSubjectsForMobileShading)
				{
					DependentView->VisibleLightInfos[GetLightSceneInfo().Id].MobileCSMSubjectPrimitives.AddSubjectPrimitive(PrimitiveSceneInfo, PrimitiveId);
				}
				bWasAdded = true;
			}
			// EShadowMeshSelection::All is intentional because ProxySupportsGPUScene being false may indicate that some VFs (e.g., some LODs) support GPUScene while others don't
			// thus we have to leave the final decision until the mesh batches are produced.
			else if (bDynamicRelevance && EnumHasAnyFlags(MeshSelectionMask, Proxy->SupportsGPUScene() ? EShadowMeshSelection::VSM : EShadowMeshSelection::All))
			{
				// Add the primitive to the subject primitive list.
				DynamicSubjectPrimitives.Add(PrimitiveSceneInfo);

				if (bShouldRecordShadowSubjectsForMobileShading)
				{
					DependentView->VisibleLightInfos[GetLightSceneInfo().Id].MobileCSMSubjectPrimitives.AddSubjectPrimitive(PrimitiveSceneInfo, PrimitiveId);
				}
				bWasAdded = true;
			}
		}

		// Add translucent shadow casting primitives to SubjectTranslucentPrimitives
		if (bTranslucentShadow && bTranslucentRelevance && bShadowRelevance)
		{
			SubjectTranslucentPrimitives.Add(PrimitiveSceneInfo);
			bWasAdded = true;
		}

		if (bVolumetricShadow && bHeterogeneousVolumeRelevance)
		{
			SubjectHeterogeneousVolumePrimitives.Add(PrimitiveSceneInfo);
			bWasAdded = true;
		}
	}
	return bWasAdded;
}

bool FProjectedShadowInfo::TestPrimitiveFarCascadeConditions(bool bPrimitiveCastsFarShadow, const FBoxSphereBounds& Bounds) const
{
	if (CascadeSettings.bFarShadowCascade)
	{
		return bPrimitiveCastsFarShadow;
	}

	if (bVSM && IsWholeSceneDirectionalShadow() && MaxNonFarCascadeDistance > 0.0f)
	{
		check(DependentView);

		FVector ViewOrigin = DependentView->GetShadowViewMatrices().GetViewOrigin();
		float ViewDistance = (Bounds.Origin - ViewOrigin) | DependentView->GetViewDirection();
		if (ViewDistance - Bounds.SphereRadius > MaxNonFarCascadeDistance)
		{
			return bPrimitiveCastsFarShadow;
		}
	}
	return true;
}

uint64 FProjectedShadowInfo::AddSubjectPrimitive_AnyThread(
	const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact,
	TArrayView<FViewInfo> ViewArray,
	ERHIFeatureLevel::Type FeatureLevel,
	FAddSubjectPrimitiveStats& OutStats,
	FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer) const
{
	// Ray traced shadows use the GPU managed distance field object buffers, no CPU culling should be used
	check(!bRayTracedDistanceField);

	FAddSubjectPrimitiveResult Result;
	Result.Qword = 0;

	if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Mobile)
	{
		const bool bShouldRecordShadowSubjectsForMobile = GetLightSceneInfo().ShouldRecordShadowSubjectsForMobile();

		if (bShouldRecordShadowSubjectsForMobile)
		{
			Result.bAddOnRenderThread = true;
			Result.bShouldRecordShadowSubjectsForMobile = true;
			++OutStats.NumDeferredPrimitives;
			return Result.Qword;
		}
	}

	FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneInfoCompact.PrimitiveSceneInfo;

	if (!ReceiverPrimitives.Contains(PrimitiveSceneInfo)
		// Far cascade only casts from primitives marked for it
		&& TestPrimitiveFarCascadeConditions(PrimitiveSceneInfoCompact.Proxy->CastsFarShadow(), FBoxSphereBounds(PrimitiveSceneInfoCompact.Bounds)))
	{
		FViewInfo* CurrentView;
		const bool bWholeSceneDirectionalShadow = IsWholeSceneDirectionalShadow();

		if (bWholeSceneDirectionalShadow)
		{
			CurrentView = DependentView;
		}
		else if (SelectedViewIndex != INDEX_NONE)
		{
			CurrentView = &ViewArray[SelectedViewIndex];
		}
		else
		{
			checkf(ViewArray.Num(),
				TEXT("bWholeSceneShadow=%s, CascadeSettings.ShadowSplitIndex=%d, bDirectionalLight=%s"),
				bWholeSceneShadow ? TEXT("true") : TEXT("false"),
				CascadeSettings.ShadowSplitIndex,
				bDirectionalLight ? TEXT("true") : TEXT("false"));

			if (ViewArray.Num() > 1)
			{
				Result.bAddOnRenderThread = true;
				++OutStats.NumDeferredPrimitives;
				return Result.Qword;
			}

			CurrentView = &ViewArray[0];
		}

		bool bOpaque = false;
		bool bTranslucentRelevance = false;
		bool bShadowRelevance = false;
		bool bStaticRelevance = false;
		bool bMayBeFading = false;
		bool bDynamicRelevance = false;
		bool bHeterogeneousVolumeRelevance = false;

		const int32 PrimitiveId = PrimitiveSceneInfo->GetIndex();
		FPrimitiveViewRelevance& ViewRelevance = CurrentView->PrimitiveViewRelevanceMap[PrimitiveId];

		if (!ViewRelevance.bInitializedThisFrame)
		{
			if (CurrentView->IsPerspectiveProjection())
			{
				bool bFadingIn;
				// Compute the distance between the view and the primitive.
				const float DistanceSquared = (PrimitiveSceneInfoCompact.Bounds.Origin - CurrentView->GetShadowViewMatrices().GetViewOrigin()).SizeSquared();

				if (CurrentView->IsDistanceCulled_AnyThread(
					DistanceSquared,
					PrimitiveSceneInfoCompact.MinDrawDistance,
					PrimitiveSceneInfoCompact.MaxDrawDistance,
					PrimitiveSceneInfo,
					bMayBeFading,
					bFadingIn))
				{
					return 0;
				}

				if (bMayBeFading)
				{
					Result.bNeedPrimitiveFadingStateUpdate = true;
					Result.bFadingIn = bFadingIn;
				}
			}

			// Respect HLOD visibility which can hide child LOD primitives
			if (CurrentView->ViewState &&
				CurrentView->ViewState->HLODVisibilityState.IsValidPrimitiveIndex(PrimitiveId) &&
				CurrentView->ViewState->HLODVisibilityState.IsNodeForcedHidden(PrimitiveId))
			{
				return 0;
			}

			if ((CurrentView->ShowOnlyPrimitives.IsSet() &&
				!CurrentView->ShowOnlyPrimitives->Contains(PrimitiveSceneInfoCompact.Proxy->GetPrimitiveComponentId())) ||
				CurrentView->HiddenPrimitives.Contains(PrimitiveSceneInfoCompact.Proxy->GetPrimitiveComponentId()))
			{
				return 0;
			}

			// Compute the subject primitive's view relevance since it wasn't cached
			// Update the main view's PrimitiveViewRelevanceMap
			ViewRelevance = PrimitiveSceneInfoCompact.Proxy->GetViewRelevance(CurrentView);
		}

		bOpaque = ViewRelevance.bOpaque || ViewRelevance.bMasked;
		bTranslucentRelevance = ViewRelevance.HasTranslucency();
		bShadowRelevance = ViewRelevance.bShadowRelevance;
		bStaticRelevance = ViewRelevance.bStaticRelevance;
		bDynamicRelevance = ViewRelevance.bDynamicRelevance;
		bHeterogeneousVolumeRelevance = ViewRelevance.bHasVolumeMaterialDomain && PrimitiveSceneInfo->Proxy->IsHeterogeneousVolume();

		if (!bShadowRelevance)
		{
			return 0;
		}

		// Update the primitive component's last render time. Allows the component to update when using bCastWhenHidden.
		const float CurrentWorldTime = CurrentView->Family->Time.GetWorldTimeSeconds();
		PrimitiveSceneInfo->UpdateComponentLastRenderTime(CurrentWorldTime, /*bUpdateLastRenderTimeOnScreen=*/false);

		if (bOpaque)
		{
			bool bDrawingStaticMeshes = false;

			if (PrimitiveSceneInfo->StaticMeshes.Num() > 0)
			{
				if (bWholeSceneShadow)
				{
					const FCompactBoxSphereBounds& Bounds = PrimitiveSceneInfoCompact.Bounds;
					const float DistanceSquared = (Bounds.Origin - CurrentView->GetShadowViewMatrices().GetViewOrigin()).SizeSquared();
					const float LODScaleSquared = FMath::Square(CurrentView->LODDistanceFactor);
					const bool bDrawShadowDepth = FMath::Square(Bounds.SphereRadius) > FMath::Square(GMinScreenRadiusForShadowCaster) * DistanceSquared * LODScaleSquared;
					if (!bDrawShadowDepth)
					{
						// cull object if it's too small to be considered as shadow caster
						return 0;
					}
				}

				if (bStaticRelevance)
				{
					bDrawingStaticMeshes |= ShouldDrawStaticMeshes_AnyThread(
						*CurrentView,
						PrimitiveSceneInfoCompact,
						bMayBeFading,
						Result,
						OutStats,
						OverflowBuffer);
				}
			}

			// EShadowMeshSelection::All is intentional because bSupportsGPUScene being false may indicate that some VFs (e.g., some LODs) support GPUScene while others don't
			// thus we have to leave the final decision until the mesh batches are produced.
			if (!bDrawingStaticMeshes && bDynamicRelevance && EnumHasAnyFlags(MeshSelectionMask, PrimitiveSceneInfoCompact.PrimitiveFlagsCompact.bSupportsGPUScene ? EShadowMeshSelection::VSM : EShadowMeshSelection::All))
			{
				Result.bDynamicSubjectPrimitive = true;
				++OutStats.NumDynamicSubs;
			}
		}

		if (bTranslucentShadow && bTranslucentRelevance)
		{
			Result.bTranslucentSubjectPrimitive = true;
			++OutStats.NumTranslucentSubs;
		}

		if (bVolumetricShadow && bHeterogeneousVolumeRelevance)
		{
			Result.bHeterogeneousVolumeSubjectPrimitive = true;
			++OutStats.NumHeterogeneousVolumeSubs;
		}
	}

	return Result.Qword;
}

void FProjectedShadowInfo::PresizeSubjectPrimitiveArrays(const FAddSubjectPrimitiveStats& Stats)
{
	ShadowDepthPassVisibleCommands.Reserve(ShadowDepthPassVisibleCommands.Num() + Stats.NumDeferredPrimitives * 2 + Stats.NumCachedMDCCopies);
	SubjectMeshCommandBuildRequests.Reserve(SubjectMeshCommandBuildRequests.Num() + Stats.NumMDCBuildRequests);
	SubjectMeshCommandBuildFlags.Reserve(SubjectMeshCommandBuildFlags.Num() + Stats.NumMDCBuildRequests);
	DynamicSubjectPrimitives.Reserve(DynamicSubjectPrimitives.Num() + Stats.NumDeferredPrimitives + Stats.NumDynamicSubs);
	SubjectTranslucentPrimitives.Reserve(SubjectTranslucentPrimitives.Num() + Stats.NumTranslucentSubs);
	SubjectHeterogeneousVolumePrimitives.Reserve(SubjectHeterogeneousVolumePrimitives.Num() + Stats.NumHeterogeneousVolumeSubs);
}

void FProjectedShadowInfo::FinalizeAddSubjectPrimitive(
	FDynamicShadowsTaskData& TaskData,
	const FAddSubjectPrimitiveOp& Op,
	TArrayView<FViewInfo> ViewArray,
	FFinalizeAddSubjectPrimitiveContext& Context)
{
	FPrimitiveSceneInfo* PrimitiveSceneInfo = Op.PrimitiveSceneInfo;
	const FAddSubjectPrimitiveResult& Result = Op.Result;

	if (CacheMode != SDCM_Uncached && IsWholeSceneDirectionalShadow())
	{
		checkSlow(!PrimitiveSceneInfo->Proxy->IsMeshShapeOftenMoving() || CacheMode == SDCM_CSMScrolling || CacheMode == SDCM_MovablePrimitivesOnly);

		FCachedShadowMapData& CachedShadowMapData = PrimitiveSceneInfo->Scene->GetExtension<FCSMSceneExtension>().GetCachedShadowMapDataRef(LightSceneInfo->Id, CascadeSettings.ShadowSplitIndex);

		if (CacheMode == SDCM_StaticPrimitivesOnly)
		{
			// Record all static meshes in the cached shadow map.
			CachedShadowMapData.StaticShadowSubjectPersistentPrimitiveIdMap[PrimitiveSceneInfo->GetPersistentIndex().Index] = true;
		}
		else if (!PrimitiveSceneInfo->Proxy->IsMeshShapeOftenMoving())
		{
			// Count the number of extra draw calls of static meshes for filling the scrolling area.
			FPlatformAtomics::InterlockedAdd(&CachedShadowMapData.LastFrameExtraStaticShadowSubjects, 1);
		}
	}

	if (Result.bAddOnRenderThread)
	{
		if (AddSubjectPrimitive(PrimitiveSceneInfo, ViewArray, Result.bShouldRecordShadowSubjectsForMobile) && VirtualShadowMapClipmap.IsValid())
		{
			VirtualShadowMapClipmap->OnPrimitiveRendered(PrimitiveSceneInfo);
		}
		return;
	}

	// If we have a clipmap, we need to track the state of the primitive such that we may invalidate pages when the culling changes (e.g., the primitive crosses a range boundary)
	// Do this after the bAddOnRenderThread -> AddSubjectPrimitive return to avoid double-processing in the event that bAddOnRenderThread is set.
	if (VirtualShadowMapClipmap.IsValid())
	{
		VirtualShadowMapClipmap->OnPrimitiveRendered(PrimitiveSceneInfo);
	}

	if (Result.bNeedPrimitiveFadingStateUpdate)
	{
		FViewInfo& View = IsWholeSceneDirectionalShadow() ? *DependentView : ViewArray[0];
		if (View.UpdatePrimitiveFadingState(PrimitiveSceneInfo, Result.bFadingIn))
		{
			if (Result.bOverflowed)
			{
				Context.OverflowedMDCIndices += Result.NumMDCIndices;
				Context.OverflowedMeshIndices += Result.NumMeshIndices;
			}
			return;
		}
	}

	if (Result.bCopyCachedMeshDrawCommand)
	{
		check(!Result.bDynamicSubjectPrimitive);
		const uint16* MDCIndices;
		int32 IdxBias;
		int32 NumMDCs = Result.GetMDCIndices(Context, MDCIndices, IdxBias);

		const EMeshPass::Type MeshPassType = GetShadowMeshPassType(GetShadowDepthType());

		for (int32 Idx = 0; Idx < NumMDCs; ++Idx)
		{
			const int32 CmdIdx = (int32)MDCIndices[Idx] + IdxBias;
			const FCachedMeshDrawCommandInfo& CmdInfo = PrimitiveSceneInfo->StaticMeshCommandInfos[CmdIdx];
			const FScene* Scene = PrimitiveSceneInfo->Scene;
			const FMeshDrawCommand* CachedCmd = CmdInfo.StateBucketId >= 0 ?
				&Scene->CachedMeshDrawCommandStateBuckets[MeshPassType].GetByElementId(CmdInfo.StateBucketId).Key :
				&Scene->CachedDrawLists[MeshPassType].MeshDrawCommands[CmdInfo.CommandIndex];
			const EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags = GetCullingPayloadFlags(Result.bIsLodRange, CmdInfo.CullingPayload.LodIndex == Result.LodRangeMin, CmdInfo.CullingPayload.LodIndex == Result.LodRangeMax);

			const FMeshDrawCommandPrimitiveIdInfo PrimitiveIdInfo = PrimitiveSceneInfo->GetMDCIdInfo();
			FVisibleMeshDrawCommand& VisibleCmd = ShadowDepthPassVisibleCommands[ShadowDepthPassVisibleCommands.AddUninitialized()];
			VisibleCmd.Setup(
				CachedCmd, 
				PrimitiveIdInfo, 
				CmdInfo.StateBucketId, 
				CmdInfo.MeshFillMode, 
				CmdInfo.MeshCullMode, 
				CmdInfo.Flags, 
				CmdInfo.SortKey, 
				CmdInfo.CullingPayload,
				CullingPayloadFlags);
		}
	}

	if (Result.bRequestMeshCommandBuild)
	{
		check(!Result.bDynamicSubjectPrimitive);
		const uint16* MeshIndices;
		int32 IdxBias;
		int32 NumMeshes = Result.GetMeshIndices(Context, MeshIndices, IdxBias);

		for (int32 Idx = 0; Idx < NumMeshes; ++Idx)
		{
			const int32 MeshIdx = (int32)MeshIndices[Idx] + IdxBias;
			const FStaticMeshBatchRelevance& MeshRelevance = PrimitiveSceneInfo->StaticMeshRelevances[MeshIdx];
			const FStaticMeshBatch& MeshBatch = PrimitiveSceneInfo->StaticMeshes[MeshIdx];

			NumSubjectMeshCommandBuildRequestElements += MeshRelevance.NumElements;
			SubjectMeshCommandBuildRequests.Add(&MeshBatch);
			SubjectMeshCommandBuildFlags.Add(GetCullingPayloadFlags(Result.bIsLodRange, MeshRelevance.GetLODIndex() == Result.LodRangeMin, MeshRelevance.GetLODIndex() == Result.LodRangeMax));
		}
	}

	if (Result.bDynamicSubjectPrimitive)
	{
		DynamicSubjectPrimitives.Add(PrimitiveSceneInfo);
	}

	if (Result.bTranslucentSubjectPrimitive)
	{
		SubjectTranslucentPrimitives.Add(PrimitiveSceneInfo);
	}

	if (Result.bHeterogeneousVolumeSubjectPrimitive)
	{
		SubjectHeterogeneousVolumePrimitives.Add(PrimitiveSceneInfo);
	}
}

bool FProjectedShadowInfo::HasSubjectPrims() const
{
	return DynamicSubjectPrimitives.Num() > 0
		|| SubjectTranslucentPrimitives.Num() > 0
		|| SubjectHeterogeneousVolumePrimitives.Num() > 0
		|| ShadowDepthPass.HasAnyDraw()
		|| SubjectMeshCommandBuildRequests.Num() > 0
		|| ShadowDepthPassVisibleCommands.Num() > 0
		|| (bContainsNaniteSubjects && bNaniteGeometry);
}

void FProjectedShadowInfo::AddReceiverPrimitive(FPrimitiveSceneInfo* PrimitiveSceneInfo)
{
	// Add the primitive to the receiver primitive list.
	ReceiverPrimitives.Add(PrimitiveSceneInfo);
}

FCullingVolume FProjectedShadowInfo::GetCullingVolume() const
{
	FCullingVolume CullingVolume;
	CullingVolume.WorldToVolumeTranslation = PreShadowTranslation;

	if (VirtualShadowMapClipmap)
	{
		const bool bIsCached = VirtualShadowMapClipmap->GetCacheEntry() && !VirtualShadowMapClipmap->GetCacheEntry()->IsUncached();

		// We can only do this culling if the light is both uncached & it is using the accurate bounds (i.e., r.Shadow.Virtual.Clipmap.UseConservativeCulling is turned off).
		if (!bIsCached && !CascadeSettings.ShadowBoundsAccurate.Planes.IsEmpty())
		{
			CullingVolume.ConvexVolume = CascadeSettings.ShadowBoundsAccurate;
			// ShadowBoundsAccurate is in world-space
			CullingVolume.WorldToVolumeTranslation = FVector3d::ZeroVector;
		}
		else
		{
			CullingVolume.Sphere = VirtualShadowMapClipmap->GetBoundingSphere();
			CullingVolume.ConvexVolume = VirtualShadowMapClipmap->GetViewFrustumBounds();
		}
	}
	else if (!CascadeSettings.ShadowBoundsAccurate.Planes.IsEmpty())
	{
		CullingVolume.ConvexVolume = CascadeSettings.ShadowBoundsAccurate;
		// ShadowBoundsAccurate is in world-space
		CullingVolume.WorldToVolumeTranslation = FVector3d::ZeroVector;
	}
	else
	{
		CullingVolume.Sphere = GetLightSceneInfo().Proxy->GetBoundingSphere();
		CullingVolume.ConvexVolume = CasterOuterFrustum;
	}
	return CullingVolume;
}

void FProjectedShadowInfo::SetupCullingQuery(FSceneRenderer& SceneRenderer)
{
	if (FSceneCullingRenderer* SceneCullingRenderer = SceneRenderer.GetSceneExtensionsRenderers().GetRendererPtr<FSceneCullingRenderer>())
	{
		if (SceneCullingRenderer->UseHierarchicalCPUCulling())
		{
			check(DependentView || SelectedViewIndex != INDEX_NONE);
			const FViewInfo& View = DependentView ? *DependentView : SceneRenderer.Views[SelectedViewIndex];
			SceneCullingRendererCullingQuery = SceneCullingRenderer->CreateHierarchicalCullingQuery(GetCullingVolume());
		}
	}
}								

void FProjectedShadowInfo::SetupMeshDrawCommandsForShadowDepth(FSceneRenderer& Renderer, FInstanceCullingManager& InstanceCullingManager)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SetupMeshDrawCommandsForShadowDepth);

	FShadowDepthPassMeshProcessor* MeshPassProcessor = new FShadowDepthPassMeshProcessor(
		Renderer.Scene,
		Renderer.Scene->GetFeatureLevel(),
		ShadowDepthView,
		GetShadowDepthType(),
		nullptr);

	if (Renderer.ShouldDumpMeshDrawCommandInstancingStats())
	{
		FString PassNameForStats;
		GetShadowTypeNameForDrawEvent(PassNameForStats);
		ShadowDepthPass.SetDumpInstancingStats(TEXT("ShadowDepth ") + PassNameForStats);
	}
	
	ViewIds.Reset();

	Nanite::FPackedViewParams BaseParams{};
	BaseParams.MaxPixelsPerEdgeMultipler = 1.0f;
	BaseParams.RasterContextSize = FIntPoint(ResolutionX, ResolutionY);

	// Only enable min screen radius culling for uncached non-VSM
	if (!bPerObjectOpaqueShadow && 
		((!HasVirtualShadowMap() && CacheMode != SDCM_StaticPrimitivesOnly) 
		|| (VirtualShadowMapPerLightCacheEntry.IsValid() && VirtualShadowMapPerLightCacheEntry->IsUncached())))
	{
		BaseParams.Flags |= NANITE_VIEW_MIN_SCREEN_RADIUS_CULL;
		BaseParams.MinBoundsRadius = GMinScreenRadiusForShadowCaster;
	}

	if (bOnePassPointLightShadow)
	{
		ViewIds.AddDefaulted(6);
		for (int32 CubemapFaceIndex = 0; CubemapFaceIndex < 6; CubemapFaceIndex++)
		{
			// We always render to a whole face at once
			const FIntRect ShadowViewRect = FIntRect(X, Y, ResolutionX, ResolutionY);
			// Setup packed view
			TArray<Nanite::FPackedView, SceneRenderingAllocator> PackedViews;
			{
				Nanite::FPackedViewParams Params = BaseParams;
				Params.ViewMatrices = GetShadowDepthRenderingViewMatrices(CubemapFaceIndex);
				// TODO: Real prev frame matrices
				Params.PrevViewMatrices = Params.ViewMatrices;
				Params.ViewRect = ShadowViewRect;
				Nanite::SetCullingViewOverrides(ShadowDepthView, Params);
				ViewIds[CubemapFaceIndex] = InstanceCullingManager.RegisterView(Params);
			}
		}
	}
	else if (VirtualShadowMapClipmap.IsValid())
	{
		// TODO: Register view per clip level such that they are culled early (?)
		Nanite::FPackedViewParams Params = BaseParams;
		// Note: To ensure conservative culling, we get the coarsest clip-level view since it covers the finer ones.
		Params.ViewMatrices = VirtualShadowMapClipmap->GetViewMatrices(VirtualShadowMapClipmap->GetLevelCount() - 1);
		// TODO: Real prev frame matrices
		Params.PrevViewMatrices = Params.ViewMatrices;
		Params.ViewRect = GetInnerViewRect();
		Nanite::SetCullingViewOverrides(ShadowDepthView, Params);
		ViewIds.Add(InstanceCullingManager.RegisterView(Params));
	}
	else
	{
		Nanite::FPackedViewParams Params = BaseParams;
		Params.ViewMatrices = GetShadowDepthRenderingViewMatrices();
		// TODO: Real prev frame matrices
		Params.PrevViewMatrices = Params.ViewMatrices;
		Params.ViewRect = GetInnerViewRect();
		if (IsWholeSceneDirectionalShadow())
		{
			Params.Flags &= ~NANITE_VIEW_FLAG_NEAR_CLIP;
		}
		Nanite::SetCullingViewOverrides(ShadowDepthView, Params);
		ViewIds.Add(InstanceCullingManager.RegisterView(Params));
	}
	// Note: Needed to support legacy, non-GPU-Scene culled, primitives, this is merely used to allocate enough space for CPU-side replication.
	const bool bMayUseHostCubeFaceReplication = bOnePassPointLightShadow && !HasVirtualShadowMap();
	// Note: Iteracts with FShadowDepthPassMeshProcessor::Process and must be an overestimate of the actual replication done there.
	const uint32 InstanceFactor = bMayUseHostCubeFaceReplication ? 6 : 1;

	// Ensure all work goes down the one path to simplify processing
	EBatchProcessingMode SingleInstanceProcessingMode = HasVirtualShadowMap() ? EBatchProcessingMode::Generic : EBatchProcessingMode::UnCulled;

	ShadowDepthPass.DispatchPassSetup(
		Renderer.Scene,
		*ShadowDepthView,
		FInstanceCullingContext(TEXT("ShadowDepth"), Renderer.ShaderPlatform, &InstanceCullingManager, ViewIds, nullptr, EInstanceCullingMode::Normal, EInstanceCullingFlags::NoInstanceOrderPreservation, SingleInstanceProcessingMode, SceneCullingRendererCullingQuery),
		EMeshPass::Num,
		FExclusiveDepthStencil::DepthNop_StencilNop,
		MeshPassProcessor,
		DynamicSubjectMeshElements,
		nullptr,
		NumDynamicSubjectMeshElements * InstanceFactor,
		SubjectMeshCommandBuildRequests,
		SubjectMeshCommandBuildFlags,
		NumSubjectMeshCommandBuildRequestElements * InstanceFactor,
		ShadowDepthPassVisibleCommands);

	UE::TScopeLock ScopeLock(Renderer.DispatchedShadowDepthPassesMutex);
	Renderer.DispatchedShadowDepthPasses.Add(&ShadowDepthPass);
}

void FProjectedShadowInfo::SetupMeshDrawCommandsForProjectionStenciling(FSceneRenderer& Renderer, FInstanceCullingManager& InstanceCullingManager)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SetupMeshDrawCommandsForShadowDepth);

	const EShadingPath ShadingPath = GetFeatureLevelShadingPath(Renderer.FeatureLevel);
	static const auto EnableModulatedSelfShadowCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Shadow.EnableModulatedSelfShadow"));
	const bool bMobileModulatedShadowsAllowSelfShadow = !bSelfShadowOnly && (ShadingPath == EShadingPath::Mobile && !EnableModulatedSelfShadowCVar->GetValueOnRenderThread() && LightSceneInfo->Proxy && LightSceneInfo->Proxy->CastsModulatedShadows());
	if (bPreShadow || bSelfShadowOnly || bMobileModulatedShadowsAllowSelfShadow)
	{
		ProjectionStencilingPasses.Empty(Renderer.Views.Num());

		for (int32 ViewIndex = 0; ViewIndex < Renderer.Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Renderer.Views[ViewIndex];

			// Skip stereo pair RHs
			if (View.ShouldRenderView())
			{
				// Note: not using FMemstack as the data is not cleared in the right order.
				ProjectionStencilingPasses.Add(new FSimpleMeshDrawCommandPass(View, &InstanceCullingManager, true));
				FSimpleMeshDrawCommandPass& ProjectionStencilingPass = *ProjectionStencilingPasses.Last();

				FMeshPassProcessorRenderState DrawRenderState;
				DrawRenderState.SetBlendState(TStaticBlendState<CW_NONE>::GetRHI());

				if (bMobileModulatedShadowsAllowSelfShadow)
				{
					checkf(bPreShadow == false, TEXT("The mobile renderer does not support preshadows."));

					DrawRenderState.SetDepthStencilState(
						TStaticDepthStencilState<
						false, CF_DepthNearOrEqual,
						true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
						true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
						0xff, 0xff
						>::GetRHI());
					DrawRenderState.SetStencilRef(0);
				}
				else
				{
					// Set stencil to one.
					DrawRenderState.SetDepthStencilState(
						TStaticDepthStencilState<
						false, CF_DepthNearOrEqual,
						true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
						false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
						ShadowStencilMask, ShadowStencilMask
						>::GetRHI());

					DrawRenderState.SetStencilRef(1);
				}

				FDepthPassMeshProcessor DepthPassMeshProcessor(
					EMeshPass::DepthPass,
					Renderer.Scene,
					Renderer.Scene->GetFeatureLevel(),
					&View,
					DrawRenderState,
					false,
					DDM_AllOccluders,
					false,
					false,
					ProjectionStencilingPass.GetDynamicPassMeshDrawListContext(),
					true);

				// Pre-shadows mask by receiver elements, self-shadow mask by subject elements.
				// Note that self-shadow pre-shadows still mask by receiver elements.
				const PrimitiveArrayType& MaskPrimitives = bPreShadow ? ReceiverPrimitives : DynamicSubjectPrimitives;

				for (int32 PrimitiveIndex = 0, PrimitiveCount = MaskPrimitives.Num(); PrimitiveIndex < PrimitiveCount; PrimitiveIndex++)
				{
					const FPrimitiveSceneInfo* ReceiverPrimitiveSceneInfo = MaskPrimitives[PrimitiveIndex];

					if (View.PrimitiveVisibilityMap[ReceiverPrimitiveSceneInfo->GetIndex()])
					{
						const FPrimitiveViewRelevance& ViewRelevance = View.PrimitiveViewRelevanceMap[ReceiverPrimitiveSceneInfo->GetIndex()];

						if (ViewRelevance.bRenderInMainPass && ViewRelevance.bStaticRelevance)
						{
							for (int32 StaticMeshIdx = 0; StaticMeshIdx < ReceiverPrimitiveSceneInfo->StaticMeshes.Num(); StaticMeshIdx++)
							{
								const FStaticMeshBatch& StaticMesh = ReceiverPrimitiveSceneInfo->StaticMeshes[StaticMeshIdx];

								if (View.StaticMeshVisibilityMap[StaticMesh.Id])
								{
									const uint64 DefaultBatchElementMask = ~0ul;
									DepthPassMeshProcessor.AddMeshBatch(StaticMesh, DefaultBatchElementMask, StaticMesh.PrimitiveSceneInfo->Proxy);
								}
							}
						}

						if (ViewRelevance.bRenderInMainPass && ViewRelevance.bDynamicRelevance)
						{
							const FInt32Range MeshBatchRange = View.GetDynamicMeshElementRange(ReceiverPrimitiveSceneInfo->GetIndex());

							for (int32 MeshBatchIndex = MeshBatchRange.GetLowerBoundValue(); MeshBatchIndex < MeshBatchRange.GetUpperBoundValue(); ++MeshBatchIndex)
							{
								const FMeshBatchAndRelevance& MeshAndRelevance = View.DynamicMeshElements[MeshBatchIndex];
								const uint64 BatchElementMask = ~0ull;

								DepthPassMeshProcessor.AddMeshBatch(*MeshAndRelevance.Mesh, BatchElementMask, MeshAndRelevance.PrimitiveSceneProxy);
							}
						}
					}
				}

				if (bSelfShadowOnly && !bPreShadow && !bMobileModulatedShadowsAllowSelfShadow)
				{
					for (int32 MeshBatchIndex = 0; MeshBatchIndex < SubjectMeshCommandBuildRequests.Num(); ++MeshBatchIndex)
					{
						const FStaticMeshBatch& StaticMesh = *SubjectMeshCommandBuildRequests[MeshBatchIndex];
						const uint64 DefaultBatchElementMask = ~0ul;
						DepthPassMeshProcessor.AddMeshBatch(StaticMesh, DefaultBatchElementMask, StaticMesh.PrimitiveSceneInfo->Proxy);
					}
				}
			}
			else
			{
				ProjectionStencilingPasses.Add(nullptr);
			}
		}
	}
}

bool FProjectedShadowInfo::GatherDynamicMeshElements(
	FMeshElementCollector& MeshCollector,
	FSceneRenderer& Renderer,
	FVisibleLightInfo& VisibleLightInfo,
	TArray<const FSceneView*>& ReusedViewsArray,
	EGatherDynamicMeshElementsPass Pass)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Shadow_GatherDynamicMeshElements);
	TRACE_CPUPROFILER_EVENT_SCOPE(FProjectedShadowInfo::GatherDynamicMeshElements);

	check(ShadowDepthView);

	bool bProcessedAllPrimitives = true;

	if (DynamicSubjectPrimitives.Num() > 0 || ReceiverPrimitives.Num() > 0 || SubjectTranslucentPrimitives.Num() > 0 || SubjectHeterogeneousVolumePrimitives.Num() > 0)
	{
		ReusedViewsArray[0] = ShadowDepthView;

		if (bPreShadow && GPreshadowsForceLowestLOD)
		{
			ShadowDepthView->DrawDynamicFlags = EDrawDynamicFlags::ForceLowestLOD;
		}

		if (CascadeSettings.bFarShadowCascade)
		{
			(int32&)ShadowDepthView->DrawDynamicFlags |= (int32)EDrawDynamicFlags::FarShadowCascade;
		}

		if (IsWholeSceneDirectionalShadow())
		{
			ShadowDepthView->SetPreShadowTranslation(FVector(0, 0, 0));
			ShadowDepthView->SetDynamicMeshElementsShadowCullFrustum(&CascadeSettings.ShadowBoundsAccurate);
			bProcessedAllPrimitives &= GatherDynamicMeshElementsArray(MeshCollector, DynamicSubjectPrimitives, ReusedViewsArray, Renderer.ViewFamily, DynamicSubjectMeshElements, NumDynamicSubjectMeshElements, Pass);
			ShadowDepthView->SetPreShadowTranslation(PreShadowTranslation);
		}
		else
		{
			ShadowDepthView->SetPreShadowTranslation(PreShadowTranslation);
			ShadowDepthView->SetDynamicMeshElementsShadowCullFrustum(&CasterOuterFrustum);
			bProcessedAllPrimitives &= GatherDynamicMeshElementsArray(MeshCollector, DynamicSubjectPrimitives, ReusedViewsArray, Renderer.ViewFamily, DynamicSubjectMeshElements, NumDynamicSubjectMeshElements, Pass);
		}
		MeshCollector.ClearViewMeshArrays();

		ShadowDepthView->DrawDynamicFlags = EDrawDynamicFlags::None;

		int32 NumDynamicSubjectTranslucentMeshElements = 0;
		ShadowDepthView->SetDynamicMeshElementsShadowCullFrustum(&CasterOuterFrustum);
		bProcessedAllPrimitives &= GatherDynamicMeshElementsArray(MeshCollector, SubjectTranslucentPrimitives, ReusedViewsArray, Renderer.ViewFamily, DynamicSubjectTranslucentMeshElements, NumDynamicSubjectTranslucentMeshElements, Pass);
		MeshCollector.ClearViewMeshArrays();

		int32 NumDynamicSubjectHeterogeneousVolumeMeshElements = 0;
		bProcessedAllPrimitives &= GatherDynamicHeterogeneousVolumeMeshElementsArray(Renderer, MeshCollector, SubjectHeterogeneousVolumePrimitives, ReusedViewsArray, Renderer.ViewFamily, DynamicSubjectHeterogeneousVolumeMeshElements, NumDynamicSubjectHeterogeneousVolumeMeshElements, Pass);
		MeshCollector.ClearViewMeshArrays();
	}

	return bProcessedAllPrimitives;
}

bool FProjectedShadowInfo::GatherDynamicMeshElementsArray(
	FMeshElementCollector& MeshCollector,
	const PrimitiveArrayType& Primitives,
	const TArray<const FSceneView*>& Views,
	const FSceneViewFamily& ViewFamily,
	TArray<FMeshBatchAndRelevance,SceneRenderingAllocator>& OutDynamicMeshElements,
	int32& OutNumDynamicSubjectMeshElements,
	EGatherDynamicMeshElementsPass Pass)
{
	// Simple elements not supported in shadow passes
	FSimpleElementCollector DynamicSubjectSimpleElements;

	MeshCollector.AddViewMeshArrays(
		ShadowDepthView,
		&OutDynamicMeshElements, 
		&DynamicSubjectSimpleElements, 
		&ShadowDepthView->DynamicPrimitiveCollector);

	const uint32 PrimitiveCount = Primitives.Num();

	bool bProcessedAllPrimitives = true;

	for (uint32 PrimitiveIndex = 0; PrimitiveIndex < PrimitiveCount; ++PrimitiveIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicPrimitive);
		const FPrimitiveSceneInfo* PrimitiveSceneInfo = Primitives[PrimitiveIndex];
		const FPrimitiveSceneProxy* PrimitiveSceneProxy = PrimitiveSceneInfo->Proxy;

		// Lookup the primitive's cached view relevance
		FPrimitiveViewRelevance ViewRelevance = ShadowDepthView->PrimitiveViewRelevanceMap[PrimitiveSceneInfo->GetIndex()];

		if (!ViewRelevance.bInitializedThisFrame)
		{
			// Compute the subject primitive's view relevance since it wasn't cached
			ViewRelevance = PrimitiveSceneProxy->GetViewRelevance(ShadowDepthView);
		}

		// Only draw if the subject primitive is shadow relevant.
		if (ViewRelevance.bShadowRelevance && ViewRelevance.bDynamicRelevance)
		{
			if (Pass != EGatherDynamicMeshElementsPass::All)
			{
				const bool bSerialPrimitivesOnly = Pass == EGatherDynamicMeshElementsPass::Serial;

				// Only process primitives that match the correct pass.
				if (bSerialPrimitivesOnly == PrimitiveSceneProxy->SupportsParallelGDME())
				{
					bProcessedAllPrimitives = false;
					continue;
				}
			}

			MeshCollector.SetPrimitive(PrimitiveSceneProxy, PrimitiveSceneInfo->DefaultDynamicHitProxyId);

			PrimitiveSceneProxy->GetDynamicMeshElements(Views, ViewFamily, 0x1, MeshCollector);
		}
	}

	OutNumDynamicSubjectMeshElements += MeshCollector.GetMeshElementCount(0);
	return bProcessedAllPrimitives;
}

bool FProjectedShadowInfo::GatherDynamicHeterogeneousVolumeMeshElementsArray(
	FSceneRenderer& Renderer,
	FMeshElementCollector& MeshCollector,
	const PrimitiveArrayType& Primitives,
	const TArray<const FSceneView*>& Views,
	const FSceneViewFamily& ViewFamily,
	TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>& OutDynamicMeshElements,
	int32& OutNumDynamicSubjectMeshElements,
	EGatherDynamicMeshElementsPass Pass)
{
	bool bProcessedAllPrimitives = GatherDynamicMeshElementsArray(
		MeshCollector,
		Primitives,
		Views,
		ViewFamily,
		OutDynamicMeshElements,
		OutNumDynamicSubjectMeshElements,
		Pass
	);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		uint32 MeshElementCount = MeshCollector.GetMeshElementCount(ViewIndex);
		for (uint32 Index = 0; Index < MeshElementCount; ++Index)
		{
			FMeshBatchAndRelevance MeshBatchAndRelevance = (*MeshCollector.MeshBatches[ViewIndex])[Index];
			OutDynamicMeshElements.Add(MeshBatchAndRelevance);
		}

		OutNumDynamicSubjectMeshElements += MeshElementCount;
	}

	return bProcessedAllPrimitives;
}

/** 
 * @param View view to check visibility in
 * @return true if this shadow info has any subject prims visible in the view
 */
bool FProjectedShadowInfo::SubjectsVisible(const FViewInfo& View) const
{
	checkSlow(!IsWholeSceneDirectionalShadow());
	for(int32 PrimitiveIndex = 0;PrimitiveIndex < DynamicSubjectPrimitives.Num();PrimitiveIndex++)
	{
		const FPrimitiveSceneInfo* SubjectPrimitiveSceneInfo = DynamicSubjectPrimitives[PrimitiveIndex];
		if(View.PrimitiveVisibilityMap[SubjectPrimitiveSceneInfo->GetIndex()])
		{
			return true;
		}
	}
	return false;
}

/** 
 * Clears arrays allocated with the scene rendering allocator.
 * Cached preshadows are reused across frames so scene rendering allocations will be invalid.
 */
void FProjectedShadowInfo::ClearTransientArrays()
{
	SceneCullingRendererCullingQuery = nullptr;
	NumDynamicSubjectMeshElements = 0;
	NumSubjectMeshCommandBuildRequestElements = 0;

	SubjectTranslucentPrimitives.Empty();
	SubjectHeterogeneousVolumePrimitives.Empty();
	DynamicSubjectPrimitives.Empty();
	ReceiverPrimitives.Empty();
	DynamicSubjectMeshElements.Empty();
	DynamicSubjectTranslucentMeshElements.Empty();
	DynamicSubjectHeterogeneousVolumeMeshElements.Empty();

	ShadowDepthPassVisibleCommands.Empty();
	ShadowDepthPass.Cleanup();

	SubjectMeshCommandBuildRequests.Empty();
	SubjectMeshCommandBuildFlags.Empty();

	for (auto ProjectionStencilingPass : ProjectionStencilingPasses)
	{
		delete ProjectionStencilingPass;
	}
	ProjectionStencilingPasses.Reset();

	DynamicMeshDrawCommandStorage.MeshDrawCommands.Empty();
	GraphicsMinimalPipelineStateSet.Empty();
}

void FProjectedShadowInfo::ComputeScissorRectOptim()
{
	ScissorRectOptim.Min = FIntPoint(0, 0);
	ScissorRectOptim.Max = FIntPoint(ResolutionX, ResolutionY);

	// get the far plane corners in world space
	const FMatrix& ViewMatrix = DependentView->ViewMatrices.GetWorldToView();
	const FMatrix& ProjectionMatrix = DependentView->ViewMatrices.GetViewToClip();
	const FVector CameraDirection = ViewMatrix.GetColumn(2);
	const FVector ViewOrigin = DependentView->ViewMatrices.GetViewOrigin();

	// Support asymmetric projection
	// Get FOV and AspectRatio from the view's projection matrix.
	float AspectRatio = ProjectionMatrix.M[1][1] / ProjectionMatrix.M[0][0];
	bool bIsPerspectiveProjection = true;

	// Build the camera frustum for this cascade
	float HalfHorizontalFOV = bIsPerspectiveProjection ? FMath::Atan(1.0f / ProjectionMatrix.M[0][0]) : UE_PI / 4.0f;
	float HalfVerticalFOV = bIsPerspectiveProjection ? FMath::Atan(1.0f / ProjectionMatrix.M[1][1]) : FMath::Atan((FMath::Tan(UE_PI / 4.0f) / AspectRatio));
	float AsymmetricFOVScaleX = ProjectionMatrix.M[2][0];
	float AsymmetricFOVScaleY = ProjectionMatrix.M[2][1];

	// Far plane
	const float EndHorizontalTotalLength = CascadeSettings.SplitFar * FMath::Tan(HalfHorizontalFOV);
	const float EndVerticalTotalLength = CascadeSettings.SplitFar * FMath::Tan(HalfVerticalFOV);
	const FVector EndCameraLeftOffset = ViewMatrix.GetColumn(0) * -EndHorizontalTotalLength * (1 + AsymmetricFOVScaleX);
	const FVector EndCameraRightOffset = ViewMatrix.GetColumn(0) * EndHorizontalTotalLength * (1 - AsymmetricFOVScaleX);
	const FVector EndCameraBottomOffset = ViewMatrix.GetColumn(1) * -EndVerticalTotalLength * (1 + AsymmetricFOVScaleY);
	const FVector EndCameraTopOffset = ViewMatrix.GetColumn(1) * EndVerticalTotalLength * (1 - AsymmetricFOVScaleY);

	FVector FrustumCorners[5];
	FrustumCorners[0] = ViewOrigin + CameraDirection * CascadeSettings.SplitFar + EndCameraRightOffset + EndCameraTopOffset;//Far  Top    Right
	FrustumCorners[1] = ViewOrigin + CameraDirection * CascadeSettings.SplitFar + EndCameraRightOffset + EndCameraBottomOffset;//Far  Bottom Right
	FrustumCorners[2] = ViewOrigin + CameraDirection * CascadeSettings.SplitFar + EndCameraLeftOffset + EndCameraTopOffset;//Far  Top    Left
	FrustumCorners[3] = ViewOrigin + CameraDirection * CascadeSettings.SplitFar + EndCameraLeftOffset + EndCameraBottomOffset;//Far  Bottom Left
	FrustumCorners[4] = ViewOrigin + CameraDirection * CascadeSettings.SplitFar;//view direction

	FMatrix WorldToShadow = FTranslationMatrix(PreShadowTranslation) *
		FMatrix(TranslatedWorldToClipInnerMatrix) *
		FMatrix(
			FPlane(0.5f, 0, 0, 0),
			FPlane(0, -0.5f, 0, 0),
			FPlane(0, 0, InvMaxSubjectDepth, 0),
			FPlane(
				1.0f / (ResolutionX + BorderSize * 2) + 0.5f,
				1.0f / (ResolutionY + BorderSize * 2) + 0.5f,
				0,
				1
			)
		);

	int32 FullResX = ResolutionX + BorderSize * 2;
	int32 FullResY = ResolutionY + BorderSize * 2;
	FVector4 FrustumClipShadowCorners[6];
	FVector4 ScaleView = FVector4(FullResX, FullResY, 1.0, 1.0);

	// Transform each corners of the view frustum in Shadow clip space
	for (int Index = 0; Index < 5; Index++)
	{
		FrustumClipShadowCorners[Index] = WorldToShadow.TransformPosition(FrustumCorners[Index]);
		FrustumClipShadowCorners[Index] /= FrustumClipShadowCorners[Index].W;
		FrustumClipShadowCorners[Index] *= ScaleView;
	}

	// Transform the origin of the view frustum in Shadow Clip Space
	FrustumClipShadowCorners[5] = WorldToShadow.TransformPosition(ViewOrigin);
	FrustumClipShadowCorners[5] /= FrustumClipShadowCorners[4].W;
	FrustumClipShadowCorners[5] *= ScaleView;

	auto ComputeScissorIntersection = [FullResX, FullResY](const FLine2d& Line)
	{
		FLine2d Borders[4] = { FLine2d(FVector2d(0.0, 0.0), FVector2d(1.0, 0.0)),
			FLine2d(FVector2d(FullResX, 0.0), FVector2d(0.0, 1.0)),
			FLine2d(FVector2d(FullResX, FullResY), FVector2d(-1.0, 0.0)),
			FLine2d(FVector2d(0.0, FullResY), FVector2d(0.0, -1.0)) };

		FVector2D Result = FVector2D::ZeroVector;

		float MinDistance = 100000.0f;
		bool bIntersectionvalid = false;

		for (int Index = 0; Index < 4; Index++)
		{
			FVector2D IntPoint;
			if (Line.IntersectionPoint(Borders[Index], IntPoint))
			{
				//check if the point is behind
				FVector2D IntVector = (IntPoint - Line.Origin);
				if (FVector2D::DotProduct(IntVector.GetSafeNormal(), Line.Direction) > 0.0)
				{
					float IntersectionDistance = FVector2D::Distance(IntPoint, Line.Origin);
					if (IntersectionDistance < MinDistance)
					{
						Result = IntPoint;
						MinDistance = IntersectionDistance;
						bIntersectionvalid = true;
					}
				}
			}
		}
		return FVector4(Result.X, Result.Y, 0.0, 0.0);
	};

	// compute the intersection with shadow render target borders
	FVector2D FrustrumOrigin2D = FVector2D(FrustumClipShadowCorners[5].X, FrustumClipShadowCorners[5].Y);

	if (FrustrumOrigin2D.X >= 0.0 && FrustrumOrigin2D.X <= FullResX &&
		FrustrumOrigin2D.Y >= 0.0 && FrustrumOrigin2D.Y <= FullResY)
	{
		// Compute the intersection of the view frustum corners in shadow clip space
		for (int Index = 0; Index < 5; Index++)
		{			
			FLine2d FrustumLine = FLine2d(FrustrumOrigin2D, (FVector2D(FrustumClipShadowCorners[Index].X, FrustumClipShadowCorners[Index].Y) - FrustrumOrigin2D).GetSafeNormal());
			FrustumClipShadowCorners[Index] = ComputeScissorIntersection(FrustumLine);
		}

		// Get the scissor rect min and max values from the intersection result
		FIntRect ScissorMinMax(FIntPoint(FullResX, FullResX), FIntPoint(0, 0));

		for (int Index = 0; Index < 6; Index++)
		{
			if (FrustumClipShadowCorners[Index].X < ScissorMinMax.Min.X)
			{
				ScissorMinMax.Min.X = FMath::Clamp(FrustumClipShadowCorners[Index].X, 0, FullResX);
			}
			if (FrustumClipShadowCorners[Index].Y < ScissorMinMax.Min.Y)
			{
				ScissorMinMax.Min.Y = FMath::Clamp(FrustumClipShadowCorners[Index].Y, 0, FullResY);
			}
			if (FrustumClipShadowCorners[Index].X > ScissorMinMax.Max.X)
			{
				ScissorMinMax.Max.X = FMath::Clamp(FrustumClipShadowCorners[Index].X, 0, FullResX);
			}
			if (FrustumClipShadowCorners[Index].Y > ScissorMinMax.Max.Y)
			{
				ScissorMinMax.Max.Y = FMath::Clamp(FrustumClipShadowCorners[Index].Y, 0, FullResY);
			}
		}
		
		ScissorRectOptim = ScissorMinMax;
		ScissorRectOptim += FIntPoint(X, Y);
	}
}

FSceneViewState::FProjectedShadowKey::FProjectedShadowKey(const FProjectedShadowInfo& ProjectedShadowInfo)
	: PrimitiveId(ProjectedShadowInfo.GetParentSceneInfo() ? ProjectedShadowInfo.GetParentSceneInfo()->PrimitiveComponentId : FPrimitiveComponentId())
	, LightComponentId(ProjectedShadowInfo.GetLightSceneInfo().Proxy->GetLightComponentId())
	, ShadowSplitIndex(ProjectedShadowInfo.CascadeSettings.ShadowSplitIndex)
	, bTranslucentShadow(ProjectedShadowInfo.bTranslucentShadow)
{
}

/** Returns a cached preshadow matching the input criteria if one exists. */
TRefCountPtr<FProjectedShadowInfo> FSceneRenderer::GetCachedPreshadow(
	const FLightPrimitiveInteraction* InParentInteraction, 
	const FProjectedShadowInitializer& Initializer,
	const FBoxSphereBounds& Bounds,
	uint32 InResolutionX)
{
	if (ShouldUseCachePreshadows() && !Views[0].bIsSceneCapture)
	{
		const FPrimitiveSceneInfo* PrimitiveInfo = InParentInteraction->GetPrimitiveSceneInfo();
		const FLightSceneInfo* LightInfo = InParentInteraction->GetLight();
		const FSphere QueryBounds(Bounds.Origin, Bounds.SphereRadius);

		for (int32 ShadowIndex = 0; ShadowIndex < Scene->CachedPreshadows.Num(); ShadowIndex++)
		{
			TRefCountPtr<FProjectedShadowInfo> CachedShadow = Scene->CachedPreshadows[ShadowIndex];
			// Only reuse a cached preshadow if it was created for the same primitive and light
			if (CachedShadow->GetParentSceneInfo() == PrimitiveInfo
				&& &CachedShadow->GetLightSceneInfo() == LightInfo
				// Only reuse if it contains the bounds being queried, with some tolerance
				&& QueryBounds.IsInside(CachedShadow->ShadowBounds, CachedShadow->ShadowBounds.W * .04f)
				// Only reuse if the resolution matches
				&& CachedShadow->ResolutionX == InResolutionX
				&& CachedShadow->bAllocated)
			{
				// Reset any allocations using the scene rendering allocator, 
				// Since those will point to freed memory now that we are using the shadow on a different frame than it was created on.
				CachedShadow->ClearTransientArrays();
				return CachedShadow;
			}
		}
	}
	// No matching cached preshadow was found
	return NULL;
}

struct FComparePreshadows
{
	FORCEINLINE bool operator()(const TRefCountPtr<FProjectedShadowInfo>& A, const TRefCountPtr<FProjectedShadowInfo>& B) const
	{
		if (B->ResolutionX * B->ResolutionY < A->ResolutionX * A->ResolutionY)
		{
			return true;
		}

		return false;
	}
};

/** Removes stale shadows and attempts to add new preshadows to the cache. */
void FSceneRenderer::UpdatePreshadowCache()
{
	if (ShouldUseCachePreshadows() && !Views[0].bIsSceneCapture)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdatePreshadowCache);
		if (Scene->PreshadowCacheLayout.GetSizeX() == 0)
		{
			// Initialize the texture layout if necessary
			const FIntPoint PreshadowCacheBufferSize = GetPreShadowCacheTextureResolution(FeatureLevel);
			Scene->PreshadowCacheLayout = FTextureLayout(1, 1, PreshadowCacheBufferSize.X, PreshadowCacheBufferSize.Y, false, ETextureLayoutAspectRatio::None, false);
		}

		// Iterate through the cached preshadows, removing those that are not going to be rendered this frame
		for (int32 CachedShadowIndex = Scene->CachedPreshadows.Num() - 1; CachedShadowIndex >= 0; CachedShadowIndex--)
		{
			TRefCountPtr<FProjectedShadowInfo> CachedShadow = Scene->CachedPreshadows[CachedShadowIndex];
			bool bShadowBeingRenderedThisFrame = false;

			for (const FVisibleLightInfo& VisibleLightInfo : VisibleLightInfos)
			{
				bShadowBeingRenderedThisFrame = VisibleLightInfo.ProjectedPreShadows.Find(CachedShadow) != INDEX_NONE;
				if (bShadowBeingRenderedThisFrame)
				{
					break;
				}
			}

			if (!bShadowBeingRenderedThisFrame)
			{
				// Must succeed, since we added it to the layout earlier
				verify(Scene->PreshadowCacheLayout.RemoveElement(
					CachedShadow->X,
					CachedShadow->Y,
					CachedShadow->ResolutionX + CachedShadow->BorderSize * 2,
					CachedShadow->ResolutionY + CachedShadow->BorderSize * 2));
				Scene->CachedPreshadows.RemoveAt(CachedShadowIndex);
			}
		}

		TArray<TRefCountPtr<FProjectedShadowInfo>, SceneRenderingAllocator> UncachedPreShadows;

		// Gather a list of preshadows that can be cached
		for (const FVisibleLightInfo& VisibleLightInfo : VisibleLightInfos)
		{
			for (int32 ShadowIndex = 0; ShadowIndex < VisibleLightInfo.ProjectedPreShadows.Num(); ShadowIndex++)
			{
				TRefCountPtr<FProjectedShadowInfo> CurrentShadow = VisibleLightInfo.ProjectedPreShadows[ShadowIndex];
				checkSlow(CurrentShadow->bPreShadow);

				if (!CurrentShadow->bAllocatedInPreshadowCache)
				{
					UncachedPreShadows.Add(CurrentShadow);
				}
			}
		}

		// Sort them from largest to smallest, based on the assumption that larger preshadows will have more objects in their depth only pass
		UncachedPreShadows.Sort(FComparePreshadows());

		for (int32 ShadowIndex = 0; ShadowIndex < UncachedPreShadows.Num(); ShadowIndex++)
		{
			TRefCountPtr<FProjectedShadowInfo> CurrentShadow = UncachedPreShadows[ShadowIndex];

			// Try to find space for the preshadow in the texture layout
			if (Scene->PreshadowCacheLayout.AddElement(
				CurrentShadow->X,
				CurrentShadow->Y,
				CurrentShadow->ResolutionX + CurrentShadow->BorderSize * 2,
				CurrentShadow->ResolutionY + CurrentShadow->BorderSize * 2))
			{
				// Mark the preshadow as existing in the cache
				// It must now use the preshadow cache render target to render and read its depths instead of the usual shadow depth buffers
				CurrentShadow->bAllocatedInPreshadowCache = true;
				// Indicate that the shadow's X and Y have been initialized
				CurrentShadow->bAllocated = true;
				Scene->CachedPreshadows.Add(CurrentShadow);
			}
		}
	}
}

bool ShouldCreateObjectShadowForStationaryLight(const FLightSceneInfo* LightSceneInfo, const FPrimitiveSceneProxy* PrimitiveSceneProxy, bool bInteractionShadowMapped) 
{
	const bool bCreateObjectShadowForStationaryLight = 
		LightSceneInfo->bCreatePerObjectShadowsForDynamicObjects
		&& LightSceneInfo->IsPrecomputedLightingValid()
		&& LightSceneInfo->Proxy->GetShadowMapChannel() != INDEX_NONE
		// Create a per-object shadow if the object does not want static lighting and needs to integrate with the static shadowing of a stationary light
		// Or if the object wants static lighting but does not have a built shadowmap (Eg has been moved in the editor)
		&& (!PrimitiveSceneProxy->HasStaticLighting() || !bInteractionShadowMapped);

	return bCreateObjectShadowForStationaryLight;
}

void FSceneRenderer::SetupInteractionShadows(
	FDynamicShadowsTaskData& TaskData,
	FLightPrimitiveInteraction* Interaction, 
	FVisibleLightInfo& VisibleLightInfo, 
	bool bStaticSceneOnly,
	const TArray<FProjectedShadowInfo*,SceneRenderingAllocator>& ViewDependentWholeSceneShadows,
	TArray<FProjectedShadowInfo*,SceneRenderingAllocator>& PreShadows)
{
	check(!UseVirtualShadowMaps(ShaderPlatform));

	// too high on hit count to leave on
	// SCOPE_CYCLE_COUNTER(STAT_SetupInteractionShadows);

	FPrimitiveSceneInfo* PrimitiveSceneInfo = Interaction->GetPrimitiveSceneInfo();
	FLightSceneProxy* LightProxy = Interaction->GetLight()->Proxy;
	extern bool GUseTranslucencyShadowDepths;

	bool bShadowHandledByParent = false;

	if (PrimitiveSceneInfo->LightingAttachmentRoot.IsValid())
	{
		FAttachmentGroupSceneInfo& AttachmentGroup = Scene->AttachmentGroups.FindChecked(PrimitiveSceneInfo->LightingAttachmentRoot);
		bShadowHandledByParent = AttachmentGroup.ParentSceneInfo && AttachmentGroup.ParentSceneInfo->Proxy->LightAttachmentsAsGroup();
	}

	// Shadowing for primitives with a shadow parent will be handled by that shadow parent
	if (!bShadowHandledByParent)
	{
		const bool bCreateTranslucentObjectShadow = GUseTranslucencyShadowDepths && Interaction->HasTranslucentObjectShadow();
		const bool bCreateObjectShadowForStationaryLight = ShouldCreateObjectShadowForStationaryLight(Interaction->GetLight(), PrimitiveSceneInfo->Proxy, Interaction->IsShadowMapped());

		bool bCreateOpaqueObjectShadow = Interaction->HasInsetObjectShadow();
		if (bCreateOpaqueObjectShadow && TaskData.FeatureLevel < ERHIFeatureLevel::SM5)
		{
			bCreateOpaqueObjectShadow = (CVarMobileInsetShadows.GetValueOnRenderThread() && Interaction->HasInsetObjectShadow())
				|| (IsCapsuleDirectShadowsEnabled(TaskData.ShaderPlatform) && PrimitiveSceneInfo->Proxy->CastsCapsuleDirectShadow());
		}

		if (Interaction->HasShadow()
			// Only render shadows from objects that use static lighting during a reflection capture, since the reflection capture doesn't update at runtime
			&& (!bStaticSceneOnly || PrimitiveSceneInfo->Proxy->HasStaticLighting())
			&& (bCreateTranslucentObjectShadow || bCreateOpaqueObjectShadow || bCreateObjectShadowForStationaryLight))
		{
			// Create projected shadow infos
			CreatePerObjectProjectedShadow(TaskData, Interaction, bCreateTranslucentObjectShadow, bCreateOpaqueObjectShadow || bCreateObjectShadowForStationaryLight, ViewDependentWholeSceneShadows, PreShadows);
		}
	}
}

static float ClampShadowRolution(float UnclampedResolution, float ShadowResolutionScale, float MaxShadowResolution)
{
	float ClampedResolution = UnclampedResolution;

	if (ShadowResolutionScale > 1.0f)
	{
		// Apply ShadowResolutionScale before the MaxShadowResolution clamp if raising the resolution
		ClampedResolution *= ShadowResolutionScale;
	}

	ClampedResolution = FMath::Min<float>(ClampedResolution, MaxShadowResolution);

	if (ShadowResolutionScale <= 1.0f)
	{
		// Apply ShadowResolutionScale after the MaxShadowResolution clamp if lowering the resolution
		// Artists want to modify the softness of the shadow with ShadowResolutionScale
		ClampedResolution *= ShadowResolutionScale;
	}
	return ClampedResolution;
}


void FSceneRenderer::CreatePerObjectProjectedShadow(
	FDynamicShadowsTaskData& TaskData,
	FLightPrimitiveInteraction* Interaction, 
	bool bCreateTranslucentObjectShadow, 
	bool bCreateOpaqueObjectShadow,
	const TArray<FProjectedShadowInfo*,SceneRenderingAllocator>& ViewDependentWholeSceneShadows,
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& OutPreShadows)
{
	check(bCreateOpaqueObjectShadow || bCreateTranslucentObjectShadow);
	FPrimitiveSceneInfo* PrimitiveSceneInfo = Interaction->GetPrimitiveSceneInfo();
	const int32 PrimitiveId = PrimitiveSceneInfo->GetIndex();

	FLightSceneInfo* LightSceneInfo = Interaction->GetLight();
	FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];

	// Check if the shadow is visible in any of the views.
	bool bShadowIsPotentiallyVisibleNextFrame = false;
	bool bOpaqueShadowIsVisibleThisFrame = false;
	bool bSubjectIsVisible = false;
	bool bOpaque = false;
	bool bTranslucentRelevance = false;
	bool bTranslucentShadowIsVisibleThisFrame = false;
	int32 NumBufferedFrames = FOcclusionQueryHelpers::GetNumBufferedFrames(FeatureLevel);
	bool bCanUseShadowDepthAtlas = FProjectedShadowInfo::SupportsShadowStencilCulling(ShaderPlatform);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];

		// Lookup the primitive's cached view relevance
		FPrimitiveViewRelevance ViewRelevance = View.PrimitiveViewRelevanceMap[PrimitiveId];

		if (!ViewRelevance.bInitializedThisFrame)
		{
			// Compute the subject primitive's view relevance since it wasn't cached
			ViewRelevance = PrimitiveSceneInfo->Proxy->GetViewRelevance(&View);
		}

		// Check if the subject primitive is shadow relevant.
		const bool bPrimitiveIsShadowRelevant = ViewRelevance.bShadowRelevance;

		const FSceneViewState::FProjectedShadowKey OpaqueKey(PrimitiveSceneInfo->PrimitiveComponentId, LightSceneInfo->Proxy->GetLightComponentId(), INDEX_NONE, false);

		// Check if the shadow and preshadow are occluded.
		const bool bOpaqueShadowIsOccluded = 
			!bCreateOpaqueObjectShadow ||
			(
				!View.bIgnoreExistingQueries &&	View.State &&
				((FSceneViewState*)View.State)->IsShadowOccluded(OpaqueKey, NumBufferedFrames)
			);

		const FSceneViewState::FProjectedShadowKey TranslucentKey(PrimitiveSceneInfo->PrimitiveComponentId, LightSceneInfo->Proxy->GetLightComponentId(), INDEX_NONE, true);

		const bool bTranslucentShadowIsOccluded = 
			!bCreateTranslucentObjectShadow ||
			(
				!View.bIgnoreExistingQueries && View.State &&
				((FSceneViewState*)View.State)->IsShadowOccluded(TranslucentKey, NumBufferedFrames)
			);

		// if subject doesn't render in the main pass, it's never considered visible
		// (in this case, there will be no need to generate any preshadows for the subject)
		if (PrimitiveSceneInfo->Proxy->ShouldRenderInMainPass())
		{
			const bool bSubjectIsVisibleInThisView = View.PrimitiveVisibilityMap[PrimitiveSceneInfo->GetIndex()];
			bSubjectIsVisible |= bSubjectIsVisibleInThisView;
		}

		// The shadow is visible if it is view relevant and unoccluded.
		bOpaqueShadowIsVisibleThisFrame |= (bPrimitiveIsShadowRelevant && !bOpaqueShadowIsOccluded);
		bTranslucentShadowIsVisibleThisFrame |= (bPrimitiveIsShadowRelevant && !bTranslucentShadowIsOccluded);
		bShadowIsPotentiallyVisibleNextFrame |= bPrimitiveIsShadowRelevant;
		bOpaque |= ViewRelevance.bOpaque;
		bTranslucentRelevance |= ViewRelevance.HasTranslucency();
	}

	if (!bOpaqueShadowIsVisibleThisFrame && !bTranslucentShadowIsVisibleThisFrame && !bShadowIsPotentiallyVisibleNextFrame)
	{
		// Don't setup the shadow info for shadows which don't need to be rendered or occlusion tested.
		return;
	}

	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> ShadowGroupPrimitives;
	PrimitiveSceneInfo->GatherLightingAttachmentGroupPrimitives(ShadowGroupPrimitives);

#if ENABLE_NAN_DIAGNOSTIC
	// allow for silent failure: only possible if NaN checking is enabled.  
	if (ShadowGroupPrimitives.Num() == 0)
	{
		return;
	}
#endif

	// Compute the composite bounds of this group of shadow primitives.
	FBoxSphereBounds OriginalBounds = ShadowGroupPrimitives[0]->Proxy->GetBounds();

	if (!ensureMsgf(OriginalBounds.ContainsNaN() == false, TEXT("OriginalBound contains NaN : %s"), *OriginalBounds.ToString()))
	{
		// fix up OriginalBounds. This is going to cause flickers
		OriginalBounds = FBoxSphereBounds(FVector::ZeroVector, FVector(1.f), 1.f);
	}

	for (int32 ChildIndex = 1; ChildIndex < ShadowGroupPrimitives.Num(); ChildIndex++)
	{
		const FPrimitiveSceneInfo* ShadowChild = ShadowGroupPrimitives[ChildIndex];
		if (ShadowChild->Proxy->CastsDynamicShadow())
		{
			FBoxSphereBounds ChildBound = ShadowChild->Proxy->GetBounds();
			OriginalBounds = OriginalBounds + ChildBound;

			if (!ensureMsgf(OriginalBounds.ContainsNaN() == false, TEXT("Child %s contains NaN : %s"), *ShadowChild->Proxy->GetOwnerName().ToString(), *ChildBound.ToString()))
			{
				// fix up OriginalBounds. This is going to cause flickers
				OriginalBounds = FBoxSphereBounds(FVector::ZeroVector, FVector(1.f), 1.f);
			}
		}
	}

	// Shadowing constants.
	
	const uint32 InsetDownscaleFactor = FMath::Max<int32>(1, CVarInsetDownscaleFactor.GetValueOnRenderThread());
	const uint32 MaxShadowResolutionSetting = GetCachedScalabilityCVars().MaxShadowResolution;
	const FIntPoint ShadowBufferResolution = GetShadowDepthTextureResolution(FeatureLevel) / InsetDownscaleFactor;
	const uint32 MaxShadowResolution = FMath::Min<int32>(MaxShadowResolutionSetting, ShadowBufferResolution.X) - SHADOW_BORDER * 2;
	const uint32 MaxShadowResolutionY = FMath::Min<int32>(MaxShadowResolutionSetting, ShadowBufferResolution.Y) - SHADOW_BORDER * 2;
	const uint32 MinShadowResolution     = FMath::Max<int32>(0, CVarMinShadowResolution.GetValueOnRenderThread());
	const uint32 ShadowFadeResolution    = FMath::Max<int32>(0, CVarShadowFadeResolution.GetValueOnRenderThread());
	const uint32 MinPreShadowResolution  = FMath::Max<int32>(0, CVarMinPreShadowResolution.GetValueOnRenderThread());
	const uint32 PreShadowFadeResolution = FMath::Max<int32>(0, CVarPreShadowFadeResolution.GetValueOnRenderThread());
	
	// Compute the maximum resolution required for the shadow by any view. Also keep track of the unclamped resolution for fading.
	uint32 MaxDesiredResolution = 0;
	float MaxScreenPercent = 0;
	TArray<float, TInlineAllocator<2> > ResolutionFadeAlphas;
	TArray<float, TInlineAllocator<2> > ResolutionPreShadowFadeAlphas;
	float MaxResolutionFadeAlpha = 0;
	float MaxResolutionPreShadowFadeAlpha = 0;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		const FViewMatrices& ShadowViewMatrices = View.GetShadowViewMatrices();

		// Determine the size of the subject's bounding sphere in this view.
		const FVector ShadowViewOrigin = View.ViewMatrices.GetViewOrigin();
		float ShadowViewDistFromBounds = (OriginalBounds.Origin - ShadowViewOrigin).Size();
		const float ScreenRadius = ShadowViewMatrices.GetScreenScale() *
			OriginalBounds.SphereRadius /
			FMath::Max(ShadowViewDistFromBounds, 1.0f);
		// Early catch for invalid CalculateShadowFadeAlpha()
		ensureMsgf(ScreenRadius >= 0.0f, TEXT("ShadowViewMatrices.ScreenScale %f, OriginalBounds.SphereRadius %f, ShadowViewDistFromBounds %f"), ShadowViewMatrices.GetScreenScale(), OriginalBounds.SphereRadius, ShadowViewDistFromBounds);

		const float ScreenPercent = FMath::Max(
			1.0f / 2.0f * ShadowViewMatrices.GetProjectionScale().X,
			1.0f / 2.0f * ShadowViewMatrices.GetProjectionScale().Y
			) *
			OriginalBounds.SphereRadius /
			FMath::Max(ShadowViewDistFromBounds, 1.0f);

		MaxScreenPercent = FMath::Max(MaxScreenPercent, ScreenPercent);

		// Determine the amount of shadow buffer resolution needed for this view.
		const float UnclampedResolution = ScreenRadius * CVarShadowTexelsPerPixel.GetValueOnRenderThread();
		const float ClampedResolution = ClampShadowRolution(UnclampedResolution, LightSceneInfo->Proxy->GetShadowResolutionScale(), MaxShadowResolution);

		{
			const float FadeCalcResolution = GDoesShadowFadeUseResolutionScale ? ClampedResolution : UnclampedResolution;
			const float ViewSpecificAlpha = CalculateShadowFadeAlpha(FadeCalcResolution, ShadowFadeResolution, MinShadowResolution) * LightSceneInfo->Proxy->GetShadowAmount();
			MaxResolutionFadeAlpha = FMath::Max(MaxResolutionFadeAlpha, ViewSpecificAlpha);
			ResolutionFadeAlphas.Add(ViewSpecificAlpha);
			const float ViewSpecificPreShadowAlpha = CalculateShadowFadeAlpha(FadeCalcResolution * CVarPreShadowResolutionFactor.GetValueOnRenderThread(), PreShadowFadeResolution, MinPreShadowResolution) * LightSceneInfo->Proxy->GetShadowAmount();
			MaxResolutionPreShadowFadeAlpha = FMath::Max(MaxResolutionPreShadowFadeAlpha, ViewSpecificPreShadowAlpha);
			ResolutionPreShadowFadeAlphas.Add(ViewSpecificPreShadowAlpha);
		}

		MaxDesiredResolution = FMath::Max(
			MaxDesiredResolution,
			FMath::Max<uint32>(
				ClampedResolution,
				FMath::Min<int32>(MinShadowResolution, ShadowBufferResolution.X - SHADOW_BORDER * 2)
				)
			);
	}

	FBoxSphereBounds Bounds = OriginalBounds;

	const bool bRenderPreShadow = 
		CVarAllowPreshadows.GetValueOnRenderThread() 
		&& LightSceneInfo->Proxy->HasStaticShadowing()
		// Preshadow only affects the subject's pixels
		&& bSubjectIsVisible 
		// Only objects with dynamic lighting should create a preshadow
		// Unless we're in the editor and need to preview an object without built lighting
		&& (!PrimitiveSceneInfo->Proxy->HasStaticLighting() || !Interaction->IsShadowMapped())
		// Disable preshadows from directional lights for primitives that use single sample shadowing, the shadow factor will be written into the precomputed shadow mask in the GBuffer instead
		&& !(PrimitiveSceneInfo->Proxy->UseSingleSampleShadowFromStationaryLights() && LightSceneInfo->Proxy->GetLightType() == LightType_Directional)
		&& Scene->GetFeatureLevel() >= ERHIFeatureLevel::SM5;

	if (bRenderPreShadow && ShouldUseCachePreshadows())
	{
		float PreshadowExpandFraction = FMath::Max(CVarPreshadowExpandFraction.GetValueOnRenderThread(), 0.0f);

		// If we're creating a preshadow, expand the bounds somewhat so that the preshadow will be cached more often as the shadow caster moves around.
		//@todo - only expand the preshadow bounds for this, not the per object shadow.
		Bounds.SphereRadius += (Bounds.BoxExtent * PreshadowExpandFraction).Size();
		Bounds.BoxExtent *= PreshadowExpandFraction + 1.0f;
	}

	// Compute the projected shadow initializer for this primitive-light pair.
	FPerObjectProjectedShadowInitializer ShadowInitializer;

	if ((MaxResolutionFadeAlpha > 1.0f / 256.0f || (bRenderPreShadow && MaxResolutionPreShadowFadeAlpha > 1.0f / 256.0f))
		&& LightSceneInfo->Proxy->GetPerObjectProjectedShadowInitializer(Bounds, ShadowInitializer))
	{
		const float MaxFadeAlpha = MaxResolutionFadeAlpha;

		// Only create a shadow from this object if it hasn't completely faded away
		if (CVarAllowPerObjectShadows.GetValueOnRenderThread() && MaxFadeAlpha > 1.0f / 256.0f)
		{
			// Round down to the nearest power of two so that resolution changes are always doubling or halving the resolution, which increases filtering stability
			// Use the max resolution if the desired resolution is larger than that
			int32 SizeX = bCanUseShadowDepthAtlas	? (1 << (FMath::CeilLogTwo(MaxDesiredResolution) - 1))
													: (FMath::RoundUpToPowerOfTwo(MaxDesiredResolution) - SHADOW_BORDER * 2); // see AllocateAtlasedShadowDepthTargets, we round up target size there

			SizeX = FMath::Min<int32>(SizeX, MaxShadowResolution);


			if (bOpaque && bCreateOpaqueObjectShadow && (bOpaqueShadowIsVisibleThisFrame || bShadowIsPotentiallyVisibleNextFrame))
			{
				// Create a projected shadow for this interaction's shadow.
				FProjectedShadowInfo* ProjectedShadowInfo = Allocator.Create<FProjectedShadowInfo>();
				ProjectedShadowInfo->bPerObjectOpaqueShadow = true;
				ProjectedShadowInfo->SubjectPrimitiveComponentIndex = PrimitiveSceneInfo->PrimitiveComponentId.PrimIDValue;

				// Disable Nanite geometry in per-object shadows, otherwise they will contain double shadow for Nanite stuff since it is not filtered as host-side geo.
				// As a result, per-object shadows for any Nanite object will not work (but that can be included in virtual SMs).
				// NOTE: This also means that per-object shadows such as for movable stuff in stationary lights will also not work, if they are nanite. However, this path should probably be removed.
				// If we need to keep this the only clear solution is to enable filtering the primitives in the GPU scene also, so Nanite geo can be partitioned into different SMs as well.
				ProjectedShadowInfo->bNaniteGeometry = false;

				if(ProjectedShadowInfo->SetupPerObjectProjection(
					LightSceneInfo,
					PrimitiveSceneInfo,
					ShadowInitializer,
					false,					// no preshadow
					SizeX,
					MaxShadowResolutionY,
					SHADOW_BORDER,
					MaxScreenPercent,
					false))					// no translucent shadow
				{
					ProjectedShadowInfo->FadeAlphas = ResolutionFadeAlphas;

					if (bOpaqueShadowIsVisibleThisFrame)
					{
						VisibleLightInfo.AllProjectedShadows.Add(ProjectedShadowInfo);

						for (int32 ChildIndex = 0, ChildCount = ShadowGroupPrimitives.Num(); ChildIndex < ChildCount; ChildIndex++)
						{
							FPrimitiveSceneInfo* ShadowChild = ShadowGroupPrimitives[ChildIndex];
							ProjectedShadowInfo->AddSubjectPrimitive(ShadowChild, Views, false);
						}
					}
					else if (bShadowIsPotentiallyVisibleNextFrame)
					{
						VisibleLightInfo.OccludedPerObjectShadows.Add(ProjectedShadowInfo);
					}
				}
			}

			if (AllowTranslucencyPerObjectShadows(Scene->GetShaderPlatform())
				&& bTranslucentRelevance
				&& bCreateTranslucentObjectShadow 
				&& (bTranslucentShadowIsVisibleThisFrame || bShadowIsPotentiallyVisibleNextFrame))
			{
				// Create a projected shadow for this interaction's shadow.
				FProjectedShadowInfo* ProjectedShadowInfo = Allocator.Create<FProjectedShadowInfo>();

				const FIntPoint TranslucentShadowTextureResolution = GetTranslucentShadowDepthTextureResolution();

				if(ProjectedShadowInfo->SetupPerObjectProjection(
					LightSceneInfo,
					PrimitiveSceneInfo,
					ShadowInitializer,
					false,					// no preshadow
					// Size was computed for the full res opaque shadow, convert to downsampled translucent shadow size with proper clamping
					FMath::Clamp<int32>(SizeX / GetTranslucentShadowDownsampleFactor(), 1, TranslucentShadowTextureResolution.X - SHADOW_BORDER * 2),
					FMath::Clamp<int32>(MaxShadowResolutionY / GetTranslucentShadowDownsampleFactor(), 1, TranslucentShadowTextureResolution.Y - SHADOW_BORDER * 2),
					SHADOW_BORDER,
					MaxScreenPercent,
					true))					// translucent shadow
				{
					ProjectedShadowInfo->FadeAlphas = ResolutionFadeAlphas;

					if (bTranslucentShadowIsVisibleThisFrame)
					{
						VisibleLightInfo.AllProjectedShadows.Add(ProjectedShadowInfo);

						for (int32 ChildIndex = 0, ChildCount = ShadowGroupPrimitives.Num(); ChildIndex < ChildCount; ChildIndex++)
						{
							FPrimitiveSceneInfo* ShadowChild = ShadowGroupPrimitives[ChildIndex];
							ProjectedShadowInfo->AddSubjectPrimitive(ShadowChild, Views, false);
						}
					}
					else if (bShadowIsPotentiallyVisibleNextFrame)
					{
						VisibleLightInfo.OccludedPerObjectShadows.Add(ProjectedShadowInfo);
					}
				}
			}
		}

		const float MaxPreFadeAlpha = MaxResolutionPreShadowFadeAlpha;

		// If the subject is visible in at least one view, create a preshadow for static primitives shadowing the subject.
		if (MaxPreFadeAlpha > 1.0f / 256.0f 
			&& bRenderPreShadow
			&& bOpaque)
		{
			// Round down to the nearest power of two so that resolution changes are always doubling or halving the resolution, which increases filtering stability.
			int32 PreshadowSizeX = 1 << (FMath::CeilLogTwo(FMath::TruncToInt(MaxDesiredResolution * CVarPreShadowResolutionFactor.GetValueOnRenderThread())) - 1);

			const FIntPoint PreshadowCacheResolution = GetPreShadowCacheTextureResolution(FeatureLevel);
			checkSlow(PreshadowSizeX <= PreshadowCacheResolution.X);
			bool bIsOutsideWholeSceneShadow = true;

			for (int32 i = 0; i < ViewDependentWholeSceneShadows.Num(); i++)
			{
				const FProjectedShadowInfo* WholeSceneShadow = ViewDependentWholeSceneShadows[i];
				const FVector2D DistanceFadeValues = WholeSceneShadow->GetLightSceneInfo().Proxy->GetDirectionalLightDistanceFadeParameters(Scene->GetFeatureLevel(), WholeSceneShadow->GetLightSceneInfo().IsPrecomputedLightingValid(), WholeSceneShadow->DependentView->MaxShadowCascades);
				const float DistanceFromShadowCenterSquared = (WholeSceneShadow->ShadowBounds.Center - Bounds.Origin).SizeSquared();
				//@todo - if view dependent whole scene shadows are ever supported in splitscreen, 
				// We can only disable the preshadow at this point if it is inside a whole scene shadow for all views
				const float DistanceFromViewSquared = ((FVector)WholeSceneShadow->DependentView->GetShadowViewMatrices().GetViewOrigin() - Bounds.Origin).SizeSquared();
				// Mark the preshadow as inside the whole scene shadow if its bounding sphere is inside the near fade distance
				if (DistanceFromShadowCenterSquared < FMath::Square(FMath::Max(WholeSceneShadow->ShadowBounds.W - Bounds.SphereRadius, 0.0f))
					//@todo - why is this extra threshold required?
					&& DistanceFromViewSquared < FMath::Square(FMath::Max(DistanceFadeValues.X - 200.0f - Bounds.SphereRadius, 0.0f)))
				{
					bIsOutsideWholeSceneShadow = false;
					break;
				}
			}

			// Only create opaque preshadows when part of the caster is outside the whole scene shadow.
			if (bIsOutsideWholeSceneShadow)
			{
				// Try to reuse a preshadow from the cache
				TRefCountPtr<FProjectedShadowInfo> ProjectedPreShadowInfo = GetCachedPreshadow(Interaction, ShadowInitializer, OriginalBounds, PreshadowSizeX);

				bool bOk = true;

				if(!ProjectedPreShadowInfo)
				{
					// Create a new projected shadow for this interaction's preshadow
					// Not using the scene rendering mem stack because this shadow info may need to persist for multiple frames if it gets cached
					ProjectedPreShadowInfo = new FProjectedShadowInfo;

					bOk = ProjectedPreShadowInfo->SetupPerObjectProjection(
						LightSceneInfo,
						PrimitiveSceneInfo,
						ShadowInitializer,
						true,				// preshadow
						PreshadowSizeX,
						FMath::TruncToInt(MaxShadowResolutionY * CVarPreShadowResolutionFactor.GetValueOnRenderThread()),
						SHADOW_BORDER,
						MaxScreenPercent,
						false				// not translucent shadow
						);
				}

				if (bOk)
				{

					// Update fade alpha on the cached preshadow
					ProjectedPreShadowInfo->FadeAlphas = ResolutionPreShadowFadeAlphas;

					VisibleLightInfo.AllProjectedShadows.Add(ProjectedPreShadowInfo);
					VisibleLightInfo.ProjectedPreShadows.Add(ProjectedPreShadowInfo);

					// Only add to OutPreShadows if the preshadow doesn't already have depths cached, 
					// Since OutPreShadows is used to generate information only used when rendering the shadow depths.
					if (!ProjectedPreShadowInfo->bDepthsCached && ProjectedPreShadowInfo->CasterOuterFrustum.PermutedPlanes.Num())
					{
						OutPreShadows.Add(ProjectedPreShadowInfo);
					}

					for (int32 ChildIndex = 0; ChildIndex < ShadowGroupPrimitives.Num(); ChildIndex++)
					{
						FPrimitiveSceneInfo* ShadowChild = ShadowGroupPrimitives[ChildIndex];
						bool bChildIsVisibleInAnyView = false;
						for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
						{
							const FViewInfo& View = Views[ViewIndex];
							if (View.PrimitiveVisibilityMap[ShadowChild->GetIndex()])
							{
								bChildIsVisibleInAnyView = true;
								break;
							}
						}
						if (bChildIsVisibleInAnyView)
						{
							ProjectedPreShadowInfo->AddReceiverPrimitive(ShadowChild);
						}
					}
				}
			}
		}
	}
}

static bool CanFallbackToOldShadowMapCache(const FShadowMapRenderTargetsRefCounted& CachedShadowMap, const FIntPoint& MaxShadowResolution)
{
	return CachedShadowMap.IsValid()
		&& CachedShadowMap.GetSize().X <= MaxShadowResolution.X
		&& CachedShadowMap.GetSize().Y <= MaxShadowResolution.Y;
}

void ComputeWholeSceneShadowCacheModes(
	const FLightSceneInfo* LightSceneInfo,
	bool bCubeShadowMap,
	float RealTime,
	float ActualDesiredResolution,
	const FIntPoint& MaxShadowResolution,
	FScene* Scene,
	FWholeSceneProjectedShadowInitializer& InOutProjectedShadowInitializer,
	const int64 CachedShadowMapsSize,
	FIntPoint& InOutShadowMapSize,
	uint32& InOutNumPointShadowCachesUpdatedThisFrame,
	uint32& InOutNumSpotShadowCachesUpdatedThisFrame,
	int32& OutNumShadowMaps, 
	EShadowDepthCacheMode* OutCacheModes)
{
	// Strategy:
	// - Try to fallback if over budget. Budget is defined as number of updates currently
	// - Only allow fallback for updates caused by resolution changes
	// - Always render if cache doesn't exist or has been released
	uint32* NumCachesUpdatedThisFrame = nullptr;
	uint32 MaxCacheUpdatesAllowed = 0;

	switch (LightSceneInfo->Proxy->GetLightType())
	{
	case LightType_Point:
	case LightType_Rect:
		NumCachesUpdatedThisFrame = &InOutNumPointShadowCachesUpdatedThisFrame;
		MaxCacheUpdatesAllowed = static_cast<uint32>(GMaxNumPointShadowCacheUpdatesPerFrame);
		break;
	case LightType_Spot:
		NumCachesUpdatedThisFrame = &InOutNumSpotShadowCachesUpdatedThisFrame;
		MaxCacheUpdatesAllowed = static_cast<uint32>(GMaxNumSpotShadowCacheUpdatesPerFrame);
		break;
	default:
		checkf(false, TEXT("Directional light isn't handled here"));
		break;
	}

	FCSMSceneExtension& CSMShadowScene = Scene->GetExtension<FCSMSceneExtension>();

	if (GCacheWholeSceneShadows 
		&& (!bCubeShadowMap || RHISupportsGeometryShaders(GShaderPlatformForFeatureLevel[Scene->GetFeatureLevel()]) || (RHISupportsVertexShaderLayer(Scene->GetShaderPlatform()) && GRHISupportsArrayIndexFromAnyShader)))
	{
		TArray<FCachedShadowMapData>* CachedShadowMapDatas = CSMShadowScene.GetCachedShadowMapDatas(LightSceneInfo->Id);

		if (CachedShadowMapDatas)
		{
			FCachedShadowMapData* CachedShadowMapData = &(*CachedShadowMapDatas)[0];

			if (InOutProjectedShadowInitializer.IsCachedShadowValid(CachedShadowMapData->Initializer) 
				&& CachedShadowMapData->bCachedShadowMapHasNaniteGeometry)
			{
				if (CachedShadowMapData->ShadowMap.IsValid() && CachedShadowMapData->ShadowMap.GetSize() == InOutShadowMapSize)
				{
					OutNumShadowMaps = 1;
					OutCacheModes[0] = SDCM_MovablePrimitivesOnly;
				}
				else
				{
					TryToCacheShadowMap(Scene, CachedShadowMapsSize, OutNumShadowMaps, OutCacheModes, *NumCachesUpdatedThisFrame,
					[&]()
					{
						// Check if update is caused by resolution change
						if (CanFallbackToOldShadowMapCache(CachedShadowMapData->ShadowMap, MaxShadowResolution))
						{
							FIntPoint ExistingShadowMapSize = CachedShadowMapData->ShadowMap.GetSize();
							bool bOverBudget = *NumCachesUpdatedThisFrame > MaxCacheUpdatesAllowed;
							bool bRejectedByGuardBand = false;

							// Only allow shrinking if actual desired resolution has dropped enough.
							// This creates a guard band and hence avoid thrashing
							if (!bOverBudget
								&& (InOutShadowMapSize.X < ExistingShadowMapSize.X
								|| InOutShadowMapSize.Y < ExistingShadowMapSize.Y))
							{
								FVector2D VecNewSize = static_cast<FVector2D>(InOutShadowMapSize);
								FVector2D VecExistingSize = static_cast<FVector2D>(ExistingShadowMapSize);
								FVector2D VecDesiredSize(ActualDesiredResolution, ActualDesiredResolution);
#if DO_CHECK
								checkf(ExistingShadowMapSize.X > 0 && ExistingShadowMapSize.Y > 0,
									TEXT("%d, %d"), ExistingShadowMapSize.X, ExistingShadowMapSize.Y);
#endif
								FVector2D DropRatio = (VecExistingSize - VecDesiredSize) / (VecExistingSize - VecNewSize);
								float MaxDropRatio = FMath::Max(
									InOutShadowMapSize.X < ExistingShadowMapSize.X ? DropRatio.X : 0.f,
									InOutShadowMapSize.Y < ExistingShadowMapSize.Y ? DropRatio.Y : 0.f);

								// MaxDropRatio <= 0 can happen when max shadow map resolution is lowered (for example,
								// by changing quality settings). In that case, just let it happen.
								bRejectedByGuardBand = MaxDropRatio > 0.f && MaxDropRatio < 0.5f;
							}

							if (bOverBudget || bRejectedByGuardBand)
							{
								// Fallback to existing shadow cache
								InOutShadowMapSize = CachedShadowMapData->ShadowMap.GetSize();
								InOutProjectedShadowInitializer = CachedShadowMapData->Initializer;
								OutNumShadowMaps = 1;
								OutCacheModes[0] = SDCM_MovablePrimitivesOnly;
								--*NumCachesUpdatedThisFrame;
							}
						}
					},
					[&]()
					{
						CachedShadowMapData->InvalidateCachedShadow();
					});
				}
			}
			else
			{
				OutNumShadowMaps = 1;
				OutCacheModes[0] = SDCM_Uncached;
				CachedShadowMapData->InvalidateCachedShadow();
			}
			
			CachedShadowMapData->Initializer = InOutProjectedShadowInitializer;
			CachedShadowMapData->LastUsedTime = RealTime;
		}
		else
		{
			TryToCacheShadowMap(Scene, CachedShadowMapsSize, OutNumShadowMaps, OutCacheModes, *NumCachesUpdatedThisFrame,
			[&]()
			{
				CSMShadowScene.GetAllCachedShadowMaps().Add(LightSceneInfo->Id).Add(FCachedShadowMapData(InOutProjectedShadowInitializer, RealTime));
			},
			[&]()
			{
			});
		}
	}
	else
	{
		OutNumShadowMaps = 1;
		OutCacheModes[0] = SDCM_Uncached;
		CSMShadowScene.GetAllCachedShadowMaps().Remove(LightSceneInfo->Id);
	}

	if (OutNumShadowMaps > 0)
	{
		int32 NumOcclusionQueryableShadows = 0;

		for (int32 i = 0; i < OutNumShadowMaps; i++)
		{
			NumOcclusionQueryableShadows += IsShadowCacheModeOcclusionQueryable(OutCacheModes[i]);
		}

		// Verify only one of the shadows will be occlusion queried, since they are all for the same light bounds
		check(NumOcclusionQueryableShadows == 1);
	}
}

void ComputeViewDependentWholeSceneShadowCacheModes(
	const FLightSceneInfo* LightSceneInfo,
	float RealTime,
	FScene* Scene,
	const FWholeSceneProjectedShadowInitializer& ProjectedShadowInitializer,
	const int64 CachedShadowMapsSize,
	const FIntPoint& ShadowMapSize,
	uint32& InOutNumCSMCachesUpdatedThisFrame,
	int32& OutNumShadowMaps,
	EShadowDepthCacheMode* OutCacheModes)
{
	FCSMSceneExtension& CSMShadowScene = Scene->GetExtension<FCSMSceneExtension>();

	if (CVarCSMCaching.GetValueOnAnyThread() == 1 && !UseVirtualShadowMaps(Scene->GetShaderPlatform(), Scene->GetFeatureLevel()))
	{
		TArray<FCachedShadowMapData>* CachedShadowMapDatas = CSMShadowScene.GetCachedShadowMapDatas(LightSceneInfo->Id);

		checkSlow(ProjectedShadowInitializer.CascadeSettings.ShadowSplitIndex >= 0);
		FCachedShadowMapData* CachedShadowMapData = (CachedShadowMapDatas && CachedShadowMapDatas->Num() > ProjectedShadowInitializer.CascadeSettings.ShadowSplitIndex) ? &(*CachedShadowMapDatas)[ProjectedShadowInitializer.CascadeSettings.ShadowSplitIndex] : nullptr;

		if (CachedShadowMapData)
		{
			const FWholeSceneProjectedShadowInitializer& CachedShadowInitializer = CachedShadowMapData->Initializer;

			// The cached shadow map could be valid if the light direction doesn't change.
			bool bIsValidCachedViewDependentWholeSceneShadow = ProjectedShadowInitializer.WorldToLight == CachedShadowInitializer.WorldToLight
				&& ProjectedShadowInitializer.WAxis == CachedShadowInitializer.WAxis
				&& ProjectedShadowInitializer.bOnePassPointLightShadow == CachedShadowInitializer.bOnePassPointLightShadow
				&& ProjectedShadowInitializer.bRayTracedDistanceField == CachedShadowInitializer.bRayTracedDistanceField
				&& ProjectedShadowInitializer.SubjectBounds.SphereRadius == CachedShadowInitializer.SubjectBounds.SphereRadius;

			if (bIsValidCachedViewDependentWholeSceneShadow)
			{
				bool bExactlyEqual = ProjectedShadowInitializer.PreShadowTranslation == CachedShadowInitializer.PreShadowTranslation
					&& ProjectedShadowInitializer.Scales == CachedShadowInitializer.Scales
					&& ProjectedShadowInitializer.SubjectBounds.Origin == CachedShadowInitializer.SubjectBounds.Origin
					&& ProjectedShadowInitializer.MinLightW == CachedShadowInitializer.MinLightW
					&& ProjectedShadowInitializer.MaxDistanceToCastInLightW == CachedShadowInitializer.MaxDistanceToCastInLightW;

				// Reuse the cached shadow map if it is valid and the extra draw calls of the static meshes are less than a throttle
				if (CachedShadowMapData->ShadowMap.IsValid() && CachedShadowMapData->ShadowBufferResolution == ShadowMapSize && CachedShadowMapData->LastFrameExtraStaticShadowSubjects <= CVarCSMScrollingMaxExtraStaticShadowSubjects.GetValueOnAnyThread())
				{
					if (bExactlyEqual)
					{
						OutNumShadowMaps = 1;
						OutCacheModes[0] = SDCM_MovablePrimitivesOnly;
					}
					else
					{
						const FVector FaceDirection(1, 0, 0);
						FVector	XAxis, YAxis;
						FaceDirection.FindBestAxisVectors(XAxis, YAxis);
						const FMatrix WorldToLight = ProjectedShadowInitializer.WorldToLight * FBasisVectorMatrix(-XAxis, YAxis, FaceDirection.GetSafeNormal(), FVector::ZeroVector);

						FVector UpVector = FVector(WorldToLight.M[0][1], WorldToLight.M[1][1], WorldToLight.M[2][1]);
						FVector RightVector = FVector(WorldToLight.M[0][0], WorldToLight.M[1][0], WorldToLight.M[2][0]);

						// Projected the centers to the light coordinate
						FVector ProjectedCurrentCenter = WorldToLight.TransformPosition(-ProjectedShadowInitializer.PreShadowTranslation);
						FVector ProjectedCachedCenter = WorldToLight.TransformPosition(-CachedShadowMapData->Initializer.PreShadowTranslation);
						FVector ProjectedCenterVector = ProjectedCachedCenter - ProjectedCurrentCenter;

						FBox CurrentViewport = FBox(FVector(-ProjectedShadowInitializer.SubjectBounds.SphereRadius, -ProjectedShadowInitializer.SubjectBounds.SphereRadius, 0.0f), FVector(ProjectedShadowInitializer.SubjectBounds.SphereRadius, ProjectedShadowInitializer.SubjectBounds.SphereRadius, 0.0f));
						FBox CachedViewport = FBox(FVector(ProjectedCenterVector.X - CachedShadowMapData->Initializer.SubjectBounds.SphereRadius, ProjectedCenterVector.Y - CachedShadowMapData->Initializer.SubjectBounds.SphereRadius, 0.0f), FVector(ProjectedCenterVector.X + CachedShadowMapData->Initializer.SubjectBounds.SphereRadius, ProjectedCenterVector.Y + CachedShadowMapData->Initializer.SubjectBounds.SphereRadius, 0.0f));

						FBox OverlappedViewport = CurrentViewport.Overlap(CachedViewport);

						FVector OverlappedArea = OverlappedViewport.GetSize();

						float OverlappedAreaRatio = (OverlappedArea.X * OverlappedArea.Y) / (CachedShadowMapData->Initializer.SubjectBounds.SphereRadius * CachedShadowMapData->Initializer.SubjectBounds.SphereRadius * 4);

						// if the overlapped area of the cached shadow map and the current shadow map smaller than a throttle, update the cached shadow map, otherwise scroll the shadow map instead.
						if (OverlappedAreaRatio > CVarCSMScrollingOverlapAreaThrottle.GetValueOnAnyThread())
						{
							OutNumShadowMaps = 1;
							OutCacheModes[0] = SDCM_CSMScrolling;
						}
						else
						{
							OutNumShadowMaps = 2;
							OutCacheModes[0] = SDCM_StaticPrimitivesOnly;
							OutCacheModes[1] = SDCM_MovablePrimitivesOnly;
							++InOutNumCSMCachesUpdatedThisFrame;

							CachedShadowMapData->Initializer = ProjectedShadowInitializer;
							CachedShadowMapData->InvalidateCachedShadow();
							checkSlow(CachedShadowMapData->StaticShadowSubjectPersistentPrimitiveIdMap.Num() == Scene->GetMaxPersistentPrimitiveIndex());
						}
					}
				}
				else
				{
					TryToCacheShadowMap(Scene, CachedShadowMapsSize, OutNumShadowMaps, OutCacheModes, InOutNumCSMCachesUpdatedThisFrame,
					[&]()
					{
						CachedShadowMapData->ShadowBufferResolution = ShadowMapSize;
						CachedShadowMapData->Initializer = ProjectedShadowInitializer;
						checkSlow(CachedShadowMapData->StaticShadowSubjectPersistentPrimitiveIdMap.Num() == Scene->GetMaxPersistentPrimitiveIndex());
					},
					[&]()
					{
						CachedShadowMapData->InvalidateCachedShadow();
						CachedShadowMapData->Initializer = ProjectedShadowInitializer;
					});
				}
			}
			else
			{
				OutNumShadowMaps = 1;
				OutCacheModes[0] = SDCM_Uncached;
				CachedShadowMapData->InvalidateCachedShadow();
				CachedShadowMapData->Initializer = ProjectedShadowInitializer;
			}

			CachedShadowMapData->LastUsedTime = RealTime;
		}
		else
		{
			TryToCacheShadowMap(Scene, CachedShadowMapsSize, OutNumShadowMaps, OutCacheModes, InOutNumCSMCachesUpdatedThisFrame,
			[&]()
			{
				if (CachedShadowMapDatas == nullptr)
				{
					CachedShadowMapDatas = &CSMShadowScene.GetAllCachedShadowMaps().Add(LightSceneInfo->Id);
				}
				CachedShadowMapDatas->Add(FCachedShadowMapData(ProjectedShadowInitializer, RealTime));
				CachedShadowMapData = &CachedShadowMapDatas->Last();
				CachedShadowMapData->ShadowBufferResolution = ShadowMapSize;
				CachedShadowMapData->StaticShadowSubjectPersistentPrimitiveIdMap.Init(false, Scene->GetMaxPersistentPrimitiveIndex());
			},
			[&]()
			{
			});
		}
	}
	else
	{
		OutNumShadowMaps = 1;
		OutCacheModes[0] = SDCM_Uncached;
		CSMShadowScene.GetAllCachedShadowMaps().Remove(LightSceneInfo->Id);
	}

	if (OutNumShadowMaps > 0)
	{
		int32 NumOcclusionQueryableShadows = 0;

		for (int32 i = 0; i < OutNumShadowMaps; i++)
		{
			NumOcclusionQueryableShadows += IsShadowCacheModeOcclusionQueryable(OutCacheModes[i]);
		}

		// Verify only one of the shadows will be occlusion queried, since they are all for the same light bounds
		check(NumOcclusionQueryableShadows == 1);
	}
}

typedef TArray<FConvexVolume, TInlineAllocator<8>> FLightViewFrustumConvexHulls;

// Builds a shadow convex hull based on frustum and and (point/spot) light position
// Based on: https://udn.unrealengine.com/questions/410475/improved-culling-of-shadow-casters-for-spotlights.html
void BuildLightViewFrustumConvexHull(const FVector& LightOrigin, const FConvexVolume& Frustum, FConvexVolume& ConvexHull)
{
	// This function works with 4 frustum planes
	// We may occasionally see a 5th if OverrideFarClippingPlaneDistance is used such as in the Editor, but we ignore that 
	// virtual plane (which we assume is the last in the list) for the purposes of culling here.
	const int32 EdgeCount = 4;
	const int32 PlaneCount = 4;
	check(Frustum.Planes.Num() == 4 || Frustum.Planes.Num() == 5);

	enum EFrustumPlanes
	{
		FLeft,
		FRight,
		FTop,
		FBottom
	};

	const EFrustumPlanes Edges[EdgeCount][2] =
	{
		{ FLeft , FTop }, { FLeft , FBottom },
		{ FRight, FTop }, { FRight, FBottom }
	};

	float Distance[PlaneCount];
	bool  Visible[PlaneCount];

	for (int32 PlaneIndex = 0; PlaneIndex < PlaneCount; ++PlaneIndex)
	{
		const FPlane& Plane = Frustum.Planes[PlaneIndex];
		float Dist = Plane.PlaneDot(LightOrigin);
		bool bVisible = Dist < 0.0f;

		Distance[PlaneIndex] = Dist;
		Visible[PlaneIndex] = bVisible;

		if (bVisible)
		{
			ConvexHull.Planes.Add(Plane);
		}
	}

	for (int32 EdgeIndex = 0; EdgeIndex < EdgeCount; ++EdgeIndex)
	{
		EFrustumPlanes I1 = Edges[EdgeIndex][0];
		EFrustumPlanes I2 = Edges[EdgeIndex][1];

		// Silhouette edge
		if (Visible[I1] != Visible[I2])
		{
			// Add plane that passes through edge and light origin
			FPlane Plane = Frustum.Planes[I1] * Distance[I2] - Frustum.Planes[I2] * Distance[I1];
			if (Visible[I2])
			{
				Plane = Plane.Flip();
			}
			ConvexHull.Planes.Add(Plane);
		}
	}

	ConvexHull.Init();
}

void BuildLightViewFrustumConvexHulls(const FVector& LightOrigin, const TArray<FViewInfo>& Views, FLightViewFrustumConvexHulls& ConvexHulls)
{
	if (GShadowLightViewConvexHullCull == 0)
	{
		return;
	}


	ConvexHulls.Reserve(Views.Num());
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		// for now only support perspective projection as ortho camera shadows are broken anyway
		FViewInfo const& View = Views[ViewIndex];
		if (View.IsPerspectiveProjection())
		{
			FConvexVolume ConvexHull;
			BuildLightViewFrustumConvexHull(LightOrigin, View.ViewFrustum, ConvexHull);
			ConvexHulls.Add(ConvexHull);
		}
	}
}

bool IntersectsConvexHulls(FLightViewFrustumConvexHulls const& ConvexHulls, FBoxSphereBounds const& Bounds)
{
	if (ConvexHulls.Num() == 0)
	{
		return true;
	}

	for (int32 Index = 0; Index < ConvexHulls.Num(); ++Index)
	{
		FConvexVolume const& Hull = ConvexHulls[Index];
		if (Hull.IntersectBox(Bounds.Origin, Bounds.BoxExtent))
		{
			return true;
		}
	}

	return false;
}



void FSceneRenderer::CreateWholeSceneProjectedShadow(
	FDynamicShadowsTaskData& TaskData,
	FLightSceneInfo* LightSceneInfo,
	int64 CachedShadowMapsSize,
	uint32& InOutNumPointShadowCachesUpdatedThisFrame,
	uint32& InOutNumSpotShadowCachesUpdatedThisFrame)
{
	SCOPE_CYCLE_COUNTER(STAT_CreateWholeSceneProjectedShadow);
	FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];

	// early out if shadow resoluion scale is zero
	if (CVarResolutionScaleZeroDisablesSm.GetValueOnRenderThread() != 0 && LightSceneInfo->Proxy->GetShadowResolutionScale() <= 0.0f)
	{
		return;
	}

	// Try to create a whole-scene projected shadow initializer for the light.
	TArray<FWholeSceneProjectedShadowInitializer, TInlineAllocator<6> > ProjectedShadowInitializers;
	if (LightSceneInfo->Proxy->GetWholeSceneProjectedShadowInitializer(ViewFamily, ProjectedShadowInitializers))
	{
		checkSlow(ProjectedShadowInitializers.Num() > 0);

		// Shadow resolution constants.
		const uint32 ShadowBorder = ProjectedShadowInitializers[0].bOnePassPointLightShadow ? 0 : SHADOW_BORDER;
		const uint32 EffectiveDoubleShadowBorder = ShadowBorder * 2;
		const uint32 MinShadowResolution = FMath::Max<int32>(0, CVarMinShadowResolution.GetValueOnRenderThread());
		const int32 MaxShadowResolutionSetting = GetCachedScalabilityCVars().MaxShadowResolution;
		const FIntPoint ShadowBufferResolution = GetShadowDepthTextureResolution(FeatureLevel);
		const uint32 MaxShadowResolution = FMath::Min(MaxShadowResolutionSetting, ShadowBufferResolution.X) - EffectiveDoubleShadowBorder;
		const uint32 MaxShadowResolutionY = FMath::Min(MaxShadowResolutionSetting, ShadowBufferResolution.Y) - EffectiveDoubleShadowBorder;
		const uint32 ShadowFadeResolution = FMath::Max<int32>(0, CVarShadowFadeResolution.GetValueOnRenderThread());
				
		// Compute the maximum resolution required for the shadow by any view. Also keep track of the unclamped resolution for fading.
		float MaxDesiredResolution = 0.0f;
		float MaxScreenRadius = 0.0f;

		TArray<float, TInlineAllocator<2> > FadeAlphas;
		const int32 ViewCount = Views.Num(); 
		FadeAlphas.SetNumZeroed(ViewCount);

		float MaxFadeAlpha = 0.0f;
		bool bStaticSceneOnly = false;
		bool bAnyViewIsSceneCapture = false;

		// For whole scene lights, select the view index up-front such that culling & rendering can be consistent.
		int32 SelectedViewIndex = 0;

		for(int32 ViewIndex = 0; ViewIndex < ViewCount; ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			const FVisibleLightViewInfo& VisibleLightViewInfo = View.VisibleLightInfos[LightSceneInfo->Id];

			if (!VisibleLightViewInfo.bInViewFrustum)
			{
				continue;
			}

			const float ScreenRadius = LightSceneInfo->Proxy->GetEffectiveScreenRadius(View.GetShadowViewMatrices(), View.ViewRect.Size());
			if (ScreenRadius > MaxScreenRadius)
			{
				SelectedViewIndex = ViewIndex;
				MaxScreenRadius = ScreenRadius;
			}

			// Determine the amount of shadow buffer resolution needed for this view.
			float UnclampedResolution = 1.0f;

			switch (LightSceneInfo->Proxy->GetLightType())
			{
			case LightType_Point:
				UnclampedResolution = ScreenRadius * CVarShadowTexelsPerPixelPointlight.GetValueOnRenderThread();
				break;
			case LightType_Spot:
				UnclampedResolution = ScreenRadius * CVarShadowTexelsPerPixelSpotlight.GetValueOnRenderThread();
				break;
			case LightType_Rect:
				UnclampedResolution = ScreenRadius * CVarShadowTexelsPerPixelRectlight.GetValueOnRenderThread();
				break;
			default:
				// directional lights are not handled here
				checkf(false, TEXT("Unexpected LightType %d appears in CreateWholeSceneProjectedShadow %s"),
					(int32)LightSceneInfo->Proxy->GetLightType(),
					*LightSceneInfo->Proxy->GetOwnerNameOrLabel());
			}

			const float ClampedResolution = ClampShadowRolution(UnclampedResolution, LightSceneInfo->Proxy->GetShadowResolutionScale(), MaxShadowResolution);

			{
				// pick resolution to use for fade calcs.
				// Compute FadeAlpha before or after ShadowResolutionScale contribution (some artists may want to modify the softness of the shadow, not change the fade ranges, others do)		
				const float FadeCalcResolution = GDoesShadowFadeUseResolutionScale ? ClampedResolution : UnclampedResolution;
				const float FadeAlpha = CalculateShadowFadeAlpha( FadeCalcResolution, ShadowFadeResolution, MinShadowResolution ) * LightSceneInfo->Proxy->GetShadowAmount();
				MaxFadeAlpha = FMath::Max(MaxFadeAlpha, FadeAlpha);
				FadeAlphas[ViewIndex] = FadeAlpha;
			}
		

			MaxDesiredResolution = FMath::Max(
				MaxDesiredResolution,
				FMath::Max<float>(
					ClampedResolution,
					FMath::Min<float>(MinShadowResolution, ShadowBufferResolution.X - EffectiveDoubleShadowBorder)
					)
				);

			bStaticSceneOnly = bStaticSceneOnly || View.bStaticSceneOnly;
			bAnyViewIsSceneCapture = bAnyViewIsSceneCapture || View.bIsSceneCapture;
		}

		if (MaxFadeAlpha > 1.0f / 256.0f)
		{
			for (int32 ShadowIndex = 0, ShadowCount = ProjectedShadowInitializers.Num(); ShadowIndex < ShadowCount; ShadowIndex++)
			{
				FWholeSceneProjectedShadowInitializer& ProjectedShadowInitializer = ProjectedShadowInitializers[ShadowIndex];
				
				// Consolidate repeated operation into a single site.
				auto AddInteractingPrimitives = [bStaticSceneOnly, &Views = this->Views, FeatureLevel = this->FeatureLevel, &TaskData]
				(FLightPrimitiveInteraction* InteractionList, 
					FProjectedShadowInfo* ProjectedShadowInfo, 
					bool& bContainsNaniteSubjectsOut, 
					const FLightViewFrustumConvexHulls& LightViewFrustumConvexHulls,
					const TSharedPtr<FVirtualShadowMapPerLightCacheEntry> &VirtualSmCacheEntry
				)
				{
					for (FLightPrimitiveInteraction* Interaction = InteractionList; Interaction; Interaction = Interaction->GetNextPrimitive())
					{
						if (Interaction->HasShadow()
							// If the primitive only wants to cast a self shadow don't include it in whole scene shadows.
							&& !Interaction->CastsSelfShadowOnly()
							&& (!bStaticSceneOnly || Interaction->GetPrimitiveSceneInfo()->Proxy->HasStaticLighting())
							&& !Interaction->HasInsetObjectShadow())
						{
							if (Interaction->IsNaniteMeshProxy())
							{
								bContainsNaniteSubjectsOut = true;
							}
							// EShadowMeshSelection::All is intentional because ProxySupportsGPUScene being false may indicate that some VFs (e.g., some LODs) support GPUScene while others don't
							// thus we have to leave the final decision until the mesh batches are produced.
							else if (EnumHasAnyFlags(ProjectedShadowInfo->MeshSelectionMask, Interaction->ProxySupportsGPUScene() ? EShadowMeshSelection::VSM : EShadowMeshSelection::All))
							{
								const FPrimitiveSceneInfo* PrimitiveSceneInfo = Interaction->GetPrimitiveSceneInfo();
								FBoxSphereBounds const& Bounds = PrimitiveSceneInfo->Proxy->GetBounds();
								if (IntersectsConvexHulls(LightViewFrustumConvexHulls, Bounds))
								{
									if (ProjectedShadowInfo->AddSubjectPrimitive(Interaction->GetPrimitiveSceneInfo(), Views, false)
										&& VirtualSmCacheEntry.IsValid())
									{
										// NOTE: We don't track revealed primitives for local lights
										VirtualSmCacheEntry->OnPrimitiveRendered(PrimitiveSceneInfo, false);
									}
								}
							}
						}
					}
				};
		
				// NOTE: Allow lights that explicitly opt in to distance field shadows to take precidence
				// We may want to change this directly in the proxy, but for now this maintains existing behavior
				if (LightSceneInfo->Proxy->UseVirtualShadowMaps() && !ProjectedShadowInitializer.bRayTracedDistanceField)
				{
					// VSMs should be enabled globally and proxy should never ask for more than one
					if (ensure(VirtualShadowMapArray.IsEnabled() &&	ProjectedShadowInitializers.Num() == 1))
					{
						// Create the projected shadow info.
						FProjectedShadowInfo* ProjectedShadowInfo = Allocator.Create<FProjectedShadowInfo>();

						int32 VirtualSizeX = FVirtualShadowMap::VirtualMaxResolutionXY;
						int32 VirtualSizeY = FVirtualShadowMap::VirtualMaxResolutionXY;						
					
						ProjectedShadowInfo->SetupWholeSceneProjection(
							LightSceneInfo,
							NULL,
							ProjectedShadowInitializer,
							VirtualSizeX,
							VirtualSizeY,
							VirtualSizeX,
							VirtualSizeY,
							0, // no border
							SelectedViewIndex
						);
						FShadowSceneRenderer& ShadowSceneRenderer = GetSceneExtensionsRenderers().GetRenderer<FShadowSceneRenderer>();

						TSharedPtr<FVirtualShadowMapPerLightCacheEntry> VirtualSmPerLightCacheEntry = ShadowSceneRenderer.AddLocalLightShadow(ProjectedShadowInitializer, ProjectedShadowInfo, LightSceneInfo, MaxScreenRadius);

						ProjectedShadowInfo->bVSM = true;
						ProjectedShadowInfo->MeshSelectionMask = EShadowMeshSelection::VSM;

						// Initialize to true if we are not creating LPIs for Nanite.
						bool bContainsNaniteSubjects = ShouldSkipNaniteLPIs(TaskData.Scene->GetShaderPlatform());

						// Skip mesh setup if it won't be rendered anyway
						if (ProjectedShadowInfo->bShouldRenderVSM)
						{
							ProjectedShadowInfo->SetupCullingQuery(*this);
							// Skip convex hull tests for VSM since this causes artifacts due to caching (potentially check if caching is enabled but may lead to race condition).
							// The interaction setup has already tested the light bounds (by calling FLightSceneProxy::AffectsBounds)
							FLightViewFrustumConvexHulls EmptyHull{};
							AddInteractingPrimitives(LightSceneInfo->GetDynamicInteractionOftenMovingPrimitiveList(), ProjectedShadowInfo, bContainsNaniteSubjects, EmptyHull, VirtualSmPerLightCacheEntry);
							AddInteractingPrimitives(LightSceneInfo->GetDynamicInteractionStaticPrimitiveList(), ProjectedShadowInfo, bContainsNaniteSubjects, EmptyHull, VirtualSmPerLightCacheEntry);
							ProjectedShadowInfo->bContainsNaniteSubjects = bContainsNaniteSubjects;
						}

						VisibleLightInfo.AllProjectedShadows.Add(ProjectedShadowInfo);
					}
				}
				else // No VSM, fall back to classic shadow map path
				{
					// Round down to the nearest power of two so that resolution changes are always doubling or halving the resolution, which increases filtering stability
					// Use the max resolution if the desired resolution is larger than that
					// FMath::CeilLogTwo(MaxDesiredResolution + 1.0f) instead of FMath::CeilLogTwo(MaxDesiredResolution) because FMath::CeilLogTwo takes
					// an uint32 as argument and this causes MaxDesiredResolution get truncated. For example, if MaxDesiredResolution is 256.1f,
					// FMath::CeilLogTwo returns 8 but the next line of code expects a 9 to work correctly
					int32 RoundedDesiredResolution = FMath::Max<int32>((1 << (FMath::CeilLogTwo(MaxDesiredResolution + 1.0f) - 1)) - (ShadowBorder * 2), 1);
					int32 SizeX = MaxDesiredResolution >= MaxShadowResolution ? MaxShadowResolution : RoundedDesiredResolution;
					int32 SizeY = MaxDesiredResolution >= MaxShadowResolutionY ? MaxShadowResolutionY : RoundedDesiredResolution;

					if (ProjectedShadowInitializer.bOnePassPointLightShadow)
					{
						// Round to a resolution that is supported for one pass point light shadows
						SizeX = SizeY = GetCubeShadowDepthZResolution(FeatureLevel, GetCubeShadowDepthZIndex(FeatureLevel, MaxDesiredResolution));
					}

					int32 NumShadowMaps = 1;
					EShadowDepthCacheMode CacheMode[2] = { SDCM_Uncached, SDCM_Uncached };

					if (!bAnyViewIsSceneCapture && !ProjectedShadowInitializer.bRayTracedDistanceField)
					{
						FIntPoint ShadowMapSize( SizeX + ShadowBorder * 2, SizeY + ShadowBorder * 2 );

						ComputeWholeSceneShadowCacheModes(
							LightSceneInfo,
							ProjectedShadowInitializer.bOnePassPointLightShadow,
							ViewFamily.Time.GetRealTimeSeconds(),
							MaxDesiredResolution,
							FIntPoint(MaxShadowResolution, MaxShadowResolutionY),
							Scene,
							// Below are in-out or out parameters. They can change
							ProjectedShadowInitializer,
							CachedShadowMapsSize,
							ShadowMapSize,
							InOutNumPointShadowCachesUpdatedThisFrame,
							InOutNumSpotShadowCachesUpdatedThisFrame,
							NumShadowMaps,
							CacheMode);

						SizeX = ShadowMapSize.X - ShadowBorder * 2;
						SizeY = ShadowMapSize.Y - ShadowBorder * 2;
					}

					for (int32 CacheModeIndex = 0; CacheModeIndex < NumShadowMaps; CacheModeIndex++)
					{
						// Create the projected shadow info.
						FProjectedShadowInfo* ProjectedShadowInfo = Allocator.Create<FProjectedShadowInfo>();

						ProjectedShadowInfo->SetupWholeSceneProjection(
							LightSceneInfo,
							NULL,
							ProjectedShadowInitializer,
							SizeX,
							SizeY,
							SizeX,
							SizeY,
							ShadowBorder,
							SelectedViewIndex
							);

						ProjectedShadowInfo->CacheMode = CacheMode[CacheModeIndex];
						ProjectedShadowInfo->FadeAlphas = FadeAlphas;

						bool bContainsNaniteSubjects = ShouldSkipNaniteLPIs(TaskData.Scene->GetShaderPlatform());

						// Ray traced shadows use the GPU managed distance field object buffers, no CPU culling should be used
						// Actually we probably want to have fallback even for these? Another performance regresion however (since we'd be adding culling & draw calls for DF lights).
						if (!ProjectedShadowInfo->bRayTracedDistanceField)
						{
							ProjectedShadowInfo->SetupCullingQuery(*this);

							// Build light-view convex hulls for shadow caster culling
							FLightViewFrustumConvexHulls LightViewFrustumConvexHulls{};
							if (CacheMode[CacheModeIndex] != SDCM_StaticPrimitivesOnly)
							{
								FVector const& LightOrigin = LightSceneInfo->Proxy->GetOrigin();
								BuildLightViewFrustumConvexHulls(LightOrigin, Views, LightViewFrustumConvexHulls);
							}

							bool bCastCachedShadowFromMovablePrimitives = GCachedShadowsCastFromMovablePrimitives || LightSceneInfo->Proxy->GetForceCachedShadowsForMovablePrimitives();
							if (CacheMode[CacheModeIndex] != SDCM_StaticPrimitivesOnly 
								&& (CacheMode[CacheModeIndex] != SDCM_MovablePrimitivesOnly || bCastCachedShadowFromMovablePrimitives))
							{
								// Add all the shadow casting primitives affected by the light to the shadow's subject primitive list.
								AddInteractingPrimitives(LightSceneInfo->GetDynamicInteractionOftenMovingPrimitiveList(), ProjectedShadowInfo, bContainsNaniteSubjects, LightViewFrustumConvexHulls, nullptr);
							}
						
							if (CacheMode[CacheModeIndex] != SDCM_MovablePrimitivesOnly)
							{
								AddInteractingPrimitives(LightSceneInfo->GetDynamicInteractionStaticPrimitiveList(), ProjectedShadowInfo, bContainsNaniteSubjects, LightViewFrustumConvexHulls, nullptr);
							}
						}
						ProjectedShadowInfo->bContainsNaniteSubjects = bContainsNaniteSubjects;

						bool bRenderShadow = true;
					
						if (CacheMode[CacheModeIndex] == SDCM_StaticPrimitivesOnly)
						{
							const bool bHasStaticPrimitives = ProjectedShadowInfo->HasSubjectPrims();
							bRenderShadow = bHasStaticPrimitives;
							FCachedShadowMapData& CachedShadowMapData = Scene->GetExtension<FCSMSceneExtension>().GetCachedShadowMapDataRef(ProjectedShadowInfo->GetLightSceneInfo().Id);
							CachedShadowMapData.bCachedShadowMapHasPrimitives = bHasStaticPrimitives;
							CachedShadowMapData.bCachedShadowMapHasNaniteGeometry = ProjectedShadowInfo->bNaniteGeometry;
						}

						if (bRenderShadow)
						{
							VisibleLightInfo.AllProjectedShadows.Add(ProjectedShadowInfo);
						}
					}
				}
			}
		}
	}
}

void FSceneRenderer::InitProjectedShadowVisibility(FDynamicShadowsTaskData& TaskData)
{
	SCOPE_CYCLE_COUNTER(STAT_InitProjectedShadowVisibility);
	int32 NumBufferedFrames = FOcclusionQueryHelpers::GetNumBufferedFrames(FeatureLevel);

	const bool bHairStrands = HairStrands::HasHairInstanceInScene(*Scene);

	// Initialize the views' ProjectedShadowVisibilityMaps and remove shadows without subjects.
	for(auto LightIt = Scene->Lights.CreateConstIterator(); LightIt; ++LightIt)
	{
		FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightIt.GetIndex()];

		// Allocate the light's projected shadow visibility and view relevance maps for this view.
		for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
		{
			FViewInfo& View = Views[ViewIndex];
			FVisibleLightViewInfo& VisibleLightViewInfo = View.VisibleLightInfos[LightIt.GetIndex()];
			VisibleLightViewInfo.ProjectedShadowVisibilityMap.Init(false,VisibleLightInfo.AllProjectedShadows.Num());
			VisibleLightViewInfo.ProjectedShadowViewRelevanceMap.Empty(VisibleLightInfo.AllProjectedShadows.Num());
			VisibleLightViewInfo.ProjectedShadowViewRelevanceMap.AddZeroed(VisibleLightInfo.AllProjectedShadows.Num());
		}

		for( int32 ShadowIndex=0; ShadowIndex<VisibleLightInfo.AllProjectedShadows.Num(); ShadowIndex++ )
		{
			FProjectedShadowInfo& ProjectedShadowInfo = *VisibleLightInfo.AllProjectedShadows[ShadowIndex];

			// Assign the shadow its id.
			ProjectedShadowInfo.ShadowId = ShadowIndex;

			for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
			{
				FViewInfo& View = Views[ViewIndex];

				if (ProjectedShadowInfo.DependentView && ProjectedShadowInfo.DependentView != &View)
				{
					// The view dependent projected shadow is valid for this view if it's the right eye and the projected shadow
					// is being rendered for the left eye, or if it's a cascade shared across scene capture cube faces.
					const bool bIsValidForView =
						((IStereoRendering::IsASecondaryView(View) && IStereoRendering::IsAPrimaryView(*ProjectedShadowInfo.DependentView)) ||
						 (View.bIsSceneCaptureCube && IsSceneCaptureCubeShareCascadesEnabled(View.GetShaderPlatform())))
						&& ProjectedShadowInfo.FadeAlphas.IsValidIndex(ViewIndex)
						&& ProjectedShadowInfo.FadeAlphas[ViewIndex] != 0.0f;

					if (!bIsValidForView)
					{
						continue;
					}
				}

				FVisibleLightViewInfo& VisibleLightViewInfo = View.VisibleLightInfos[LightIt.GetIndex()];

				if(VisibleLightViewInfo.bInViewFrustum)
				{
					// Compute the subject primitive's view relevance.  Note that the view won't necessarily have it cached,
					// since the primitive might not be visible.
					FPrimitiveViewRelevance ViewRelevance;
					if(ProjectedShadowInfo.GetParentSceneInfo())
					{
						ViewRelevance = ProjectedShadowInfo.GetParentSceneInfo()->Proxy->GetViewRelevance(&View);
					}
					else
					{
						ViewRelevance.bDrawRelevance = ViewRelevance.bStaticRelevance = ViewRelevance.bDynamicRelevance = ViewRelevance.bShadowRelevance = true;
					}							
					VisibleLightViewInfo.ProjectedShadowViewRelevanceMap[ShadowIndex] = ViewRelevance;

					// Check if the subject primitive's shadow is view relevant.
					const bool bPrimitiveIsShadowRelevant = ViewRelevance.bShadowRelevance;

					bool bShadowIsOccluded = false;

					// Skip occlusion result if using VSM - can result in incorrectly cached pages otherwise.
					if (!View.bIgnoreExistingQueries && View.State && !ProjectedShadowInfo.HasVirtualShadowMap())
					{
						// Check if the shadow is occluded.
						bShadowIsOccluded =
							((FSceneViewState*)View.State)->IsShadowOccluded(
							FSceneViewState::FProjectedShadowKey(ProjectedShadowInfo),
							NumBufferedFrames
							);
					}

					// The shadow is visible if it is view relevant and unoccluded.
					if(bPrimitiveIsShadowRelevant && !bShadowIsOccluded)
					{
						VisibleLightViewInfo.ProjectedShadowVisibilityMap[ShadowIndex] = true;
					}

					// Draw the shadow frustum.
					if(bPrimitiveIsShadowRelevant && !bShadowIsOccluded)
					{
						bool bDrawPreshadowFrustum = CVarDrawPreshadowFrustum.GetValueOnRenderThread() != 0;

						if (ViewFamily.EngineShowFlags.ShadowFrustums && ((bDrawPreshadowFrustum && ProjectedShadowInfo.bPreShadow) || (!bDrawPreshadowFrustum && !ProjectedShadowInfo.bPreShadow)))
						{
							TaskData.DrawDebugShadowFrustumOps.Emplace(Views[ViewIndex], ProjectedShadowInfo);
						}
					}
				}
			}

			// Register visible lights for allowing hair strands to cast shadow (directional light)
			if (!LightIt->CastRaytracedShadow && bHairStrands && LightIt->LightType == ELightComponentType::LightType_Directional)
			{
				HairStrands::AddVisibleShadowCastingLight(*Scene, Views, LightIt->LightSceneInfo);
			}
		}

		// Register visible lights for allowing hair strands to cast shadow (directional light)
		if (LightIt->CastRaytracedShadow && bHairStrands && LightIt->LightType == ELightComponentType::LightType_Directional)
		{
			HairStrands::AddVisibleShadowCastingLight(*Scene, Views, LightIt->LightSceneInfo);
		}
	}

#if !UE_BUILD_SHIPPING
	if(GDumpShadowSetup)
	{
		GDumpShadowSetup = false;

		UE_LOGF(LogRenderer, Display, "Dump Shadow Setup:");

		for (int32 ViewIndex = 0;ViewIndex < Views.Num(); ++ViewIndex)
		{
			FViewInfo& View = Views[ViewIndex];

			UE_LOGF(LogRenderer, Display, " View  %d/%d", ViewIndex, Views.Num());

			uint32 LightIndex = 0;
			for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
			{
				FLightSceneId LightId = LightSceneInfoCompact.LightSceneInfo->GetPersistentIndex();
				FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightId];
				FVisibleLightViewInfo& VisibleLightViewInfo = View.VisibleLightInfos[LightId];

				UE_LOGF(LogRenderer, Display, "  Light %d/%d:", LightId.GetIndex(), Scene->Lights.Num());

				for( int32 ShadowIndex = 0, ShadowCount = VisibleLightInfo.AllProjectedShadows.Num(); ShadowIndex < ShadowCount; ShadowIndex++ )
				{
					FProjectedShadowInfo& ProjectedShadowInfo = *VisibleLightInfo.AllProjectedShadows[ShadowIndex];

					if(VisibleLightViewInfo.bInViewFrustum)
					{
						UE_LOGF(LogRenderer, Display, "   Shadow %d/%d: ShadowId=%d",  ShadowIndex, ShadowCount, ProjectedShadowInfo.ShadowId);
						UE_LOGF(LogRenderer, Display, "    WholeSceneDir=%d SplitIndex=%d near=%f far=%f",
							ProjectedShadowInfo.IsWholeSceneDirectionalShadow(),
							ProjectedShadowInfo.CascadeSettings.ShadowSplitIndex,
							ProjectedShadowInfo.CascadeSettings.SplitNear,
							ProjectedShadowInfo.CascadeSettings.SplitFar);
						UE_LOGF(LogRenderer, Display, "    bDistField=%d bFarShadows=%d Bounds=%f,%f,%f,%f",
							ProjectedShadowInfo.bRayTracedDistanceField,
							ProjectedShadowInfo.CascadeSettings.bFarShadowCascade,
							ProjectedShadowInfo.ShadowBounds.Center.X,
							ProjectedShadowInfo.ShadowBounds.Center.Y,
							ProjectedShadowInfo.ShadowBounds.Center.Z,
							ProjectedShadowInfo.ShadowBounds.W);
						UE_LOGF(LogRenderer, Display, "    SplitFadeRegion=%f .. %f FadePlaneOffset=%f FadePlaneLength=%f",
							ProjectedShadowInfo.CascadeSettings.SplitNearFadeRegion,
							ProjectedShadowInfo.CascadeSettings.SplitFarFadeRegion,
							ProjectedShadowInfo.CascadeSettings.FadePlaneOffset,
							ProjectedShadowInfo.CascadeSettings.FadePlaneLength);			
					}
				}
			}
		}
	}
#endif // !UE_BUILD_SHIPPING
}

void FSceneRenderer::DrawDebugShadowFrustum(FViewInfo& View, FProjectedShadowInfo& ProjectedShadowInfo)
{
	FViewElementPDI ShadowFrustumPDI(&View, nullptr, nullptr);

	if (ProjectedShadowInfo.IsWholeSceneDirectionalShadow())
	{
		// Get split color
		FColor Color = FColor::White;
		switch (ProjectedShadowInfo.CascadeSettings.ShadowSplitIndex)
		{
		case 0: Color = FColor::Red; break;
		case 1: Color = FColor::Yellow; break;
		case 2: Color = FColor::Green; break;
		case 3: Color = FColor::Blue; break;
		}

		const FMatrix ViewMatrix = View.ViewMatrices.GetWorldToView();
		const FMatrix ProjectionMatrix = View.ViewMatrices.GetViewToClip();
		const FVector4 ViewOrigin = View.ViewMatrices.GetViewOrigin();

		float AspectRatio = ProjectionMatrix.M[1][1] / ProjectionMatrix.M[0][0];
		float ActualFOV = (ViewOrigin.W > 0.0f) ? FMath::Atan(1.0f / ProjectionMatrix.M[0][0]) : PI / 4.0f;

		float Near = ProjectedShadowInfo.CascadeSettings.SplitNear;
		float Mid = ProjectedShadowInfo.CascadeSettings.FadePlaneOffset;
		float Far = ProjectedShadowInfo.CascadeSettings.SplitFar;

		// Camera Subfrustum
		DrawFrustumWireframe(&ShadowFrustumPDI, (ViewMatrix * FPerspectiveMatrix(ActualFOV, AspectRatio, 1.0f, Near, Mid)).Inverse(), Color, 0);
		DrawFrustumWireframe(&ShadowFrustumPDI, (ViewMatrix * FPerspectiveMatrix(ActualFOV, AspectRatio, 1.0f, Mid, Far)).Inverse(), FColor::White, 0);

		// Shadow Map Projection Bounds
		DrawFrustumWireframe(&ShadowFrustumPDI, FMatrix(ProjectedShadowInfo.TranslatedWorldToClipInnerMatrix.Inverse()) * FTranslationMatrix(-ProjectedShadowInfo.PreShadowTranslation), Color, 0);
	}
	else
	{
		ProjectedShadowInfo.RenderFrustumWireframe(&ShadowFrustumPDI);
	}
}

void FSceneRenderer::GatherShadowDynamicMeshElements(FDynamicShadowsTaskData& TaskData)
{
	TConstArrayView<FProjectedShadowInfo*> ShadowsToSetupViews = TaskData.ShadowArrays.ShadowsToSetupViews;
	const int32 SetupShadowDepthViewBatchSize = 8;

	ParallelForTemplate(
		TEXT("SetupShadowDepthView"), ShadowsToSetupViews.Num(), SetupShadowDepthViewBatchSize,
		[this, ShadowsToSetupViews](int32 Index)
		{
			FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
			ShadowsToSetupViews[Index]->SetupShadowDepthView(this);
		},
		GSupportsParallelRenderingTasksWithSeparateRHIThread ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread
	);

	TaskData.ShadowsToGather.Reserve(128);

	for (int32 AtlasIndex = 0; AtlasIndex < SortedShadowsForShadowDepthPass.ShadowMapAtlases.Num(); AtlasIndex++)
	{
		TaskData.ShadowsToGather.Append(SortedShadowsForShadowDepthPass.ShadowMapAtlases[AtlasIndex].Shadows);
	}

	for (int32 AtlasIndex = 0; AtlasIndex < SortedShadowsForShadowDepthPass.ShadowMapCubemaps.Num(); AtlasIndex++)
	{
		TaskData.ShadowsToGather.Append(SortedShadowsForShadowDepthPass.ShadowMapCubemaps[AtlasIndex].Shadows);
	}

	TaskData.ShadowsToGather.Append(SortedShadowsForShadowDepthPass.PreshadowCache.Shadows);

	for (FSortedShadowMapAtlas& Atlas : SortedShadowsForShadowDepthPass.TranslucencyShadowMapAtlases)
	{
		TaskData.ShadowsToGather.Append(Atlas.Shadows);
	}

	if (UseNonNaniteVirtualShadowMaps(ShaderPlatform, FeatureLevel))
	{
		for (FProjectedShadowInfo* ProjectedShadowInfo : SortedShadowsForShadowDepthPass.VirtualShadowMapShadows)
		{
			if (ProjectedShadowInfo->bShouldRenderVSM)
			{
				TaskData.ShadowsToGather.Emplace(ProjectedShadowInfo);
			}
		}
	}

	if (!TaskData.ShadowsToGather.IsEmpty())
	{
		// Process all shadows in the serial path when not in multithreaded mode.
		TaskData.ShadowsToGatherInSerialPass.Init(!TaskData.bMultithreadedGDME, TaskData.ShadowsToGather.Num());

		const UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::High;

		const UE::Tasks::EExtendedTaskPriority ExtendedTaskPriority = TaskData.bMultithreadedGDME
			? UE::Tasks::EExtendedTaskPriority::None
			: UE::Tasks::EExtendedTaskPriority::Inline;

		int32 NumShadowViews = 0;
		for (FProjectedShadowInfo* ProjectedShadowInfo : TaskData.ShadowsToGather)
		{
			NumShadowViews += ProjectedShadowInfo->bOnePassPointLightShadow ? 6 : 1;
		}

		UE::Tasks::FTask MeshCollectorsTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&InstanceCullingManager = TaskData.InstanceCullingManager, NumShadowViews]
		{
			// Wait to allocate views until after the task event has triggered.
			InstanceCullingManager.AllocateViews(NumShadowViews);

		}, TaskData.BeginGatherDynamicMeshElementsTask, TaskPriority, ExtendedTaskPriority);

		if (TaskData.bMultithreadedGDME)
		{
			UE::Tasks::FTaskEvent MeshCollectorsTaskEvent{ UE_SOURCE_LOCATION };
			
			const int32 NumShadowTasks = FMath::Min<int32>(GetNumShadowDynamicMeshElementTasks(), TaskData.ShadowsToGather.Num());

			for (int32 TaskIndex = 0; TaskIndex < NumShadowTasks; ++TaskIndex)
			{
				FRHICommandList* RHICmdList = new FRHICommandList(FRHIGPUMask::All());
				RHICmdList->SwitchPipeline(ERHIPipeline::Graphics);
				TaskData.CommandLists.Emplace(RHICmdList);

				FShadowMeshCollector* MeshCollector = Allocator.Create<FShadowMeshCollector>(*RHICmdList, *this);
				TaskData.MeshCollectors.Emplace(MeshCollector);

				MeshCollectorsTaskEvent.AddPrerequisites(
					MeshCollector->GetPipe().Launch(UE_SOURCE_LOCATION,
						[this, &TaskData, MeshCollector]() mutable
				{
					FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
					TArray<const FSceneView*> LocalViews;
					LocalViews.AddZeroed(1);

					while (true)
					{
						// Atomically increment to get the next shadow to process in this task until empty.
						const int32 ShadowIndex = TaskData.ShadowsToGatherNextIndex.fetch_add(1, std::memory_order_relaxed);
						if (ShadowIndex >= TaskData.ShadowsToGather.Num())
						{
							break;
						}

						FProjectedShadowInfo& ProjectedShadowInfo = *TaskData.ShadowsToGather[ShadowIndex];
						FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[ProjectedShadowInfo.GetLightSceneInfo().Id];

						const bool bProcessedAllPrimitives = ProjectedShadowInfo.GatherDynamicMeshElements(MeshCollector->GetCollector(), *this, VisibleLightInfo, LocalViews, FProjectedShadowInfo::EGatherDynamicMeshElementsPass::Parallel);

						// Atomically mark shadows that require a second serial pass.
						if (!bProcessedAllPrimitives)
						{
							TaskData.ShadowsToGatherInSerialPass[ShadowIndex].AtomicSet(true);
						}

						// Setup mesh draw command passes immediately if we don't have any more primitives to process in the serial pass.
						if (bProcessedAllPrimitives)
						{
							ProjectedShadowInfo.SetupMeshDrawCommandsForProjectionStenciling(*this, TaskData.InstanceCullingManager);
							ProjectedShadowInfo.SetupMeshDrawCommandsForShadowDepth(*this, TaskData.InstanceCullingManager);
						}
					}

				}, MeshCollectorsTask, TaskPriority));
			}

			MeshCollectorsTaskEvent.Trigger();
			MeshCollectorsTask = MoveTemp(MeshCollectorsTaskEvent);
		}

		// Process only serial primitives if in multithreaded mode, otherwise process all of them.
		const FProjectedShadowInfo::EGatherDynamicMeshElementsPass SerialPass = TaskData.bMultithreadedGDME
			? FProjectedShadowInfo::EGatherDynamicMeshElementsPass::Serial
			: FProjectedShadowInfo::EGatherDynamicMeshElementsPass::All;

		TaskData.SetupMeshPassTask.AddPrerequisites(UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &TaskData, SerialPass] () mutable
		{
			FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
			TRACE_CPUPROFILER_EVENT_SCOPE(SetupShadowMeshPass);

			TArray<const FSceneView*> LocalViews;

			for (TConstSetBitIterator<SceneRenderingBitArrayAllocator> BitIt(TaskData.ShadowsToGatherInSerialPass); BitIt; ++BitIt)
			{
				if (LocalViews.IsEmpty())
				{
					LocalViews.AddZeroed(1);
				}

				FShadowMeshCollector& MeshCollector = TaskData.GetSerialMeshCollector();
				FProjectedShadowInfo* ProjectedShadowInfo = TaskData.ShadowsToGather[BitIt.GetIndex()];
				FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[ProjectedShadowInfo->GetLightSceneInfo().Id];
				ProjectedShadowInfo->GatherDynamicMeshElements(MeshCollector.GetCollector(), *this, VisibleLightInfo, LocalViews, SerialPass);

				// Setup mesh draw command passes now that all primitives are processed.
				ProjectedShadowInfo->SetupMeshDrawCommandsForProjectionStenciling(*this, TaskData.InstanceCullingManager);
				ProjectedShadowInfo->SetupMeshDrawCommandsForShadowDepth(*this, TaskData.InstanceCullingManager);
			}

			for (FShadowMeshCollector* MeshCollector : TaskData.MeshCollectors)
			{
				MeshCollector->Finish();
			}
	
		}, MeshCollectorsTask, TaskPriority, ExtendedTaskPriority));
	}

	TaskData.SetupMeshPassTask.Trigger();
}

FAutoConsoleTaskPriority CPrio_GatherShadowPrimitives(
	TEXT("TaskGraph.TaskPriorities.GatherShadowPrimitives"),
	TEXT("Task and thread priority for GatherShadowPrimitives tasks."),
	ENamedThreads::HighThreadPriority,
	ENamedThreads::NormalTaskPriority,
	ENamedThreads::HighTaskPriority
);

struct FGatherShadowPrimitivesPacket
{
	// Inputs
	FScenePrimitiveOctree::FNodeIndex NodeIndex = INDEX_NONE;
	int32 StartPrimitiveIndex{ 0 };
	int32 NumPrimitives{ 0 };

	// Scratch
	FPerShadowGatherStats ViewDependentWholeSceneShadowStats;

	// Output
	TArray<FAddSubjectPrimitiveOverflowedIndices> PreShadowOverflowedIndices;
	TArray<FAddSubjectPrimitiveOverflowedIndices> ViewDependentWholeSceneShadowOverflowedIndices;
	TArray<FShadowSubjectPrimitives> PreShadowSubjectPrimitives;
	TArray<FShadowSubjectPrimitives> ViewDependentWholeSceneShadowSubjectPrimitives;

	FGatherShadowPrimitivesPacket(
		FScenePrimitiveOctree::FNodeIndex InNodeIndex,
		int32 InStartPrimitiveIndex,
		int32 InNumPrimitives)
		: NodeIndex(InNodeIndex)
		, StartPrimitiveIndex(InStartPrimitiveIndex)
		, NumPrimitives(InNumPrimitives)
	{
	}

	void AnyThreadTask(FDynamicShadowsTaskData& TaskData)
	{
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

		const int32 NumPreShadows = TaskData.PreShadows.Num();
		const int32 NumVDWSShadows = TaskData.ViewDependentWholeSceneShadows.Num();

		check(TaskData.GatherStats.Num() == NumVDWSShadows);
		ViewDependentWholeSceneShadowStats.Empty(NumVDWSShadows);
		ViewDependentWholeSceneShadowStats.AddDefaulted(NumVDWSShadows);

		PreShadowOverflowedIndices.Empty(NumPreShadows);
		PreShadowOverflowedIndices.AddDefaulted(NumPreShadows);

		ViewDependentWholeSceneShadowOverflowedIndices.Empty(NumVDWSShadows);
		ViewDependentWholeSceneShadowOverflowedIndices.AddDefaulted(NumVDWSShadows);

		PreShadowSubjectPrimitives.Empty(NumPreShadows);
		PreShadowSubjectPrimitives.AddDefaulted(NumPreShadows);

		ViewDependentWholeSceneShadowSubjectPrimitives.Empty(NumVDWSShadows);
		ViewDependentWholeSceneShadowSubjectPrimitives.AddDefaulted(NumVDWSShadows);
		
		if (NodeIndex != INDEX_NONE)
		{
			// Check all the primitives in this octree node.
			for (const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact : TaskData.Scene->PrimitiveOctree.GetElementsForNode(NodeIndex))
			{
				// Nanite has its own culling
				if (PrimitiveSceneInfoCompact.PrimitiveFlagsCompact.bCastDynamicShadow &&
					(!PrimitiveSceneInfoCompact.PrimitiveFlagsCompact.bIsNaniteMesh || GSkipCullingNaniteMeshes == 0))
				{
					FilterPrimitiveForShadows(TaskData, PrimitiveSceneInfoCompact);
				}
			}
		}
		else
		{
			check(NumPrimitives > 0);

			// Check primitives in this packet's range
			for (int32 PrimitiveIndex = StartPrimitiveIndex; PrimitiveIndex < StartPrimitiveIndex + NumPrimitives; PrimitiveIndex++)
			{
				const FPrimitiveFlagsCompact PrimitiveFlagsCompact = TaskData.Scene->PrimitiveFlagsCompact[PrimitiveIndex];

				// Nanite has its own culling
				if (PrimitiveFlagsCompact.bCastDynamicShadow &&
					(!PrimitiveFlagsCompact.bIsNaniteMesh || GSkipCullingNaniteMeshes == 0))
				{
					FPrimitiveSceneInfo* PrimitiveSceneInfo = TaskData.Scene->Primitives[PrimitiveIndex];
					const FPrimitiveSceneInfoCompact PrimitiveSceneInfoCompact(PrimitiveSceneInfo);

					FilterPrimitiveForShadows(TaskData, PrimitiveSceneInfoCompact);
				}
			}
		}

		for (int32 StatIndex=0, StatNum = ViewDependentWholeSceneShadowStats.Num(); StatIndex < StatNum; StatIndex++)
		{
			TaskData.GatherStats[StatIndex].InterlockedAdd(ViewDependentWholeSceneShadowStats[StatIndex]);
		}
		ViewDependentWholeSceneShadowStats.Empty();
	}

	bool FilterForHigherPriorityShadow(FDynamicShadowsTaskData& TaskData, const FPrimitiveSceneInfo* PrimitiveSceneInfo, const FPrimitiveSceneProxy* PrimitiveProxy) const
	{
		if (UseVirtualShadowMaps(TaskData.ShaderPlatform, TaskData.FeatureLevel))
		{
			return false;
		}

		bool bCastsHigherPriorityShadow = false;

		// If light attachment root is valid, we're in a group and need to get the flag from the root.
		// If not in mobile, it is sufficient to only check inset shadow because Inset Shadow = Inset Shadow || CapsuleDirectShadow
		if (PrimitiveSceneInfo->LightingAttachmentRoot.IsValid())
		{
			const FAttachmentGroupSceneInfo& AttachmentGroup = PrimitiveSceneInfo->Scene->AttachmentGroups.FindChecked(PrimitiveSceneInfo->LightingAttachmentRoot);
			if (AttachmentGroup.ParentSceneInfo)
			{
				bCastsHigherPriorityShadow = AttachmentGroup.ParentSceneInfo->Proxy->CastsInsetShadow();
			}
		}
		else
		{
			bCastsHigherPriorityShadow = PrimitiveProxy->CastsInsetShadow();
		}

		// if we are on mobile check if inset/capsule shadows are actually cast
		if (bCastsHigherPriorityShadow && TaskData.FeatureLevel < ERHIFeatureLevel::SM5)
		{
			const bool MobileAllow = MobileUsesFullDepthPrepass(TaskData.ShaderPlatform) || (IsMobileDeferredShadingEnabled(TaskData.ShaderPlatform) && !MobileAllowFramebufferFetch(TaskData.ShaderPlatform));
			const bool bSupportsMobileInsetShadows = CVarMobileInsetShadows.GetValueOnRenderThread() && MobileAllow;
			const bool bSupportsMobileCapsuleDirectShadows = IsCapsuleDirectShadowsEnabled(TaskData.ShaderPlatform) && MobileAllow;

			// primitive casts inset shadow and inset shadow is turned on
			const bool bCastsMobileInsetShadow = bCastsHigherPriorityShadow && bSupportsMobileInsetShadows;

			// Primitive casts capsule direct shadow and capsule direct shadow is turned on
			// We may need to check against light attachment root
			bool bCastsMobileCapsuleDirectShadow = false;
			if (bSupportsMobileCapsuleDirectShadows)
			{
				if (PrimitiveSceneInfo->LightingAttachmentRoot.IsValid())
				{
					const FAttachmentGroupSceneInfo& AttachmentGroup = PrimitiveSceneInfo->Scene->AttachmentGroups.FindChecked(PrimitiveSceneInfo->LightingAttachmentRoot);
					if (AttachmentGroup.ParentSceneInfo)
					{
						bCastsMobileCapsuleDirectShadow = AttachmentGroup.ParentSceneInfo->Proxy->CastsCapsuleDirectShadow();
					}
				}
				else
				{
					bCastsMobileCapsuleDirectShadow = PrimitiveProxy->CastsCapsuleDirectShadow();
				}
			}

			bCastsHigherPriorityShadow = bCastsMobileInsetShadow || bCastsMobileCapsuleDirectShadow;
		}

		return bCastsHigherPriorityShadow;
	}

	void FilterPrimitiveForShadows(FDynamicShadowsTaskData& TaskData, const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact)
	{
		const FPrimitiveFlagsCompact& PrimitiveFlagsCompact = PrimitiveSceneInfoCompact.PrimitiveFlagsCompact;
		const FBoxSphereBounds PrimitiveBounds(PrimitiveSceneInfoCompact.Bounds);
		FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneInfoCompact.PrimitiveSceneInfo;
		const FPrimitiveSceneProxy* PrimitiveProxy = PrimitiveSceneInfoCompact.Proxy;

		// Check if the primitive is a subject for any of the preshadows.
		// Only allow preshadows from lightmapped primitives that cast both dynamic and static shadows.
		if (TaskData.PreShadows.Num() && PrimitiveFlagsCompact.bCastStaticShadow && PrimitiveFlagsCompact.bStaticLighting)
		{
			for (int32 ShadowIndex = 0, Num = TaskData.PreShadows.Num(); ShadowIndex < Num; ShadowIndex++)
			{
				FProjectedShadowInfo* RESTRICT ProjectedShadowInfo = TaskData.PreShadows[ShadowIndex];

				// Note: Culling based on the primitive's bounds BEFORE dereferencing PrimitiveSceneInfo / PrimitiveProxy
				// Check if this primitive is in the shadow's frustum.
				bool bInFrustum = ProjectedShadowInfo->CasterOuterFrustum.IntersectBox(PrimitiveBounds.Origin, ProjectedShadowInfo->PreShadowTranslation, PrimitiveBounds.BoxExtent);

				if (bInFrustum && ProjectedShadowInfo->GetLightSceneInfoCompact().AffectsPrimitive(PrimitiveBounds, PrimitiveProxy))
				{
					FAddSubjectPrimitiveResult Result;
					FAddSubjectPrimitiveStats UnusedStats;
					Result.Qword = ProjectedShadowInfo->AddSubjectPrimitive_AnyThread(PrimitiveSceneInfoCompact, TaskData.Views, TaskData.FeatureLevel, UnusedStats, PreShadowOverflowedIndices[ShadowIndex]);

					if (Result.Qword != 0)
					{
						FShadowSubjectPrimitives& SubjectPrimitives = PreShadowSubjectPrimitives[ShadowIndex];
						FAddSubjectPrimitiveOp& Op = SubjectPrimitives[SubjectPrimitives.AddUninitialized()];
						Op.PrimitiveSceneInfo = PrimitiveSceneInfo;
						Op.Result.Qword = Result.Qword;
					}
				}
			}
		}

		const FCSMSceneExtension& CSMShadowScene = TaskData.Scene->GetExtension<FCSMSceneExtension>();

		for (int32 ShadowIndex = 0, Num = TaskData.ViewDependentWholeSceneShadows.Num();ShadowIndex < Num;ShadowIndex++)
		{
			const FProjectedShadowInfo* RESTRICT ProjectedShadowInfo = TaskData.ViewDependentWholeSceneShadows[ShadowIndex];
			const FLightSceneInfo& RESTRICT LightSceneInfo = ProjectedShadowInfo->GetLightSceneInfo();
			const FLightSceneProxy& RESTRICT LightProxy = *LightSceneInfo.Proxy;

			const FVector LightDirection = LightProxy.GetDirection();
			const FVector PrimitiveToShadowCenter = ProjectedShadowInfo->ShadowBounds.Center - PrimitiveBounds.Origin;
			// Project the primitive's bounds origin onto the light vector
			const float ProjectedDistanceFromShadowOriginAlongLightDir = PrimitiveToShadowCenter | LightDirection;
			// Calculate the primitive's squared distance to the cylinder's axis
			const float PrimitiveDistanceFromCylinderAxisSq = (-LightDirection * ProjectedDistanceFromShadowOriginAlongLightDir + PrimitiveToShadowCenter).SizeSquared();
			const float CombinedRadiusSq = FMath::Square(ProjectedShadowInfo->ShadowBounds.W + PrimitiveBounds.SphereRadius);

			// Note: Culling based on the primitive's bounds BEFORE dereferencing PrimitiveSceneInfo / PrimitiveProxy
			// Check if this primitive is in the shadow's cylinder
			if (PrimitiveDistanceFromCylinderAxisSq < CombinedRadiusSq
				// If the primitive is further along the cone axis than the shadow bounds origin, 
				// Check if the primitive is inside the spherical cap of the cascade's bounds
				&& !(ProjectedDistanceFromShadowOriginAlongLightDir < 0 && PrimitiveToShadowCenter.SizeSquared() > CombinedRadiusSq)
				// Test against the convex hull containing the extruded shadow bounds
				&& ProjectedShadowInfo->CascadeSettings.ShadowBoundsAccurate.IntersectBox(PrimitiveBounds.Origin, PrimitiveBounds.BoxExtent))
			{
				// Distance culling for RSMs
				const float MinScreenRadiusForShadowCaster = GMinScreenRadiusForShadowCaster;

				bool bScreenSpaceSizeCulled = false;
				check(ProjectedShadowInfo->DependentView);

				{
					const float DistanceSquared = (PrimitiveBounds.Origin - ProjectedShadowInfo->DependentView->GetShadowViewMatrices().GetViewOrigin()).SizeSquared();
					const float LODScaleSquared = FMath::Square(ProjectedShadowInfo->DependentView->LODDistanceFactor);
					bScreenSpaceSizeCulled = (FMath::Square(PrimitiveBounds.SphereRadius) < FMath::Square(MinScreenRadiusForShadowCaster) * DistanceSquared * LODScaleSquared) ||
						IsScreenSizeCulled(*ProjectedShadowInfo->DependentView, PrimitiveBounds, TaskData.MinScreenBoundsRadiusSquared);
				}

				bool bShadowRelevance = true;

				if (ProjectedShadowInfo->CacheMode == SDCM_MovablePrimitivesOnly)
				{
					const FCachedShadowMapData& CachedShadowMapData = CSMShadowScene.GetCachedShadowMapDataRef(LightSceneInfo.Id, ProjectedShadowInfo->CascadeSettings.ShadowSplitIndex);

					// if the mesh is dynamic, it should cast csm.
					bShadowRelevance = PrimitiveProxy->IsMeshShapeOftenMoving();
				}
				else if (ProjectedShadowInfo->CacheMode == SDCM_StaticPrimitivesOnly)
				{
					bShadowRelevance = !PrimitiveProxy->IsMeshShapeOftenMoving();
				}
				else if (ProjectedShadowInfo->CacheMode == SDCM_CSMScrolling)
				{
					const FCachedShadowMapData& CachedShadowMapData = CSMShadowScene.GetCachedShadowMapDataRef(LightSceneInfo.Id, ProjectedShadowInfo->CascadeSettings.ShadowSplitIndex);

					// if the mesh is dynamic or a new added mesh, it should cast csm.
					bShadowRelevance = PrimitiveProxy->IsMeshShapeOftenMoving() || !CachedShadowMapData.StaticShadowSubjectPersistentPrimitiveIdMap[PrimitiveSceneInfo->GetPersistentIndex().Index];

					// if the static mesh is rendered in the cached shadow map, check if it intersects with the non-overlapped area
					if (!bShadowRelevance)
					{
						for (int32 i = 0; i < ProjectedShadowInfo->CSMScrollingExtraCullingPlanes.Num() && !bShadowRelevance; ++i)
						{
							const FPlane& CSMScrollingExtraCullingPlane = ProjectedShadowInfo->CSMScrollingExtraCullingPlanes[i];
							bShadowRelevance |= FMath::PlaneAABBRelativePosition(CSMScrollingExtraCullingPlane, PrimitiveBounds.GetBox()) <= 0;
						}
					}
				}

				// Filter out when inset or capsule shadows are drawn
				bool bCastsHigherPriorityShadow = FilterForHigherPriorityShadow(TaskData, PrimitiveSceneInfo, PrimitiveProxy);

				if (!bScreenSpaceSizeCulled
					&& bShadowRelevance
					&& ProjectedShadowInfo->GetLightSceneInfoCompact().AffectsPrimitive(PrimitiveBounds, PrimitiveProxy)
					// Include all primitives for movable lights, but only statically shadowed primitives from a light with static shadowing,
					// Since lights with static shadowing still create per-object shadows for primitives without static shadowing.
					&& (!LightProxy.HasStaticLighting() || (!LightSceneInfo.IsPrecomputedLightingValid() || LightProxy.UseCSMForDynamicObjects()))
					// Exclude primitives that will create their own per-object shadow, except when rendering RSMs
					&& (!bCastsHigherPriorityShadow)
					// Exclude primitives that will create a per-object shadow from a stationary light
					&& !ShouldCreateObjectShadowForStationaryLight(&LightSceneInfo, PrimitiveProxy, true)
					// Only render shadows from objects that use static lighting during a reflection capture, since the reflection capture doesn't update at runtime
					&& (!TaskData.bStaticSceneOnly || PrimitiveProxy->HasStaticLighting())
					// Render dynamic lit objects if CSMForDynamicObjects is enabled.
					&& (!LightProxy.UseCSMForDynamicObjects() || !PrimitiveProxy->HasStaticLighting() || TaskData.bMobileAllowStaticCSM))
				{
					FAddSubjectPrimitiveResult Result;
					Result.Qword = ProjectedShadowInfo->AddSubjectPrimitive_AnyThread(
						PrimitiveSceneInfoCompact,
						TArrayView<FViewInfo>(),
						TaskData.FeatureLevel,
						ViewDependentWholeSceneShadowStats[ShadowIndex],
						ViewDependentWholeSceneShadowOverflowedIndices[ShadowIndex]);

					if (Result.Qword != 0)
					{
						FShadowSubjectPrimitives& SubjectPrimitives = ViewDependentWholeSceneShadowSubjectPrimitives[ShadowIndex];
						if (!SubjectPrimitives.Num())
						{
							SubjectPrimitives.Reserve(16);
						}
						FAddSubjectPrimitiveOp& Op = SubjectPrimitives[SubjectPrimitives.AddUninitialized()];
						Op.PrimitiveSceneInfo = PrimitiveSceneInfo;
						Op.Result.Qword = Result.Qword;
					}
				}
			}
		}
	}

	void AnyThreadFinalize(FDynamicShadowsTaskData& TaskData)
	{
		for (int32 ShadowIndex = 0; ShadowIndex < PreShadowSubjectPrimitives.Num(); ShadowIndex++)
		{
			FProjectedShadowInfo* ProjectedShadowInfo = TaskData.PreShadows[ShadowIndex];
			const FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer = PreShadowOverflowedIndices[ShadowIndex];
			FFinalizeAddSubjectPrimitiveContext Context;
			Context.OverflowedMDCIndices = OverflowBuffer.MDCIndices.GetData();
			Context.OverflowedMeshIndices = OverflowBuffer.MeshIndices.GetData();

			for (const FAddSubjectPrimitiveOp& PrimitiveOp : PreShadowSubjectPrimitives[ShadowIndex])
			{
				ProjectedShadowInfo->FinalizeAddSubjectPrimitive(TaskData, PrimitiveOp, TaskData.Views, Context);
			}
		}

		for (int32 ShadowIndex = 0; ShadowIndex < ViewDependentWholeSceneShadowSubjectPrimitives.Num(); ShadowIndex++)
		{
			FProjectedShadowInfo* ProjectedShadowInfo = TaskData.ViewDependentWholeSceneShadows[ShadowIndex];
			const FAddSubjectPrimitiveOverflowedIndices& OverflowBuffer = ViewDependentWholeSceneShadowOverflowedIndices[ShadowIndex];
			FFinalizeAddSubjectPrimitiveContext Context;
			Context.OverflowedMDCIndices = OverflowBuffer.MDCIndices.GetData();
			Context.OverflowedMeshIndices = OverflowBuffer.MeshIndices.GetData();

			for (const FAddSubjectPrimitiveOp& PrimitiveOp : ViewDependentWholeSceneShadowSubjectPrimitives[ShadowIndex])
			{
				ProjectedShadowInfo->FinalizeAddSubjectPrimitive(TaskData, PrimitiveOp, TArrayView<FViewInfo>(), Context);
			}

			if (ProjectedShadowInfo->CacheMode == SDCM_StaticPrimitivesOnly && ProjectedShadowInfo->IsWholeSceneDirectionalShadow())
			{
				FCachedShadowMapData* CachedShadowMapData = const_cast<FCachedShadowMapData*>(TaskData.Scene->GetExtension<FCSMSceneExtension>().GetCachedShadowMapData(ProjectedShadowInfo->GetLightSceneInfo().Id, ProjectedShadowInfo->CascadeSettings.ShadowSplitIndex));
				checkSlow(CachedShadowMapData != nullptr);
				CachedShadowMapData->bCachedShadowMapHasPrimitives = ProjectedShadowInfo->HasSubjectPrims();
			}
		}
	}
};

struct FGatherShadowPrimitivesPrepareTask
{
	FDynamicShadowsTaskData& TaskData;

	FGatherShadowPrimitivesPrepareTask(FDynamicShadowsTaskData& InTaskData)
		: TaskData(InTaskData)
	{
	}

	static FORCEINLINE TStatId GetStatId()
	{
		return TStatId();
	}

	static ENamedThreads::Type GetDesiredThread()
	{
		return CPrio_GatherShadowPrimitives.Get();
	}

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		SCOPED_NAMED_EVENT_TEXT("FGatherShadowPrimitivesPrepareTask", FColor::Green);
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

		if (TaskData.bMultithreadedCreateAndFilterShadows)
		{
			TaskData.SceneRenderer->CreateDynamicShadows(TaskData);
		}

		AnyThreadTask();

		// Kick off any child tasks if we added them

		FGraphEventArray Prereqs;
		Prereqs.Reserve(TaskData.Packets.Num());

		if (const int32 NumPackets = TaskData.Packets.Num(); NumPackets > 0)
		{
			const int32 NumTasks = FMath::Min<int32>(FTaskGraphInterface::Get().GetNumWorkerThreads(), NumPackets);
			const int32 PacketsPerTask = FMath::DivideAndRoundUp(NumPackets, NumTasks);

			for (int32 TaskIndex = 0; TaskIndex < NumTasks; TaskIndex++)
			{
				const int32 FirstPacketToProcess = TaskIndex * PacketsPerTask;
				const int32 NumPacketsToProcess = FMath::Min(PacketsPerTask, FMath::Max(0, NumPackets - FirstPacketToProcess));

				if (NumPacketsToProcess > 0)
				{
					Prereqs.Emplace(FFunctionGraphTask::CreateAndDispatchWhenReady([TaskData = &TaskData, FirstPacketToProcess, NumPacketsToProcess]()
						{
							SCOPED_NAMED_EVENT_TEXT("FGatherShadowPrimitivesTask", FColor::Green);
							for (int32 PacketIndex = 0; PacketIndex < NumPacketsToProcess; PacketIndex++)
							{
								if (int32 PacketToProcess = FirstPacketToProcess + PacketIndex; PacketToProcess < TaskData->Packets.Num())
								{
									TaskData->Packets[PacketToProcess]->AnyThreadTask(*TaskData);
								}
							}
						},
						TStatId(),
						nullptr,
						CPrio_GatherShadowPrimitives.Get()));

					MyCompletionGraphEvent->DontCompleteUntil(Prereqs.Last());
				}
			}
		}

		if (TaskData.bMultithreadedCreateAndFilterShadows)
		{
			FGraphEventRef FinalizeTask = FFunctionGraphTask::CreateAndDispatchWhenReady([TaskData = &TaskData]()
				{
					SCOPED_NAMED_EVENT_TEXT("FGatherShadowPrimitivesFinalizeTask", FColor::Green);
					FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

					// Reserve only once up to total number of elements gathered from stats to avoid reallocations, especially important for really dense scenes
					TaskData->PresizeViewDependentWholeSceneShadows();
					
					for (FGatherShadowPrimitivesPacket* Packet : TaskData->Packets)
					{
						Packet->AnyThreadFinalize(*TaskData);
						delete Packet;
					}
					TaskData->Packets.Empty();
					TaskData->SceneRenderer->FilterDynamicShadows(*TaskData);
				},
				TStatId(),
				&Prereqs,
				CPrio_GatherShadowPrimitives.Get());

			MyCompletionGraphEvent->DontCompleteUntil(FinalizeTask);

			if (TaskData.bMultithreadedGDME)
			{
				Prereqs.Reset();
				Prereqs.Emplace(FinalizeTask);

				FFunctionGraphTask::CreateAndDispatchWhenReady([TaskData = &TaskData]
					{
						SCOPED_NAMED_EVENT_TEXT("GatherShadowDynamicMeshElements", FColor::Green);
						FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
						TaskData->SceneRenderer->GatherShadowDynamicMeshElements(*TaskData);
					},
					TStatId(),
					&Prereqs,
					CPrio_GatherShadowPrimitives.Get());
			}
		}
	}

	void AddSubTask(FScenePrimitiveOctree::FNodeIndex NodeIndex, int32 StartPrimitiveIndex, int32 NumPrimitives)
	{
		TaskData.Packets.Add(new FGatherShadowPrimitivesPacket(NodeIndex, StartPrimitiveIndex, NumPrimitives));
	}

	void AnyThreadTask()
	{
		TaskData.GatherStats.AddDefaulted(TaskData.ViewDependentWholeSceneShadows.Num());

		if (GUseOctreeForShadowCulling)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ShadowSceneOctreeTraversal);

			TaskData.Packets.Reserve(100);

			// Find primitives that are in a shadow frustum in the octree.
			TaskData.Scene->PrimitiveOctree.FindNodesWithPredicate(
				[this](FScenePrimitiveOctree::FNodeIndex /*ParentNodeIndex*/, FScenePrimitiveOctree::FNodeIndex /*NodeIndex*/, const FBoxCenterAndExtent& NodeBounds)
				{
					// Check that the child node is in the frustum for at least one shadow.

					// Check for subjects of preshadows.
					for (FProjectedShadowInfo* ProjectedShadowInfo : TaskData.PreShadows)
					{
						check(ProjectedShadowInfo->CasterOuterFrustum.PermutedPlanes.Num());
						// Check if this primitive is in the shadow's frustum.
						if (ProjectedShadowInfo->CasterOuterFrustum.IntersectBox(
							NodeBounds.Center + ProjectedShadowInfo->PreShadowTranslation,
							NodeBounds.Extent))
						{
							return true;
						}
					}

					for (FProjectedShadowInfo* ProjectedShadowInfo : TaskData.ViewDependentWholeSceneShadows)
					{
						//check(ProjectedShadowInfo->CasterFrustum.PermutedPlanes.Num());
						// Check if this primitive is in the shadow's frustum.
						if (ProjectedShadowInfo->CasterOuterFrustum.IntersectBox(
							NodeBounds.Center + ProjectedShadowInfo->PreShadowTranslation,
							NodeBounds.Extent))
						{
							return true;
						}
					}

					// If the child node was in the frustum of at least one preshadow, push it on
					// the iterator's pending node stack.
					return false;
				},
				[this](FScenePrimitiveOctree::FNodeIndex /*ParentNodeIndex*/, FScenePrimitiveOctree::FNodeIndex NodeIndex, const FBoxCenterAndExtent& /*NodeBounds*/)
				{
					if (TaskData.Scene->PrimitiveOctree.GetElementsForNode(NodeIndex).Num() > 0)
					{
						AddSubTask(NodeIndex, 0, 0);
					}
				});
		}
		else
		{
			const int32 PacketSize = CVarParallelGatherNumPrimitivesPerPacket.GetValueOnAnyThread();
			const int32 NumPackets = FMath::DivideAndRoundUp(TaskData.Scene->Primitives.Num(), PacketSize);

			TaskData.Packets.Reserve(NumPackets);

			for (int32 PacketIndex = 0; PacketIndex < NumPackets; PacketIndex++)
			{
				const int32 StartPrimitiveIndex = PacketIndex * PacketSize;
				const int32 NumPrimitives = FMath::Min(PacketSize, TaskData.Scene->Primitives.Num() - StartPrimitiveIndex);

				AddSubTask(INDEX_NONE, StartPrimitiveIndex, NumPrimitives);
			}
		}
	}
};

void FSceneRenderer::BeginGatherShadowPrimitives(FDynamicShadowsTaskData* TaskData, IVisibilityTaskData* VisibilityTaskData)
{
	SCOPE_CYCLE_COUNTER(STAT_GatherShadowPrimitivesTime);

	check(TaskData);

	UE::Tasks::FTaskEvent ShadowSetupPrerequisites{ UE_SOURCE_LOCATION };
	if (VisibilityTaskData)
	{
		ShadowSetupPrerequisites.AddPrerequisites(VisibilityTaskData->GetComputeRelevanceTask());
		ShadowSetupPrerequisites.AddPrerequisites(VisibilityTaskData->GetLightVisibilityTask());
	}

	if (FShadowSceneRenderer* ShadowSceneRenderer = GetSceneExtensionsRenderers().GetRendererPtr<FShadowSceneRenderer>())
	{
		ShadowSetupPrerequisites.AddPrerequisites(ShadowSceneRenderer->GetRendererSetupTask());
	}

	ShadowSetupPrerequisites.AddPrerequisites(Scene->GetCreateLightPrimitiveInteractionsTask());
	ShadowSetupPrerequisites.AddPrerequisites(Scene->GetCacheMeshDrawCommandsTask());
	ShadowSetupPrerequisites.Trigger();

	// The FSceneRenderBuilder clean-up task clears shadow mesh passes. Cached pre-shadows are the only shadow type that is persistently
	// reused from render-to-render and those need to be cleaned up prior to generating new mesh draw commands for the current scene.

	if (!TaskData->bMultithreadedCreateAndFilterShadows)
	{
		ShadowSetupPrerequisites.Wait();
		FSceneRenderBuilder::WaitForAsyncCleanupTask();
		CreateDynamicShadows(*TaskData);
	}

	if (!TaskData->bMultithreaded || !TaskData->bRunningEarly)
	{
		if (TaskData->PreShadows.Num() || TaskData->ViewDependentWholeSceneShadows.Num())
		{
			ShadowSetupPrerequisites.Wait();
			FSceneRenderBuilder::WaitForAsyncCleanupTask();
			FGatherShadowPrimitivesPrepareTask(*TaskData).AnyThreadTask();

			if (!TaskData->bRunningEarly)
			{
				ParallelFor(TaskData->Packets.Num(),
					[TaskData](int32 Index)
					{
						TaskData->Packets[Index]->AnyThreadTask(*TaskData);
					},
					!TaskData->bMultithreaded);
			}
			else
			{
				for (FGatherShadowPrimitivesPacket* Packet : TaskData->Packets)
				{
					Packet->AnyThreadTask(*TaskData);
				}
			}
		}
	}
	else
	{
		FGraphEventArray Prereqs;
		Prereqs.Emplace(CreateCompatibilityGraphEvent(ShadowSetupPrerequisites));
		Prereqs.Emplace(FSceneRenderBuilder::GetAsyncCleanupTask());

		TaskData->TaskEvent = TGraphTask<FGatherShadowPrimitivesPrepareTask>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(*TaskData);
	}
}

static void DebugDrawPrimitiveOctree(FViewInfo& View, const FScenePrimitiveOctree& PrimitiveOctree)
{
	FViewElementPDI PrimitivePDI(&View, nullptr, nullptr);
	PrimitiveOctree.FindNodesWithPredicate(
		[&](FScenePrimitiveOctree::FNodeIndex ParentNodeIndex, FScenePrimitiveOctree::FNodeIndex CurrentNodeIndex, const FBoxCenterAndExtent& Bounds) -> bool
		{
			return true;
		},
		[&](FScenePrimitiveOctree::FNodeIndex ParentNodeIndex, FScenePrimitiveOctree::FNodeIndex CurrentNodeIndex, const FBoxCenterAndExtent& Bounds)
		{
			DrawWireBox(&PrimitivePDI, Bounds.GetBox(), FLinearColor(1,0,1), SDPG_Foreground);
		}
	);
}

void FSceneRenderer::FinishGatherShadowPrimitives(FDynamicShadowsTaskData* TaskData)
{
	SCOPED_NAMED_EVENT_TEXT("FSceneRenderer::FinishGatherShadowPrimitives", FColor::Green);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RenderThreadFinalize);
	CSV_SCOPED_SET_WAIT_STAT(Shadows)

	check(TaskData);

	if (TaskData->TaskEvent)
	{
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(TaskData->TaskEvent, ENamedThreads::GetRenderThread_Local());
		TaskData->TaskEvent = nullptr;
	}

	if (!TaskData->bMultithreadedCreateAndFilterShadows)
	{
		SCOPED_NAMED_EVENT_TEXT("FGatherShadowPrimitivesTask", FColor::Green);
		
		// Reserve only once up to total number of elements gathered from stats to avoid reallocations, especially important for really dense scenes
		TaskData->PresizeViewDependentWholeSceneShadows();
		
		for (FGatherShadowPrimitivesPacket* Packet : TaskData->Packets)
		{
			Packet->AnyThreadFinalize(*TaskData);
			delete Packet;
		}
		TaskData->Packets.Empty();
		FilterDynamicShadows(*TaskData);
	}

	TaskData->FilterDynamicShadowsTask.Wait();

	if (!TaskData->bMultithreadedGDME)
	{
		GatherShadowDynamicMeshElements(*TaskData);
	}

	for (FDrawDebugShadowFrustumOp Op : TaskData->DrawDebugShadowFrustumOps)
	{
		DrawDebugShadowFrustum(*Op.View, *Op.ProjectedShadowInfo);
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (CVarVisualizePrimitiveOctree.GetValueOnRenderThread() != 0)
	{
		DebugDrawPrimitiveOctree(TaskData->Views[0], TaskData->Scene->PrimitiveOctree);
	}

	if (!TaskData->ShadowArrays.OnePassShadowUnsupportedLights.IsEmpty())
	{
		OnGetOnScreenMessages.AddLambda([Scene = Scene, OnePassShadowUnsupportedLights = MoveTemp(TaskData->ShadowArrays.OnePassShadowUnsupportedLights)](FScreenMessageWriter& ScreenMessageWriter)->void
		{
			static const FText Message = NSLOCTEXT("Renderer", "PointLightVsLayer", "RUNTIME DOES NOT SUPPORT WHOLE SCENE POINT LIGHT SHADOWS (Missing Vertexshader Layer support): ");
			ScreenMessageWriter.DrawLine(Message);
			for (FSceneRendererLightBitArray::FIterator It(OnePassShadowUnsupportedLights); It; ++It)
			{
				FLightSceneId LightId = It.GetIndex();
				if (Scene->Lights.IsValidIndex(LightId))
				{
					ScreenMessageWriter.DrawLine(FText::FromString(Scene->Lights[LightId].LightSceneInfo->Proxy->GetOwnerNameOrLabel()), 35);
				}
			}
		});
	}
#endif
}

static bool NeedsUnatlasedCSMDepthsWorkaround(ERHIFeatureLevel::Type FeatureLevel)
{
	// UE-42131: Excluding mobile from this, mobile renderer relies on the depth texture border.
	return GRHINeedsUnatlasedCSMDepthsWorkaround && (FeatureLevel >= ERHIFeatureLevel::SM5);
}

static bool UseSceneCaptureCubeSharedCascade(const FSceneRenderer& Renderer, const FLightSceneInfo& LightSceneInfo)
{
	// If the CVar is changed during a time sliced render of a reflection capture, and the shadow atlas was already rendered,
	// keep the shared cascade feature enabled for the remaining time sliced faces.
	return Renderer.Views[0].bIsSceneCaptureCube && LightSceneInfo.Type == LightType_Directional &&
		(IsSceneCaptureCubeShareCascadesEnabled(Renderer.ShaderPlatform) || (Renderer.Views[0].bIsReflectionCapture && Renderer.Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame));
}

void FSceneRenderer::AddViewDependentWholeSceneShadowsForView(
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ShadowInfos, 
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ShadowInfosThatNeedCulling,
	FVisibleLightInfo& VisibleLightInfo, 
	FLightSceneInfo& LightSceneInfo,
	const int64 CachedShadowMapsSize,
	uint32& NumCSMCachesUpdatedThisFrame)
{
	SCOPE_CYCLE_COUNTER(STAT_AddViewDependentWholeSceneShadowsForView);

	const bool bNeedsVirtualShadowMap = LightSceneInfo.Proxy->UseVirtualShadowMaps();
	const bool bCubeSharedCascade = UseSceneCaptureCubeSharedCascade(*this, LightSceneInfo);
	const bool bTimesliced = bCubeSharedCascade && Views[0].bIsReflectionCapture && Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame;

	// Allow each view to create a whole scene view dependent shadow
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];

		const float LightShadowAmount = LightSceneInfo.Proxy->GetShadowAmount();
		TArray<float, TInlineAllocator<2> > FadeAlphas;
		if (bCubeSharedCascade)
		{
			// All views sharing the cascade get the same shadow amount
			FadeAlphas.Init(LightShadowAmount, Views.Num());
		}
		else
		{
			FadeAlphas.Init(0.0f, Views.Num());
			FadeAlphas[ViewIndex] = LightShadowAmount;
		}

		if (IStereoRendering::IsAPrimaryView(View))
		{
			for (int FadeAlphaIndex = ViewIndex + 1; FadeAlphaIndex < Views.Num(); FadeAlphaIndex++)
			{
				if (Views.IsValidIndex(FadeAlphaIndex)
					&& IStereoRendering::IsASecondaryView(Views[FadeAlphaIndex]))
				{
					FadeAlphas[FadeAlphaIndex] = LightShadowAmount;
				}
				else if (IStereoRendering::IsAPrimaryView(Views[FadeAlphaIndex]))
				{
					break;
				}
			}
		}
		
		// If rendering in stereo mode we render shadow depths only for the left eye, but project for both eyes!
		// Similarly, for the cube capture shared cascade, we only render shadows with the first view.
		if (IStereoRendering::IsAPrimaryView(View) && (!bCubeSharedCascade || ViewIndex == 0))
		{
			const bool bExtraDistanceFieldCascade = LightSceneInfo.ShouldRenderLightViewIndependent()
				&& LightSceneInfo.Proxy->ShouldCreateRayTracedCascade(View.GetFeatureLevel(), LightSceneInfo.IsPrecomputedLightingValid(), View.MaxShadowCascades);

			const int32 ProjectionCount = LightSceneInfo.Proxy->GetNumViewDependentWholeSceneShadows(View, LightSceneInfo.IsPrecomputedLightingValid()) + (bExtraDistanceFieldCascade?1:0);

			static_assert(INDEX_NONE == -1, "INDEX_NONE != -1!");

			float MaxNonFarCascadeDistance = 0.0f;

			bool bVsmUseFarShadowRules = CVarVsmUseFarShadowRules.GetValueOnRenderThread() != 0;
			if (!bNeedsVirtualShadowMap || bVsmUseFarShadowRules)
			{
				for (int32 Index = 0; Index < ProjectionCount; Index++)
				{
					FWholeSceneProjectedShadowInitializer ProjectedShadowInitializer;

					int32 LocalIndex = Index;

					// Indexing like this puts the ray traced shadow cascade last (might not be needed)
					if(bExtraDistanceFieldCascade && LocalIndex + 1 == ProjectionCount)
					{
						LocalIndex = INDEX_NONE;
					}

					if (LightSceneInfo.Proxy->GetViewDependentWholeSceneProjectedShadowInitializer(View, LocalIndex, LightSceneInfo.IsPrecomputedLightingValid(), ProjectedShadowInitializer))
					{
						// Also needed for VSM-only when UseFarShadowCulling=1
						if (!ProjectedShadowInitializer.CascadeSettings.bFarShadowCascade && !ProjectedShadowInitializer.bRayTracedDistanceField)
						{
							MaxNonFarCascadeDistance = FMath::Max(MaxNonFarCascadeDistance, ProjectedShadowInitializer.CascadeSettings.SplitFar);
						}

						if (bNeedsVirtualShadowMap)
						{
							// VSM disables CSM, but DF shadows can still be enabled
							if (!ProjectedShadowInitializer.bRayTracedDistanceField)
							{
								continue;
							}
						}

						uint32 ShadowBorder = NeedsUnatlasedCSMDepthsWorkaround( FeatureLevel ) ? 0 : SHADOW_BORDER;
					
						const int32 MaxCSMResolution = GetCachedScalabilityCVars().MaxCSMShadowResolution;
						const int32 MinCSMResolution = 32;
						FIntPoint ShadowBufferResolution = FIntPoint( FMath::Clamp( MaxCSMResolution, MinCSMResolution, (int32)GMaxShadowDepthBufferSizeX ) - ShadowBorder * 2,
																	  FMath::Clamp( MaxCSMResolution, MinCSMResolution, (int32)GMaxShadowDepthBufferSizeY ) - ShadowBorder * 2 );

						int32 NumShadowMaps = 1;
						EShadowDepthCacheMode CacheMode[2] = { SDCM_Uncached, SDCM_Uncached };

						if (!View.bIsSceneCapture && !ProjectedShadowInitializer.bRayTracedDistanceField)
						{
							ComputeViewDependentWholeSceneShadowCacheModes(
								&LightSceneInfo,
								ViewFamily.Time.GetRealTimeSeconds(),
								Scene,
								ProjectedShadowInitializer,
								CachedShadowMapsSize,
								ShadowBufferResolution,
								NumCSMCachesUpdatedThisFrame,
								NumShadowMaps,
								CacheMode);
						}

						for (int32 CacheModeIndex = 0; CacheModeIndex < NumShadowMaps; CacheModeIndex++)
						{
							// Create the projected shadow info.
							FProjectedShadowInfo* ProjectedShadowInfo = Allocator.Create<FProjectedShadowInfo>();
							ProjectedShadowInfo->SetupWholeSceneProjection(
								&LightSceneInfo,
								&View,
								ProjectedShadowInitializer,
								ShadowBufferResolution.X,
								ShadowBufferResolution.Y,
								ShadowBufferResolution.X,
								ShadowBufferResolution.Y,
								ShadowBorder
							);
							ProjectedShadowInfo->FadeAlphas = FadeAlphas;
							ProjectedShadowInfo->ProjectionIndex = Index;
							ProjectedShadowInfo->CacheMode = CacheMode[CacheModeIndex];

							VisibleLightInfo.AllProjectedShadows.Add(ProjectedShadowInfo);
							ShadowInfos.Add(ProjectedShadowInfo);

							// Ray traced shadows use the GPU managed distance field object buffers, no CPU culling needed
							if (!ProjectedShadowInfo->bRayTracedDistanceField)
							{
								// If shadows are timesliced, we are reusing results from a previous frame, and shouldn't kick off expensive primitive culling work
								if (!bTimesliced)
								{
									ProjectedShadowInfo->SetupCullingQuery(*this);

									ShadowInfosThatNeedCulling.Add(ProjectedShadowInfo);
								}

								if (CacheMode[CacheModeIndex] != SDCM_Uncached)
								{
									FCachedShadowMapData& CachedShadowMapData = Scene->GetExtension<FCSMSceneExtension>().GetCachedShadowMapDataRef(LightSceneInfo.Id, ProjectedShadowInfo->CascadeSettings.ShadowSplitIndex);

									CachedShadowMapData.LastFrameExtraStaticShadowSubjects = 0;

									if (CacheMode[CacheModeIndex] == SDCM_StaticPrimitivesOnly)
									{
										CachedShadowMapData.MaxSubjectZ = ProjectedShadowInfo->MaxSubjectZ;
										CachedShadowMapData.MinSubjectZ = ProjectedShadowInfo->MinSubjectZ;
										CachedShadowMapData.PreShadowTranslation = ProjectedShadowInfo->PreShadowTranslation;
									}
									else if (CacheMode[CacheModeIndex] == SDCM_CSMScrolling)
									{
										const FVector FaceDirection(1, 0, 0);
										FVector	XAxis, YAxis;
										FaceDirection.FindBestAxisVectors(XAxis, YAxis);
										const FMatrix WorldToLight = ProjectedShadowInitializer.WorldToLight * FBasisVectorMatrix(-XAxis, YAxis, FaceDirection.GetSafeNormal(), FVector::ZeroVector);
										const FMatrix LightToWorld = WorldToLight.InverseFast();

										FVector UpVector = LightToWorld.TransformVector(FVector(0.0f, 1.0f, 0.0f));
										FVector RightVector = LightToWorld.TransformVector(FVector(1.0f, 0.0f, 0.0f));

										// Projected the centers to the light coordinate, recalculate the projected center because the ProjectedShadowInfo's PreShadowTranslation was modified
										FVector ProjectedCurrentCenter = WorldToLight.TransformPosition(-ProjectedShadowInfo->PreShadowTranslation);
										FVector ProjectedCachedCenter = WorldToLight.TransformPosition(-CachedShadowMapData.PreShadowTranslation);
										FVector ProjectedCenterVector = ProjectedCachedCenter - ProjectedCurrentCenter;

										ProjectedShadowInfo->CSMScrollingZOffset = ProjectedCenterVector.Z;

										FVector2D BorderScale = FVector2D(float(ProjectedShadowInfo->ResolutionX + 2 * ProjectedShadowInfo->BorderSize) / ProjectedShadowInfo->ResolutionX, float(ProjectedShadowInfo->ResolutionY + 2 * ProjectedShadowInfo->BorderSize) / ProjectedShadowInfo->ResolutionY);
										FVector2D CurrentRadius = ProjectedShadowInitializer.SubjectBounds.SphereRadius * BorderScale;
										FVector2D CachedRadius = CachedShadowMapData.Initializer.SubjectBounds.SphereRadius * BorderScale;

										FBox CurrentViewport = FBox(FVector(-CurrentRadius.X, -CurrentRadius.Y, 0.0f), FVector(CurrentRadius.X, CurrentRadius.Y, 0.0f));
										FBox CachedViewport = FBox(FVector(ProjectedCenterVector.X - CachedRadius.X, ProjectedCenterVector.Y - CachedRadius.Y, 0.0f), FVector(ProjectedCenterVector.X + CachedRadius.X, ProjectedCenterVector.Y + CachedRadius.Y, 0.0f));

										FBox OverlappedViewport = CurrentViewport.Overlap(CachedViewport);

										float CachedViewportWidth = CachedViewport.Max.X - CachedViewport.Min.X;
										float CachedViewportHeight = CachedViewport.Max.Y - CachedViewport.Min.Y;
										ProjectedShadowInfo->OverlappedUVOnCachedShadowMap = FVector4f((OverlappedViewport.Min.X - CachedViewport.Min.X) / CachedViewportWidth
											, (CachedViewport.Max.Y - OverlappedViewport.Max.Y) / CachedViewportHeight
											, (OverlappedViewport.Max.X - CachedViewport.Min.X) / CachedViewportWidth
											, (CachedViewport.Max.Y - OverlappedViewport.Min.Y) / CachedViewportHeight);

										float CurrentViewportWidth = CurrentViewport.Max.X - CurrentViewport.Min.X;
										float CurrentViewportHeight = CurrentViewport.Max.Y - CurrentViewport.Min.Y;
										ProjectedShadowInfo->OverlappedUVOnCurrentShadowMap = FVector4f((OverlappedViewport.Min.X - CurrentViewport.Min.X) / CurrentViewportWidth
											, (CurrentViewport.Max.Y - OverlappedViewport.Max.Y) / CurrentViewportHeight
											, (OverlappedViewport.Max.X - CurrentViewport.Min.X) / CurrentViewportWidth
											, (CurrentViewport.Max.Y - OverlappedViewport.Min.Y) / CurrentViewportHeight);

										TArray<FPlane, TInlineAllocator<4>>& ExtraCullingPlanes = ProjectedShadowInfo->CSMScrollingExtraCullingPlanes;

										ExtraCullingPlanes.Empty();

										//Top
										if (CurrentViewport.Max.Y > OverlappedViewport.Max.Y)
										{
											ExtraCullingPlanes.Add(FPlane(LightToWorld.TransformPosition(ProjectedCurrentCenter + FVector(0.0f, OverlappedViewport.Max.Y, 0.0f)), -UpVector));
										}

										//Bottom
										if (CurrentViewport.Min.Y < OverlappedViewport.Min.Y)
										{
											ExtraCullingPlanes.Add(FPlane(LightToWorld.TransformPosition(ProjectedCurrentCenter + FVector(0.0f, OverlappedViewport.Min.Y, 0.0f)), UpVector));
										}

										//Left
										if (CurrentViewport.Min.X < OverlappedViewport.Min.X)
										{
											ExtraCullingPlanes.Add(FPlane(LightToWorld.TransformPosition(ProjectedCurrentCenter + FVector(OverlappedViewport.Min.X, 0.0f, 0.0f)), RightVector));
										}

										//Right
										if (CurrentViewport.Max.X > OverlappedViewport.Max.X)
										{
											ExtraCullingPlanes.Add(FPlane(LightToWorld.TransformPosition(ProjectedCurrentCenter + FVector(OverlappedViewport.Max.X, 0.0f, 0.0f)), -RightVector));
										}
									}
								}
							}
						}
					}
				}
			}

			// Note: it is possible for a non-directional light to set up a proxy that requests a view-dependent SM (or several)
			// Unmodified UE does not use this, so the virtual SM ignores this path for the time being (more likely to be removed).
			const bool bDirectionalLight = LightSceneInfo.Proxy->GetLightType() == LightType_Directional;
			if (bNeedsVirtualShadowMap && bDirectionalLight)
			{
				if (ensure(VirtualShadowMapArray.IsEnabled()))
				{
					FShadowSceneRenderer& ShadowSceneRenderer = GetSceneExtensionsRenderers().GetRenderer<FShadowSceneRenderer>();
					ShadowSceneRenderer.AddDirectionalLightShadow(LightSceneInfo, View, MaxNonFarCascadeDistance, ShadowInfosThatNeedCulling);
				}
			}
		}
	}
}

void FSceneRenderer::AllocateShadowDepthTargets(FDynamicShadowsTaskData& TaskData)
{
	SCOPED_NAMED_EVENT_TEXT("FSceneRenderer::AllocateShadowDepthTargets", FColor::Magenta);

	const bool bMobile = FeatureLevel < ERHIFeatureLevel::SM5;

	const FFilteredShadowArrays& ShadowArrays = TaskData.ShadowArrays;

	FRHICommandList& RHICmdList = *TaskData.RHICmdListForAllocateTargets;

	if (ShadowArrays.CachedPreShadows.Num() > 0)
	{
		if (!Scene->PreShadowCacheDepthZ)
		{
			FPooledRenderTargetDesc Desc(FPooledRenderTargetDesc::Create2DDesc(GetPreShadowCacheTextureResolution(FeatureLevel), PF_ShadowDepth, FClearValueBinding::None, TexCreate_None, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource, false));
			GRenderTargetPool.FindFreeElement(RHICmdList, Desc, Scene->PreShadowCacheDepthZ, TEXT("PreShadowCacheDepthZ"));
		}

		SortedShadowsForShadowDepthPass.PreshadowCache.RenderTargets.DepthTarget = Scene->PreShadowCacheDepthZ;
		for (int32 ShadowIndex = 0; ShadowIndex < ShadowArrays.CachedPreShadows.Num(); ShadowIndex++)
		{
			FProjectedShadowInfo* ProjectedShadowInfo = ShadowArrays.CachedPreShadows[ShadowIndex];
			ProjectedShadowInfo->RenderTargets.DepthTarget = Scene->PreShadowCacheDepthZ.GetReference();
			SortedShadowsForShadowDepthPass.PreshadowCache.Shadows.Add(ProjectedShadowInfo);
		}
	}

	AllocateOnePassPointLightDepthTargets(RHICmdList, ShadowArrays.WholeScenePointShadows);
	AllocateCachedShadowDepthTargets(RHICmdList, ShadowArrays.CachedSpotlightShadows);
	AllocateCachedShadowDepthTargets(RHICmdList, ShadowArrays.CachedWholeSceneDirectionalShadows);
	if (bMobile)
	{
		AllocateMobileCSMAndSpotLightShadowDepthTargets(RHICmdList, ShadowArrays.WholeSceneDirectionalShadows);
	}
	else
	{
		AllocateCSMDepthTargets(RHICmdList, ShadowArrays.WholeSceneDirectionalShadows, SortedShadowsForShadowDepthPass.ShadowMapAtlases);
	}
	AllocateAtlasedShadowDepthTargets(RHICmdList, ShadowArrays.Shadows, SortedShadowsForShadowDepthPass.ShadowMapAtlases);
	AllocateTranslucentShadowDepthTargets(RHICmdList, ShadowArrays.TranslucentShadows);

	// Update translucent shadow map uniform buffers.
	for (int32 TranslucentShadowIndex = 0; TranslucentShadowIndex < ShadowArrays.TranslucentShadows.Num(); ++TranslucentShadowIndex)
	{
		FProjectedShadowInfo* ShadowInfo = ShadowArrays.TranslucentShadows[TranslucentShadowIndex];
		const int32 PrimitiveIndex = ShadowInfo->GetParentSceneInfo()->GetIndex();

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			FViewInfo& View = Views[ViewIndex];
			FUniformBufferRHIRef* UniformBufferPtr = View.TranslucentSelfShadowUniformBufferMap.Find(PrimitiveIndex);

			if (UniformBufferPtr)
			{
				FTranslucentSelfShadowUniformParameters Parameters;
				SetupTranslucentSelfShadowUniformParameters(ShadowInfo, Parameters);
				RHICmdList.UpdateUniformBuffer(*UniformBufferPtr, &Parameters);
			}
		}
	}

	SET_MEMORY_STAT(STAT_CachedShadowmapMemory, Scene->GetExtension<FCSMSceneExtension>().GetCachedWholeSceneShadowMapsSize());
	SET_MEMORY_STAT(STAT_ShadowmapAtlasMemory, SortedShadowsForShadowDepthPass.ComputeMemorySize());
}

struct FLayoutAndAssignedShadows
{
	FLayoutAndAssignedShadows(int32 MaxTextureSize) :
		TextureLayout(1, 1, MaxTextureSize, MaxTextureSize, false, ETextureLayoutAspectRatio::None, false)
	{}

	FLayoutAndAssignedShadows(FIntPoint MaxTextureSize) :
		TextureLayout(1, 1, MaxTextureSize.X, MaxTextureSize.Y, false, ETextureLayoutAspectRatio::None, false)
	{}

	FTextureLayout TextureLayout;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> Shadows;
};

void FSceneRenderer::AllocateAtlasedShadowDepthTargets(
	FRHICommandListBase& RHICmdList,
	TConstArrayView<FProjectedShadowInfo*> Shadows,
	TArray<FSortedShadowMapAtlas,SceneRenderingAllocator>& OutAtlases)
{
	if (Shadows.Num() == 0)
	{
		return;
	}

	const FIntPoint MaxTextureSize = GetShadowDepthTextureResolution(FeatureLevel);

	TArray<FLayoutAndAssignedShadows, SceneRenderingAllocator> Layouts;

	// If shadow casters use stencil culling they can share a depth atlas
	bool bCanUseShadowDepthAtlas = FProjectedShadowInfo::SupportsShadowStencilCulling(ShaderPlatform);

	// Allocate shared atlas
	if (bCanUseShadowDepthAtlas)
	{
		Layouts.Add(FLayoutAndAssignedShadows(MaxTextureSize));
	}

	for (int32 ShadowIndex = 0; ShadowIndex < Shadows.Num(); ShadowIndex++)
	{
		FProjectedShadowInfo* ProjectedShadowInfo = Shadows[ShadowIndex];

		// Atlased shadows need a border
		check(ProjectedShadowInfo->BorderSize != 0);
		check(!ProjectedShadowInfo->bAllocated);

		if (ProjectedShadowInfo->CacheMode == SDCM_MovablePrimitivesOnly && !ProjectedShadowInfo->HasSubjectPrims())
		{
			const FCachedShadowMapData& CachedShadowMapData = Scene->GetExtension<FCSMSceneExtension>().GetCachedShadowMapDataRef(ProjectedShadowInfo->GetLightSceneInfo().Id);
			ProjectedShadowInfo->X = ProjectedShadowInfo->Y = 0;
			ProjectedShadowInfo->bAllocated = true;
			// Skip the shadow depth pass since there are no movable primitives to composite, project from the cached shadowmap directly which contains static primitive depths
			ProjectedShadowInfo->RenderTargets.CopyReferencesFromRenderTargets(CachedShadowMapData.ShadowMap);
		}
		else
		{
			auto addCaster = [&](FLayoutAndAssignedShadows& Layout, FProjectedShadowInfo* ProjectedShadowInfo) -> bool
			{
				if (Layout.TextureLayout.AddElement(
					ProjectedShadowInfo->X,
					ProjectedShadowInfo->Y,
					ProjectedShadowInfo->ResolutionX + ProjectedShadowInfo->BorderSize * 2,
					ProjectedShadowInfo->ResolutionY + ProjectedShadowInfo->BorderSize * 2)
					)
				{
					ProjectedShadowInfo->bAllocated = true;
					Layout.Shadows.Add(ProjectedShadowInfo);
					return true;
				}

				return false;
			};

			if (bCanUseShadowDepthAtlas)
			{
				// Avoid infinite loop if texture cannot be allocated even on a fresh atlas
				// This should not occur, but good to have a safeguard and will still trigger the check() below
				for (int32 Attempt = 0; Attempt < 2; ++Attempt)
				{
					if (!addCaster(Layouts.Last(), ProjectedShadowInfo))
					{
						// Out of space, add a new atlas and try again
						Layouts.Add(FLayoutAndAssignedShadows(MaxTextureSize));
						continue;
					}

					break;
				}
			}
			else
			{
				// Unique shadow map per caster
				Layouts.Add(FLayoutAndAssignedShadows(MaxTextureSize));
				addCaster(Layouts.Last(), ProjectedShadowInfo);
			}

			check(ProjectedShadowInfo->bAllocated);
		}
	}

	const TCHAR *ShadowMapName = bCanUseShadowDepthAtlas ? TEXT("ShadowDepthAtlas") : TEXT("ShadowDepth");

	for (int32 LayoutIndex = 0; LayoutIndex < Layouts.Num(); LayoutIndex++)
	{
		const FLayoutAndAssignedShadows& CurrentLayout = Layouts[LayoutIndex];

		OutAtlases.AddDefaulted();
		FSortedShadowMapAtlas& ShadowMapAtlas = OutAtlases.Last();

		// Snap atlas sizes to powers of 2 to keep them more consistent for the render target pool
		FIntPoint AtlasSize(
			FMath::Min((uint32)MaxTextureSize.X, FMath::RoundUpToPowerOfTwo(CurrentLayout.TextureLayout.GetSizeX())),
			FMath::Min((uint32)MaxTextureSize.Y, FMath::RoundUpToPowerOfTwo(CurrentLayout.TextureLayout.GetSizeY())));

		if (CVarAlwaysAllocateMaxResolutionAtlases.GetValueOnRenderThread() != 0)
		{
			AtlasSize = MaxTextureSize;
		}

		FPooledRenderTargetDesc ShadowMapDesc2D = FPooledRenderTargetDesc::Create2DDesc(AtlasSize, PF_ShadowDepth, FClearValueBinding::DepthOne, TexCreate_None, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource, false);
		ShadowMapDesc2D.Flags |= GFastVRamConfig.ShadowPerObject;
		GRenderTargetPool.FindFreeElement(RHICmdList, ShadowMapDesc2D, ShadowMapAtlas.RenderTargets.DepthTarget, ShadowMapName);

		for (int32 ShadowIndex = 0; ShadowIndex < CurrentLayout.Shadows.Num(); ShadowIndex++)
		{
			FProjectedShadowInfo* ProjectedShadowInfo = CurrentLayout.Shadows[ShadowIndex];

			if (ProjectedShadowInfo->bAllocated)
			{
				ProjectedShadowInfo->RenderTargets.CopyReferencesFromRenderTargets(ShadowMapAtlas.RenderTargets);
				ShadowMapAtlas.Shadows.Add(ProjectedShadowInfo);
			}
		}
	}
}

/**
* Helper class to get the name of an indexed rendertarget, keeping the pointers around (this is required by the rendertarget pool)
*/
class RenderTargetNameSet
{
private:
	FString Prefix;
	TArray<FString> Names;

public:
	RenderTargetNameSet(const TCHAR* Prefix)
		: Prefix(Prefix), Names()
	{ }

	const TCHAR* Get(const int32 Index)
	{
		const int32 Count = Index + 1;
		while (Names.Num() < Count)
		{
			if (Index == 0)
			{
				Names.Add(Prefix);
			}
			else
			{
				Names.Emplace(FString::Printf(TEXT("%s%d"), *Prefix, Names.Num()));
			}
		}
		return *Names[Index];
	}
};

void FSceneRenderer::AllocateCachedShadowDepthTargets(FRHICommandListBase& RHICmdList, TConstArrayView<FProjectedShadowInfo*> CachedShadows)
{
	FCSMSceneExtension& CSMShadowScene = Scene->GetExtension<FCSMSceneExtension>();
	for (int32 ShadowIndex = 0; ShadowIndex < CachedShadows.Num(); ShadowIndex++)
	{
		FProjectedShadowInfo* ProjectedShadowInfo = CachedShadows[ShadowIndex];

		bool bIsWholeSceneDirectionalShadow = ProjectedShadowInfo->IsWholeSceneDirectionalShadow();

		SortedShadowsForShadowDepthPass.ShadowMapAtlases.AddDefaulted();
		FSortedShadowMapAtlas& ShadowMap = SortedShadowsForShadowDepthPass.ShadowMapAtlases.Last();

		FIntPoint ShadowResolution(ProjectedShadowInfo->ResolutionX + ProjectedShadowInfo->BorderSize * 2, ProjectedShadowInfo->ResolutionY + ProjectedShadowInfo->BorderSize * 2);
		FPooledRenderTargetDesc ShadowMapDesc2D = FPooledRenderTargetDesc::Create2DDesc(ShadowResolution, PF_ShadowDepth, FClearValueBinding::DepthOne, TexCreate_None, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource, false, 1, false);
		static RenderTargetNameSet CachedSplitDepthRTNames(TEXT("CachedShadowDepthMap_Split"));
		const TCHAR* RTDebugName = bIsWholeSceneDirectionalShadow ? CachedSplitDepthRTNames.Get(ProjectedShadowInfo->CascadeSettings.ShadowSplitIndex) : TEXT("CachedShadowDepthMap");
		GRenderTargetPool.FindFreeElement(RHICmdList, ShadowMapDesc2D, ShadowMap.RenderTargets.DepthTarget, RTDebugName);

		check(ProjectedShadowInfo->CacheMode == SDCM_StaticPrimitivesOnly);
		FCachedShadowMapData& CachedShadowMapData = CSMShadowScene.GetCachedShadowMapDataRef(ProjectedShadowInfo->GetLightSceneInfo().Id, FMath::Max(ProjectedShadowInfo->CascadeSettings.ShadowSplitIndex, 0));
		CachedShadowMapData.ShadowMap = ShadowMap.RenderTargets;

		ProjectedShadowInfo->X = ProjectedShadowInfo->Y = 0;
		ProjectedShadowInfo->bAllocated = true;
		ProjectedShadowInfo->RenderTargets.CopyReferencesFromRenderTargets(ShadowMap.RenderTargets);

		ShadowMap.Shadows.Add(ProjectedShadowInfo);
	}
}

void FSceneRenderer::AllocateCSMDepthTargets(
	FRHICommandListBase& RHICmdList,
	TConstArrayView<FProjectedShadowInfo*> WholeSceneDirectionalShadows,
	TArray<FSortedShadowMapAtlas, SceneRenderingAllocator>& OutAtlases
	)
{
	if (WholeSceneDirectionalShadows.Num() > 0)
	{
		const bool bAllowAtlasing = !NeedsUnatlasedCSMDepthsWorkaround(FeatureLevel);

		const int32 MaxTextureSize = 1 << (GMaxTextureMipCount - 1);
		TArray<FLayoutAndAssignedShadows, SceneRenderingAllocator> Layouts;
		Layouts.Add(FLayoutAndAssignedShadows(MaxTextureSize));

		for (int32 ShadowIndex = 0; ShadowIndex < WholeSceneDirectionalShadows.Num(); ShadowIndex++)
		{
			if (!bAllowAtlasing && ShadowIndex > 0)
			{
				Layouts.Add(FLayoutAndAssignedShadows(MaxTextureSize));
			}

			FProjectedShadowInfo* ProjectedShadowInfo = WholeSceneDirectionalShadows[ShadowIndex];

			// Atlased shadows need a border
			check(!bAllowAtlasing || ProjectedShadowInfo->BorderSize != 0);
			check(!ProjectedShadowInfo->bAllocated);

			if (Layouts.Last().TextureLayout.AddElement(
				ProjectedShadowInfo->X,
				ProjectedShadowInfo->Y,
				ProjectedShadowInfo->ResolutionX + ProjectedShadowInfo->BorderSize * 2,
				ProjectedShadowInfo->ResolutionY + ProjectedShadowInfo->BorderSize * 2)
				)
			{
				ProjectedShadowInfo->bAllocated = true;
				Layouts.Last().Shadows.Add(ProjectedShadowInfo);
			}
		}

		for (int32 LayoutIndex = 0; LayoutIndex < Layouts.Num(); LayoutIndex++)
		{
			const FLayoutAndAssignedShadows& CurrentLayout = Layouts[LayoutIndex];
			OutAtlases.AddDefaulted();
			FSortedShadowMapAtlas& ShadowMapAtlas = OutAtlases.Last();

			FIntPoint WholeSceneAtlasSize(CurrentLayout.TextureLayout.GetSizeX(), CurrentLayout.TextureLayout.GetSizeY());
			FPooledRenderTargetDesc WholeSceneShadowMapDesc2D(FPooledRenderTargetDesc::Create2DDesc(WholeSceneAtlasSize, PF_ShadowDepth, FClearValueBinding::DepthOne, TexCreate_None, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource, false));
			WholeSceneShadowMapDesc2D.Flags |= GFastVRamConfig.ShadowCSM;
			static RenderTargetNameSet CsmRtNames(TEXT("WholeSceneShadowmap"));
			GRenderTargetPool.FindFreeElement(RHICmdList, WholeSceneShadowMapDesc2D, ShadowMapAtlas.RenderTargets.DepthTarget, CsmRtNames.Get(LayoutIndex));

			for (FProjectedShadowInfo* ProjectedShadowInfo : CurrentLayout.Shadows)
			{
				check(ProjectedShadowInfo->bAllocated);
				ProjectedShadowInfo->RenderTargets.CopyReferencesFromRenderTargets(ShadowMapAtlas.RenderTargets);
				ShadowMapAtlas.Shadows.Add(ProjectedShadowInfo);
			}
		}
	}
}

void FSceneRenderer::AllocateOnePassPointLightDepthTargets(FRHICommandListBase& RHICmdList, TConstArrayView<FProjectedShadowInfo*> WholeScenePointShadows)
{
	if (FeatureLevel >= ERHIFeatureLevel::SM5 || DoesRuntimeSupportOnePassPointLightShadows(ShaderPlatform))
	{
		FCSMSceneExtension& CSMShadowScene = Scene->GetExtension<FCSMSceneExtension>();
		for (int32 ShadowIndex = 0; ShadowIndex < WholeScenePointShadows.Num(); ShadowIndex++)
		{
			FProjectedShadowInfo* ProjectedShadowInfo = WholeScenePointShadows[ShadowIndex];
			check(ProjectedShadowInfo->BorderSize == 0);

			if (ProjectedShadowInfo->CacheMode == SDCM_MovablePrimitivesOnly && !ProjectedShadowInfo->HasSubjectPrims())
			{
				const FCachedShadowMapData& CachedShadowMapData = CSMShadowScene.GetCachedShadowMapDataRef(ProjectedShadowInfo->GetLightSceneInfo().Id);
				ProjectedShadowInfo->X = ProjectedShadowInfo->Y = 0;
				ProjectedShadowInfo->bAllocated = true;
				// Skip the shadow depth pass since there are no movable primitives to composite, project from the cached shadowmap directly which contains static primitive depths
				check(CachedShadowMapData.ShadowMap.IsValid());
				ProjectedShadowInfo->RenderTargets.CopyReferencesFromRenderTargets(CachedShadowMapData.ShadowMap);
			}
			else
			{
				SortedShadowsForShadowDepthPass.ShadowMapCubemaps.AddDefaulted();
				FSortedShadowMapAtlas& ShadowMapCubemap = SortedShadowsForShadowDepthPass.ShadowMapCubemaps.Last();

				FPooledRenderTargetDesc Desc(FPooledRenderTargetDesc::CreateCubemapDesc(ProjectedShadowInfo->ResolutionX, PF_ShadowDepth, FClearValueBinding::DepthFar, TexCreate_None, TexCreate_DepthStencilTargetable | TexCreate_NoFastClear | TexCreate_ShaderResource, false, 1, 1, false));
				Desc.Flags |= GFastVRamConfig.ShadowPointLight;
				GRenderTargetPool.FindFreeElement(RHICmdList, Desc, ShadowMapCubemap.RenderTargets.DepthTarget, TEXT("CubeShadowDepthZ") );

				if (ProjectedShadowInfo->CacheMode == SDCM_StaticPrimitivesOnly)
				{
					FCachedShadowMapData& CachedShadowMapData = CSMShadowScene.GetCachedShadowMapDataRef(ProjectedShadowInfo->GetLightSceneInfo().Id);
					CachedShadowMapData.ShadowMap = ShadowMapCubemap.RenderTargets;
				}

				ProjectedShadowInfo->X = ProjectedShadowInfo->Y = 0;
				ProjectedShadowInfo->bAllocated = true;
				ProjectedShadowInfo->RenderTargets.CopyReferencesFromRenderTargets(ShadowMapCubemap.RenderTargets);

				ShadowMapCubemap.Shadows.Add(ProjectedShadowInfo);
			}
		}
	}
}

// for easier use of "VisualizeTexture"
TCHAR* const GetTranslucencyShadowTransmissionName(uint32 Id)
{
	// (TCHAR*) for non VisualStudio
	switch(Id)
	{
		case 0: return (TCHAR*)TEXT("TranslucencyShadowTransmission0");
		case 1: return (TCHAR*)TEXT("TranslucencyShadowTransmission1");

		default:
			check(0);
	}
	return (TCHAR*)TEXT("InvalidName");
}

void FSceneRenderer::AllocateTranslucentShadowDepthTargets(FRHICommandListBase& RHICmdList, TConstArrayView<FProjectedShadowInfo*> TranslucentShadows)
{
	if (TranslucentShadows.Num() > 0 && FeatureLevel >= ERHIFeatureLevel::SM5)
	{
		const FIntPoint TranslucentShadowBufferResolution = GetTranslucentShadowDepthTextureResolution();

		// Start with an empty atlas for per-object shadows (don't allow packing object shadows into the CSM atlas atm)
		SortedShadowsForShadowDepthPass.TranslucencyShadowMapAtlases.AddDefaulted();

		FTextureLayout CurrentShadowLayout(1, 1, TranslucentShadowBufferResolution.X, TranslucentShadowBufferResolution.Y, false, ETextureLayoutAspectRatio::None, false);

		for (int32 ShadowIndex = 0; ShadowIndex < TranslucentShadows.Num(); ShadowIndex++)
		{
			FProjectedShadowInfo* ProjectedShadowInfo = TranslucentShadows[ShadowIndex];

			check(ProjectedShadowInfo->BorderSize != 0);
			check(!ProjectedShadowInfo->bAllocated);

			if (CurrentShadowLayout.AddElement(
				ProjectedShadowInfo->X,
				ProjectedShadowInfo->Y,
				ProjectedShadowInfo->ResolutionX + ProjectedShadowInfo->BorderSize * 2,
				ProjectedShadowInfo->ResolutionY + ProjectedShadowInfo->BorderSize * 2)
				)
			{
				ProjectedShadowInfo->bAllocated = true;
			}
			else
			{
				CurrentShadowLayout = FTextureLayout(1, 1, TranslucentShadowBufferResolution.X, TranslucentShadowBufferResolution.Y, false, ETextureLayoutAspectRatio::None, false);
				SortedShadowsForShadowDepthPass.TranslucencyShadowMapAtlases.AddDefaulted();

				if (CurrentShadowLayout.AddElement(
					ProjectedShadowInfo->X,
					ProjectedShadowInfo->Y,
					ProjectedShadowInfo->ResolutionX + ProjectedShadowInfo->BorderSize * 2,
					ProjectedShadowInfo->ResolutionY + ProjectedShadowInfo->BorderSize * 2)
					)
				{
					ProjectedShadowInfo->bAllocated = true;
				}
			}

			check(ProjectedShadowInfo->bAllocated);

			FSortedShadowMapAtlas& ShadowMapAtlas = SortedShadowsForShadowDepthPass.TranslucencyShadowMapAtlases.Last();

			if (ShadowMapAtlas.RenderTargets.ColorTargets.Num() == 0)
			{
				ShadowMapAtlas.RenderTargets.ColorTargets.Empty(NumTranslucencyShadowSurfaces);
				ShadowMapAtlas.RenderTargets.ColorTargets.AddDefaulted(NumTranslucencyShadowSurfaces);

				for (int32 SurfaceIndex = 0; SurfaceIndex < NumTranslucencyShadowSurfaces; SurfaceIndex++)
				{
					// Using PF_FloatRGBA because Fourier coefficients used by Fourier opacity maps have a large range and can be negative
					FPooledRenderTargetDesc Desc(FPooledRenderTargetDesc::Create2DDesc(TranslucentShadowBufferResolution, PF_FloatRGBA, FClearValueBinding::None, TexCreate_None, TexCreate_RenderTargetable | TexCreate_ShaderResource, false));
					GRenderTargetPool.FindFreeElement(RHICmdList, Desc, ShadowMapAtlas.RenderTargets.ColorTargets[SurfaceIndex], GetTranslucencyShadowTransmissionName(SurfaceIndex));
				}
			}

			ProjectedShadowInfo->RenderTargets.CopyReferencesFromRenderTargets(ShadowMapAtlas.RenderTargets);
			ShadowMapAtlas.Shadows.Add(ProjectedShadowInfo);
		}
	}
}

void FSceneRenderer::CreateDynamicShadows(FDynamicShadowsTaskData& TaskData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateDynamicShadows);

	const bool bMobile = FeatureLevel < ERHIFeatureLevel::SM5;
	const bool bHairStrands = HairStrands::HasHairInstanceInScene(*Scene);

	const bool bProjectEnablePointLightShadows = FReadOnlyCVARCache::EnablePointLightShadows(ShaderPlatform);
	const bool bProjectEnableMovableSpotLightShadows = !bMobile || IsMobileMovableSpotlightShadowsEnabled(ShaderPlatform);
	const bool bMobileInsetShadows = CVarMobileInsetShadows.GetValueOnRenderThread();
	const bool bCapsuleDirectShadows = IsCapsuleDirectShadowsEnabled(ShaderPlatform);

	uint32 NumPointShadowCachesUpdatedThisFrame = 0;
	uint32 NumSpotShadowCachesUpdatedThisFrame = 0;
	uint32 NumCSMCachesUpdatedThisFrame = 0;

	const bool bStaticSceneOnly = TaskData.bStaticSceneOnly;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& PreShadows = TaskData.PreShadows;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> ViewDependentWholeSceneShadows;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ViewDependentWholeSceneShadowsThatNeedCulling = TaskData.ViewDependentWholeSceneShadows;

	{
		SCOPE_CYCLE_COUNTER(STAT_InitDynamicShadowsTime);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ShadowInitDynamic);

		if (GetShadowQuality() > 0)
		{
			int64 CachedShadowMapsSize = -1;

			const auto GetCachedShadowMapsSize = [&]
			{
				if (CachedShadowMapsSize < 0)
				{
					// This call is quite expensive, so compute it lazily.
					CachedShadowMapsSize = Scene->GetExtension<FCSMSceneExtension>().GetCachedWholeSceneShadowMapsSize();
				}
				return CachedShadowMapsSize;
			};

			for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
			{
				FLightSceneInfo* LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;

				FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];

				const ELightOcclusionType OcclusionType = GetLightOcclusionType(LightSceneInfoCompact, ViewFamily);

				// Register visible lights for allowing hair strands to cast shadow (non-directional light)
				if (bHairStrands && OcclusionType == ELightOcclusionType::Raytraced && LightSceneInfo->Proxy->GetLightType() != LightType_Directional)
				{
					if (LightSceneInfoCompact.bCastStaticShadow || LightSceneInfoCompact.bCastDynamicShadow)
					{
						for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
						{
							// View frustums are only checked when lights have visible primitives or have modulated shadows,
							// so we don't need to check for that again here
							if (LightSceneInfo->ShouldRenderLight(Views[ViewIndex]))
							{
								HairStrands::AddVisibleShadowCastingLight(*Scene, Views, LightSceneInfo);
								break;
							}
						}
					}
				}

				if (OcclusionType != ELightOcclusionType::Shadowmap &&
					OcclusionType != ELightOcclusionType::MegaLightsVSM)
				{
					continue;
				}

				// Only consider lights that may have shadows.
				if (LightSceneInfoCompact.bCastStaticShadow || LightSceneInfoCompact.bCastDynamicShadow)
				{
					// see if the light is visible in any view
					bool bIsVisibleInAnyView = false;

					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
					{
						// View frustums are only checked when lights have visible primitives or have modulated shadows,
						// so we don't need to check for that again here
						bIsVisibleInAnyView = LightSceneInfo->ShouldRenderLight(Views[ViewIndex]);

						if (bIsVisibleInAnyView)
						{
							break;
						}
					}

					if (bIsVisibleInAnyView)
					{
						FScopeCycleCounter Context(LightSceneInfo->Proxy->GetStatId());

						const bool bAllowStaticLighting = IsStaticLightingAllowed();
						const bool bPointLightShadow = LightSceneInfoCompact.LightType == LightType_Point || LightSceneInfoCompact.LightType == LightType_Rect;
						const bool bDirectionalLightShadow = LightSceneInfoCompact.LightType == LightType_Directional;
						const bool bSpotLightShadow = LightSceneInfoCompact.LightType == LightType_Spot;

						// Only create whole scene shadows for lights that don't precompute shadowing (movable lights)
						const bool bShouldCreateShadowForMovableLight =
							LightSceneInfoCompact.bCastDynamicShadow
							&& (!LightSceneInfo->Proxy->HasStaticShadowing() || !bAllowStaticLighting);

						const bool bCreateShadowForMovableLight =
							bShouldCreateShadowForMovableLight
							&& (!bPointLightShadow || bProjectEnablePointLightShadows)
							&& (!bSpotLightShadow || bProjectEnableMovableSpotLightShadows);

						// Also create a whole scene shadow for lights with precomputed shadows that are unbuilt
						const bool bShouldCreateShadowToPreviewStaticLight =
							LightSceneInfo->Proxy->HasStaticShadowing()
							&& LightSceneInfoCompact.bCastStaticShadow
							&& !LightSceneInfo->IsPrecomputedLightingValid();

						const bool bCreateShadowToPreviewStaticLight =
							bShouldCreateShadowToPreviewStaticLight
							&& (!bPointLightShadow || bProjectEnablePointLightShadows)
							// Stationary point light and spot light shadow are unsupported on mobile
							&& (!bMobile || bDirectionalLightShadow);

						// Create a whole scene shadow for lights that want static shadowing but didn't get assigned to a valid shadowmap channel due to overlap
						const bool bShouldCreateShadowForOverflowStaticShadowing =
							LightSceneInfo->Proxy->HasStaticShadowing()
							&& !LightSceneInfo->Proxy->HasStaticLighting()
							&& LightSceneInfoCompact.bCastStaticShadow
							&& LightSceneInfo->IsPrecomputedLightingValid()
							&& LightSceneInfo->Proxy->GetShadowMapChannel() == INDEX_NONE;

						const bool bCreateShadowForOverflowStaticShadowing =
							bShouldCreateShadowForOverflowStaticShadowing
							&& (!bPointLightShadow || bProjectEnablePointLightShadows)
							// Stationary point light and spot light shadow are unsupported on mobile
							&& (!bMobile || bDirectionalLightShadow);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
						bUsedPointLightWholeSceneShadow = bUsedPointLightWholeSceneShadow || ((bShouldCreateShadowForMovableLight || bShouldCreateShadowForOverflowStaticShadowing || bShouldCreateShadowToPreviewStaticLight) && bPointLightShadow);
#endif

						if (bCreateShadowForMovableLight || bCreateShadowToPreviewStaticLight || bCreateShadowForOverflowStaticShadowing)
						{
							// Try to create a whole scene projected shadow.
							CreateWholeSceneProjectedShadow(TaskData, LightSceneInfo, GetCachedShadowMapsSize(), NumPointShadowCachesUpdatedThisFrame, NumSpotShadowCachesUpdatedThisFrame);
						}

						// Register visible lights for allowing hair strands to cast shadow (non-directional light)
						if (bHairStrands && LightSceneInfo->Proxy->GetLightType() != LightType_Directional)
						{
							HairStrands::AddVisibleShadowCastingLight(*Scene, Views, LightSceneInfo);
						}

						// Allow movable and stationary lights to create CSM, or static lights that are unbuilt
						if ((!LightSceneInfo->Proxy->HasStaticLighting() && LightSceneInfoCompact.bCastDynamicShadow) || bCreateShadowToPreviewStaticLight)
						{
							static_assert(UE_ARRAY_COUNT(Scene->MobileDirectionalLights) == 3, "All array entries for MobileDirectionalLights must be checked");
							if (!bMobile ||
								((LightSceneInfo->Proxy->UseCSMForDynamicObjects() || !LightSceneInfo->Proxy->HasStaticShadowing())
									// Mobile uses the scene's MobileDirectionalLights only for whole scene shadows.
									&& (LightSceneInfo == Scene->MobileDirectionalLights[0] || LightSceneInfo == Scene->MobileDirectionalLights[1] || LightSceneInfo == Scene->MobileDirectionalLights[2])))
							{
								AddViewDependentWholeSceneShadowsForView(ViewDependentWholeSceneShadows, ViewDependentWholeSceneShadowsThatNeedCulling, VisibleLightInfo, *LightSceneInfo, GetCachedShadowMapsSize(), NumCSMCachesUpdatedThisFrame);
							}

							if (!bMobile || bMobileInsetShadows || bCapsuleDirectShadows || (LightSceneInfo->Proxy->CastsModulatedShadows() && !LightSceneInfo->Proxy->UseCSMForDynamicObjects() && LightSceneInfo->Proxy->HasStaticShadowing()))
							{
								if (!LightSceneInfo->Proxy->UseVirtualShadowMaps())
								{
									const TArray<FLightPrimitiveInteraction*>* InteractionShadowPrimitives = LightSceneInfo->GetInteractionShadowPrimitives();

									if (InteractionShadowPrimitives)
									{
										const int32 NumPrims = InteractionShadowPrimitives->Num();
										for (int32 Idx = 0; Idx < NumPrims; ++Idx)
										{
											SetupInteractionShadows(TaskData, (*InteractionShadowPrimitives)[Idx], VisibleLightInfo, bStaticSceneOnly, ViewDependentWholeSceneShadows, PreShadows);
										}
									}
									else
									{
										// Look for individual primitives with a dynamic shadow.
										for (FLightPrimitiveInteraction* Interaction = LightSceneInfo->GetDynamicInteractionOftenMovingPrimitiveList();
											Interaction;
											Interaction = Interaction->GetNextPrimitive()
											)
										{
											SetupInteractionShadows(TaskData, Interaction, VisibleLightInfo, bStaticSceneOnly, ViewDependentWholeSceneShadows, PreShadows);
										}

										for (FLightPrimitiveInteraction* Interaction = LightSceneInfo->GetDynamicInteractionStaticPrimitiveList();
											Interaction;
											Interaction = Interaction->GetNextPrimitive()
											)
										{
											SetupInteractionShadows(TaskData, Interaction, VisibleLightInfo, bStaticSceneOnly, ViewDependentWholeSceneShadows, PreShadows);
										}
									}
								}
							}
						}

						if (!TaskData.bHasRayTracedDistanceFieldShadows)
						{
							for (int32 ShadowIndex = 0; ShadowIndex < VisibleLightInfo.AllProjectedShadows.Num(); ShadowIndex++)
							{
								const FProjectedShadowInfo* ProjectedShadowInfo = VisibleLightInfo.AllProjectedShadows[ShadowIndex];

								if (ProjectedShadowInfo->bRayTracedDistanceField)
								{
									TaskData.bHasRayTracedDistanceFieldShadows = true;
									break;
								}
							}
						}
					}
				}
			}
		}

		// Remove cache entries that haven't been used in a while
		Scene->GetExtension<FCSMSceneExtension>().RemoveExpiredCacheEntries(ViewFamily.Time.GetRealTimeSeconds());

		CSV_CUSTOM_STAT(LightCount, UpdatedShadowMaps, float(NumPointShadowCachesUpdatedThisFrame + NumSpotShadowCachesUpdatedThisFrame + NumCSMCachesUpdatedThisFrame), ECsvCustomStatOp::Set);

		// Calculate visibility of the projected shadows.
		InitProjectedShadowVisibility(TaskData);

		// At this point the shadows are initialized and we can start gathering lights for shading.
		TaskData.BeginGatherAndSortLightsTask.Trigger();
	}

	// Clear old preshadows and attempt to add new ones to the cache
	UpdatePreshadowCache();

	if (FShadowSceneRenderer* ShadowSceneRenderer = GetSceneExtensionsRenderers().GetRendererPtr<FShadowSceneRenderer>())
	{
		ShadowSceneRenderer->PostInitDynamicShadowsSetup();
	}
}

void WaitForFilterDynamicShadowsTask(FDynamicShadowsTaskData* TaskData)
{
	if (TaskData)
	{
		TaskData->FilterDynamicShadowsTask.Wait();
	}
}

void FSceneRenderer::FilterDynamicShadows(FDynamicShadowsTaskData& TaskData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateFilterShadowPrimitives);

	FFilteredShadowArrays& ShadowArrays = TaskData.ShadowArrays;

	const bool bMobile = Scene->GetFeatureLevel() < ERHIFeatureLevel::SM5;
	// Whether to merge spot light shadows into directional light atlas
	const bool bMergeMobileDynamicSpotlightShadows = bMobile && !IsMobileDeferredShadingEnabled(ShaderPlatform);
	
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> WholeSceneDirectionalShadowsForLight;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> MobileDynamicSpotlightShadows;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	auto MarkUnsupportedOnePassShadow = [&ShadowArrays](FLightSceneId LightId)
	{
		ShadowArrays.OnePassShadowUnsupportedLights.PadToId(LightId, false);
		ShadowArrays.OnePassShadowUnsupportedLights[LightId] = true;
	};
#else 
	auto MarkUnsupportedOnePassShadow = [](FLightSceneId LightId) {};
#endif

	for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)		
	{
		const FLightSceneInfo* LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;
		FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];

		for (int32 ShadowIndex = 0; ShadowIndex < VisibleLightInfo.AllProjectedShadows.Num(); ShadowIndex++)
		{
			FProjectedShadowInfo* ProjectedShadowInfo = VisibleLightInfo.AllProjectedShadows[ShadowIndex];

			// Check that the shadow is visible in at least one view before rendering it.
			bool bShadowIsVisible = false;

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				const FViewInfo& View = Views[ViewIndex];

				if (ProjectedShadowInfo->DependentView && ProjectedShadowInfo->DependentView != &View)
				{
					continue;
				}

				const FVisibleLightViewInfo& VisibleLightViewInfo = View.VisibleLightInfos[LightSceneInfo->Id];
				const FPrimitiveViewRelevance ViewRelevance = VisibleLightViewInfo.ProjectedShadowViewRelevanceMap[ShadowIndex];
				const bool bHasViewRelevance = (ProjectedShadowInfo->bTranslucentShadow && ViewRelevance.HasTranslucency())
					|| (!ProjectedShadowInfo->bTranslucentShadow && ViewRelevance.bOpaque);

				bShadowIsVisible |= bHasViewRelevance && VisibleLightViewInfo.ProjectedShadowVisibilityMap[ShadowIndex];
			}

			// Fully cached VSM should not be skipped, since they need to be queued for projection.
			if (!ProjectedShadowInfo->bShouldRenderVSM)
			{
			}
			// Skip the shadow rendering if there is no primitive in the cached shadow map and the current shadow map either
			else if ((ProjectedShadowInfo->CacheMode == SDCM_StaticPrimitivesOnly || (ProjectedShadowInfo->CacheMode == SDCM_MovablePrimitivesOnly && !ProjectedShadowInfo->IsWholeSceneDirectionalShadow())) && !ProjectedShadowInfo->HasSubjectPrims())
			{
				const FCachedShadowMapData& CachedShadowMapData = Scene->GetExtension<FCSMSceneExtension>().GetCachedShadowMapDataRef(ProjectedShadowInfo->GetLightSceneInfo().Id, FMath::Max(ProjectedShadowInfo->CascadeSettings.ShadowSplitIndex, 0));

				// A shadowmap for movable primitives when there are no movable primitives would normally read directly from the cached shadowmap
				// However if the cached shadowmap also had no primitives then we need to skip rendering the shadow entirely
				if (!CachedShadowMapData.bCachedShadowMapHasPrimitives)
				{
					bShadowIsVisible = false;
				}
			}
			// If uncached and no primitives, skip allocations etc
			// Note: cached PreShadows, for some reason, has CacheMode == SDCM_Uncached so need to skip them here.
			// Ray traced shadows use the GPU managed distance field object buffers, no CPU culling should be used
			else if (
				(ProjectedShadowInfo->CacheMode == SDCM_Uncached && !ProjectedShadowInfo->bPreShadow)
				&& (!ProjectedShadowInfo->HasSubjectPrims() && !ProjectedShadowInfo->bRayTracedDistanceField)
				&& !ProjectedShadowInfo->IsWholeSceneDirectionalShadow())
			{
				bShadowIsVisible = false;
			}

			if ((IsForwardShadingEnabled(ShaderPlatform) || (IsMobilePlatform(ShaderPlatform) && MobileUsesShadowMaskTexture(ShaderPlatform)))
				&& ProjectedShadowInfo->GetLightSceneInfo().GetDynamicShadowMapChannel() == -1)
			{
				// With forward shading, dynamic shadows are projected into channels of the light attenuation texture based on their assigned DynamicShadowMapChannel
				bShadowIsVisible = false;
			}

			// Skip one-pass point light shadows if rendering to them is not suported, unless VSM or DF
			if (bShadowIsVisible
				&& !ProjectedShadowInfo->HasVirtualShadowMap()
				&& !ProjectedShadowInfo->bRayTracedDistanceField
				&& ProjectedShadowInfo->bOnePassPointLightShadow
				&& !DoesRuntimeSupportOnePassPointLightShadows(ShaderPlatform))
			{
				// Also note this to produce a log message & on-screen warning
				MarkUnsupportedOnePassShadow(ProjectedShadowInfo->GetLightSceneInfo().Id);
				bShadowIsVisible = false;
			}

			// Shadows must be disabled if timesliced shadows are enabled, but no timeslice is active for the given shadow map.  This
			// generally happens in the editor where a shadow caster is modified in a way that forces the render proxy to be recreated
			// or destroyed.
			if (IsMobilePlatform(ShaderPlatform) &&
				Views[0].IsRuntimeReflectionCapture() &&
				ProjectedShadowInfo->IsWholeSceneDirectionalShadow() &&
				Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame &&
				Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame != LightSceneInfo->Proxy->TimesliceAtlasFrame)
			{
				bShadowIsVisible = false;
				Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFailed = true;
			}

			if (bShadowIsVisible)
			{
				// Visible shadow stats
				if (ProjectedShadowInfo->bWholeSceneShadow)
				{
					INC_DWORD_STAT(STAT_WholeSceneShadows);

					if (ProjectedShadowInfo->CacheMode == SDCM_MovablePrimitivesOnly)
					{
						INC_DWORD_STAT(STAT_CachedWholeSceneShadows);
					}
				}
				else if (ProjectedShadowInfo->bPreShadow)
				{
					INC_DWORD_STAT(STAT_PreShadows);
				}
				else
				{
					INC_DWORD_STAT(STAT_PerObjectShadows);
				}

				bool bNeedsProjection = ProjectedShadowInfo->CacheMode != SDCM_StaticPrimitivesOnly
					//// Filter out everything but PerObjectOpaqueShadows, Distance field shadows and capsule shadows for ES31
					&& (!bMobile || ProjectedShadowInfo->bPerObjectOpaqueShadow || ProjectedShadowInfo->bRayTracedDistanceField || ProjectedShadowInfo->bWholeSceneShadow || (ProjectedShadowInfo->bCapsuleShadow && IsMobileCapsuleShadowsEnabled(ShaderPlatform)));

				if (bNeedsProjection)
				{
					if (ProjectedShadowInfo->bCapsuleShadow)
					{
						VisibleLightInfo.CapsuleShadowsToProject.Add(ProjectedShadowInfo);
					}
					else
					{
						if (ProjectedShadowInfo->bRayTracedDistanceField)
						{
							ShadowArrays.ProjectedDistanceFieldShadows.Add(ProjectedShadowInfo);
						}
						VisibleLightInfo.ShadowsToProject.Add(ProjectedShadowInfo);
					}
				}

				const bool bNeedsShadowmapSetup = !ProjectedShadowInfo->bCapsuleShadow && !ProjectedShadowInfo->bRayTracedDistanceField;

				if (bNeedsShadowmapSetup)
				{
					// Certain shadow types skip the depth pass when there are no movable primitives to composite.
					bool bAlwaysHasDepthPass = true;

					if (ProjectedShadowInfo->HasVirtualShadowMap())
					{
						SortedShadowsForShadowDepthPass.VirtualShadowMapShadows.Add(ProjectedShadowInfo);
					}
					else if (ProjectedShadowInfo->bPreShadow && ProjectedShadowInfo->bAllocatedInPreshadowCache)
					{
						ShadowArrays.CachedPreShadows.Add(ProjectedShadowInfo);
					}
					else if (ProjectedShadowInfo->bDirectionalLight && ProjectedShadowInfo->bWholeSceneShadow)
					{
						if (ProjectedShadowInfo->CacheMode == SDCM_StaticPrimitivesOnly)
						{
							ShadowArrays.CachedWholeSceneDirectionalShadows.Add(ProjectedShadowInfo);
						}
						else
						{
							WholeSceneDirectionalShadowsForLight.Add(ProjectedShadowInfo);
						}
					}
					else if (ProjectedShadowInfo->bOnePassPointLightShadow)
					{
						ShadowArrays.WholeScenePointShadows.Add(ProjectedShadowInfo);
						bAlwaysHasDepthPass = false;
					}
					else if (ProjectedShadowInfo->bTranslucentShadow)
					{
						ShadowArrays.TranslucentShadows.Add(ProjectedShadowInfo);
					}
					else if (ProjectedShadowInfo->CacheMode == SDCM_StaticPrimitivesOnly)
					{
						check(ProjectedShadowInfo->bWholeSceneShadow);
						ShadowArrays.CachedSpotlightShadows.Add(ProjectedShadowInfo);
					}
					else if (bMergeMobileDynamicSpotlightShadows && ProjectedShadowInfo->bWholeSceneShadow)
					{
						MobileDynamicSpotlightShadows.Add(ProjectedShadowInfo);
					}
					else
					{
						ShadowArrays.Shadows.Add(ProjectedShadowInfo);
						bAlwaysHasDepthPass = false;
					}

					if (bAlwaysHasDepthPass || ProjectedShadowInfo->CacheMode != SDCM_MovablePrimitivesOnly || ProjectedShadowInfo->HasSubjectPrims())
					{
						ShadowArrays.ShadowsToSetupViews.Add(ProjectedShadowInfo);
					}
				}
			}
		}

		// Sort cascades, this is needed for blending between cascades to work, and also to enable fetching the farthest cascade as index [0]
		VisibleLightInfo.ShadowsToProject.Sort(FCompareFProjectedShadowInfoBySplitIndex());

		if (!bMobile)
		{
			// Only allocate CSM targets if at least one of the SMs in the CSM has any primitives, but then always allocate all of them as some algorithms depend on this (notably the forward shading structures).
			// Don't perform this if VSM are not enabled.
			bool bAnyHasSubjectPrims = !VirtualShadowMapArray.IsEnabled();
			for (const FProjectedShadowInfo* ProjectedShadowInfo : WholeSceneDirectionalShadowsForLight)
			{
				if (bAnyHasSubjectPrims)
				{
					break;
				}

				bAnyHasSubjectPrims |= ProjectedShadowInfo->HasSubjectPrims();
			}
			if (bAnyHasSubjectPrims)
			{
				ShadowArrays.WholeSceneDirectionalShadows.Append(WholeSceneDirectionalShadowsForLight);
			}
		}
		else
		{
			//Only one directional light could cast csm on mobile, so we could delay allocation for it and see if we could combine any spotlight shadow with it.
			ShadowArrays.WholeSceneDirectionalShadows.Append(WholeSceneDirectionalShadowsForLight);
		}

		WholeSceneDirectionalShadowsForLight.Reset();
	}

	if (MobileDynamicSpotlightShadows.Num() > 0)
	{
		// AllocateMobileCSMAndSpotLightShadowDepthTargets would only allocate a single large render target for all shadows, so if the requirement exceeds the MaxTextureSize, the rest of the shadows will not get space for rendering
		// So we sort spotlight shadows and append them at the last to make sure csm will get space in any case.
		MobileDynamicSpotlightShadows.Sort(FCompareFProjectedShadowInfoByResolution());

		//Limit the number of spotlights shadow for performance reason
		static const auto MobileMaxVisibleMovableSpotLightShadowsCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.MaxVisibleMovableSpotLightShadows"));
		if (MobileMaxVisibleMovableSpotLightShadowsCVar)
		{
			int32 MobileMaxVisibleMovableSpotLightShadows = MobileMaxVisibleMovableSpotLightShadowsCVar->GetValueOnRenderThread();
			MobileDynamicSpotlightShadows.SetNum(FMath::Min(MobileDynamicSpotlightShadows.Num(), MobileMaxVisibleMovableSpotLightShadows), EAllowShrinking::No);
		}

		ShadowArrays.WholeSceneDirectionalShadows.Append(MobileDynamicSpotlightShadows);
		MobileDynamicSpotlightShadows.Reset();
	}

	// Sort the projected shadows by resolution.
	ShadowArrays.TranslucentShadows.Sort(FCompareFProjectedShadowInfoByResolution());

	AllocateShadowDepthTargets(TaskData);

	TaskData.FilterDynamicShadowsTask.Trigger();
}

FDynamicShadowsTaskData* FSceneRenderer::BeginInitDynamicShadows(FRDGBuilder& GraphBuilder, bool bRunningEarly, IVisibilityTaskData* VisibilityTaskData, FInstanceCullingManager& InstanceCullingManager)
{
	SCOPE_CYCLE_COUNTER(STAT_DynamicShadowSetupTime);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(InitViews_Shadows);
	SCOPED_NAMED_EVENT_TEXT("FSceneRenderer::BeginInitDynamicShadows", FColor::Magenta);

	FDynamicShadowsTaskData* DynamicShadowTaskData = Allocator.Create<FDynamicShadowsTaskData>(GraphBuilder.RHICmdList, this, InstanceCullingManager, bRunningEarly);

	// Gathers the list of primitives used to draw various shadow types
	BeginGatherShadowPrimitives(DynamicShadowTaskData, VisibilityTaskData);

	return DynamicShadowTaskData;
}


void FSceneRenderer::FinishInitDynamicShadows(FRDGBuilder& GraphBuilder, FDynamicShadowsTaskData* TaskData)
{
	SCOPED_NAMED_EVENT_TEXT("FSceneRenderer::FinishInitDynamicShadows", FColor::Magenta);
	SCOPE_CYCLE_COUNTER(STAT_InitDynamicShadowsTime);
	SCOPE_CYCLE_COUNTER(STAT_DynamicShadowSetupTime);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ShadowInitDynamic);

	check(TaskData);

	FinishGatherShadowPrimitives(TaskData);

	if (FShadowSceneRenderer* ShadowSceneRenderer = GetSceneExtensionsRenderers().GetRendererPtr<FShadowSceneRenderer>())
	{
		ShadowSceneRenderer->DispatchVirtualShadowMapViewAndCullingSetup(GraphBuilder, SortedShadowsForShadowDepthPass.VirtualShadowMapShadows);
		ShadowSceneRenderer->PostSetupDebugRender();
	}
}

void FSceneRenderer::FinishDynamicShadowMeshPassSetup(FRDGBuilder& GraphBuilder, FDynamicShadowsTaskData* TaskData)
{
	// This can be called early in InitDynamicShadows.
	if (TaskData->bFinishedMeshPassSetup)
	{
		return;
	}
	
	CSV_SCOPED_SET_WAIT_STAT(ShadowsSetupMeshPass)
	TaskData->SetupMeshPassTask.Wait();
	TaskData->MeshCollectors.Empty();

	for (FRHICommandListImmediate::FQueuedCommandList& QueueCmdList : TaskData->CommandLists)
	{
		QueueCmdList.CmdList->FinishRecording();
	}
	GraphBuilder.RHICmdList.QueueAsyncCommandListSubmit(TaskData->CommandLists);
	TaskData->CommandLists.Empty();

	DynamicReadBufferForShadows.Commit(GraphBuilder.RHICmdList);

	// Ensure all shadow view dynamic primitives are uploaded before shadow-culling batching pass.
	// TODO: automate this such that:
	//  1. we only process views that need it (have dynamic primitives)
	//  2. it is integrated in the GPU-scene (it already collects the dynamic primives and know about them...)
	//  3. BUT: we need to touch the views to update the GPUScene buffer references in the FViewInfo
	//          so need to refactor that into its own binding point, probably. Or something.
	for (FSortedShadowMapAtlas& ShadowMapAtlas : SortedShadowsForShadowDepthPass.ShadowMapAtlases)
	{
		for (FProjectedShadowInfo* ProjectedShadowInfo : ShadowMapAtlas.Shadows)
		{
			Scene->GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, *ProjectedShadowInfo->ShadowDepthView);
		}
	}
	for (FSortedShadowMapAtlas& ShadowMap : SortedShadowsForShadowDepthPass.ShadowMapCubemaps)
	{
		check(ShadowMap.Shadows.Num() == 1);
		FProjectedShadowInfo* ProjectedShadowInfo = ShadowMap.Shadows[0];
		Scene->GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, *ProjectedShadowInfo->ShadowDepthView);
	}
	for (FProjectedShadowInfo* ProjectedShadowInfo : SortedShadowsForShadowDepthPass.PreshadowCache.Shadows)
	{
		if (!ProjectedShadowInfo->bDepthsCached)
		{
			Scene->GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, *ProjectedShadowInfo->ShadowDepthView);
		}
	}
	for (const FSortedShadowMapAtlas& ShadowMapAtlas : SortedShadowsForShadowDepthPass.TranslucencyShadowMapAtlases)
	{
		for (FProjectedShadowInfo* ProjectedShadowInfo : ShadowMapAtlas.Shadows)
		{
			Scene->GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, *ProjectedShadowInfo->ShadowDepthView);
		}
	}
	for (FProjectedShadowInfo* ProjectedShadowInfo : SortedShadowsForShadowDepthPass.VirtualShadowMapShadows)
	{
		Scene->GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, *ProjectedShadowInfo->ShadowDepthView, false, GetShadowInvalidatingInstancesInterface(ProjectedShadowInfo->DependentView) );
	}

	TaskData->bFinishedMeshPassSetup = true;
}

FDynamicShadowsTaskData* FSceneRenderer::InitDynamicShadows(FRDGBuilder& GraphBuilder, FInstanceCullingManager& InstanceCullingManager)
{
	FDynamicShadowsTaskData* TaskData = BeginInitDynamicShadows(GraphBuilder, false, nullptr, InstanceCullingManager);
	FinishInitDynamicShadows(GraphBuilder, TaskData);
	FinishDynamicShadowMeshPassSetup(GraphBuilder, TaskData);
	return TaskData;
}

void FSceneRenderer::AllocateMobileCSMAndSpotLightShadowDepthTargets(FRHICommandListBase& RHICmdList, TConstArrayView<FProjectedShadowInfo*> MobileCSMAndSpotLightShadows)
{
	if (MobileCSMAndSpotLightShadows.Num() > 0)
	{
		// Decide if we want to generate or reuse a timesliced shadow atlas layout.  There is no need to timeslice if all 6 views are being rendered at once.
		bool bRuntimeReflectionTimeslice = false;
		bool bRuntimeReflectionTimesliceFirst = false;
		if (Views[0].IsRuntimeReflectionCapture() && Views.Num() < 6)
		{
			// Check if this the first render of a timeslice.  It should include the first view, and either be a render of multiple views or a render
			// with FaceCount == 0, indicating it's a standalone shadow render occurring before the faces are rendered.
			if (Views[0].UnscaledViewRect.Min.X == 0 && Views[0].UnscaledViewRect.Min.Y == 0 && (Views.Num() > 1 || Scene->ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_RT == 0))
			{
				// Check the CVar for the first view.  Track the atlas frame if so.
				bRuntimeReflectionTimeslice = IsSceneCaptureCubeShareCascadesEnabled(ShaderPlatform);
				if (bRuntimeReflectionTimeslice)
				{
					bRuntimeReflectionTimesliceFirst = true;
				}
			}
			else
			{
				// For subsequent views, check whether the timesliced atlas was generated with the first view.  TimesliceAtlasFrame is cleared to
				// zero before a new runtime reflection capture is kicked off.
				bRuntimeReflectionTimeslice = Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame != 0;
			}
		}

		const int32 MaxTextureSize = 1 << (GMaxTextureMipCount - 1);
		FLayoutAndAssignedShadows MobileCSMAndSpotLightShadowLayout(MaxTextureSize);

		for (int32 ShadowIndex = 0; ShadowIndex < MobileCSMAndSpotLightShadows.Num(); ShadowIndex++)
		{
			FProjectedShadowInfo* ProjectedShadowInfo = MobileCSMAndSpotLightShadows[ShadowIndex];

			// Atlased shadows need a border
			checkSlow(ProjectedShadowInfo->BorderSize != 0);
			checkSlow(!ProjectedShadowInfo->bAllocated);

			FLightSceneProxy* LightSceneProxy = ProjectedShadowInfo->GetLightSceneInfo().Proxy;

			if (bRuntimeReflectionTimeslice && !bRuntimeReflectionTimesliceFirst)
			{
				// We should have filtered out the shadow as non-renderable earlier if timeslicing is enabled, but no valid atlas location is available
				check(LightSceneProxy->TimesliceAtlasFrame == Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame);
			
				const FReflectionEnvironmentCachedShadow& CachedShadow = Scene->ReflectionSceneData.RuntimeCaptureShadows.Shadows[LightSceneProxy->TimesliceAtlasIndex[ProjectedShadowInfo->ProjectionIndex]];
				ProjectedShadowInfo->X = CachedShadow.X;
				ProjectedShadowInfo->Y = CachedShadow.Y;

				ProjectedShadowInfo->bAllocated = true;
				MobileCSMAndSpotLightShadowLayout.Shadows.Add(ProjectedShadowInfo);
			}
			else if (MobileCSMAndSpotLightShadowLayout.TextureLayout.AddElement(
				ProjectedShadowInfo->X,
				ProjectedShadowInfo->Y,
				ProjectedShadowInfo->ResolutionX + ProjectedShadowInfo->BorderSize * 2,
				ProjectedShadowInfo->ResolutionY + ProjectedShadowInfo->BorderSize * 2)
				)
			{
				ProjectedShadowInfo->bAllocated = true;
				MobileCSMAndSpotLightShadowLayout.Shadows.Add(ProjectedShadowInfo);

				// Save off the cached atlas location and frame
				if (bRuntimeReflectionTimeslice)
				{
					int32 TimesliceAtlasIndex = Scene->ReflectionSceneData.RuntimeCaptureShadows.Shadows.AddDefaulted();
					LightSceneProxy->TimesliceAtlasIndex[ProjectedShadowInfo->ProjectionIndex] = TimesliceAtlasIndex;
					LightSceneProxy->TimesliceAtlasFrame = GFrameNumberRenderThread;

					FReflectionEnvironmentCachedShadow& CachedShadow = Scene->ReflectionSceneData.RuntimeCaptureShadows.Shadows[TimesliceAtlasIndex];
					CachedShadow.X = ProjectedShadowInfo->X;
					CachedShadow.Y = ProjectedShadowInfo->Y;
					CachedShadow.Position = LightSceneProxy->Position;
					CachedShadow.WorldToLight = LightSceneProxy->WorldToLight;
					CachedShadow.LightToWorld = LightSceneProxy->LightToWorld;
				}
			}
		}

		FIntPoint WholeSceneAtlasSize = (bRuntimeReflectionTimeslice && !bRuntimeReflectionTimesliceFirst) ?
			Scene->ReflectionSceneData.RuntimeCaptureShadows.AtlasSize :
			FIntPoint(MobileCSMAndSpotLightShadowLayout.TextureLayout.GetSizeX(), MobileCSMAndSpotLightShadowLayout.TextureLayout.GetSizeY());

		if (WholeSceneAtlasSize.X > 0)
		{
			SortedShadowsForShadowDepthPass.ShadowMapAtlases.AddDefaulted();
			FSortedShadowMapAtlas& ShadowMapAtlas = SortedShadowsForShadowDepthPass.ShadowMapAtlases.Last();

			if (CVarMobileShadowmapRoundUpToPowerOfTwo.GetValueOnRenderThread() != 0)
			{
				WholeSceneAtlasSize.X = 1 << FMath::CeilLogTwo(WholeSceneAtlasSize.X);
				WholeSceneAtlasSize.Y = 1 << FMath::CeilLogTwo(WholeSceneAtlasSize.Y);
			}

			if (bRuntimeReflectionTimeslice && !bRuntimeReflectionTimesliceFirst)
			{
				// Reuse timesliced atlas rendered on a previous frame
				ShadowMapAtlas.RenderTargets.DepthTarget = Scene->ReflectionSceneData.RuntimeCaptureShadows.AtlasRenderTarget;
			}
			else
			{
				FPooledRenderTargetDesc WholeSceneShadowMapDesc2D(FPooledRenderTargetDesc::Create2DDesc(WholeSceneAtlasSize, PF_ShadowDepth, FClearValueBinding::DepthOne, TexCreate_None, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource, false));
				WholeSceneShadowMapDesc2D.Flags |= GFastVRamConfig.ShadowCSM;
				GRenderTargetPool.FindFreeElement(RHICmdList, WholeSceneShadowMapDesc2D, ShadowMapAtlas.RenderTargets.DepthTarget,
					bRuntimeReflectionTimeslice ? TEXT("RuntimeCaptureShadowAtlas") : TEXT("MobileCSMAndSpotLightShadowmap"));

				// Store off timesliced atlas information for subsequent frames
				if (bRuntimeReflectionTimeslice)
				{
					Scene->ReflectionSceneData.RuntimeCaptureShadows.AtlasRenderTarget = ShadowMapAtlas.RenderTargets.DepthTarget;
					Scene->ReflectionSceneData.RuntimeCaptureShadows.AtlasSize = WholeSceneAtlasSize;
					Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame = GFrameNumberRenderThread;
				}
			}

			for (int32 ShadowIndex = 0; ShadowIndex < MobileCSMAndSpotLightShadowLayout.Shadows.Num(); ShadowIndex++)
			{
				FProjectedShadowInfo* ProjectedShadowInfo = MobileCSMAndSpotLightShadowLayout.Shadows[ShadowIndex];

				if (ProjectedShadowInfo->bAllocated)
				{
					ProjectedShadowInfo->RenderTargets.CopyReferencesFromRenderTargets(ShadowMapAtlas.RenderTargets);
					ShadowMapAtlas.Shadows.Add(ProjectedShadowInfo);
				}
			}
		}
	}
}
