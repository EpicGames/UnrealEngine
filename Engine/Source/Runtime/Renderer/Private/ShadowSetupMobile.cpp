// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ShadowSetupMobile.cpp: Shadow setup implementation for mobile specific features.
=============================================================================*/

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "Engine/World.h"
#include "EngineDefines.h"
#include "ConvexVolume.h"
#include "RendererInterface.h"
#include "Math/GenericOctree.h"
#include "LightSceneInfo.h"
#include "SceneRendering.h"
#include "DynamicPrimitiveDrawing.h"
#include "ScenePrivate.h"
#include "MeshPassProcessor.h"
#include "UObject/UObjectIterator.h"
#include "RenderCore.h"
#include "ShadowRendering.h"
#include "StaticMeshBatch.h"

static TAutoConsoleVariable<int32> CVarCsmShaderCullingDebugGfx(
	TEXT("r.Mobile.Shadow.CSMShaderCullingDebugGfx"),
	0,
	TEXT(""),
	ECVF_RenderThreadSafe);

const uint32 CSMShaderCullingMethodDefault = 1;
static TAutoConsoleVariable<int32> CVarsCsmShaderCullingMethod(
	TEXT("r.Mobile.Shadow.CSMShaderCullingMethod"),
	CSMShaderCullingMethodDefault,
	TEXT("Method to determine which primitives will receive CSM shaders:\n")
	TEXT("0 - disabled (all primitives will receive CSM)\n")
	TEXT("1 - Light frustum, all primitives whose bounding box is within CSM receiving distance. (default)\n")
	TEXT("2 - Combined caster bounds, all primitives whose bounds are within CSM receiving distance and the capsule of the combined bounds of all casters.\n")
	TEXT("3 - Light frustum + caster bounds, all primitives whose bounds are within CSM receiving distance and capsule of at least one caster. (slowest)\n")
	TEXT("4 - Cull all. Prevent primitives from receiving CSM shadows.\n")
	TEXT("5 - Disable culling if mobile distance field shadowing is used for all views.\n")
	TEXT("Combine with 16 to change primitive bounding test to spheres instead of box. (i.e. 18 == combined casters + sphere test)")
	,ECVF_RenderThreadSafe);

static void OnCsmShaderCullingMethodChanged()
{
	// Cannot do this in editors because feature levels can change
#if !WITH_EDITOR
	static int32 PrevValue = CSMShaderCullingMethodDefault;
	const int32 CurValue = (CVarsCsmShaderCullingMethod.GetValueOnGameThread() & 0xF);
	
	if (CurValue != PrevValue && (CurValue == 5 || PrevValue == 5))
	{
		PrevValue = CurValue;

		if (CurValue == 5)
		{
			TArray<ERHIFeatureLevel::Type> UsedFeatureLevels;
			for (TObjectIterator<UWorld> It; It; ++It)
			{
				const UWorld* World = *It;
				if (World && World->Scene)
				{
					UsedFeatureLevels.AddUnique(World->Scene->GetFeatureLevel());
				}
			}

			bool bBasePassAlwaysUseCSM = UsedFeatureLevels.Num() > 0;
			for (ERHIFeatureLevel::Type FeatureLevel : UsedFeatureLevels)
			{
				bBasePassAlwaysUseCSM = bBasePassAlwaysUseCSM && MobileBasePassAlwaysUsesCSM(GetFeatureLevelShaderPlatform(FeatureLevel));
			}

			const EMeshPassFlags NewFlags = bBasePassAlwaysUseCSM ? EMeshPassFlags::MainView : (EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
			FPassProcessorManager::SetPassFlags(EShadingPath::Mobile, EMeshPass::MobileBasePassCSM, NewFlags);
		}
		else
		{
			FPassProcessorManager::SetPassFlags(EShadingPath::Mobile, EMeshPass::MobileBasePassCSM, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
		}
	}
#endif
}

static FAutoConsoleVariableSink CVarsCsmShaderCullingMethodSink(FConsoleCommandDelegate::CreateStatic(&OnCsmShaderCullingMethodChanged));

static bool CouldStaticMeshEverReceiveCSMFromStationaryLight(ERHIFeatureLevel::Type FeatureLevel, const FPrimitiveSceneInfo* PrimitiveSceneInfo, const FStaticMeshBatch& StaticMesh)
{
	// test if static shadows are allowed in the first place:
	static auto* CVarMobileAllowDistanceFieldShadows = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.AllowDistanceFieldShadows"));
	const bool bMobileAllowDistanceFieldShadows = CVarMobileAllowDistanceFieldShadows->GetValueOnRenderThread() == 1;

	bool bHasCSMApplicableLightInteraction = bMobileAllowDistanceFieldShadows && StaticMesh.LCI && StaticMesh.LCI->GetLightMapInteraction(FeatureLevel).GetType() == LMIT_Texture;
	bool bHasCSMApplicableShadowInteraction = bHasCSMApplicableLightInteraction && StaticMesh.LCI && StaticMesh.LCI->GetShadowMapInteraction(FeatureLevel).GetType() == SMIT_Texture;

	return (bHasCSMApplicableLightInteraction && bHasCSMApplicableShadowInteraction) ||
		(!bHasCSMApplicableLightInteraction && PrimitiveSceneInfo->Proxy->IsMovable());
}

static bool EnableStaticMeshCSMVisibilityState(bool bMovableLight, const FPrimitiveSceneInfo* PrimitiveSceneInfo, FMobileCSMVisibilityInfo& MobileCSMVisibilityInfo, FViewInfo& View)
{
	bool bFoundReceiver = false;
	if (MobileCSMVisibilityInfo.MobilePrimitiveCSMReceiverVisibilityMap[PrimitiveSceneInfo->GetIndex()])
	{
		return bFoundReceiver;
	}

	MobileCSMVisibilityInfo.MobilePrimitiveCSMReceiverVisibilityMap[PrimitiveSceneInfo->GetIndex()] = true;
	INC_DWORD_STAT_BY(STAT_CSMStaticPrimitiveReceivers, 1);
	for (int32 MeshIndex = 0; MeshIndex < PrimitiveSceneInfo->StaticMeshes.Num(); MeshIndex++)
	{
		const FStaticMeshBatch& StaticMesh = PrimitiveSceneInfo->StaticMeshes[MeshIndex];

		bool bHasCSMApplicableShadowInteraction = View.StaticMeshVisibilityMap[StaticMesh.Id] && StaticMesh.LCI;
		bHasCSMApplicableShadowInteraction = bHasCSMApplicableShadowInteraction && StaticMesh.LCI->GetShadowMapInteraction(View.GetFeatureLevel()).GetType() == SMIT_Texture;

		if (bMovableLight || CouldStaticMeshEverReceiveCSMFromStationaryLight(View.GetFeatureLevel(), PrimitiveSceneInfo, StaticMesh))
		{
			const FMaterialRenderProxy* MaterialRenderProxy = StaticMesh.MaterialRenderProxy;
			const FMaterial& Material = MaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);
			if (Material.GetShadingModels().IsLit())
			{
				// CSM enabled list
				MobileCSMVisibilityInfo.MobileCSMStaticMeshVisibilityMap[StaticMesh.Id] = MobileCSMVisibilityInfo.MobileNonCSMStaticMeshVisibilityMap[StaticMesh.Id];
				// CSM excluded list
				MobileCSMVisibilityInfo.MobileNonCSMStaticMeshVisibilityMap[StaticMesh.Id] = false;
				
				INC_DWORD_STAT_BY(STAT_CSMStaticMeshReceivers, 1);
				bFoundReceiver = true;
			}
		}
	}
	return 
		bFoundReceiver || 
		// Dynamic primitives do not have static meshes
		PrimitiveSceneInfo->StaticMeshes.Num() == 0;
}

template<typename TReceiverFunc>
static bool MobileDetermineStaticMeshesCSMVisibilityStateInner(
	FScene* Scene,
	FViewInfo& View,
	const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact,
	FProjectedShadowInfo* ProjectedShadowInfo,
	TReceiverFunc IsReceiverFunc
	)
{
	FConvexVolume ViewVolume;
	const FLightSceneInfo& LightSceneInfo = ProjectedShadowInfo->GetLightSceneInfo();
	FLightSceneProxy* RESTRICT LightProxy = LightSceneInfo.Proxy;
	FVector LightDir = LightProxy->GetDirection();

	FPrimitiveSceneInfo* RESTRICT	PrimitiveSceneInfo = PrimitiveSceneInfoCompact.PrimitiveSceneInfo;
	FPrimitiveSceneProxy* RESTRICT	PrimitiveProxy = PrimitiveSceneInfoCompact.Proxy;
	const FBoxSphereBounds			PrimitiveBounds(PrimitiveSceneInfoCompact.Bounds);
	bool bFoundCSMReceiver = false;
	if (PrimitiveProxy->WillEverBeLit() && PrimitiveProxy->ShouldReceiveMobileCSMShadows()
		&& (PrimitiveProxy->GetLightingChannelMask() & LightProxy->GetLightingChannelMask()) != 0)
	{
		const FVector LightDirection = LightProxy->GetDirection();
		const FVector PrimitiveToShadowCenter = ProjectedShadowInfo->ShadowBounds.Center - PrimitiveBounds.Origin;
		// Project the primitive's bounds origin onto the light vector
		const FVector::FReal ProjectedDistanceFromShadowOriginAlongLightDir = PrimitiveToShadowCenter | LightDirection;
		// Calculate the primitive's squared distance to the cylinder's axis
		const FVector::FReal PrimitiveDistanceFromCylinderAxisSq = (-LightDirection * ProjectedDistanceFromShadowOriginAlongLightDir + PrimitiveToShadowCenter).SizeSquared();
		const FVector::FReal CombinedRadiusSq = FMath::Square(ProjectedShadowInfo->ShadowBounds.W + PrimitiveBounds.SphereRadius);

		// Include all primitives for movable lights, but only statically shadowed primitives from a light with static shadowing,
		// Since lights with static shadowing still create per-object shadows for primitives without static shadowing.
		if ((!LightProxy->HasStaticLighting() || (!LightSceneInfo.IsPrecomputedLightingValid() || LightProxy->UseCSMForDynamicObjects()))
			// Check if this primitive is in the shadow's cylinder
			&& PrimitiveDistanceFromCylinderAxisSq < CombinedRadiusSq
			// Check if the primitive is closer than the cylinder cap toward the light
			// next line is commented as it breaks large world shadows, if this was meant to be an optimization we should think about a better solution
			//// && ProjectedDistanceFromShadowOriginAlongLightDir - PrimitiveBounds.SphereRadius < -ProjectedShadowInfo->MinPreSubjectZ
			// If the primitive is further along the cone axis than the shadow bounds origin, 
			// Check if the primitive is inside the spherical cap of the cascade's bounds
			&& !(ProjectedDistanceFromShadowOriginAlongLightDir < 0
				&& PrimitiveToShadowCenter.SizeSquared() > CombinedRadiusSq))
		{
			FVisibleLightViewInfo& VisibleLightViewInfo = View.VisibleLightInfos[LightSceneInfo.Id];

			const FPrimitiveViewRelevance& Relevance = View.PrimitiveViewRelevanceMap[PrimitiveSceneInfo->GetIndex()];
			const bool bLit = (Relevance.ShadingModelMask != (1 << MSM_Unlit));
			bool bCanReceiveDynamicShadow =
				bLit
				&& (Relevance.bOpaque || Relevance.bMasked)
				&& IsReceiverFunc(PrimitiveBounds.Origin, PrimitiveBounds.BoxExtent, PrimitiveBounds.SphereRadius);

			if (bCanReceiveDynamicShadow)
			{
				bool bMovableLightUsingCSM = LightProxy->IsMovable() && LightSceneInfo.ShouldRenderViewIndependentWholeSceneShadows();
				bFoundCSMReceiver = EnableStaticMeshCSMVisibilityState(bMovableLightUsingCSM, PrimitiveSceneInfo, View.MobileCSMVisibilityInfo, View);
			}
		}
	}
	return bFoundCSMReceiver;
}

template<typename TReceiverFunc>
static bool MobileDetermineStaticMeshesCSMVisibilityState(FScene* Scene, FViewInfo& View, FProjectedShadowInfo* WholeSceneShadow, TReceiverFunc IsReceiverFunc)
{
	bool bFoundReceiver = false;
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ShadowOctreeTraversal);
		// Find primitives that are in a shadow frustum in the octree.
		Scene->PrimitiveOctree.FindElementsWithPredicate([&IsReceiverFunc](FScenePrimitiveOctree::FNodeIndex /*ParentNodeIndex*/, FScenePrimitiveOctree::FNodeIndex /*NodeIndex*/, const FBoxCenterAndExtent& NodeBounds)
		{
			return IsReceiverFunc(FVector(NodeBounds.Center), FVector(NodeBounds.Extent), NodeBounds.Extent.Size3());
		},
		[&bFoundReceiver, Scene, &View, WholeSceneShadow, &IsReceiverFunc](FScenePrimitiveOctree::FNodeIndex /*ParentNodeIndex*/, const FPrimitiveSceneInfoCompact& Primitive)
		{
			// gather the shadows for this one primitive
			bFoundReceiver = MobileDetermineStaticMeshesCSMVisibilityStateInner(Scene, View, Primitive, WholeSceneShadow, IsReceiverFunc) || bFoundReceiver;
		});
	}
	return bFoundReceiver;
}

static void VisualizeMobileDynamicCSMSubjectCapsules(FViewInfo& View, FLightSceneInfo* LightSceneInfo, FProjectedShadowInfo* ProjectedShadowInfo)
{
	auto DrawDebugCapsule = [](FViewInfo& InView, const FLightSceneInfo* InLightSceneInfo, const FVector& Start, float CastLength, float CapsuleRadius)
	{
		const FMatrix& LightToWorld = InLightSceneInfo->Proxy->GetLightToWorld();
		FViewElementPDI ShadowFrustumPDI(&InView, nullptr, nullptr);
		FVector Dir = LightToWorld.GetUnitAxis(EAxis::X);
		FVector End = Start + (Dir*CastLength);
		DrawWireSphere(&ShadowFrustumPDI, FTransform(Start), FColor::White, CapsuleRadius, 40, 0);
		DrawWireCapsule(&ShadowFrustumPDI, Start + Dir*0.5f*CastLength, LightToWorld.GetUnitAxis(EAxis::Z), LightToWorld.GetUnitAxis(EAxis::Y), Dir,
			FColor(231, 0, 0, 255), CapsuleRadius, 0.5f * CastLength + CapsuleRadius, 25, SDPG_World);
		ShadowFrustumPDI.DrawLine(Start, End, FColor::Black, 0);
	};

	FVisibleLightViewInfo& VisibleLightViewInfo = View.VisibleLightInfos[LightSceneInfo->Id];
	FMobileCSMSubjectPrimitives& MobileCSMSubjectPrimitives = VisibleLightViewInfo.MobileCSMSubjectPrimitives;
	FVector LightDir = LightSceneInfo->Proxy->GetDirection();
	const FVector::FReal ShadowCastLength = WORLD_MAX;
	const uint32 CullingMethod = CVarsCsmShaderCullingMethod.GetValueOnRenderThread() & 0xF;
	const bool bSphereTest = (CVarsCsmShaderCullingMethod.GetValueOnRenderThread() & 0x10) != 0;

	switch (CullingMethod)
	{
		case 2:
		{
			// Combined bounds
			FVector CombinedCasterStart;
			FVector CombinedCasterEnd;
			FBoxSphereBounds::Builder CombinedBoundsBuilder;
			for (auto& Caster : MobileCSMSubjectPrimitives.GetShadowSubjectPrimitives())
			{
				CombinedBoundsBuilder += Caster->Proxy->GetBounds();
			}
			FBoxSphereBounds CombinedBounds(CombinedBoundsBuilder);
			CombinedCasterStart = CombinedBounds.Origin;
			CombinedCasterEnd = CombinedBounds.Origin + (LightDir * ShadowCastLength);

			DrawDebugCapsule(View, LightSceneInfo, CombinedCasterStart, ShadowCastLength, CombinedBounds.SphereRadius);
			break;
		}
		case 3:
		{
			// All casters.
			for (auto& Caster : MobileCSMSubjectPrimitives.GetShadowSubjectPrimitives())
			{
				const FBoxSphereBounds& CasterBounds = Caster->Proxy->GetBounds();
				const FVector& CasterStart = CasterBounds.Origin;
				const FVector CasterEnd = CasterStart + (LightDir * ShadowCastLength);
				DrawDebugCapsule(View, LightSceneInfo, CasterStart, ShadowCastLength, CasterBounds.SphereRadius);
			}
			break;
		}
		default:
		{
			if (CullingMethod >= 1 && CullingMethod <= 3)
			{
				// all culling modes draw the receiver frustum.
				FViewElementPDI ShadowFrustumPDI(&View, nullptr, nullptr);
				FMatrix Receiver(ProjectedShadowInfo->InvReceiverInnerMatrix);
				DrawFrustumWireframe(&ShadowFrustumPDI, Receiver * FTranslationMatrix(-ProjectedShadowInfo->PreShadowTranslation), FColor::Cyan, 0);
			}
		}
		break;
	}
}

/** Finds the visible dynamic shadows for each view. */
FDynamicShadowsTaskData* FMobileSceneRenderer::InitDynamicShadows(FRDGBuilder& GraphBuilder, FInstanceCullingManager& InstanceCullingManager)
{
	const bool bCombinedStaticAndCSMEnabled = FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers();
	const bool bMobileEnableMovableLightCSMShaderCulling = FReadOnlyCVARCache::MobileEnableMovableLightCSMShaderCulling();

	// initialize CSMVisibilityInfo for each eligible light.
	for (FLightSceneInfo* MobileDirectionalLightSceneInfo : Scene->MobileDirectionalLights)
	{
		const bool bShouldRecordShadowSubjectsForMobile = MobileDirectionalLightSceneInfo ? MobileDirectionalLightSceneInfo->ShouldRecordShadowSubjectsForMobile() : false;

		if (bShouldRecordShadowSubjectsForMobile)
		{
			int32 PrimitiveCount = Scene->Primitives.Num();
			for (auto& View : Views)
			{
				// Mirrors logic from FScene::GetMobileDirectionalLightForView, see comment there for more info
				if (MobileDirectionalLightSceneInfo->Proxy->GetViewLightingChannelMask() & View.ViewLightingChannelMask)
				{
					FMobileCSMSubjectPrimitives& MobileCSMSubjectPrimitives = View.VisibleLightInfos[MobileDirectionalLightSceneInfo->Id].MobileCSMSubjectPrimitives;
					MobileCSMSubjectPrimitives.InitShadowSubjectPrimitives(PrimitiveCount);
				}
			}
		}
	}

	FDynamicShadowsTaskData* TaskData = FSceneRenderer::InitDynamicShadows(GraphBuilder, InstanceCullingManager);

	bool bAlwaysUseCSM = false;
	const bool bSkipCSMShaderCulling = MobileBasePassAlwaysUsesCSM(Scene->GetShaderPlatform());

	if (bSkipCSMShaderCulling)
	{
		bAlwaysUseCSM = true;
	}
	else
	{
		PrepareViewVisibilityLists();

		for (FLightSceneInfo* MobileDirectionalLightSceneInfo : Scene->MobileDirectionalLights)
		{
			const FLightSceneProxy* LightSceneProxy = MobileDirectionalLightSceneInfo ? MobileDirectionalLightSceneInfo->Proxy : nullptr;
			if (LightSceneProxy)
			{
				bool bLightHasCombinedStaticAndCSMEnabled = bCombinedStaticAndCSMEnabled && LightSceneProxy->UseCSMForDynamicObjects();
				bool bMovableLightUsingCSM = bMobileEnableMovableLightCSMShaderCulling && LightSceneProxy->IsMovable() && MobileDirectionalLightSceneInfo->ShouldRenderViewIndependentWholeSceneShadows();

				// non-csm culling movable light will force all draws to use CSM shaders.
				// TODO: Cases in which a light channel uses a shadow casting non-csm culled movable light we only really need to use CSM on primitives that match the light channel.
				bAlwaysUseCSM = bAlwaysUseCSM || (!bMobileEnableMovableLightCSMShaderCulling && LightSceneProxy->IsMovable() && MobileDirectionalLightSceneInfo->ShouldRenderViewIndependentWholeSceneShadows());
				if (bLightHasCombinedStaticAndCSMEnabled || bMovableLightUsingCSM)
				{
					BuildCSMVisibilityState(MobileDirectionalLightSceneInfo);
				}
			}
		}
	}

	for (auto& View : Views)
	{
		FMobileCSMVisibilityInfo& MobileCSMVisibilityInfo = View.MobileCSMVisibilityInfo;
		MobileCSMVisibilityInfo.bAlwaysUseCSM = bAlwaysUseCSM;
		if (bSkipCSMShaderCulling)
		{
			MobileCSMVisibilityInfo.bMobileDynamicCSMInUse = true;
		}
	}

	{
		// Check for modulated shadows. 
		bModulatedShadowsInUse = false;
		for(const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
		{
			FLightSceneInfo* LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;
			FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];
			// Mobile renderer only projects modulated shadows.
			bModulatedShadowsInUse = VisibleLightInfo.ShadowsToProject.Num() > 0;
			if (bModulatedShadowsInUse)
			{
				break;
			}
		}
	}

	return TaskData;
}

// Build a single FProjectedShadowInfo whose bounds encompass the entire CSM receiver range
// (near + far cascades), used for the "is this primitive a CSM receiver?" test.
bool BuildSingleCascadeShadowInfo(FViewInfo &View, FVisibleLightInfoArray& VisibleLightInfos, FLightSceneInfo* LightSceneInfo, FProjectedShadowInfo& OUTSingleCascadeInfo)
{
	const FLightSceneProxy* Proxy = LightSceneInfo->Proxy;
	const bool bPrecomputedLightingIsValid = LightSceneInfo->IsPrecomputedLightingValid();

	const float SplitNear = View.NearClippingDistance;
	const float SplitFar = Proxy->GetMaxShadowReceiverDistance(View, bPrecomputedLightingIsValid);
	if (SplitFar <= SplitNear)
	{
		return false;
	}

	const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];
	if (VisibleLightInfo.AllProjectedShadows.Num() == 0)
	{
		return false;
	}

	FWholeSceneProjectedShadowInitializer WholeSceneInitializer;
	FShadowCascadeSettings& Cascade = WholeSceneInitializer.CascadeSettings;

	FSphere Bounds = Proxy->GetShadowSplitBoundsDepthRange(View, View.ViewMatrices.GetViewOrigin(), SplitNear, SplitFar, &Cascade);
	Bounds.W = FMath::CeilToFloat(Bounds.W);

	const float ShadowExtent = Bounds.W / FMath::Sqrt(3.0f);
	const FBoxSphereBounds SubjectBounds(Bounds.Center, FVector(ShadowExtent, ShadowExtent, ShadowExtent), Bounds.W);

	WholeSceneInitializer.PreShadowTranslation      = -Bounds.Center;
	WholeSceneInitializer.WorldToLight              = FInverseRotationMatrix(Proxy->GetDirection().GetSafeNormal().Rotation());
	WholeSceneInitializer.Scales                    = FVector2D(1.0f / Bounds.W, 1.0f / Bounds.W);
	WholeSceneInitializer.SubjectBounds             = FBoxSphereBounds(FVector::ZeroVector, SubjectBounds.BoxExtent, SubjectBounds.SphereRadius);
	WholeSceneInitializer.WAxis                     = FVector4(0, 0, 0, 1);
	WholeSceneInitializer.MinLightW                 = FMath::Min<float>(-UE_OLD_HALF_WORLD_MAX, -SubjectBounds.SphereRadius);
	WholeSceneInitializer.MaxDistanceToCastInLightW = SubjectBounds.SphereRadius - WholeSceneInitializer.MinLightW;
	WholeSceneInitializer.bRayTracedDistanceField   = false;

	Cascade.SplitNear         = SplitNear;
	Cascade.SplitFar          = SplitFar;
	Cascade.FadePlaneOffset   = SplitFar;
	Cascade.FadePlaneLength   = 0.0f;
	Cascade.ShadowSplitIndex  = 0;
	Cascade.bFarShadowCascade = false;

	const FProjectedShadowInfo* Existing = VisibleLightInfo.AllProjectedShadows[0];
	OUTSingleCascadeInfo.SetupWholeSceneProjection(
		LightSceneInfo,
		&View,
		WholeSceneInitializer,
		Existing->ResolutionX,
		Existing->ResolutionY,
		Existing->ResolutionX,
		Existing->ResolutionY,
		Existing->BorderSize
	);

	// SetupWholeSceneProjection re-queries ShadowBounds from the proxy using cascade 0,
	// which loses the full near+far CSM range. Restore the sphere we actually built from.
	OUTSingleCascadeInfo.ShadowBounds = FSphere(Bounds.Center, Bounds.W);
	return true;
}

// Build visibility lists of CSM receivers and non-csm receivers.
void FMobileSceneRenderer::BuildCSMVisibilityState(FLightSceneInfo* LightSceneInfo)
{
	SCOPE_CYCLE_COUNTER(STAT_BuildCSMVisibilityState);

	const uint32 CSMCullingMethod = CVarsCsmShaderCullingMethod.GetValueOnRenderThread() & 0xF;
	const bool bSphereTest = (CVarsCsmShaderCullingMethod.GetValueOnRenderThread() & 0x10) != 0;

	bool bMovableLightUsingCSM = LightSceneInfo->Proxy->IsMovable() && LightSceneInfo->ShouldRenderViewIndependentWholeSceneShadows();

	if (LightSceneInfo->Proxy->CastsDynamicShadow() && 
		(bMovableLightUsingCSM || (LightSceneInfo->Proxy->HasStaticShadowing() && LightSceneInfo->Proxy->UseCSMForDynamicObjects()))
		)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			bool bStaticCSMReceiversFound = false;
			FViewInfo& View = Views[ViewIndex];

			// Mirrors logic from FScene::GetMobileDirectionalLightForView, see comment there for more info
			if (!(LightSceneInfo->Proxy->GetViewLightingChannelMask() & View.ViewLightingChannelMask))
			{
				continue;
			}

			FProjectedShadowInfo SingleCascadeInfo;
			if (BuildSingleCascadeShadowInfo(View, VisibleLightInfos, LightSceneInfo, SingleCascadeInfo) == false)
			{
				continue;
			}

			FProjectedShadowInfo* ProjectedShadowInfo = &SingleCascadeInfo;

			if (ViewFamily.EngineShowFlags.ShadowFrustums)
			{
				FViewElementPDI ShadowFrustumPDI(&View, nullptr, nullptr);
				
				const FMatrix ViewMatrix = View.ViewMatrices.GetWorldToView();
				const FMatrix ProjectionMatrix = View.ViewMatrices.GetViewToClip();
				const FVector4 ViewOrigin = View.ViewMatrices.GetViewOrigin();

				float AspectRatio = ProjectionMatrix.M[1][1] / ProjectionMatrix.M[0][0];
				float ActualFOV = (ViewOrigin.W > 0.0f) ? FMath::Atan(1.0f / ProjectionMatrix.M[0][0]) : PI / 4.0f;

				float Near = ProjectedShadowInfo->CascadeSettings.SplitNear;
				float Mid = ProjectedShadowInfo->CascadeSettings.FadePlaneOffset;
				float Far = ProjectedShadowInfo->CascadeSettings.SplitFar;

				DrawFrustumWireframe(&ShadowFrustumPDI, (ViewMatrix * FPerspectiveMatrix(ActualFOV, AspectRatio, 1.0f, Near, Far)).Inverse(), FColor::Emerald, 0);
				DrawFrustumWireframe(&ShadowFrustumPDI, FMatrix(ProjectedShadowInfo->TranslatedWorldToClipInnerMatrix.Inverse()) * FTranslationMatrix(-ProjectedShadowInfo->PreShadowTranslation), FColor::Cyan, 0);
			}

			FViewInfo* ShadowSubjectView = ProjectedShadowInfo->DependentView ? ProjectedShadowInfo->DependentView : &View;
			FVisibleLightViewInfo& VisibleLightViewInfo = ShadowSubjectView->VisibleLightInfos[LightSceneInfo->Id];
			FMobileCSMSubjectPrimitives& MobileCSMSubjectPrimitives = VisibleLightViewInfo.MobileCSMSubjectPrimitives;
			FMobileCSMVisibilityInfo& MobileCSMVisibilityInfo = View.MobileCSMVisibilityInfo;
			FVector LightDir = LightSceneInfo->Proxy->GetDirection();
			const float ShadowCastLength = WORLD_MAX;

			const auto& ShadowSubjectPrimitives = MobileCSMSubjectPrimitives.GetShadowSubjectPrimitives();
			if (ShadowSubjectPrimitives.Num() != 0 || CSMCullingMethod == 0 || CSMCullingMethod == 1)
			{

				FConvexVolume ViewFrustum;
				GetViewFrustumBounds(ViewFrustum, View.ViewMatrices.GetWorldToClip(), true);
				//FConvexVolume& ShadowReceiverFrustum = ProjectedShadowInfo->CascadeSettings.ShadowBoundsAccurate;
				//FVector PreShadowTranslation = FVector(0, 0, 0);
				FConvexVolume& ShadowReceiverFrustum = ProjectedShadowInfo->ReceiverInnerFrustum;
				FVector& PreShadowTranslation = ProjectedShadowInfo->PreShadowTranslation;


				// Common receiver test functions.
				// Test receiver bounding box against view+shadow frustum only
				auto IsShadowReceiver = [&ViewFrustum, &ShadowReceiverFrustum, &PreShadowTranslation](const FVector& PrimOrigin, const FVector& PrimExtent)
				{
					return ViewFrustum.IntersectBox(PrimOrigin, PrimExtent)
						&& ShadowReceiverFrustum.IntersectBox(PrimOrigin + PreShadowTranslation, PrimExtent);
				};

				//Test against caster capsule vs bounds sphere
				auto IsShadowReceiverCasterVsSphere = [](const FVector& PrimOrigin, float PrimRadius, const FVector& CasterStart, const FVector& CasterEnd, float CasterRadius)
				{
					return FMath::PointDistToSegmentSquared(PrimOrigin, CasterStart, CasterEnd) < FMath::Square(PrimRadius + CasterRadius);
				};

				// Test receiver against single caster capsule vs bounding box
				auto IsShadowReceiverCasterVsBox = [](const FVector& PrimOrigin, const FVector& PrimExtent, const FVector& CasterStart, const FVector& CasterEnd, float CasterRadius)
				{
					FBox PrimBox(PrimOrigin - (PrimExtent + CasterRadius), PrimOrigin + (PrimExtent + CasterRadius));
					FVector Direction = CasterEnd - CasterStart;

					return FMath::LineBoxIntersection(PrimBox, CasterStart, CasterEnd, Direction);
				};

				switch (CSMCullingMethod)
				{
					case 0:
					{
						// Set all prims to receive CSM
						for (FSceneSetBitIterator BitIt(View.PrimitiveVisibilityMap); BitIt; ++BitIt)
						{
							EnableStaticMeshCSMVisibilityState(bMovableLightUsingCSM, Scene->Primitives[BitIt.GetIndex()], MobileCSMVisibilityInfo, View);
						}
						bStaticCSMReceiversFound = MobileCSMVisibilityInfo.bMobileDynamicCSMInUse = true;						
						break;
					}
					case 1:
					{
						auto IsShadowReceiverFrustumOnly = [&IsShadowReceiver](const FVector& PrimOrigin, const FVector& PrimExtent, float PrimRadius)
						{
							return IsShadowReceiver(PrimOrigin, PrimExtent);
						};
						bStaticCSMReceiversFound = MobileDetermineStaticMeshesCSMVisibilityState(Scene, View, ProjectedShadowInfo, IsShadowReceiverFrustumOnly);
						break;
					}
					case 2: // combined casters:
					{
						// Calculate combined bounds
						FBoxSphereBounds::Builder CombinedBoundsBuilder;
						for (auto& Caster : ShadowSubjectPrimitives)
						{
							CombinedBoundsBuilder += Caster->Proxy->GetBounds();
						}

						FBoxSphereBounds CombinedBounds(CombinedBoundsBuilder);
						FVector CombinedCasterStart = CombinedBounds.Origin;
						FVector CombinedCasterEnd = CombinedBounds.Origin + (LightDir * ShadowCastLength);

						if (bSphereTest)
						{
							// Test against view+shadow frustums and caster capsule vs bounding sphere
							auto IsShadowReceiverCombined = [&IsShadowReceiver, &IsShadowReceiverCasterVsSphere, &CombinedBounds, &CombinedCasterStart, &CombinedCasterEnd](const FVector& PrimOrigin, const FVector& PrimExtent, float PrimRadius)
							{
								return IsShadowReceiver(PrimOrigin, PrimExtent) && IsShadowReceiverCasterVsSphere(PrimOrigin, PrimRadius, CombinedCasterStart, CombinedCasterEnd, CombinedBounds.SphereRadius);
							};
							bStaticCSMReceiversFound = MobileDetermineStaticMeshesCSMVisibilityState(Scene, View, ProjectedShadowInfo, IsShadowReceiverCombined);
						}
						else
						{
							// Test against view+shadow frustums and caster capsule vs bounding box
							auto IsShadowReceiverCombinedBox = [&IsShadowReceiver, &IsShadowReceiverCasterVsBox, &CombinedBounds, &CombinedCasterStart, &CombinedCasterEnd](const FVector& PrimOrigin, const FVector& PrimExtent, float PrimRadius)
							{
								return IsShadowReceiver(PrimOrigin, PrimExtent) && IsShadowReceiverCasterVsBox(PrimOrigin, PrimExtent, CombinedCasterStart, CombinedCasterEnd, CombinedBounds.SphereRadius);
							};

							bStaticCSMReceiversFound = MobileDetermineStaticMeshesCSMVisibilityState(Scene, View, ProjectedShadowInfo, IsShadowReceiverCombinedBox);
						}
						break;
					}
					case 3: // All casters:
					{
						if (bSphereTest)
						{
							auto IsShadowReceiverAllCastersVsSphere = [&ShadowSubjectPrimitives, &IsShadowReceiverCasterVsSphere, &LightDir, &ShadowCastLength](const FVector& PrimOrigin, float PrimRadius)
							{
								for (auto& Caster : ShadowSubjectPrimitives)
								{
									const FBoxSphereBounds& CasterBounds = Caster->Proxy->GetBounds();
									const FVector& CasterStart = CasterBounds.Origin;
									float CasterRadius = CasterBounds.SphereRadius;
									const FVector CasterEnd = CasterStart + (LightDir * ShadowCastLength);

									if (IsShadowReceiverCasterVsSphere(PrimOrigin, PrimRadius, CasterStart, CasterEnd, CasterRadius))
									{
										return true;
									}
								}
								return false;
							};
							// Test against view+shadow frustums and all caster capsules vs bounding sphere
							auto IsShadowReceiverSphereAllCasters = [&IsShadowReceiver, &IsShadowReceiverAllCastersVsSphere](const FVector& PrimOrigin, const FVector& PrimExtent, float PrimRadius)
							{
								return IsShadowReceiver(PrimOrigin, PrimExtent) && IsShadowReceiverAllCastersVsSphere(PrimOrigin, PrimRadius);
							};

							bStaticCSMReceiversFound = MobileDetermineStaticMeshesCSMVisibilityState(Scene, View, ProjectedShadowInfo, IsShadowReceiverSphereAllCasters);
						}
						else
						{
							// Test against all caster capsules vs bounding box
							auto IsShadowReceiverAllCastersVsBox = [&ShadowSubjectPrimitives, &IsShadowReceiverCasterVsBox, &LightDir, &ShadowCastLength](const FVector& PrimOrigin, const FVector& PrimExtent)
							{
								for (auto& Caster : ShadowSubjectPrimitives)
								{
									const FBoxSphereBounds& CasterBounds = Caster->Proxy->GetBounds();
									const FVector& CasterStart = CasterBounds.Origin;
									const FVector CasterEnd = CasterStart + (LightDir * ShadowCastLength);
									float CasterRadius = CasterBounds.SphereRadius;

									if (IsShadowReceiverCasterVsBox(PrimOrigin, PrimExtent, CasterStart, CasterEnd, CasterRadius))
									{
										return true;
									}
								}
								return false;
							};
							// Test against view+shadow frustums and all caster capsules vs bounding box
							auto IsShadowReceiverBoxAllCasters = [&IsShadowReceiver, &IsShadowReceiverAllCastersVsBox](const FVector& PrimOrigin, const FVector& PrimExtent, float PrimRadius)
							{
								return IsShadowReceiver(PrimOrigin, PrimExtent) && IsShadowReceiverAllCastersVsBox(PrimOrigin, PrimExtent);
							};
							bStaticCSMReceiversFound = MobileDetermineStaticMeshesCSMVisibilityState(Scene, View, ProjectedShadowInfo, IsShadowReceiverBoxAllCasters);
						}
					}
					break;
					case 4:
					{
						bStaticCSMReceiversFound = MobileCSMVisibilityInfo.bMobileDynamicCSMInUse = false;
					}
					break;
				}

				if (CVarCsmShaderCullingDebugGfx.GetValueOnRenderThread())
				{
					VisualizeMobileDynamicCSMSubjectCapsules(View, LightSceneInfo, ProjectedShadowInfo);
				}
				INC_DWORD_STAT_BY(STAT_CSMSubjects, ShadowSubjectPrimitives.Num());
			}
			MobileCSMVisibilityInfo.bMobileDynamicCSMInUse = bStaticCSMReceiversFound;
		}
	}
}

void FMobileSceneRenderer::GatherShadowPointLights(
	const FViewInfo& View,
	const FSortedLightSetSceneInfo& SortedLightSet,
	TArray<int32, TInlineAllocator<8>>& OutShadowPointLightIndexes)
{
	if (!FReadOnlyCVARCache::EnablePointLightShadows(ShaderPlatform))
	{
		return;
	}

	int32 NumLights = SortedLightSet.SortedLights.Num();
	for (int32 LightIdx = SortedLightSet.UnbatchedLightStart; LightIdx < NumLights; ++LightIdx)
	{
		const FSortedLightSceneInfo& SortedLight = SortedLightSet.SortedLights[LightIdx];
		const FLightSceneInfo& LightSceneInfo = *SortedLight.LightSceneInfo;

		if (LightSceneInfo.IsAValidShadowPointLight(View, VisibleLightInfos, ShaderPlatform))
		{
			OutShadowPointLightIndexes.Add(LightIdx);
		}
	}
}
