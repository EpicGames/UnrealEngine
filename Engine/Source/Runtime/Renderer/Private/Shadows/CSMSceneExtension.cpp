// Copyright Epic Games, Inc. All Rights Reserved.

#include "CSMSceneExtension.h"
#include "ScenePrivate.h"
#include "ScenePrimitiveUpdates.h"

IMPLEMENT_SCENE_EXTENSION(FCSMSceneExtension);

FCachedShadowMapData::FCachedShadowMapData(const FWholeSceneProjectedShadowInitializer& InInitializer, float InLastUsedTime)
	: Initializer(InInitializer)
	, LastUsedTime(InLastUsedTime)
	, bCachedShadowMapHasPrimitives(true)
	, bCachedShadowMapHasNaniteGeometry(false)
	, LastFrameExtraStaticShadowSubjects(0)
{
}

void FCSMSceneExtension::InitExtension(FScene& InScene)
{
}

ISceneExtensionUpdater* FCSMSceneExtension::CreateUpdater()
{
	return new FUpdater(*this);
}

void FCSMSceneExtension::RemoveExpiredCacheEntries(float CurrentRealTime)
{
	for (TMap<FLightSceneId, TArray<FCachedShadowMapData>>::TIterator CachedShadowMapIt(CachedShadowMaps); CachedShadowMapIt; ++CachedShadowMapIt)
	{
		TArray<FCachedShadowMapData>& ShadowMapDatas = CachedShadowMapIt.Value();

		for (auto& ShadowMapData : ShadowMapDatas)
		{
			if (ShadowMapData.ShadowMap.IsValid() && CurrentRealTime - ShadowMapData.LastUsedTime > 2.0f)
			{
				ShadowMapData.InvalidateCachedShadow();
			}
		}
	}

	CSV_CUSTOM_STAT_GLOBAL(ShadowCacheUsageMB, (float(GetCachedWholeSceneShadowMapsSize()) / 1024) / 1024, ECsvCustomStatOp::Set);
}

int64 FCSMSceneExtension::GetCachedWholeSceneShadowMapsSize() const
{
	int64 CachedShadowmapMemory = 0;

	for (TMap<FLightSceneId, TArray<FCachedShadowMapData>>::TConstIterator CachedShadowMapIt(CachedShadowMaps); CachedShadowMapIt; ++CachedShadowMapIt)
	{
		const TArray<FCachedShadowMapData>& ShadowMapDatas = CachedShadowMapIt.Value();

		for (const auto& ShadowMapData : ShadowMapDatas)
		{
			if (ShadowMapData.ShadowMap.IsValid())
			{
				CachedShadowmapMemory += ShadowMapData.ShadowMap.ComputeMemorySize();
			}
		}
	}

	return CachedShadowmapMemory;
}

TArray<FCachedShadowMapData>* FCSMSceneExtension::GetCachedShadowMapDatas(FLightSceneId LightId)
{
	return CachedShadowMaps.Find(LightId);
}

const TArray<FCachedShadowMapData>* FCSMSceneExtension::GetCachedShadowMapDatas(FLightSceneId LightId) const
{
	return CachedShadowMaps.Find(LightId);
}

FCachedShadowMapData& FCSMSceneExtension::GetCachedShadowMapDataRef(FLightSceneId LightId, int32 ShadowMapIndex)
{
	TArray<FCachedShadowMapData>& CachedShadowMapDatas = CachedShadowMaps.FindChecked(LightId);
	checkSlow(ShadowMapIndex >= 0 && ShadowMapIndex < CachedShadowMapDatas.Num());
	return CachedShadowMapDatas[ShadowMapIndex];
}

const FCachedShadowMapData& FCSMSceneExtension::GetCachedShadowMapDataRef(FLightSceneId LightId, int32 ShadowMapIndex) const
{
	const TArray<FCachedShadowMapData>& CachedShadowMapDatas = CachedShadowMaps.FindChecked(LightId);
	checkSlow(ShadowMapIndex >= 0 && ShadowMapIndex < CachedShadowMapDatas.Num());
	return CachedShadowMapDatas[ShadowMapIndex];
}

const FCachedShadowMapData* FCSMSceneExtension::GetCachedShadowMapData(FLightSceneId LightId, int32 ShadowMapIndex) const
{
	const TArray<FCachedShadowMapData>* CachedShadowMapDatas = CachedShadowMaps.Find(LightId);
	if (CachedShadowMapDatas && ShadowMapIndex >= 0 && ShadowMapIndex < CachedShadowMapDatas->Num())
	{
		return &(*CachedShadowMapDatas)[ShadowMapIndex];
	}
	return nullptr;
}

void FCSMSceneExtension::FUpdater::PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet)
{
	// Invalidate cached shadows whenever a shadow-casting primitive is removed
	FScene* ScenePtr = static_cast<FScene*>(&CSMScene.Scene);

	for (const FLightSceneInfo* LightSceneInfo : ScenePtr->DirectionalLights)
	{
		TArray<FCachedShadowMapData>* CachedShadowMapDatas = CSMScene.GetCachedShadowMapDatas(LightSceneInfo->Id);

		if (CachedShadowMapDatas)
		{
			for (FCachedShadowMapData& CachedShadowMapData : *CachedShadowMapDatas)
			{
				for (FPersistentPrimitiveIndex PersistentPrimitiveIndex : ChangeSet.RemovedPrimitiveIds)
				{
					if (CachedShadowMapData.StaticShadowSubjectPersistentPrimitiveIdMap[PersistentPrimitiveIndex.Index] == true)
					{
						CachedShadowMapData.InvalidateCachedShadow();
						break;
					}
				}
			}
		}
	}
}

void FCSMSceneExtension::FUpdater::PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet)
{
	// Resize primitive tracking bitmaps to match new scene size
	FScene* ScenePtr = static_cast<FScene*>(&CSMScene.Scene);
	const int32 MaxPersistentPrimitiveIndex = ScenePtr->GetMaxPersistentPrimitiveIndex();

	for (auto& Pair : CSMScene.CachedShadowMaps)
	{
		for (FCachedShadowMapData& CachedShadowMapData : Pair.Value)
		{
			CachedShadowMapData.StaticShadowSubjectPersistentPrimitiveIdMap.SetNum(MaxPersistentPrimitiveIndex, false);
		}
	}
}

void FCSMSceneExtension::FUpdater::PreLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet)
{
	// Remove cached shadow maps for removed lights
	for (FLightSceneId LightId : LightSceneChangeSet.RemovedLightIds)
	{
		CSMScene.CachedShadowMaps.Remove(LightId);
	}
}
