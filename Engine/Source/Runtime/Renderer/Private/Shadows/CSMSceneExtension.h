// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneExtensions.h"
#include "SceneManagement.h"
#include "SceneRendering.h"

class FScene;
struct FLightSceneChangeSet;
class FScenePreUpdateChangeSet;
class FScenePostUpdateChangeSet;
class FSceneUniformBuffer;


/**
 * Cached shadow map data for whole-scene shadows (CSM, point, spot).
 */
class FCachedShadowMapData
{
public:
	FWholeSceneProjectedShadowInitializer Initializer;
	FShadowMapRenderTargetsRefCounted ShadowMap;
	float LastUsedTime;
	bool bCachedShadowMapHasPrimitives;
	bool bCachedShadowMapHasNaniteGeometry;

	/**
	 * The static meshes cast shadow on this cached csm
	 */
	TBitArray<> StaticShadowSubjectPersistentPrimitiveIdMap;

	FIntPoint ShadowBufferResolution;
	FVector PreShadowTranslation;
	float MaxSubjectZ;
	float MinSubjectZ;

	/**
	 * The extra static meshes cast shadow in last frame, if it exceeds the r.Shadow.MaxCSMScrollingStaticShadowSubjects, the cached csm should be rebuilt.
	 */
	int32 LastFrameExtraStaticShadowSubjects;

	void InvalidateCachedShadow()
	{
		ShadowMap.Release();
		StaticShadowSubjectPersistentPrimitiveIdMap.SetRange(0, StaticShadowSubjectPersistentPrimitiveIdMap.Num(), false);
	}

	FCachedShadowMapData(const FWholeSceneProjectedShadowInitializer& InInitializer, float InLastUsedTime);
};

/**
 * Scene extension for Cascaded Shadow Map (CSM) state management.
 * This extension owns all persistent CSM-related state.
 */
class FCSMSceneExtension : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FCSMSceneExtension);

public:
	using ISceneExtension::ISceneExtension;

	//~ ISceneExtension interface
	virtual void InitExtension(FScene& InScene) override;
	virtual ISceneExtensionUpdater* CreateUpdater() override;

	/**
	 * Remove cached shadow map entries that haven't been used recently and report stats.
	 */
	void RemoveExpiredCacheEntries(float CurrentRealTime);

	/**
	 * Get the total memory used by all cached shadow maps.
	 */
	int64 GetCachedWholeSceneShadowMapsSize() const;

	/**
	 * Get cached shadow map data array for a light, or nullptr if not found.
	 */
	TArray<FCachedShadowMapData>* GetCachedShadowMapDatas(FLightSceneId LightID);
	const TArray<FCachedShadowMapData>* GetCachedShadowMapDatas(FLightSceneId LightID) const;

	/**
	 * Get cached shadow map data for a light at a specific index (checked slow).
	 */
	FCachedShadowMapData& GetCachedShadowMapDataRef(FLightSceneId LightID, int32 ShadowMapIndex = 0);
	const FCachedShadowMapData& GetCachedShadowMapDataRef(FLightSceneId LightID, int32 ShadowMapIndex = 0) const;

	/**
	 * Get cached shadow map data for a light at a specific index, or nullptr if invalid.
	 */
	const FCachedShadowMapData* GetCachedShadowMapData(FLightSceneId LightID, int32 ShadowMapIndex = 0) const;

	/**
	 * Get the map of all cached shadow maps (mutable).
	 */
	TMap<FLightSceneId, TArray<FCachedShadowMapData>>& GetAllCachedShadowMaps() { return CachedShadowMaps; }
	const TMap<FLightSceneId, TArray<FCachedShadowMapData>>& GetAllCachedShadowMaps() const { return CachedShadowMaps; }

private:
	class FUpdater : public ISceneExtensionUpdater
	{
	public:
		DECLARE_SCENE_EXTENSION_UPDATER(FUpdater, FCSMSceneExtension);

		FUpdater(FCSMSceneExtension& InCSMScene) : CSMScene(InCSMScene) {}

		virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override;
		virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override;
		virtual void PreLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet) override;

		FCSMSceneExtension& CSMScene;
	};

	friend class FUpdater;

	/** Map from light id to the cached shadowmap data for that light. */
	TMap<FLightSceneId, TArray<FCachedShadowMapData>> CachedShadowMaps;
};
