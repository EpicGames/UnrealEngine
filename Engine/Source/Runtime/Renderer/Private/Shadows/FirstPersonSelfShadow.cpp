// Copyright Epic Games, Inc. All Rights Reserved.

#include "FirstPersonSelfShadow.h"
#include "LightSceneProxy.h"
#include "SceneTextures.h"
#include "../LightRendering.h"
#include "../ShadowRendering.h"
#include "../FirstPersonSceneExtension.h"
#include "RenderGraphUtils.h"
#include "PixelShaderUtils.h"
#include "HZB.h"

static TAutoConsoleVariable<int32> CVarFirstPersonSelfShadow(
	TEXT("r.FirstPerson.SelfShadow"),
	0,
	TEXT("Enables self shadows for first person primitives. Self shadows are achieved with HZB screen space traces. Use r.FirstPerson.SelfShadow.LightTypes to control which shadow casting light types should cast self shadows."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarFirstPersonSelfShadowLightTypes(
	TEXT("r.FirstPerson.SelfShadow.LightTypes"),
	0,
	TEXT("Controls which light types should cast self shadows for first person primitives. 0: Directional Lights Only, 1: Local Lights Only, 2: All Lights"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarFirstPersonSelfShadowCheckerboardMode(
	TEXT("r.FirstPerson.SelfShadow.DownsampleCheckerboardMode"),
	0,
	TEXT("Controls how to downsample depth and normals for first person self shadows. 0: always pick closest depth, 1: always pick farthest depth, 2: alternate closest/farthest in a checkerboard pattern."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarFirstPersonSelfShadowMaxTraceDistance(
	TEXT("r.FirstPerson.SelfShadow.MaxTraceDistance"),
	100.0f,
	TEXT("Maximum world space trace distance for shadow rays."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFirstPersonSelfShadowMaxIterations(
	TEXT("r.FirstPerson.SelfShadow.MaxHZBTraceIterations"),
	512,
	TEXT("Maximum number of HZB traversal iterations during the first person self shadow screen trace. Lowering this number can improve performance at the cost of potential shadow artifacts."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarFirstPersonSelfShadowRelativeDepthThickness(
	TEXT("r.FirstPerson.SelfShadow.RelativeDepthThickness"),
	0.2f,
	TEXT("Relative depth thickness behind which a screen space tracing hit is ignored."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFirstPersonSelfShadowMinimumHZBTraceOccupancy(
	TEXT("r.FirstPerson.SelfShadow.MinimumHZBTraceOccupancy"),
	0,
	TEXT("Minimum wave thread occupancy below which HZB tracing is aborted. Setting this to a value higher than 0 can improve performance at the cost of potential shadow artifacts."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarFirstPersonSelfShadowBilateralFilterDepthThreshold(
	TEXT("r.FirstPerson.SelfShadow.BilateralFilterDepthThreshold"),
	1.0f,
	TEXT("Scale applied to depth differences used to weigh sample contributions when filtering and upsampling first person self shadows. A higher value makes the result softer but may lead to leaking of light/shadow across geometric edges."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFirstPersonSelfShadowRawFullResolution(
	TEXT("r.FirstPerson.SelfShadow.RawFullResolution"),
	0,
	TEXT("Runs first person self shadows at full resolution and without filtering, resulting in pixel perfect shadows. This mostly serves as a ground truth to compare the half-resolution shadow to."),
	ECVF_RenderThreadSafe);

enum class EFPLightSourceShape
{
	Directional,
	Point,
	Rect,

	MAX
};

class FFirstPersonSelfShadowInputsDownsamplePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FFirstPersonSelfShadowInputsDownsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FFirstPersonSelfShadowInputsDownsamplePS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER(uint32, CheckerboardMode)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return HasFirstPersonGBufferBit(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFirstPersonSelfShadowInputsDownsamplePS, "/Engine/Private/FirstPersonSelfShadow.usf", "FirstPersonSelfShadowDownsamplePS", SF_Pixel);

class FFirstPersonSelfShadowTracingPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FFirstPersonSelfShadowTracingPS, Global)
	SHADER_USE_PARAMETER_STRUCT(FFirstPersonSelfShadowTracingPS, FGlobalShader);

	class FSourceShapeDim : SHADER_PERMUTATION_ENUM_CLASS("LIGHT_SOURCE_SHAPE", EFPLightSourceShape);
	class FRawFullResolution : SHADER_PERMUTATION_BOOL("RAW_FULL_RESOLUTION_FP_SELF_SHADOWS");
	using FPermutationDomain = TShaderPermutationDomain<FSourceShapeDim, FRawFullResolution>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)		
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FDeferredLightUniformStruct, DeferredLight)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, DownsampledInputsTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, DownsampledDepthTexture)
		SHADER_PARAMETER(float, HZBMaxTraceDistance)
		SHADER_PARAMETER(float, HZBMaxIterations)
		SHADER_PARAMETER(float, HZBRelativeDepthThickness)
		SHADER_PARAMETER(uint32, HZBMinimumTracingThreadOccupancy)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return HasFirstPersonGBufferBit(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFirstPersonSelfShadowTracingPS, "/Engine/Private/FirstPersonSelfShadow.usf", "FirstPersonSelfShadowTracePS", SF_Pixel);


class FFirstPersonSelfShadowBlurPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FFirstPersonSelfShadowBlurPS, Global)
	SHADER_USE_PARAMETER_STRUCT(FFirstPersonSelfShadowBlurPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputsTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, DepthTexture)
		SHADER_PARAMETER(float, InvDepthThreshold)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return HasFirstPersonGBufferBit(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFirstPersonSelfShadowBlurPS, "/Engine/Private/FirstPersonSelfShadow.usf", "FirstPersonSelfShadowBlurPS", SF_Pixel);


class FFirstPersonSelfShadowUpsamplePS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FFirstPersonSelfShadowUpsamplePS, Global)
	SHADER_USE_PARAMETER_STRUCT(FFirstPersonSelfShadowUpsamplePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, DownsampledDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, ShadowFactorsTexture)
		SHADER_PARAMETER(FVector2f, DownsampledInvBufferSize)
		SHADER_PARAMETER(float, InvDepthThreshold)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return HasFirstPersonGBufferBit(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFirstPersonSelfShadowUpsamplePS, "/Engine/Private/FirstPersonSelfShadow.usf", "FirstPersonSelfShadowUpsamplePS", SF_Pixel);


bool ShouldRenderFirstPersonSelfShadow(const FSceneViewFamily& ViewFamily)
{
	return ViewFamily.EngineShowFlags.DirectLighting
		&& CVarFirstPersonSelfShadow.GetValueOnRenderThread() != 0 
		&& HasFirstPersonGBufferBit(ViewFamily.GetShaderPlatform());
}

bool LightCastsFirstPersonSelfShadow(const FLightSceneInfo& LightSceneInfo)
{
	const bool bCastsShadow = LightSceneInfo.Proxy->CastsDynamicShadow();
	const bool bSelfShadowEnabled = CVarFirstPersonSelfShadow.GetValueOnRenderThread() != 0;
	const int32 EnabledSelfShadowLightTypes = FMath::Clamp(CVarFirstPersonSelfShadowLightTypes.GetValueOnRenderThread(), 0, 2);
	const bool bLocalLightSelfShadowEnabled = EnabledSelfShadowLightTypes > 0;
	const bool bDirectionalLightSelfShadowEnabled = EnabledSelfShadowLightTypes != 1;
	const bool bIsDirectionalLight = LightSceneInfo.Type == LightType_Directional;
	return bCastsShadow && bSelfShadowEnabled && ((bIsDirectionalLight && bDirectionalLightSelfShadowEnabled) || (!bIsDirectionalLight && bLocalLightSelfShadowEnabled));
}

static bool IsViewFirstPersonSelfShadowRelevant(const FViewInfo& View, const FFirstPersonViewBounds& FirstPersonViewBounds, const FLightSceneInfo& LightSceneInfo)
{
	// First person primitives can be expected to all be very close to one another, so lights will usually either fully affect all of them or none,
	// which is why we use a single FBoxSphereBounds object for all first person primitives visible in the view.
	return FirstPersonViewBounds.bHasFirstPersonPrimitives && LightSceneInfo.ShouldRenderLight(View) && LightSceneInfo.Proxy->AffectsBounds(FirstPersonViewBounds.FirstPersonBounds);
}

bool ShouldRenderFirstPersonSelfShadowForLight(const FSceneRendererBase& SceneRenderer, const FSceneViewFamily& ViewFamily, const TArray<FViewInfo>& Views, const FLightSceneInfo& LightSceneInfo)
{
	const bool bSelfShadowSupported = ShouldRenderFirstPersonSelfShadow(ViewFamily);
	const bool bLightCastsSelfShadow = LightCastsFirstPersonSelfShadow(LightSceneInfo);
	if (bSelfShadowSupported && bLightCastsSelfShadow)
	{
		const FFirstPersonSceneExtensionRenderer* FPRenderer = SceneRenderer.GetSceneExtensionsRenderers().GetRendererPtr<FFirstPersonSceneExtensionRenderer>();
		if (ensure(FPRenderer))
		{
			// Return true if any view is relevant, not only if all views are relevant. We can filter out individual views later.
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
			{
				if (IsViewFirstPersonSelfShadowRelevant(Views[ViewIndex], FPRenderer->GetFirstPersonViewBounds(Views[ViewIndex]), LightSceneInfo))
				{
					return true;
				}
			}
		}
	}
	return false;
}

FFirstPersonSelfShadowInputs CreateFirstPersonSelfShadowInputs(FRDGBuilder& GraphBuilder, const TArray<FViewInfo>& Views, const FMinimalSceneTextures& SceneTextures)
{
	FFirstPersonSelfShadowInputs Result{};
	Result.SceneTextures = &SceneTextures;

	const bool bRawFullResolution = CVarFirstPersonSelfShadowRawFullResolution.GetValueOnRenderThread() != 0;
	if (bRawFullResolution)
	{
		// No need to downsample anything in this case; just store a pointer to the scene textures for later
		return Result;
	}

	for (int32 ViewIndex = 0, ViewCount = Views.Num(); ViewIndex < ViewCount; ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		auto& DownsampledTextures = Result.DownsampledInputs.AddDefaulted_GetRef();
		DownsampledTextures.Resolution = GetDownscaledExtent(View.ViewRect.Size(), FIntPoint(2));
		DownsampledTextures.Normals = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(DownsampledTextures.Resolution, PF_R8G8B8A8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource), TEXT("FPDownsampledNormals"));
		DownsampledTextures.DepthStencil = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(DownsampledTextures.Resolution, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource), TEXT("FPDownsampledDepthStencil"));

		// Remap cvar (closest/furthest/CB depth) to min/max/CB for the shader.
		uint32 CheckerboardMode = 0;
		switch (FMath::Clamp(CVarFirstPersonSelfShadowCheckerboardMode.GetValueOnRenderThread(), 0, 2))
		{
		case 0: CheckerboardMode = 1; break;
		case 1: CheckerboardMode = 0; break;
		case 2: CheckerboardMode = 2; break;
		}

		FFirstPersonSelfShadowInputsDownsamplePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFirstPersonSelfShadowInputsDownsamplePS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = SceneTextures.UniformBuffer;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->CheckerboardMode = CheckerboardMode;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(DownsampledTextures.Normals, ERenderTargetLoadAction::ENoAction);
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DownsampledTextures.DepthStencil, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);

		TShaderMapRef<FFirstPersonSelfShadowInputsDownsamplePS> PixelShader(View.ShaderMap);

		// Set all non-first person pixels to the far depth value in the downsampled depth buffer so we can use the hardware to only do work for first person pixels.
		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("FPSelfShadowsDownsampleDepthNormal (View: %i)", ViewIndex),
			PixelShader,
			PassParameters,
			FIntRect(FIntPoint::ZeroValue, DownsampledTextures.Resolution),
			nullptr /*BlendState*/,
			nullptr /*RasterizerState*/,
			TStaticDepthStencilState<true, CF_Always>::GetRHI());
	}

	return Result;
}

void RenderFirstPersonSelfShadow(FRDGBuilder& GraphBuilder, const FSceneRendererBase& SceneRenderer, const TArray<FViewInfo>& Views, FRDGTextureRef ScreenShadowMaskTexture, const FFirstPersonSelfShadowInputs& Inputs, const FLightSceneInfo& LightSceneInfo)
{
	check(ScreenShadowMaskTexture);
	check(LightCastsFirstPersonSelfShadow(LightSceneInfo));

	const FFirstPersonSceneExtensionRenderer* FPRenderer = SceneRenderer.GetSceneExtensionsRenderers().GetRendererPtr<FFirstPersonSceneExtensionRenderer>();
	if (!ensure(FPRenderer))
	{
		return;
	}
	const FLightSceneProxy* RESTRICT LightProxy = LightSceneInfo.Proxy;
	const ELightComponentType LightType = (ELightComponentType)LightProxy->GetLightType();
	const bool bIsRadial = LightType != LightType_Directional;
	const bool bRawFullResolution = CVarFirstPersonSelfShadowRawFullResolution.GetValueOnRenderThread() != 0;
	const float InvDepthThreshold = 1.0f / FMath::Max(UE_SMALL_NUMBER, CVarFirstPersonSelfShadowBilateralFilterDepthThreshold.GetValueOnRenderThread());

	for (int32 ViewIndex = 0, ViewCount = Views.Num(); ViewIndex < ViewCount; ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (!IsViewFirstPersonSelfShadowRelevant(View, FPRenderer->GetFirstPersonViewBounds(View), LightSceneInfo))
		{
			continue;
		}

		const FIntPoint DownsampledResolution = !bRawFullResolution ? Inputs.DownsampledInputs[ViewIndex].Resolution : FIntPoint::ZeroValue;

		FRDGTextureRef FPShadowsTexture = nullptr;
		FRDGTextureRef FPDenoisedShadowsTexture = nullptr;
		if (!bRawFullResolution)
		{
			FPShadowsTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(DownsampledResolution, PF_R8, FClearValueBinding::White, TexCreate_RenderTargetable | TexCreate_ShaderResource), TEXT("FPShadows"));
			FPDenoisedShadowsTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(DownsampledResolution, PF_R8, FClearValueBinding::White, TexCreate_RenderTargetable | TexCreate_ShaderResource), TEXT("FPDenoisedShadows"));
		}

		auto* DeferredLightStruct = GraphBuilder.AllocParameters<FDeferredLightUniformStruct>();
		*DeferredLightStruct = GetDeferredLightParameters(View, LightSceneInfo);
		TRDGUniformBufferRef<FDeferredLightUniformStruct> DeferredLightUniformBuffer = GraphBuilder.CreateUniformBuffer(DeferredLightStruct);

		// Trace screen space rays
		{
			check(IsHZBValid(View, EHZBType::ClosestHZB, true /*bCheckedIfProduced*/));

			FFirstPersonSelfShadowTracingPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFirstPersonSelfShadowTracingPS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->SceneTextures = Inputs.SceneTextures->UniformBuffer;
			PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
			PassParameters->DeferredLight = DeferredLightUniformBuffer;
			PassParameters->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::ClosestHZB);
			PassParameters->HZBMaxTraceDistance = FMath::Max(0.0f, CVarFirstPersonSelfShadowMaxTraceDistance.GetValueOnRenderThread());
			PassParameters->HZBMaxIterations = FMath::Max(1.0f, CVarFirstPersonSelfShadowMaxIterations.GetValueOnRenderThread());
			PassParameters->HZBRelativeDepthThickness = FMath::Max(UE_SMALL_NUMBER, CVarFirstPersonSelfShadowRelativeDepthThickness.GetValueOnRenderThread());
			PassParameters->HZBMinimumTracingThreadOccupancy = FMath::Max(0, CVarFirstPersonSelfShadowMinimumHZBTraceOccupancy.GetValueOnRenderThread());
			if (!bRawFullResolution)
			{
				PassParameters->DownsampledInputsTexture = GraphBuilder.CreateSRV(Inputs.DownsampledInputs[ViewIndex].Normals);
				PassParameters->DownsampledDepthTexture = GraphBuilder.CreateSRV(Inputs.DownsampledInputs[ViewIndex].DepthStencil);
				PassParameters->RenderTargets[0] = FRenderTargetBinding(FPShadowsTexture, ERenderTargetLoadAction::ENoAction);
				PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Inputs.DownsampledInputs[ViewIndex].DepthStencil, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthRead_StencilNop);
			}
			else
			{
				PassParameters->DownsampledInputsTexture = nullptr;
				PassParameters->DownsampledDepthTexture = nullptr;
				PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenShadowMaskTexture, ERenderTargetLoadAction::ELoad);
			}

			FFirstPersonSelfShadowTracingPS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FFirstPersonSelfShadowTracingPS::FSourceShapeDim>(bIsRadial ? (LightProxy->IsRectLight() ? EFPLightSourceShape::Rect : EFPLightSourceShape::Point) : EFPLightSourceShape::Directional);
			PermutationVector.Set<FFirstPersonSelfShadowTracingPS::FRawFullResolution>(bRawFullResolution);
			TShaderMapRef<FFirstPersonSelfShadowTracingPS> PixelShader(View.ShaderMap, PermutationVector);

			FRHIBlendState* BlendState = nullptr;
			FRHIDepthStencilState* DepthStencilState = nullptr;
			if (!bRawFullResolution)
			{
				// Default blend state, but early out on depth such that we only process first person pixels.
				BlendState = TStaticBlendState<>::GetRHI();
				DepthStencilState = TStaticDepthStencilState<false, CF_NotEqual>::GetRHI();
			}
			else
			{
				// Use the shadow projection blend state when running at full resolution as we directly render into the screen shadow mask texture.
				BlendState = FProjectedShadowInfo::GetBlendStateForProjection(LightSceneInfo.GetDynamicShadowMapChannel(), LightType == LightType_Directional, false, false, false);
				DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			}

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				View.ShaderMap,
				RDG_EVENT_NAME("FirstPersonSelfShadowTracing: %s (View: %i)", *LightProxy->GetOwnerNameOrLabel(), ViewIndex),
				PixelShader,
				PassParameters,
				!bRawFullResolution ? FIntRect(FIntPoint::ZeroValue, DownsampledResolution) : View.ViewRect,
				BlendState,
				nullptr /*RasterizerState*/,
				DepthStencilState);
		}

		// Apply a 3x3 blur with some contact hardening depending on shadow caster distance. This helps with achieving a nicer upsampled shadow while still giving some small scale details.
		if (!bRawFullResolution)
		{
			FFirstPersonSelfShadowBlurPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFirstPersonSelfShadowBlurPS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->InputsTexture = GraphBuilder.CreateSRV(FPShadowsTexture);
			PassParameters->DepthTexture = GraphBuilder.CreateSRV(Inputs.DownsampledInputs[ViewIndex].DepthStencil);
			PassParameters->InvDepthThreshold = InvDepthThreshold;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(FPDenoisedShadowsTexture, ERenderTargetLoadAction::ENoAction);
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Inputs.DownsampledInputs[ViewIndex].DepthStencil, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthRead_StencilNop);

			TShaderMapRef<FFirstPersonSelfShadowBlurPS> PixelShader(View.ShaderMap);

			// Default blend state, but early out on depth such that we only process first person pixels.
			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				View.ShaderMap,
				RDG_EVENT_NAME("FirstPersonSelfShadowBlur: %s (View: %i)", *LightProxy->GetOwnerNameOrLabel(), ViewIndex),
				PixelShader,
				PassParameters,
				FIntRect(FIntPoint::ZeroValue, DownsampledResolution),
				nullptr /*BlendState*/,
				nullptr /*RasterizerState*/,
				TStaticDepthStencilState<false, CF_NotEqual>::GetRHI());
		}

		// Upsample into ScreenShadowMaskTexture
		if (!bRawFullResolution)
		{
			FFirstPersonSelfShadowUpsamplePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFirstPersonSelfShadowUpsamplePS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->SceneTextures = Inputs.SceneTextures->UniformBuffer;
			PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
			PassParameters->DownsampledDepthTexture = GraphBuilder.CreateSRV(Inputs.DownsampledInputs[ViewIndex].DepthStencil);
			PassParameters->ShadowFactorsTexture = GraphBuilder.CreateSRV(FPDenoisedShadowsTexture);
			PassParameters->DownsampledInvBufferSize = FVector2f(1.0f) / FVector2f(DownsampledResolution);
			PassParameters->InvDepthThreshold = InvDepthThreshold;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenShadowMaskTexture, ERenderTargetLoadAction::ELoad);

			TShaderMapRef<FFirstPersonSelfShadowUpsamplePS> PixelShader(View.ShaderMap);

			FRHIBlendState* BlendState = FProjectedShadowInfo::GetBlendStateForProjection(LightSceneInfo.GetDynamicShadowMapChannel(), LightType == LightType_Directional, false, false, false);

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				View.ShaderMap,
				RDG_EVENT_NAME("FirstPersonSelfShadowUpsample: %s (View: %i)", *LightProxy->GetOwnerNameOrLabel(), ViewIndex),
				PixelShader,
				PassParameters,
				View.ViewRect,
				BlendState);
		}
	}
}

