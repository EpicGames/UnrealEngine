// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"

class FSceneRendererBase;
class FLightSceneInfo;
struct FMinimalSceneTextures;

// Light independent inputs required for rendering first person self-shadow.
struct FFirstPersonSelfShadowInputs
{
	struct FDownsampledTextures
	{
		FIntPoint Resolution = FIntPoint::ZeroValue;
		FRDGTextureRef Normals = nullptr;
		FRDGTextureRef DepthStencil = nullptr;
	};
	const FMinimalSceneTextures* SceneTextures = nullptr;
	TArray<FDownsampledTextures, TInlineAllocator<4>> DownsampledInputs;
};

// Whether to render first person self-shadow at all.
bool ShouldRenderFirstPersonSelfShadow(const FSceneViewFamily& ViewFamily);

// Whether the light could cast first person self-shadow. This is similar in spirit to calling Casts*Shadow() on the light proxy
// and does not check for other conditions unrelated to the light itself, such as whether it is relevant for a given view or
// if the current platform and configuration supports first person self-shadow at all. For these cases, consider calling
// ShouldRenderFirstPersonSelfShadowForLight instead.
bool LightCastsFirstPersonSelfShadow(const FLightSceneInfo& LightSceneInfo);

// Whether to render first person self-shadow for a particular light.
bool ShouldRenderFirstPersonSelfShadowForLight(const FSceneRendererBase& SceneRenderer, const FSceneViewFamily& ViewFamily, const TArray<FViewInfo>& Views, const FLightSceneInfo& LightSceneInfo);

// Creates the required light independent inputs for RenderFirstPersonSelfShadow().
FFirstPersonSelfShadowInputs CreateFirstPersonSelfShadowInputs(FRDGBuilder& GraphBuilder, const TArray<FViewInfo>& Views, const FMinimalSceneTextures& SceneTextures);

// Renders first person self-shadow for the passed in light to the given ScreenShadowMaskTexture. Self-shadow is achieved by doing screen space shadow traces for first person pixels in the GBuffer.
void RenderFirstPersonSelfShadow(FRDGBuilder& GraphBuilder, const FSceneRendererBase& SceneRenderer, const TArray<FViewInfo>& Views, FRDGTextureRef ScreenShadowMaskTexture, const FFirstPersonSelfShadowInputs& Inputs, const FLightSceneInfo& LightSceneInfo);

