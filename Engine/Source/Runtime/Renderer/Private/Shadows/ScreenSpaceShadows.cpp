// Copyright Epic Games, Inc. All Rights Reserved.

/*
=================================================================================
	ScreenSpaceShadows.cpp: Functionality for rendering screen space shadows
=================================================================================
*/

#include "ScreenSpaceShadows.h"

#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "ShadowRendering.h"
#include "SceneRendering.h"
#include "RenderGraphUtils.h"
#include "PixelShaderUtils.h"

#include "bend_sss_cpu.h"

enum class ContactShadowsMethod
{
	StochasticJittering = 0,
	BendSSS = 1,
};

static int32 GContactShadowsMethod = 0;
static FAutoConsoleVariableRef CVarContactShadowsMethod(
	TEXT("r.ContactShadows.Standalone.Method"),
	GContactShadowsMethod,
	TEXT("Technique to use to calculate Contact (Screen Space) Shadows:\n")
	TEXT("0 - Stochastic Jittering.\n")
	TEXT("1 - Bend Screen Space Shadows."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static float GBendShadowsOverrideSurfaceThickness = 0.005f;
static FAutoConsoleVariableRef CVarBendShadowsOverrideSurfaceThickness(
	TEXT("r.ContactShadows.Bend.OverrideSurfaceThickness"),
	GBendShadowsOverrideSurfaceThickness,
	TEXT("How thick the surface represented by a pixel is assumed to be when determining whether a ray intersects it."),
	ECVF_RenderThreadSafe
);

enum class EContactShadowsIntensityMode
{
	PrimitiveFlag,
	DepthBasedApproximation,
	ForceCastingIntensity,

	MAX
};

static int32 GContactShadowsIntensityMode = 0;
static FAutoConsoleVariableRef CVarContactShadowsIntensityMode(
	TEXT("r.ContactShadows.Intensity.Mode"),
	GContactShadowsIntensityMode,
	TEXT("Control how contact shadow intensity is calculated:\n")
	TEXT("0 - Respect bCastContactShadow flag on Primitive Component.\n")
	TEXT("1 - Depth based approximation.\n")
	TEXT("2 - Use Casting Intensity."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static float GContactShadowsIntensityFadeStart = 1600; // 16m
static FAutoConsoleVariableRef CVarContactShadowsIntensityFadeStart(
	TEXT("r.ContactShadows.Intensity.FadeStart"),
	GContactShadowsIntensityFadeStart,
	TEXT("Depth value at which contact shadows starts fading from NonCastingIntensity to CastingIntensity.\n")
	TEXT("Only used when r.ContactShadows.Intensity.Mode=1"),
	ECVF_RenderThreadSafe
);

static float GContactShadowsIntensityFadeLength = 800; // 8m
static FAutoConsoleVariableRef CVarContactShadowsIntensityFadeLength(
	TEXT("r.ContactShadows.Intensity.FadeLength"),
	GContactShadowsIntensityFadeLength,
	TEXT("Length of the fading interval from NonCastingIntensity to CastingIntensity.\n")
	TEXT("Only used when r.ContactShadows.Intensity.Mode=1"),
	ECVF_RenderThreadSafe
);

extern void GetLightContactShadowParameters(const FLightSceneProxy* Proxy, float& OutLength, bool& bOutLengthInWS, float& OutCastingIntensity, float& OutNonCastingIntensity);

const int32 GScreenSpaceShadowsTileSizeX = 8;
const int32 GScreenSpaceShadowsTileSizeY = 8;

int32 GetScreenSpaceShadowDownsampleFactor()
{
	return 2;
}

FIntPoint GetBufferSizeForScreenSpaceShadows(const FViewInfo& View)
{
	return FIntPoint::DivideAndRoundDown(View.GetSceneTexturesConfig().Extent, GetScreenSpaceShadowDownsampleFactor());
}

class FScreenSpaceShadowsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScreenSpaceShadowsCS);
	SHADER_USE_PARAMETER_STRUCT(FScreenSpaceShadowsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float2>, RWShadowFactors)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint2>, StencilTexture)
		SHADER_PARAMETER(FVector3f, LightDirection)
		SHADER_PARAMETER(float, ContactShadowLength)
		SHADER_PARAMETER(uint32, bContactShadowLengthInWS)
		SHADER_PARAMETER(float, ContactShadowCastingIntensity)
		SHADER_PARAMETER(float, ContactShadowNonCastingIntensity)
		SHADER_PARAMETER(FIntRect, ScissorRectMinAndSize)
		SHADER_PARAMETER(uint32, DownsampleFactor)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GScreenSpaceShadowsTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GScreenSpaceShadowsTileSizeY);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), 1);
		OutEnvironment.SetDefine(TEXT("PLATFORM_SUPPORTS_TYPED_UAV_LOAD"), (int32)RHISupports4ComponentUAVReadWrite(Parameters.Platform));
	}
};

IMPLEMENT_GLOBAL_SHADER(FScreenSpaceShadowsCS, "/Engine/Private/ScreenSpaceShadows.usf", "ScreenSpaceShadowsCS", SF_Compute);

class FScreenSpaceShadowsBendCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScreenSpaceShadowsBendCS);
	SHADER_USE_PARAMETER_STRUCT(FScreenSpaceShadowsBendCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, OutputTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint2>, StencilTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, PointBorderSampler)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER(FVector3f, LightDirection)
		SHADER_PARAMETER(float, ContactShadowLength)
		SHADER_PARAMETER(uint32, bContactShadowLengthInWS)
		SHADER_PARAMETER(float, ContactShadowCastingIntensity)
		SHADER_PARAMETER(float, ContactShadowNonCastingIntensity)
		SHADER_PARAMETER(float, ContactShadowIntensityFadeStart)
		SHADER_PARAMETER(float, ContactShadowIntensityFadeOneOverLength)
		SHADER_PARAMETER(float, SurfaceThickness)
		SHADER_PARAMETER(FIntRect, ScissorRectMinAndSize)
		SHADER_PARAMETER(uint32, DownsampleFactor)
		SHADER_PARAMETER(FVector2f, InvDepthTextureSize)
		SHADER_PARAMETER(FVector4f, LightCoordinate)
		SHADER_PARAMETER(FIntVector, WaveOffset)
	END_SHADER_PARAMETER_STRUCT()

	class FIntensityModeDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_INTENSITY_MODE", EContactShadowsIntensityMode);
	using FPermutationDomain = TShaderPermutationDomain<FIntensityModeDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), 64);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), 1);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), 1);
		OutEnvironment.SetDefine(TEXT("PLATFORM_SUPPORTS_TYPED_UAV_LOAD"), (int32)RHISupports4ComponentUAVReadWrite(Parameters.Platform));
		OutEnvironment.SetDefine(TEXT("BEND_SSS"), 1);

		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}
};

IMPLEMENT_GLOBAL_SHADER(FScreenSpaceShadowsBendCS, "/Engine/Private/ScreenSpaceShadows.usf", "ScreenSpaceShadowsBendCS", SF_Compute);

class FScreenSpaceShadowsUpsamplePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScreenSpaceShadowsUpsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FScreenSpaceShadowsUpsamplePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowFactorsTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowFactorsSampler)
		SHADER_PARAMETER(FIntRect, ScissorRectMinAndSize)
		SHADER_PARAMETER(FVector2f, ShadowFactorsUVBilinearMax)
		SHADER_PARAMETER(float, OneOverDownsampleFactor)
	END_SHADER_PARAMETER_STRUCT()

	class FUpsample : SHADER_PERMUTATION_BOOL("SHADOW_FACTORS_UPSAMPLE_REQUIRED");
	using FPermutationDomain = TShaderPermutationDomain<FUpsample>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("UPSAMPLE_PASS"), 1);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FScreenSpaceShadowsUpsamplePS, "/Engine/Private/ScreenSpaceShadows.usf", "ScreenSpaceShadowsUpsamplePS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FScreenSpaceShadowsUpsample, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FScreenSpaceShadowsUpsamplePS::FParameters, PS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void UpsampleScreenSpaceShadows(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View,
	FIntRect ScissorRect,
	bool bProjectingForForwardShading,
	const FLightSceneInfo* LightSceneInfo,
	FScreenPassTexture ShadowsTexture,
	int32 DownsampleFactor,
	FRDGTextureRef ScreenShadowMaskTexture
)
{
	const FVector2f ShadowsTextureExtent(ShadowsTexture.Texture->Desc.Extent);
	const FVector2f ShadowsTextureExtentInverse(1.0f / ShadowsTextureExtent.X, 1.0f / ShadowsTextureExtent.Y);
	const FVector2f ShadowsTextureViewportMax(ShadowsTexture.ViewRect.Max);

	FScreenSpaceShadowsUpsample* PassParameters = GraphBuilder.AllocParameters<FScreenSpaceShadowsUpsample>();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenShadowMaskTexture, GetLoadActionIfProduced(ScreenShadowMaskTexture, ERenderTargetLoadAction::EClear));

	PassParameters->PS.View = GetShaderBinding(View.ViewUniformBuffer);
	PassParameters->PS.SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.GetFeatureLevel());
	PassParameters->PS.ShadowFactorsTexture = ShadowsTexture.Texture;
	PassParameters->PS.ShadowFactorsSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	PassParameters->PS.ScissorRectMinAndSize = FIntRect(ScissorRect.Min, ScissorRect.Size());
	PassParameters->PS.ShadowFactorsUVBilinearMax = (ShadowsTextureViewportMax - 0.5f) * ShadowsTextureExtentInverse;
	PassParameters->PS.OneOverDownsampleFactor = 1.0f / DownsampleFactor;

	FScreenSpaceShadowsUpsamplePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FScreenSpaceShadowsUpsamplePS::FUpsample>(DownsampleFactor != 1);
	auto PixelShader = View.ShaderMap->GetShader<FScreenSpaceShadowsUpsamplePS>(PermutationVector);

	// blend separately from CSM / DF Shadows since those interact with static lighting
	// this matches behavior of GetShadowTerms(...) 
	const bool bIsWholeSceneDirectionalShadow = false;

	FRHIBlendState* BlendState = FProjectedShadowInfo::GetBlendStateForProjection(
		LightSceneInfo->GetDynamicShadowMapChannel(),
		bIsWholeSceneDirectionalShadow,
		false,
		bProjectingForForwardShading,
		false);

	ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Upsample"),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, &View, PixelShader, BlendState, ScissorRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(ScissorRect.Min.X, ScissorRect.Min.Y, 0.0f, ScissorRect.Max.X, ScissorRect.Max.Y, 1.0f);
			RHICmdList.SetScissorRect(true, ScissorRect.Min.X, ScissorRect.Min.Y, ScissorRect.Max.X, ScissorRect.Max.Y);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			FPixelShaderUtils::InitFullscreenPipelineState(RHICmdList, View.ShaderMap, PixelShader, GraphicsPSOInit);

			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.BlendState = BlendState;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

			FPixelShaderUtils::DrawFullscreenTriangle(RHICmdList);
			RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
		});
}

void RenderScreenSpaceShadows(
	FRDGBuilder& GraphBuilder,
	bool bAsyncCompute,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View,
	FIntRect ScissorRect,
	bool bProjectingForForwardShading,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef ScreenShadowMaskTexture)
{
	check(ScissorRect.Area() > 0);

	const FIntPoint BufferSize = GetBufferSizeForScreenSpaceShadows(View);
	FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(BufferSize, PF_G16R16F, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource));
	FRDGTextureRef ShadowsTexture = GraphBuilder.CreateTexture(Desc, TEXT("ScreenSpaceShadows"));

	// ScreenSpaceShadowsCS always outputs at rect with min = (0,0)
	FIntRect ShadowsTextureViewRect(0, 0, ScissorRect.Width() / GetScreenSpaceShadowDownsampleFactor(), ScissorRect.Height() / GetScreenSpaceShadowDownsampleFactor());

	{
		FScreenSpaceShadowsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FScreenSpaceShadowsCS::FParameters>();

		PassParameters->RWShadowFactors = GraphBuilder.CreateUAV(ShadowsTexture);
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.GetFeatureLevel());
		PassParameters->StencilTexture = SceneTextures.Stencil;

		PassParameters->ScissorRectMinAndSize = FIntRect(ScissorRect.Min, ScissorRect.Size());
		PassParameters->DownsampleFactor = GetScreenSpaceShadowDownsampleFactor();

		const FLightSceneProxy* LightProxy = LightSceneInfo->Proxy;
		FLightRenderParameters LightParameters;
		LightProxy->GetLightShaderParameters(LightParameters);

		PassParameters->LightDirection = LightParameters.Direction;

		float ContactShadowLength;
		bool bContactShadowLengthInWS;
		float ContactShadowCastingIntensity;
		float ContactShadowNonCastingIntensity;
		GetLightContactShadowParameters(LightProxy, ContactShadowLength, bContactShadowLengthInWS, ContactShadowCastingIntensity, ContactShadowNonCastingIntensity);

		PassParameters->ContactShadowLength = ContactShadowLength;
		PassParameters->bContactShadowLengthInWS = bContactShadowLengthInWS;
		PassParameters->ContactShadowCastingIntensity = ContactShadowCastingIntensity;
		PassParameters->ContactShadowNonCastingIntensity = ContactShadowNonCastingIntensity;

		auto ComputeShader = View.ShaderMap->GetShader<FScreenSpaceShadowsCS>();

		uint32 GroupSizeX = FMath::DivideAndRoundUp(ShadowsTextureViewRect.Width(), GScreenSpaceShadowsTileSizeX);
		uint32 GroupSizeY = FMath::DivideAndRoundUp(ShadowsTextureViewRect.Height(), GScreenSpaceShadowsTileSizeY);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ScreenSpaceShadowing %ux%u", GroupSizeX * GScreenSpaceShadowsTileSizeX, GroupSizeY * GScreenSpaceShadowsTileSizeY),
			bAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FIntVector(GroupSizeX, GroupSizeY, 1));
	}

	UpsampleScreenSpaceShadows(
		GraphBuilder,
		SceneTextures,
		View,
		ScissorRect,
		bProjectingForForwardShading,
		LightSceneInfo,
		FScreenPassTexture(ShadowsTexture, ShadowsTextureViewRect),
		GetScreenSpaceShadowDownsampleFactor(),
		ScreenShadowMaskTexture);
}

void RenderScreenSpaceShadowsBend(
	FRDGBuilder& GraphBuilder,
	bool bAsyncCompute,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View,
	FIntRect ScissorRect,
	bool bProjectingForForwardShading,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef ScreenShadowMaskTexture)
{
	check(ScissorRect.Area() > 0);

	const int32 DownsampleFactor = 1;

	const FIntPoint BufferSize = View.GetSceneTexturesConfig().Extent;
	FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(BufferSize, PF_R16F, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource));
	FRDGTextureRef ShadowsTexture = GraphBuilder.CreateTexture(Desc, TEXT("ScreenSpaceShadows"));

	const FRDGTextureDesc& DepthDesc = SceneTextures.Depth.Resolve->Desc;

	{
		const FLightSceneProxy* LightProxy = LightSceneInfo->Proxy;

		const FMatrix ViewProjection = View.ViewMatrices.GetWorldToClip();
		const FVector4 LightDirection4 = FVector4(-LightProxy->GetDirection(), 0.0f);
		const FVector4 LightDirection4Clip = ViewProjection.TransformFVector4(LightDirection4);

		FVector4f LightProjection = (FVector4f)LightDirection4Clip;
		FIntPoint MinRenderBounds = ScissorRect.Min;
		FIntPoint MaxRenderBounds = ScissorRect.Max;
		Bend::DispatchList DispatchList = Bend::BuildDispatchList((float*)&LightProjection, (int32*)&BufferSize, (int32*)&MinRenderBounds, (int32*)&MaxRenderBounds);

		for (int32 DispatchIndex = 0; DispatchIndex < DispatchList.DispatchCount; ++DispatchIndex)
		{
			const Bend::DispatchData& Dispatch = DispatchList.Dispatch[DispatchIndex];

			FScreenSpaceShadowsBendCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FScreenSpaceShadowsBendCS::FParameters>();

			PassParameters->OutputTexture = GraphBuilder.CreateUAV(ShadowsTexture);

			PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.GetFeatureLevel());
			PassParameters->InvDepthTextureSize = FVector2f(1.0f / DepthDesc.Extent.X, 1.0f / DepthDesc.Extent.Y);

			PassParameters->StencilTexture = SceneTextures.Stencil;

			// A point sampler, with Wrap Mode set to Clamp-To-Border-Color (D3D12_TEXTURE_ADDRESS_MODE_BORDER), and Border Color set to "FarDepthValue" (typically zero), or some other far-depth value out of DepthBounds.
			// If you have issues where invalid shadows are appearing from off-screen, it is likely that this sampler is not correctly setup
			PassParameters->PointBorderSampler = TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border, 0, 1, 0>::GetRHI();

			PassParameters->View = View.ViewUniformBuffer;

			PassParameters->ScissorRectMinAndSize = FIntRect(ScissorRect.Min, ScissorRect.Size());
			PassParameters->DownsampleFactor = DownsampleFactor;

			FLightRenderParameters LightParameters;
			LightProxy->GetLightShaderParameters(LightParameters);

			PassParameters->LightDirection = LightParameters.Direction;

			float ContactShadowLength;
			bool bContactShadowLengthInWS;
			float ContactShadowCastingIntensity;
			float ContactShadowNonCastingIntensity;
			GetLightContactShadowParameters(LightProxy, ContactShadowLength, bContactShadowLengthInWS, ContactShadowCastingIntensity, ContactShadowNonCastingIntensity);

			PassParameters->ContactShadowLength = ContactShadowLength;
			PassParameters->bContactShadowLengthInWS = bContactShadowLengthInWS;
			PassParameters->ContactShadowCastingIntensity = ContactShadowCastingIntensity;
			PassParameters->ContactShadowNonCastingIntensity = ContactShadowNonCastingIntensity;
			PassParameters->ContactShadowIntensityFadeStart = GContactShadowsIntensityFadeStart;
			PassParameters->ContactShadowIntensityFadeOneOverLength = 1.0f / GContactShadowsIntensityFadeLength;
			PassParameters->SurfaceThickness = GBendShadowsOverrideSurfaceThickness;

			PassParameters->LightCoordinate = FVector4f(DispatchList.LightCoordinate_Shader[0], DispatchList.LightCoordinate_Shader[1], DispatchList.LightCoordinate_Shader[2], DispatchList.LightCoordinate_Shader[3]);
			PassParameters->WaveOffset = FIntVector(Dispatch.WaveOffset_Shader[0], Dispatch.WaveOffset_Shader[1], 0);

			FScreenSpaceShadowsBendCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FScreenSpaceShadowsBendCS::FIntensityModeDim>((EContactShadowsIntensityMode)GContactShadowsIntensityMode);

			auto ComputeShader = View.ShaderMap->GetShader<FScreenSpaceShadowsBendCS>(PermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ScreenSpaceShadowing (Bend)"),
				bAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FIntVector(Dispatch.WaveCount[0], Dispatch.WaveCount[1], Dispatch.WaveCount[2]));
		}
	}

	UpsampleScreenSpaceShadows(
		GraphBuilder,
		SceneTextures,
		View,
		ScissorRect,
		bProjectingForForwardShading,
		LightSceneInfo,
		FScreenPassTexture(ShadowsTexture, ScissorRect),
		DownsampleFactor,
		ScreenShadowMaskTexture);
}

void RenderScreenSpaceShadows(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const TArray<FViewInfo>& Views,
	const FLightSceneInfo* LightSceneInfo,
	bool bProjectingForForwardShading,
	FRDGTextureRef ScreenShadowMaskTexture)
{
	static auto* ContactShadowsCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ContactShadows"));

	if (ScreenShadowMaskTexture == nullptr || (ContactShadowsCVar && ContactShadowsCVar->GetValueOnRenderThread() == 0))
	{
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "ScreenSpaceShadows");

	const FLightSceneProxy* LightSceneProxy = LightSceneInfo->Proxy;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		if (!View.Family->EngineShowFlags.ContactShadows)
		{
			continue;
		}

		FIntRect ScissorRect;
		if (!LightSceneProxy->GetScissorRect(ScissorRect, View, View.ViewRect))
		{
			ScissorRect = View.ViewRect;
		}

		if (ScissorRect.Area() > 0)
		{
			if (GContactShadowsMethod == (uint32)ContactShadowsMethod::BendSSS)
			{
				RenderScreenSpaceShadowsBend(
					GraphBuilder,
					false,
					SceneTextures,
					View,
					ScissorRect,
					bProjectingForForwardShading,
					LightSceneInfo,
					ScreenShadowMaskTexture);
			}
			else
			{
				RenderScreenSpaceShadows(
					GraphBuilder,
					false,
					SceneTextures,
					View,
					ScissorRect,
					bProjectingForForwardShading,
					LightSceneInfo,
					ScreenShadowMaskTexture);
			}
		}
	}
}
