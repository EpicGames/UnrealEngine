// Copyright Epic Games, Inc. All Rights Reserved.

/*
===============================================================================
	ScreenSpaceShadows.h: Functionality for rendering screen space shadows
===============================================================================
*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"

struct FMinimalSceneTextures;
class FLightSceneInfo;

void RenderScreenSpaceShadows(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const TArray<FViewInfo>& Views,
	const FLightSceneInfo* LightSceneInfo,
	bool bProjectingForForwardShading,
	FRDGTextureRef ScreenShadowMaskTexture);