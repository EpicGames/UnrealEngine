// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShadowScene.h"
#include "ScenePrivate.h"
#include "Algo/ForEach.h"
#include "ShadowSceneRenderer.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "DynamicPrimitiveDrawing.h"
#include "LightSceneProxy.h"
#endif

TAutoConsoleVariable<int32> CVarDebugDrawLightActiveStateTracking(
	TEXT("r.Shadow.Scene.DebugDrawLightActiveStateTracking"),
	0,
	TEXT("."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarShadowSceneLightActiveFrameCount(
	TEXT("r.Shadow.Scene.LightActiveFrameCount"),
	10,
	TEXT("Number of frames before a light that has been moving (updated or transform changed) goes to inactive state.\n")
	TEXT("  This determines the number of frames that the MobilityFactor goes to zero over, and thus a higher number spreads invalidations out over a longer time."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarVirtualShadowMapFirstPersonClipmap(
	TEXT( "r.FirstPerson.Shadow.Virtual.Clipmap" ),
	true,
	TEXT( "Enable/Disable support for first-person clipmap for the world-space representation." ),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static bool IsFirstPersonVirtualShadowMapEnabled(EShaderPlatform ShaderPlatform)
{
	const bool bCVarEnabled = CVarVirtualShadowMapFirstPersonClipmap.GetValueOnRenderThread() != 0;
	const bool bGBufferBitSupported = HasFirstPersonGBufferBit(ShaderPlatform);
	return bCVarEnabled && bGBufferBitSupported;
}

DECLARE_DWORD_COUNTER_STAT(TEXT("Active Light Count"), STAT_ActiveLightCount, STATGROUP_ShadowRendering);

IMPLEMENT_SCENE_EXTENSION(FShadowScene);


void FShadowScene::FUpdater::PreLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet)
{
	// Don't sync if there is no work to do.
	if (!LightSceneChangeSet.RemovedLightIds.IsEmpty())
	{
		// Need to wait in case the update is performed several times in a row for some reason.
		ShadowScene.SceneChangeUpdateTask.Wait();

		// TODO: maybe make async?
		{
			// Oust all removed Ids.
			for (FLightSceneId Id : LightSceneChangeSet.RemovedLightIds)
			{
				ShadowScene.ActiveLights[Id] = false;
				if (ShadowScene.LightsCommonData.IsValidIndex(Id))
				{
					ShadowScene.LightsCommonData.RemoveAt(Id);
				}

				if (ShadowScene.Scene.Lights[Id].LightType == LightType_Directional)
				{
					int32 DirLightIndex = ShadowScene.DirectionalLights.IndexOfByPredicate([Id](const FDirectionalLightData& DirectionalLightData) { return DirectionalLightData.LightId == Id; });
					if (DirLightIndex != INDEX_NONE)
					{
						ShadowScene.DirectionalLights.RemoveAtSwap(DirLightIndex);
					}
				}
			}
		}
	}
}

void FShadowScene::FUpdater::PostLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet)
{
	// don't spawn async work for no good reason.
	constexpr int32 kMinWorkSizeForAsync = 64;

	int32 WorkSize = LightSceneChangeSet.SceneLightInfoUpdates.NumCommands();

	// Don't sync, or kick off a new job if there is no work to do.
	if (WorkSize > 0)
	{
		// Need to wait in case the update is performed several times in a row for some reason.
		ShadowScene.SceneChangeUpdateTask.Wait();
		ShadowScene.SceneChangeUpdateTask = GraphBuilder.AddSetupTask([this, LightSceneChangeSet]
		{
			// Track active lights (those that are or were moving, and thus need updating)
			ShadowScene.ActiveLights.SetNum(FMath::Max(LightSceneChangeSet.PreUpdateMaxIndex, LightSceneChangeSet.PostUpdateMaxIndex), false);

			auto UpdateLight = [&](FLightSceneId Id)
			{
				const FLightSceneInfoCompact& LightSceneInfoCompact = ShadowScene.Scene.Lights[Id];
				FLightCommonData& LightCommonData = ShadowScene.GetOrAddLightCommon(Id);

				// Mark as not rendered
				LightCommonData.FirstActiveFrameNumber = INDEX_NONE;
				// Only movable lights can become "active" (i.e., having moved recently and thus needing active update)
				if (LightSceneInfoCompact.bIsMovable)
				{
					ShadowScene.ActiveLights[Id] = true;
					LightCommonData.MobilityFactor = 1.0f;
				}
				else
				{
					ShadowScene.ActiveLights[Id] = false;
					LightCommonData.MobilityFactor = 0.0f;
					// Go straight to not active
				}	
			};

			for (FLightSceneId LightId : LightSceneChangeSet.AddedLightIds)
			{ 
				if (ShadowScene.Scene.Lights[LightId].LightType == LightType_Directional)
				{
					check(ShadowScene.DirectionalLights.FindByPredicate([LightId](const FDirectionalLightData& DirectionalLightData) { return DirectionalLightData.LightId == LightId; }) == nullptr);
					ShadowScene.DirectionalLights.Emplace(FDirectionalLightData{LightId});
				}
				UpdateLight(LightId);
			}

			for (const auto& Item : LightSceneChangeSet.SceneLightInfoUpdates.GetRangeView<FUpdateLightTransformParameters>())
			{ 
				UpdateLight(Item.SceneInfo->Id);
			}

		}, WorkSize > kMinWorkSizeForAsync);
	}
}

void FShadowScene::FUpdater::PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet)
{
	// Note: if moving this to an async task, also make sure to properly sync/depend any use of `AlwaysInvalidatingPrimitives` (which is exposed in GetAlwaysInvalidatingPrimitives)
	for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.RemovedPrimitiveSceneInfos)
	{
		if (PrimitiveSceneInfo->Proxy->GetShadowCacheInvalidationBehavior() == EShadowCacheInvalidationBehavior::Always)
		{
			ShadowScene.AlwaysInvalidatingPrimitives.RemoveSwap(PrimitiveSceneInfo);
		}
		if (ShadowScene.bEnableVirtualShadowMapFirstPersonClipmap && PrimitiveSceneInfo->Proxy->IsFirstPersonWorldSpaceRepresentation())
		{
			ShadowScene.FirstPersonWorldSpacePrimitives.RemoveSwap(PrimitiveSceneInfo);
		}
	}
}

void FShadowScene::FUpdater::PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet)
{

	for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.AddedPrimitiveSceneInfos)
	{
		if (PrimitiveSceneInfo->Proxy->GetShadowCacheInvalidationBehavior() == EShadowCacheInvalidationBehavior::Always)
		{
			ShadowScene.AlwaysInvalidatingPrimitives.Add(PrimitiveSceneInfo);
		}
		if (ShadowScene.bEnableVirtualShadowMapFirstPersonClipmap && PrimitiveSceneInfo->Proxy->IsFirstPersonWorldSpaceRepresentation())
		{
			ShadowScene.FirstPersonWorldSpacePrimitives.Add(PrimitiveSceneInfo);
		}
	}

	const bool bNewEnabled = IsFirstPersonVirtualShadowMapEnabled(ShadowScene.Scene.GetShaderPlatform());
	// When toggled we must find all primitives marked for this path
	if (bNewEnabled && !ShadowScene.bEnableVirtualShadowMapFirstPersonClipmap)
	{
		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ShadowScene.Scene.Primitives)
		{
			if (PrimitiveSceneInfo->Proxy->IsFirstPersonWorldSpaceRepresentation())
			{
				ShadowScene.FirstPersonWorldSpacePrimitives.Add(PrimitiveSceneInfo);
			}
		}
	}
	ShadowScene.bEnableVirtualShadowMapFirstPersonClipmap = bNewEnabled;

	if (!ShadowScene.bEnableVirtualShadowMapFirstPersonClipmap)
	{
		ShadowScene.FirstPersonWorldSpacePrimitives.Empty();
	}
}

void FShadowScene::UpdateForRenderedFrame(FRDGBuilder& GraphBuilder)
{
	SceneChangeUpdateTask.Wait();

	const int32 ActiveFrameCount = FMath::Max(1, CVarShadowSceneLightActiveFrameCount.GetValueOnRenderThread());
	
	// 1. FScene::FrameNumber is incremented before a call to render is being dispatched to the RT.
	int32 SceneFrameNumber = Scene.GetFrameNumberRenderThread();
	// Iterate the previously active lights and update
	for (FSceneLightBitArray::FIterator BitIt(ActiveLights); BitIt; ++BitIt)
	{
		FLightSceneId Id = BitIt.GetIndex();
		// No need to process if we already decided it is active (i.e., it was moved again this frame)
		FLightCommonData& Light = LightsCommonData[Id];
		
		// If it was not rendered before, we record the current scene frame number
		if (Light.FirstActiveFrameNumber == INDEX_NONE)
		{
			Light.FirstActiveFrameNumber = SceneFrameNumber;
			Light.MobilityFactor = 1.0f;
		}
		else if(SceneFrameNumber - Light.FirstActiveFrameNumber  < ActiveFrameCount)
		{
			// If it was updated before, but not this frame, check how many rendered frames have elapsed and transition to the inactive state
			Light.MobilityFactor = 1.0f - FMath::Clamp(float(SceneFrameNumber - Light.FirstActiveFrameNumber) / float(ActiveFrameCount), 0.0f, 1.0f);
		}
		else
		{
			// It's not been updated for more than K frames transition to non-active state
			ActiveLights[Id] = false;
			Light.MobilityFactor = 0.0f;
		}
	}

	SET_DWORD_STAT(STAT_ActiveLightCount, ActiveLights.CountSetBits());
}



float FShadowScene::GetLightMobilityFactor(FLightSceneId LightId) const 
{ 
	SceneChangeUpdateTask.Wait();

	if (IsActive(LightId))
	{
		return LightsCommonData[LightId].MobilityFactor;
	}
	return 0.0f;
}


void FShadowScene::DebugRender(TArrayView<FViewInfo> Views)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	if (CVarDebugDrawLightActiveStateTracking.GetValueOnRenderThread() != 0)
	{
		SceneChangeUpdateTask.Wait();

		for (FViewInfo& View : Views)
		{
			FViewElementPDI DebugPDI(&View, nullptr, &View.DynamicPrimitiveCollector);

			for (FSceneLightBitArray::FIterator BitIt(ActiveLights); BitIt; ++BitIt)
			{
				FLightSceneId Id = BitIt.GetIndex();
				const FLightSceneInfoCompact& LightSceneInfoCompact = Scene.Lights[Id];
				FLightSceneProxy* Proxy = LightSceneInfoCompact.LightSceneInfo->Proxy;
				FLinearColor Color = FMath::Lerp(FLinearColor(FColor::Yellow), FLinearColor(FColor::Blue), GetLightMobilityFactor(Id));
				switch (LightSceneInfoCompact.LightType)
				{
				case LightType_Directional:
					DrawWireSphereAutoSides(&DebugPDI, Proxy->GetLightToWorld().GetOrigin(), Color, FMath::Min(100.0f, Proxy->GetRadius()), SDPG_World);
					DrawWireSphereAutoSides(&DebugPDI, Proxy->GetLightToWorld().GetOrigin(), Color, FMath::Min(200.0f, Proxy->GetRadius()), SDPG_World);
					break;
				case LightType_Spot:
				{
					FTransform TransformNoScale = FTransform(Proxy->GetLightToWorld());
					TransformNoScale.RemoveScaling();

					DrawWireSphereCappedCone(&DebugPDI, TransformNoScale, Proxy->GetRadius(), FMath::RadiansToDegrees(Proxy->GetOuterConeAngle()), 16, 4, 8, Color, SDPG_World);
				}
				break;
				default:
				{
					DrawWireSphereAutoSides(&DebugPDI, Proxy->GetPosition(), Color, Proxy->GetRadius(), SDPG_World);
				}
				break;
				};
			}
		}
	}

#endif
}

void FShadowScene::WaitForSceneLightsUpdateTask()
{
	SceneChangeUpdateTask.Wait();
}

bool FShadowScene::ShouldCreateExtension(FScene& InScene)
{
	return true;
}

void FShadowScene::InitExtension(FScene& InScene)
{
}

ISceneExtensionUpdater* FShadowScene::CreateUpdater()
{
	return new FUpdater(*this);
}

ISceneExtensionRenderer* FShadowScene::CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags)
{
	if (FDeferredShadingSceneRenderer* DeferredShadingSceneRenderer = InSceneRenderer.GetDeferredShadingSceneRenderer())
	{
		return new FShadowSceneRenderer(*DeferredShadingSceneRenderer, *this);
	}

	return nullptr;
}
