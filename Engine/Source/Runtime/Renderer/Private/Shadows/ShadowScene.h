// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/SparseArray.h"
#include "Containers/ArrayView.h"
#include "Containers/Array.h"
#include "Tasks/Task.h"
#include "SceneExtensions.h"
#include "LightSceneInfo.h"

class FProjectedShadowInfo;
class FScene;
class FWholeSceneProjectedShadowInitializer;
class FRDGBuilder;
class FVirtualShadowMapPerLightCacheEntry;
struct FLightSceneChangeSet;
class FViewInfo;
class FPrimitiveSceneInfo;
class FScenePreUpdateChangeSet;
class FScenePostUpdateChangeSet;

/**
 * Persistent scene-representation for VSM shadow rendering.
 */
class FShadowScene : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FShadowScene);
public:
	using ISceneExtension::ISceneExtension;

	/**
	 * Fetch the "mobility factor" for the light, [0,1] where 0.0 means not moving, and 1.0 means was updated this frame.
	 * Does a smooth transition from 1 to 0 over N frames, defined by the cvar.
	 */
	float GetLightMobilityFactor(FLightSceneId LightId) const;

	/**
	 * Call once per rendered frame to update state that depend on number of rendered frames.
	 */
	void UpdateForRenderedFrame(FRDGBuilder& GraphBuilder);

	void DebugRender(TArrayView<FViewInfo> Views);
	
	// List of always invalidating primitives, if this gets too popular perhaps a TSet or some such is more appropriate for performance scaling.
	TArrayView<FPrimitiveSceneInfo*> GetAlwaysInvalidatingPrimitives() { return AlwaysInvalidatingPrimitives; }

	/**
	 * Wait for any scene update task started in the PostLightsUpdate.
	 */
	void WaitForSceneLightsUpdateTask();

	static bool ShouldCreateExtension(FScene& InScene);

private:

	class FUpdater : public ISceneExtensionUpdater
	{
	public:
		DECLARE_SCENE_EXTENSION_UPDATER(FUpdater, FShadowScene);

		FUpdater(FShadowScene& InShadowScene) : ShadowScene(InShadowScene) {}

		// Handle all scene changes wrt lights
		virtual void PreLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet) override;
		virtual void PostLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet &LightSceneChangeSet) override;

		/**
		 * Handle scene changes, notably track all primitives that always invalidate the shadows.
		 */
		virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override;
		virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override;

		FShadowScene& ShadowScene;
	};

	friend class FShadowSceneRenderer;
	friend class FUpdater;

	virtual void InitExtension(FScene& InScene) override;
	virtual ISceneExtensionUpdater* CreateUpdater() override;
	virtual ISceneExtensionRenderer* CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags) override;

	inline bool IsActive(FLightSceneId LightId) const { return ActiveLights[LightId]; }

	/**
	 * Data common to all light types, indexed by light scene ID
	 */
	struct FLightCommonData
	{
		/**
		 * Scene rendering frame number of the first frame that the scene was rendered after being modified (moved/added).
		 */
		int32 FirstActiveFrameNumber;
		float MobilityFactor;
	};
	TLightSparseArray<FLightCommonData> LightsCommonData;

	FLightCommonData& GetOrAddLightCommon(FLightSceneId LightId)
	{
		if (!LightsCommonData.IsValidIndex(LightId))
		{
			LightsCommonData.EmplaceAt(LightId, FLightCommonData{});
		}
		return LightsCommonData[LightId];
	}	

	/**
	 * Directional light data, not indexed by light scene ID but instead linearly searched as there are typically very few.
	 */
	struct FDirectionalLightData
	{
		FLightSceneId LightId = {};
	};
	TArray<FDirectionalLightData> DirectionalLights;

	// Bit-array marking active lights, those we deem active are ones that have been modified in a recent frame and thus need some kind of active update.
	FSceneLightBitArray ActiveLights;

	// Links to other systems etc.
	mutable UE::Tasks::FTask SceneChangeUpdateTask;

	// List of always invalidating primitives, if this gets too popular perhaps a TSet or some such is more appropriate for performance scaling.
	TArray<FPrimitiveSceneInfo*> AlwaysInvalidatingPrimitives;

	// List of primitives that are marked as "first person world-space" meaning they are to cast shadows only onto the world but not the FP geo itself
	TArray<FPrimitiveSceneInfo*> FirstPersonWorldSpacePrimitives;

	bool bEnableVirtualShadowMapFirstPersonClipmap = false;
};
