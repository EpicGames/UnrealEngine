// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShadowSceneRenderer.h"
#include "ShadowScene.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "ShadowRendering.h"
#include "VirtualShadowMaps/VirtualShadowMapCacheManager.h"
#include "VirtualShadowMaps/VirtualShadowMapProjection.h"
#include "VirtualShadowMaps/VirtualShadowMapClipmap.h"
#include "SceneCulling/SceneCulling.h"
#include "SceneCulling/SceneCullingRenderer.h"
#include "Rendering/NaniteStreamingManager.h"
#include "Nanite/NaniteCullRaster.h"
#include "HairStrands/HairStrandsTransmittance.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "DynamicPrimitiveDrawing.h"
#endif

CSV_DECLARE_CATEGORY_EXTERN(VSM);

extern TAutoConsoleVariable<int32> CVarNaniteShadowsUpdateStreaming;
extern TAutoConsoleVariable<int32> CVarVsmUseFarShadowRules;

TAutoConsoleVariable<int32> CVarVSMMaterialVisibility(
	TEXT("r.Shadow.Virtual.Nanite.MaterialVisibility"),
	0,
	TEXT("Enable Nanite CPU-side visibility filtering of draw commands, depends on r.Nanite.MaterialVisibility being enabled."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarMaxDistantLightsPerFrame(
	TEXT("r.Shadow.Virtual.MaxDistantUpdatePerFrame"),
	1,
	TEXT("Maximum number of distant lights to update each frame. Invalidated lights that were missed may be updated in a later frame (round-robin)."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarDistantLightMode(
	TEXT("r.Shadow.Virtual.DistantLightMode"),
	1,
	TEXT("Control whether distant light mode is enabled for local lights.\n0 == Off, \n1 == On (default), \n2 == Force All.\n")
	TEXT("When on, lights with a pixel footprint below the threshold are marked as distant. Updates to distant lights are throttled (force-cached), they use simpler page-table logic and the memory cost is lower."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarNaniteShadowsLODBias(
	TEXT("r.Shadow.NaniteLODBias"),
	1.0f,
	TEXT("LOD bias for nanite geometry in shadows. 0 = full detail. >0 = reduced detail."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarVirtualShadowOnePassProjection(
	TEXT("r.Shadow.Virtual.OnePassProjection"),
	1,
	TEXT("Projects all local light virtual shadow maps in a single pass for better performance."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarResolutionLodBiasLocal(
	TEXT("r.Shadow.Virtual.ResolutionLodBiasLocal"),
	0.0f,
	TEXT("Bias applied to LOD calculations for local lights. -1.0 doubles resolution, 1.0 halves it and so on."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarResolutionLodBiasLocalMoving(
	TEXT("r.Shadow.Virtual.ResolutionLodBiasLocalMoving"),
	1.0f,
	TEXT("Bias applied to LOD calculations for local lights that are moving. -1.0 doubles resolution, 1.0 halves it and so on.\n")
	TEXT("The bias transitions smoothly back to ResolutionLodBiasLocal as the light transitions to non-moving, see 'r.Shadow.Scene.LightActiveFrameCount'."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVirtualShadowMapFirstPersonClipmapFirstLevel(
	TEXT( "r.FirstPerson.Shadow.Virtual.Clipmap.FirstLevel" ),
	8,
	TEXT( "First level of the virtual clipmap. Lower values allow higher resolution shadows closer to the camera, but may increase page count." ),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVirtualShadowMapFirstPersonClipmapLastLevel(
	TEXT( "r.FirstPerson.Shadow.Virtual.Clipmap.LastLevel" ),
	18,
	TEXT( "Last level of the virtual clipmap. Indirectly determines radius the clipmap can cover. Each extra level doubles the maximum range, but may increase page count." ),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVirtualShadowMapNaniteAllowMultipassViews(
	TEXT( "r.Shadow.Virtual.Nanite.AllowMultipassViews" ),
	1,
	TEXT( "When enabled, allows multiple Nanite passes if the view count might exceed Nanite limits.\n" )
	TEXT( "This has some performance overhead and is generally not required since views are aggressively culled on the GPU, but can maintain correct rendering in some extreme cases." ),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32>  CVarForceInvalidateLocalVSM(
	TEXT("r.Shadow.Virtual.Cache.ForceInvalidateLocal"),
	0,
	TEXT("Controls local light VSM invalidation behavior:\n")
	TEXT("0: No forced invalidation (default)\n")
	TEXT("1: Force invalidate all non-distant lights\n")
	TEXT("2: Force invalidate all lights"),
	ECVF_RenderThreadSafe);

extern TAutoConsoleVariable<int32> CVarMarkPixelPagesMipModeLocal;
extern float GMinScreenRadiusForShadowCaster;

extern TAutoConsoleVariable<int32> CVarMarkCoarsePagesLocal;

bool IsVSMOnePassProjectionEnabled(const FEngineShowFlags& ShowFlags)
{
	return CVarVirtualShadowOnePassProjection.GetValueOnAnyThread() 
		// Debug outputs from projection pass do not support one pass projection
		&& (ShowFlags.VisualizeVirtualShadowMap == 0);
}

DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Total Raster Bins"), STAT_VSMNaniteBasePassTotalRasterBins, STATGROUP_ShadowRendering);
DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Visible Raster Bins"), STAT_VSMNaniteBasePassVisibleRasterBins, STATGROUP_ShadowRendering);

DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Total Shading Bins"), STAT_VSMNaniteBasePassTotalShadingBins, STATGROUP_ShadowRendering);
DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Visible Shading Bins"), STAT_VSMNaniteBasePassVisibleShadingBins, STATGROUP_ShadowRendering);

DECLARE_DWORD_COUNTER_STAT(TEXT("Distant Light Count"), STAT_DistantLightCount, STATGROUP_ShadowRendering);
DECLARE_DWORD_COUNTER_STAT(TEXT("Distant Cached Count"), STAT_DistantCachedCount, STATGROUP_ShadowRendering);

DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Light Projections (Directional)"), STAT_VSMDirectionalProjectionFull, STATGROUP_ShadowRendering);
DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Light Projections (Local Full)"), STAT_VSMLocalProjectionFull, STATGROUP_ShadowRendering);
DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Light Projections (Local One Pass Copy)"), STAT_VSMLocalProjectionOnePassCopy, STATGROUP_ShadowRendering);

FShadowSceneRenderer::FShadowSceneRenderer(FDeferredShadingSceneRenderer& InSceneRenderer, FShadowScene& InShadowScene)
	: ISceneExtensionRenderer(InSceneRenderer)
	, SceneRenderer(InSceneRenderer)
	, Scene(InShadowScene.Scene)
	, ShadowScene(InShadowScene)
	, VirtualShadowMapArray(InSceneRenderer.VirtualShadowMapArray)
	, DistantLightMode(CVarDistantLightMode.GetValueOnAnyThread())
{
}

float FShadowSceneRenderer::ComputeNaniteShadowsLODScaleFactor()
{
	return FMath::Pow(2.0f, -CVarNaniteShadowsLODBias.GetValueOnRenderThread()) * Nanite::GStreamingManager.GetQualityScaleFactor();
}

namespace
{

struct FHeapPair
{
	int32 Age;
	TSharedPtr<FVirtualShadowMapPerLightCacheEntry> CacheEntry;

	// Order for a min-heap, we always want to replace the least-old item
	bool operator <(const FHeapPair& Other) const { return Age < Other.Age; }
};

}

void FShadowSceneRenderer::PreInitViews(FRDGBuilder& GraphBuilder)
{
	// Wait for scene update task to finish
	ShadowScene.SceneChangeUpdateTask.Wait();

	// Clear the frame setups to indicate that nothing is allocated for this frame
	CommonSetups.SetNumZeroed(ShadowScene.LightsCommonData.GetMaxIndex());

	// Allocate space for each directional light in the scene, one for each view.
	bool bIsStereo = SceneRenderer.IsRenderingStereo();
	const int32 ShadowNumViews = bIsStereo ? 1 : SceneRenderer.Views.Num();

	DirectionalLights.Reserve(ShadowScene.DirectionalLights.Num() * ShadowNumViews);
	// pre-allocate indexes and setups for each directional light, strided by the view count.
	for (const auto& DirectionalLight : ShadowScene.DirectionalLights)
	{
		FLightCommonFrameSetup& CommonSetup = CommonSetups[DirectionalLight.LightId];
		CommonSetup.bIsDirectional = true;
		CommonSetup.SetupIndex = DirectionalLights.Num();
		if (bIsStereo)
		{
			// only set up one for both
			DirectionalLights.Emplace(DirectionalLight.LightId, 3u);
			check(SceneRenderer.Views.Num() == 2)
		}
		else
		{
			for (int32 ViewIndex = 0; ViewIndex < ShadowNumViews; ++ViewIndex)
			{
				DirectionalLights.Emplace(DirectionalLight.LightId, 1u << ViewIndex);
			}
		}
	}

	bNeedVSMOnePassProjection = false;
	bNeedMegaLightsProjection = false;

	ViewDatas.Reserve(SceneRenderer.Views.Num());
	for (const FViewInfo& View : SceneRenderer.Views)
	{
		const FVector2f ViewSize = FVector2f(View.ViewRect.Size());
		FVector2f RadiusClipXY = FVector2f(2.0f) / ViewSize;

		const FMatrix &ViewToClip = View.ViewMatrices.GetViewToClip();
		// TODO: is RadiusXY always symmetrical?
		FVector2f ProjScaleXY = FVector2f(static_cast<float>(ViewToClip.M[0][0]), static_cast<float>(ViewToClip.M[1][1]));
		FVector2f RadiusXY = RadiusClipXY / ProjScaleXY;
		float MinRadiusXY = FMath::Min(RadiusXY.X, RadiusXY.Y);
		float ClipToViewSizeScale = ViewToClip.M[2][3] * MinRadiusXY;
		float ClipToViewSizeBias = ViewToClip.M[3][3] * MinRadiusXY;
		ViewDatas.Emplace(FViewData{ ClipToViewSizeScale, ClipToViewSizeBias });
	}

	// Kick off shadow scene updates.
	ShadowScene.UpdateForRenderedFrame(GraphBuilder);

	// Priority queue of distant lights to update.
	const int32 MaxToUpdate = CVarMaxDistantLightsPerFrame.GetValueOnRenderThread() < 0 ? INT32_MAX : CVarMaxDistantLightsPerFrame.GetValueOnRenderThread();

	if (MaxToUpdate == 0 || !VirtualShadowMapArray.IsEnabled() || !VirtualShadowMapArray.CacheManager->IsCacheEnabled())
	{
		return;
	}	
	
	RendererSetupTask = GraphBuilder.AddSetupTask([this, MaxToUpdate]()
	{
		FVirtualShadowMapArrayCacheManager& CacheManager = *VirtualShadowMapArray.CacheManager;

		TArray<FHeapPair> DistantLightUpdateQueue;
		int32 SceneFrameNumber = int32(Scene.GetFrameNumber());
		for (auto It = CacheManager.CreateConstEntryIterator(); It; ++It)
		{
			TSharedPtr<FVirtualShadowMapPerLightCacheEntry> PerLightCacheEntry = It.Value();
			if (PerLightCacheEntry->IsFullyCached())
			{
				int32 Age = SceneFrameNumber - int32(PerLightCacheEntry->GetLastScheduledFrameNumber());
				if (DistantLightUpdateQueue.Num() < MaxToUpdate)
				{
					DistantLightUpdateQueue.HeapPush(FHeapPair{ Age, PerLightCacheEntry});
				}
				else
				{
					// Queue is full, but we found an older item
					if (DistantLightUpdateQueue.HeapTop().Age < Age)
					{
						// Replace heap top and restore heap property.
						DistantLightUpdateQueue[0] = FHeapPair{ Age, PerLightCacheEntry };
						AlgoImpl::HeapSiftDown(DistantLightUpdateQueue.GetData(), 0, DistantLightUpdateQueue.Num(), FIdentityFunctor(), TLess<FHeapPair>());
					}
				}
			}
		}

		for (const FHeapPair &HeapPair : DistantLightUpdateQueue)
		{
			// Mark frame it was scheduled, this is picked up later in AddLocalLightShadow to trigger invalidation 
			HeapPair.CacheEntry->MarkScheduled(SceneFrameNumber);
		}
	});
}

UE::Renderer::Private::IShadowInvalidatingInstances *FShadowSceneRenderer::GetInvalidatingInstancesInterface(const FSceneView *SceneView)
{
	// No need to collect invalidations if there is nothing to invalidate.
	FVirtualShadowMapArrayCacheManager* CacheManager = Scene.GetVirtualShadowMapCache();
	if (CacheManager)
	{
		// TODO: Make use of the SceneView parameter to register invalidations for view-dependent shadows appropriately.
		return CacheManager->GetInvalidatingInstancesInterface();
	}
	return nullptr;
}


static float GetResolutionLODBiasLocal(float LightMobilityFactor, float LightLODBias)
{
	return FVirtualShadowMapArray::InterpolateResolutionBias(
		CVarResolutionLodBiasLocal.GetValueOnRenderThread(),
		CVarResolutionLodBiasLocalMoving.GetValueOnRenderThread(),
		LightMobilityFactor) + LightLODBias;
}

void FShadowSceneRenderer::UpdateLocalLightProjectionShaderDataMatrices(
	const FProjectedShadowInfo* ProjectedShadowInfo,
	int32 MapIndex,
	FVirtualShadowMapProjectionShaderData* OutProjectionShaderData) const
{
	const FViewMatrices ViewMatrices = ProjectedShadowInfo->GetShadowDepthRenderingViewMatrices(MapIndex, true);
	const FDFVector3 PreViewTranslation(ProjectedShadowInfo->PreShadowTranslation);

	OutProjectionShaderData->ShadowViewToClipMatrix						= FMatrix44f(ViewMatrices.GetViewToClip());
	FMatrix TranslatedWorldToShadowUVMatrix = CalcTranslatedWorldToShadowUVMatrix(ViewMatrices.GetTranslatedViewMatrix(), ViewMatrices.GetViewToClip());
	OutProjectionShaderData->TranslatedWorldToShadowUVMatrix = FMatrix44f(TranslatedWorldToShadowUVMatrix);
	OutProjectionShaderData->ShadowUVToTranslatedWorldMatrix = FMatrix44f(TranslatedWorldToShadowUVMatrix.Inverse());

	OutProjectionShaderData->TranslatedWorldToShadowUVNormalMatrix		= FMatrix44f(CalcTranslatedWorldToShadowUVNormalMatrix( ViewMatrices.GetTranslatedViewMatrix(), ViewMatrices.GetViewToClip() ));
	OutProjectionShaderData->PreViewTranslationHigh						= PreViewTranslation.High;
	OutProjectionShaderData->PreViewTranslationLow						= PreViewTranslation.Low;

	OutProjectionShaderData->LightDirection								= FVector3f(0, 0, 0);	// Unused for local lights
	OutProjectionShaderData->ClipmapLevel_ClipmapLevelCountRemaining	= -1;					// Not a clipmap
}

FShadowSceneRenderer::FDirectionalLightShadowFrameSetup* FShadowSceneRenderer::FindDirectional(const FLightSceneId LightId, int32 ViewIndex)
{
	// may alternatively be indexed through CommonSetups[LightId].SetupIndex but it is probably more efficient to do a linear search.
	for (FDirectionalLightShadowFrameSetup& Setup : DirectionalLights)
	{
		if (LightId == Setup.LightId && (Setup.ViewMask & (1u << ViewIndex)) != 0u)
		{
			return &Setup;
		}
	}
	check(false);
	return nullptr;
}

/**
 * Calculate the radius in world-space units of a single pixel at a given depth.
 */
static float GetWorldSpacePixelFootprint(float ViewSpaceDepth, float ClipToViewSizeScale, float ClipToViewSizeBias)
{
	return ViewSpaceDepth * ClipToViewSizeScale + ClipToViewSizeBias;
}

/**
 * Compute the lowest (highest res) mip level that might be marked by any pixels inside the light influence radius for a given scene primary view.
 */
static uint32 GetConservativeMipLevelLocal(const FViewInfo& View, float ClipToViewSizeScale, float ClipToViewSizeBias, const FVector& LightOrigin, float LightRadius, float WorldToShadowFootprintScale, float ResolutionLodBias, float GlobalResolutionLodBias, uint32 MipModeLocal)
{
	// Note: not just a rotation, full world-space DP.
	FVector ViewSpaceOrigin = View.GetShadowViewMatrices().GetWorldToView().TransformPosition(LightOrigin);

	// Remove radius to arrive at minimum possible z-distance in view space, from primary view.
	float RadiusWorld = GetWorldSpacePixelFootprint(FMath::Max(0.0f, float(ViewSpaceOrigin.Z) - LightRadius), ClipToViewSizeScale, ClipToViewSizeBias);

	// Radius is the max possible shadow view space Z, which would require the max res.
	float ShadowFootprint = RadiusWorld * WorldToShadowFootprintScale / LightRadius;

	return UE::HLSL::GetMipLevelLocal(ShadowFootprint, MipModeLocal, ResolutionLodBias, GlobalResolutionLodBias);
}

TSharedPtr<FVirtualShadowMapPerLightCacheEntry> FShadowSceneRenderer::AddLocalLightShadow(const FWholeSceneProjectedShadowInitializer& ProjectedShadowInitializer, FProjectedShadowInfo* ProjectedShadowInfo, FLightSceneInfo* LightSceneInfo, float MaxScreenRadius)
{
	FVirtualShadowMapArrayCacheManager* CacheManager = VirtualShadowMapArray.CacheManager;
	FLightSceneId LightId = LightSceneInfo->Id;

	FLightCommonFrameSetup& CommonSetup = CommonSetups[LightId];
	// prevent double allocation
	check(!CommonSetup.bHasVirtualShadowMap);
	CommonSetup.bHasVirtualShadowMap = true;
	// link from ID to the allocated local shadow slot
	CommonSetup.SetupIndex = uint32(LocalLights.Num());

	FLocalLightShadowFrameSetup& LocalLightSetup = LocalLights.AddDefaulted_GetRef();

	LocalLightSetup.ProjectedShadowInfo = ProjectedShadowInfo;
	LocalLightSetup.LightSceneInfo = LightSceneInfo;

	const FLightSceneProxy* LightSceneProxy = LightSceneInfo->Proxy;
	const float ResolutionLODBiasLocal = GetResolutionLODBiasLocal(ShadowScene.GetLightMobilityFactor(LightId), LightSceneProxy->GetVSMResolutionLodBias());

	// Compute conservative mip level estimate based on radius of the bounding sphere.
	// TODO: can probably do better by finding  closest point on cone for certain scenarios? Not as important as it might seem as the worst case is for a narrow cone, but then the narrow FOV limits the required resolution.

	const FVector2f ShadowViewSize = FVector2f(FVirtualShadowMap::VirtualMaxResolutionXY, FVirtualShadowMap::VirtualMaxResolutionXY);
	const FMatrix& ShadowViewToClip = ProjectedShadowInfo->bOnePassPointLightShadow ? ProjectedShadowInfo->OnePassShadowFaceProjectionMatrix : ProjectedShadowInfo->ViewToClipOuter;
	float ShadowProjScale = ShadowViewToClip.M[0][0]; // always symmetrical
	const float WorldToShadowFootprintScale = ShadowProjScale * ShadowViewSize.X;

	uint32 MinMipLevel = FVirtualShadowMap::MaxMipLevels;
	for (int32 ViewIndex = 0; ViewIndex < SceneRenderer.Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = SceneRenderer.Views[ViewIndex];
		const FViewData& ViewData = ViewDatas[ViewIndex];

		MinMipLevel = FMath::Min(MinMipLevel, GetConservativeMipLevelLocal(
			View, 
			ViewData.ClipToViewSizeScale,
			ViewData.ClipToViewSizeBias,
			LightSceneProxy->GetOrigin(), 
			LightSceneProxy->GetRadius(),
			WorldToShadowFootprintScale,
			ResolutionLODBiasLocal, 
			CacheManager->GetGlobalResolutionLodBias(), 
			CVarMarkPixelPagesMipModeLocal.GetValueOnRenderThread()
			));
	}
	
	bool bIsDistantLight = DistantLightMode == 2;

	if (DistantLightMode == 1)
	{
		// use distant light only if we are sure that there's only one mip level.
		bIsDistantLight = MinMipLevel == (FVirtualShadowMap::MaxMipLevels - 1);
	}
	
	const int32 NumMaps = ProjectedShadowInitializer.bOnePassPointLightShadow ? 6 : 1;

	TSharedPtr<FVirtualShadowMapPerLightCacheEntry> PerLightCacheEntry = CacheManager->FindCreateLightCacheEntry(LightId, 0, NumMaps);
	LocalLightSetup.PerLightCacheEntry = PerLightCacheEntry;

	const int32 ForceInvalidateLocalMode = CVarForceInvalidateLocalVSM.GetValueOnRenderThread();
	// In mode 1 we force invalidate all non-distant lights
	// In mode 2 we force invalidate all lights, even distant ones
	bool bForceInvalidate = !CacheManager->IsCacheEnabled();
	if (ForceInvalidateLocalMode > 1)
	{
		bForceInvalidate = true;
	}
	else if (ForceInvalidateLocalMode != 0)
	{
		bForceInvalidate = bForceInvalidate || (!bIsDistantLight);
	}

	PerLightCacheEntry->UpdateLocal(
		FVirtualShadowMapPerLightCacheEntry::FLocalLightCacheKey { ProjectedShadowInitializer.WorldToLight, ProjectedShadowInitializer.PreShadowTranslation, bIsDistantLight },
		LightSceneProxy->GetOrigin(),
		LightSceneProxy->GetRadius(),
		bForceInvalidate,
		IsVirtualShadowMapLocalReceiverMaskEnabled());

	if (bIsDistantLight && PerLightCacheEntry->GetLastScheduledFrameNumber() == Scene.GetFrameNumber())
	{
		PerLightCacheEntry->Invalidate();
	}

	// Update info on the ProjectionShadowInfo; eventually this should all move into local data structures here
	ProjectedShadowInfo->VirtualShadowMapPerLightCacheEntry = PerLightCacheEntry;
	ProjectedShadowInfo->bShouldRenderVSM = !PerLightCacheEntry->IsFullyCached();

	{
		const FViewInfo* ViewForShadow = ProjectedShadowInfo->FindViewForShadow(SceneRenderer.Views);
		check(ViewForShadow);
		uint32 PackedCullingViewId = FVirtualShadowMapProjectionShaderData::PackCullingViewId(ViewForShadow->SceneRendererPrimaryViewId, ViewForShadow->PersistentViewId);
		uint32 Flags = PerLightCacheEntry->IsUncached() ? VSM_PROJ_FLAG_UNCACHED : 0U;
		Flags |= PerLightCacheEntry->ShouldUseReceiverMask() ? VSM_PROJ_FLAG_USE_RECEIVER_MASK : 0U;

		// If the coarse pages mode is 2, we suppress invalidaitons of pages not marked as "detail geometry". 
		// Coarse pages are not marked with this flag so will not be invalidated.
		if (CVarMarkCoarsePagesLocal.GetValueOnRenderThread() == 2)
		{
			Flags |= VSM_PROJ_FLAG_FORCE_CACHE_DYNAMIC_COARSE;
		}

		const FLightSceneProxy* Proxy = ProjectedShadowInfo->GetLightSceneInfo().Proxy;

		// For now just tie this to whether anything has invalidated the light (including movement)
		// This is slightly over-conservative but catches the important cases
		const bool bUpdateMatrices = PerLightCacheEntry->IsInvalidated();

		for (int32 Index = 0; Index < NumMaps; ++Index)
		{
			FVirtualShadowMapCacheEntry& VirtualSmCacheEntry = PerLightCacheEntry->ShadowMapEntries[Index];
			VirtualSmCacheEntry.Update(*PerLightCacheEntry);

			FVirtualShadowMapProjectionShaderData& ProjectionData = VirtualSmCacheEntry.ProjectionData;

			if (bUpdateMatrices)
			{
				UpdateLocalLightProjectionShaderDataMatrices(ProjectedShadowInfo, Index, &ProjectionData);
			}

			// TODO: All of this is per-light data; splitting this out to a separate structure could help
			ProjectionData.LightType			= Proxy->GetLightType();
			ProjectionData.LightSourceRadius	= Proxy->GetSourceRadius();	
			ProjectionData.LightRadius			= Proxy->GetRadius();
			ProjectionData.TexelDitherScale		= Proxy->GetVSMTexelDitherScale();
			ProjectionData.ResolutionLodBias	= ResolutionLODBiasLocal;
			ProjectionData.Flags				= Flags;
			ProjectionData.MinMipLevel			= MinMipLevel;
			ProjectionData.PackedCullingViewId	= PackedCullingViewId;
			ProjectionData.LightId				= Proxy->GetLightSceneInfo()->Id.GetIndex();
		}
	}

	// TODO: This is remarkably slow and shouldn't really need to be evaluated multiple times
	ELightOcclusionType OcclusionType = GetLightOcclusionType(*LightSceneInfo->Proxy, SceneRenderer.ViewFamily);
	// Depending on which type of projection we're going to use, mark that we need to associated path for later
	if (OcclusionType == ELightOcclusionType::Shadowmap)
	{
		bNeedVSMOnePassProjection = true;
	}
	else if (OcclusionType == ELightOcclusionType::MegaLightsVSM)
	{
		bNeedMegaLightsProjection = true;
	}
	else
	{
		// ??? Should not get into this path with other projection types
		check(false);
	}

	return PerLightCacheEntry;
}

void FShadowSceneRenderer::AddDirectionalLightShadow(FLightSceneInfo& LightSceneInfo, FViewInfo& View, float MaxNonFarCascadeDistance, TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& OutShadowInfosThatNeedCulling)
{
	const int32 ViewIndex = View.SceneRendererPrimaryViewId;
	FLightSceneId LightId = LightSceneInfo.Id;
	FVisibleLightInfo& VisibleLightInfo = SceneRenderer.VisibleLightInfos[LightId];

	FLightCommonFrameSetup& CommonSetup = CommonSetups[LightId];
	check(CommonSetup.bIsDirectional);
	CommonSetup.bHasVirtualShadowMap = true;

	// Helper function to create a projected shadow info. This is needed to:
	//  * Get the matrices included in the shadow rendering pass setup, driving nanite VSM rendering (which VisibleLightInfo.AllProjectedShadows is appended to)
	auto AddLegacySetup = [&](const TSharedPtr<FVirtualShadowMapClipmap>& Clipmap, bool bQueueforNonNaniteCulling)
	{
		FProjectedShadowInfo* ProjectedShadowInfo = SceneRenderer.Allocator.Create<FProjectedShadowInfo>();
		ProjectedShadowInfo->SetupClipmapProjection(&LightSceneInfo, &View, Clipmap, CVarVsmUseFarShadowRules.GetValueOnRenderThread() != 0 ? MaxNonFarCascadeDistance : -1.0f);
	
		// This is needed to get it into the line for ending up in the SortedShadowsForShadowDepthPass.VirtualShadowMapShadows which is what drives the shadow rendering.
		VisibleLightInfo.AllProjectedShadows.Add(ProjectedShadowInfo);
		
		ProjectedShadowInfo->VirtualShadowMapPerLightCacheEntry = Clipmap->GetCacheEntry();

		if (bQueueforNonNaniteCulling)
		{
			ProjectedShadowInfo->SetupCullingQuery(SceneRenderer);

			OutShadowInfosThatNeedCulling.Emplace(ProjectedShadowInfo);
		}

		return ProjectedShadowInfo;
	};

	// a secondary view should not allocate shadow for itself!
	check(View.GetPrimaryView() == &View);
	FDirectionalLightShadowFrameSetup& Setup = DirectionalLights[CommonSetup.SetupIndex + ViewIndex];
	check(Setup.LightId == LightId);
	check(Setup.ViewMask & (1u << ViewIndex));

	// Set up regular clipmap
	FDirectionalLightShadowFrameSetup::FClipmapInfo& RegularInfo = Setup.ClipmapInfos[EVirtualShadowTypeId::Regular];
	{
		FVirtualShadowMapClipmapConfig Config = FVirtualShadowMapClipmapConfig::GetGlobal();
		if (VirtualShadowMapArray.IsPrefilteredDistantEnabled())
		{
			Config.bHasPrefilteredDistant = true;
			// TODO: is there any point to allow disabling this? Probably a rather bad default.
			// Config.bUseReceiverMask = true;
			// Note: we must force these on, both because it makes sense but also so we can re-purpose the DynamicDepthCullRange instead of adding yet another range (for now).
			Config.bForceInvalidate = true;
			Config.bCullDynamicTightly = true;
		}

		TSharedPtr<FVirtualShadowMapClipmap> VirtualShadowMapClipmap = TSharedPtr<FVirtualShadowMapClipmap>(new FVirtualShadowMapClipmap(
			VirtualShadowMapArray,
			LightSceneInfo,
			View.ViewMatrices,
			View.ViewRect.Size(),
			&View,
			ShadowScene.GetLightMobilityFactor(LightSceneInfo.Id),
			Config));

		// NOTE: only contains "regular" clipmaps, the alternate types are internal to the system and needs to be queried for.
		VisibleLightInfo.VirtualShadowMapClipmaps.Add(VirtualShadowMapClipmap);

		RegularInfo.ProjectedShadowInfo = AddLegacySetup(VirtualShadowMapClipmap, true);
		RegularInfo.Clipmap = VirtualShadowMapClipmap;
	}

	if (!ShadowScene.FirstPersonWorldSpacePrimitives.IsEmpty())
	{
		FDirectionalLightShadowFrameSetup::FClipmapInfo& FPInfo = Setup.ClipmapInfos[EVirtualShadowTypeId::FirstPerson];

		// Clone the setup from the regular VSM clipmap
		FVirtualShadowMapClipmapConfig Config = FVirtualShadowMapClipmapConfig::GetGlobal();
		Config.ShadowTypeId = EVirtualShadowTypeId::FirstPerson;
		Config.bForceInvalidate = true;
		Config.FirstCoarseLevel = -1;
		Config.LastCoarseLevel = -1;
		Config.FirstLevel = CVarVirtualShadowMapFirstPersonClipmapFirstLevel.GetValueOnRenderThread();
		Config.LastLevel = CVarVirtualShadowMapFirstPersonClipmapLastLevel.GetValueOnRenderThread();
		Config.bIsFirstPersonShadow = true;

		FPInfo.Clipmap = TSharedPtr<FVirtualShadowMapClipmap>(new FVirtualShadowMapClipmap(
			VirtualShadowMapArray,
			LightSceneInfo,
			View.ViewMatrices,
			View.ViewRect.Size(),
			&View,
			1.0f,// mobility factor as if moving - this VSM has no persistence, though perhaps it should for HZB?
			Config
		));

		FPInfo.ProjectedShadowInfo = AddLegacySetup(FPInfo.Clipmap, false);

		// Record the first chunk
		check((ExplicitChunkDrawInstanceIds.Num() % INSTANCE_HIERARCHY_MAX_CHUNK_SIZE) == 0);
		FPInfo.ProjectedShadowInfo->ExplicitNaniteInstanceChunkStartOffset = ExplicitChunkDrawInstanceIds.Num() / INSTANCE_HIERARCHY_MAX_CHUNK_SIZE;

		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ShadowScene.FirstPersonWorldSpacePrimitives)
		{
			// This is the visibility results we want for a First Person World Space Representation (FPWSR) primitive owned by Player A:
			// 
			// 			| Main View | Regular Clipmap | FP Clipmap
			// ---------|-----------|-----------------|-----------
			// Player A |	False	|	False		  |	True
			// Player B	|	True	|	True		  | False
			// 
			// The way we achieve that is that we force FPWSR primitives to bOwnerNoSee=true and bCastHiddenShadow=false, which will result in their bShadowRelevance only being
			// true for views which do NOT own the primitive (all views except Player A). In order to also draw the primitive into Player A's FP clipmap,
			// we explicitly invert the shadow relevance here. Also in order to not draw the primitive into Player B's FP clipmap, we need to explicitly check for ownership here.
			if (PrimitiveSceneInfo->Proxy->IsOwnedBy(View.ViewActor))
			{
				if (!PrimitiveSceneInfo->Proxy->IsNaniteMesh())
				{
					FPInfo.ProjectedShadowInfo->AddSubjectPrimitive(PrimitiveSceneInfo, SceneRenderer.Views, false, true);
				}
				else
				{
					const int32 InstanceSceneDataOffset = PrimitiveSceneInfo->GetInstanceSceneDataOffset();
					// Note: for larger instance counts one could also use the RLE capabilities here
					for (int32 InstanceIndex = 0; InstanceIndex < PrimitiveSceneInfo->GetNumInstanceSceneDataEntries(); ++InstanceIndex)
					{
						ExplicitChunkDrawInstanceIds.Emplace(InstanceSceneDataOffset + InstanceIndex);
					}
				}
			}
		}

		// Round up to even chunk size
		const int32 NumExplicitChunkDrawInstanceIds = ExplicitChunkDrawInstanceIds.Num();
		const int32 ExplicitChunkDrawInstanceIdsRemainder = NumExplicitChunkDrawInstanceIds % INSTANCE_HIERARCHY_MAX_CHUNK_SIZE;
		const int32 AlignedNumExplicitChunkDrawInstanceIds = FMath::DivideAndRoundUp(NumExplicitChunkDrawInstanceIds, int32(INSTANCE_HIERARCHY_MAX_CHUNK_SIZE)) * INSTANCE_HIERARCHY_MAX_CHUNK_SIZE;
		FPInfo.ProjectedShadowInfo->ExplicitNaniteInstanceLastChunkSize = NumExplicitChunkDrawInstanceIds > 0 && ExplicitChunkDrawInstanceIdsRemainder == 0 ? INSTANCE_HIERARCHY_MAX_CHUNK_SIZE : ExplicitChunkDrawInstanceIdsRemainder;
		FPInfo.ProjectedShadowInfo->ExplicitNaniteInstanceChunkEndOffset = AlignedNumExplicitChunkDrawInstanceIds / INSTANCE_HIERARCHY_MAX_CHUNK_SIZE;
		ExplicitChunkDrawInstanceIds.SetNumZeroed(AlignedNumExplicitChunkDrawInstanceIds);
	}

	ELightOcclusionType OcclusionType = GetLightOcclusionType(*LightSceneInfo.Proxy, SceneRenderer.ViewFamily);
	// Mark that we need MegaLights projection pass if this light uses it
	// We don't need to explicitly mark VSM projection here since directional lights don't go through one pass projection,
	// and thus will naturally go down the path that will render into the screen shadow mask.
	if (OcclusionType == ELightOcclusionType::MegaLightsVSM)
	{
		bNeedMegaLightsProjection = true;
	}
}

void FShadowSceneRenderer::AllocateVirtualShadowMapIds()
{
	if (!VirtualShadowMapArray.IsEnabled())
	{
		return;
	}

	// Directional lights allocated first
	for (const FDirectionalLightShadowFrameSetup& FrameSetup : DirectionalLights)
	{
		for (const auto& ClipmapInfo : FrameSetup.ClipmapInfos)
		{
			if (!ClipmapInfo.Clipmap.IsValid())
			{
				continue;
			}

			const TSharedPtr<FVirtualShadowMapPerLightCacheEntry>& CacheEntry = ClipmapInfo.Clipmap->GetCacheEntry();
			const int32 NextVirtualShadowMapId = VirtualShadowMapArray.AllocateDirectional(CacheEntry->ShadowMapEntries.Num());
			CacheEntry->UpdateVirtualShadowMapId(NextVirtualShadowMapId);
		}
	}

	// Then local lights
	for (const FLocalLightShadowFrameSetup& FrameSetup : LocalLights)
	{
		const TSharedPtr<FVirtualShadowMapPerLightCacheEntry>& CacheEntry = FrameSetup.PerLightCacheEntry;
		const int32 NextVirtualShadowMapId = VirtualShadowMapArray.AllocateLocal(CacheEntry->IsDistantLight(), CacheEntry->ShadowMapEntries.Num());
		CacheEntry->UpdateVirtualShadowMapId(NextVirtualShadowMapId);
	}

	// Then add any unreferenced lights that we still may keep around their cached pages at the end, and allocate VSM IDs for everything
	if (VirtualShadowMapArray.CacheManager)
	{
		VirtualShadowMapArray.CacheManager->UpdateUnreferencedCacheEntries(VirtualShadowMapArray);
	}
}

void FShadowSceneRenderer::PostInitDynamicShadowsSetup()
{
	AllocateVirtualShadowMapIds();

	// Dispatch async Nanite culling job if appropriate
	if (CVarVSMMaterialVisibility.GetValueOnRenderThread() != 0)
	{
		TArray<FConvexVolume, SceneRenderingAllocator> NaniteCullingViewsVolumes;
		// If we have a clipmap that can't be culled, it'd be a complete waste of time to cull the local lights.
		bool bUnboundedClipmap = false;
		
		for (const FDirectionalLightShadowFrameSetup& DirectionalLightShadowFrameSetup : DirectionalLights)
		{
			FProjectedShadowInfo* ProjectedShadowInfo = DirectionalLightShadowFrameSetup.ClipmapInfos[EVirtualShadowTypeId::Regular].ProjectedShadowInfo;
			if (!bUnboundedClipmap && ProjectedShadowInfo && ProjectedShadowInfo->bShouldRenderVSM)
			{
				const bool bIsCached = !ProjectedShadowInfo->VirtualShadowMapPerLightCacheEntry->IsUncached();

				// We can only do this culling if the light is both uncached & it is using the accurate bounds (i.e., r.Shadow.Virtual.Clipmap.UseConservativeCulling is turned off).
				if (!bIsCached && !ProjectedShadowInfo->CascadeSettings.ShadowBoundsAccurate.Planes.IsEmpty())
				{
					NaniteCullingViewsVolumes.Add(ProjectedShadowInfo->CascadeSettings.ShadowBoundsAccurate);
				}
				else
				{
					bUnboundedClipmap = true;
				}
			}
		}

		if (!bUnboundedClipmap)
		{
			for (const FLocalLightShadowFrameSetup& LocalLightShadowFrameSetup : LocalLights)
			{
				FProjectedShadowInfo* ProjectedShadowInfo = LocalLightShadowFrameSetup.ProjectedShadowInfo;
				if (ProjectedShadowInfo->bShouldRenderVSM)
				{
					FConvexVolume WorldSpaceCasterOuterFrustum = ProjectedShadowInfo->CasterOuterFrustum;
					for (FPlane& Plane : WorldSpaceCasterOuterFrustum.Planes)
					{
						Plane = Plane.TranslateBy(-ProjectedShadowInfo->PreShadowTranslation);
					}
					WorldSpaceCasterOuterFrustum.Init();
					NaniteCullingViewsVolumes.Add(WorldSpaceCasterOuterFrustum);
				}
			}

			if (!NaniteCullingViewsVolumes.IsEmpty())
			{
				NaniteVisibilityQuery = Scene.NaniteVisibility[ENaniteMeshPass::BasePass].BeginVisibilityQuery(
					SceneRenderer.Allocator,
					Scene,
					NaniteCullingViewsVolumes,
					&Scene.NaniteRasterPipelines[ENaniteMeshPass::BasePass],
					&Scene.NaniteShadingPipelines[ENaniteMeshPass::BasePass]
				);
			}
		}
	}
}

void FShadowSceneRenderer::RenderVirtualShadowMaps(FRDGBuilder& GraphBuilder, bool bNaniteEnabled, bool bUpdateNaniteStreaming)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShadowSceneRenderer::RenderVirtualShadowMaps);

	// Always process an existing query if it exists
	if (NaniteVisibilityQuery != nullptr)
	{
#if STATS
		GraphBuilder.AddSetupTask([Query = NaniteVisibilityQuery]
		{
			const FNaniteVisibilityResults& VisibilityResults = *Nanite::GetVisibilityResults(Query);

			uint32 TotalRasterBins = 0;
			uint32 VisibleRasterBins = 0;
			VisibilityResults.GetRasterBinStats(VisibleRasterBins, TotalRasterBins);

			uint32 TotalShadingBins = 0;
			uint32 VisibleShadingBins = 0;
			VisibilityResults.GetShadingBinStats(VisibleShadingBins, TotalShadingBins);

			SET_DWORD_STAT(STAT_VSMNaniteBasePassTotalRasterBins, TotalRasterBins);
			SET_DWORD_STAT(STAT_VSMNaniteBasePassVisibleRasterBins, VisibleRasterBins);

			SET_DWORD_STAT(STAT_VSMNaniteBasePassTotalShadingBins, TotalShadingBins);
			SET_DWORD_STAT(STAT_VSMNaniteBasePassVisibleShadingBins, VisibleShadingBins);

		}, Nanite::GetVisibilityTask(NaniteVisibilityQuery));
#endif
	}

	if (VirtualShadowMapArray.GetNumShadowMaps() == 0)
	{
		return;
	}

	if (bNaniteEnabled && NaniteRenderPasses.Num() > 0)
	{
		VirtualShadowMapArray.RenderVirtualShadowMapsNanite(
			GraphBuilder,
			SceneRenderer,
			bUpdateNaniteStreaming,
			NaniteVisibilityQuery,
			NaniteRenderPasses);
	}

	if (UseNonNaniteVirtualShadowMaps(SceneRenderer.ShaderPlatform, SceneRenderer.FeatureLevel))
	{
		const TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& VirtualShadowMapShadows = SceneRenderer.SortedShadowsForShadowDepthPass.VirtualShadowMapShadows;
		VirtualShadowMapArray.RenderVirtualShadowMapsNonNanite(GraphBuilder, SceneRenderer.GetSceneUniforms(), VirtualShadowMapShadows, SceneRenderer.Views);
	}

	VirtualShadowMapArray.PostRender(GraphBuilder);
}

void FShadowSceneRenderer::DispatchVirtualShadowMapViewAndCullingSetup(FRDGBuilder& GraphBuilder, TConstArrayView<FProjectedShadowInfo*> VirtualShadowMapShadows)
{
	SCOPED_NAMED_EVENT(FShadowSceneRenderer_DispatchVirtualShadowMapViewAndCullingSetup, FColor::Orange);

	// Unconditionally update GPU physical pages (on all GPUs) with new VSM IDs/addresses
	VirtualShadowMapArray.UpdatePhysicalPageAddresses(GraphBuilder, SceneRenderer.ViewFamily);

	if (!VirtualShadowMapShadows.IsEmpty() && UseNanite(SceneRenderer.ShaderPlatform))
	{
		CreateNaniteRenderPasses(GraphBuilder, SceneRenderer.Views, VirtualShadowMapShadows);

		// Dispatch collected queries
		for (FNaniteVirtualShadowMapRenderPass& RenderPass : NaniteRenderPasses)
		{
			RenderPass.SceneInstanceCullingQuery->Dispatch(GraphBuilder);
		}
	}
}

void FShadowSceneRenderer::PostSetupDebugRender()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if ((SceneRenderer.ViewFamily.EngineShowFlags.DebugDrawDistantVirtualSMLights) && VirtualShadowMapArray.IsEnabled())
	{
		int32 NumFullyCached = 0;
		int32 NumDistant = 0;
		for (FViewInfo& View : SceneRenderer.Views)
		{
			FViewElementPDI DebugPDI(&View, nullptr, nullptr);

			for (const FLocalLightShadowFrameSetup& LightSetup : LocalLights)
			{			
				FLinearColor Color = FLinearColor(FColor::Blue);
				if (LightSetup.PerLightCacheEntry && LightSetup.PerLightCacheEntry->IsDistantLight())
				{
					++NumDistant;
					int32 FramesSinceLastRender = int32(Scene.GetFrameNumber()) - int32(LightSetup.PerLightCacheEntry->GetLastScheduledFrameNumber());
					float Fade = FMath::Min(0.8f, float(FramesSinceLastRender) / float(LocalLights.Num()));
					if (LightSetup.PerLightCacheEntry->IsFullyCached())
					{
						++NumFullyCached;
						Color = FMath::Lerp(FLinearColor(FColor::Green), FLinearColor(FColor::Red), Fade);
					}
					else
					{
						Color = FLinearColor(FColor::Purple);
					}
				}

				Color.A = 1.0f;
				if (LightSetup.LightSceneInfo->Proxy->GetLightType() == LightType_Spot)
				{
					FTransform TransformNoScale = FTransform(LightSetup.LightSceneInfo->Proxy->GetLightToWorld());
					TransformNoScale.RemoveScaling();

					DrawWireSphereCappedCone(&DebugPDI, TransformNoScale, LightSetup.LightSceneInfo->Proxy->GetRadius(), FMath::RadiansToDegrees(LightSetup.LightSceneInfo->Proxy->GetOuterConeAngle()), 16, 4, 8, Color, SDPG_World);
				}
				else
				{
					DrawWireSphereAutoSides(&DebugPDI, LightSetup.LightSceneInfo->Proxy->GetOrigin(), Color, LightSetup.LightSceneInfo->Proxy->GetRadius(), SDPG_World);
				}
			}
		}
		SET_DWORD_STAT(STAT_DistantLightCount, NumDistant);
		SET_DWORD_STAT(STAT_DistantCachedCount, NumFullyCached);
	}
#endif
}

void FShadowSceneRenderer::BeginMarkVirtualShadowMapPages(
	FRDGBuilder& GraphBuilder,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult,
	const FFrontLayerTranslucencyData& FrontLayerTranslucencyData,
	const Froxel::FRenderer& FroxelRenderer)
{
	if (!VirtualShadowMapArray.IsEnabled())
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, ShadowDepths, "VirtualShadowMapMarkPages");

	VirtualShadowMapArray.BeginMarkPages(
		GraphBuilder,
		SceneRenderer,
		SingleLayerWaterPrePassResult,
		FrontLayerTranslucencyData,
		FroxelRenderer,
		AreAnyLocalLightsPresent()
	);
}

void FShadowSceneRenderer::RenderVirtualShadowMaps(
	FRDGBuilder& GraphBuilder, 
	bool bNaniteEnabled)
{
	if (!VirtualShadowMapArray.IsEnabled())
	{
		return;
	}

	VirtualShadowMapArray.BuildPageAllocations(
		GraphBuilder,
		SceneRenderer.GetActiveSceneTextures(),
		SceneRenderer.Views
	);

	RenderVirtualShadowMaps(GraphBuilder, bNaniteEnabled, CVarNaniteShadowsUpdateStreaming.GetValueOnRenderThread() != 0);
}

void FShadowSceneRenderer::RenderVirtualShadowMapProjectionMaskBits(
	FRDGBuilder& GraphBuilder,
	FMinimalSceneTextures& SceneTextures)
{
	bShouldUseVirtualShadowMapOnePassProjection =
		VirtualShadowMapArray.IsAllocated() &&
		IsVSMOnePassProjectionEnabled(SceneRenderer.ViewFamily.EngineShowFlags) &&
		bNeedVSMOnePassProjection;

	HairTransmittanceMaskBits.Empty();
	if (bShouldUseVirtualShadowMapOnePassProjection)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "VirtualShadowMapProjectionMaskBits");

		VirtualShadowMapMaskBits = CreateVirtualShadowMapMaskBits(GraphBuilder, SceneTextures, VirtualShadowMapArray, TEXT("Shadow.Virtual.MaskBits"));
		VirtualShadowMapMaskBitsHairStrands = CreateVirtualShadowMapMaskBits(GraphBuilder, SceneTextures, VirtualShadowMapArray, TEXT("Shadow.Virtual.MaskBits(HairStrands)"));

		for (int32 ViewIndex = 0; ViewIndex < SceneRenderer.Views.Num(); ++ViewIndex)
		{
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, SceneRenderer.Views.Num() > 1, "View%d", ViewIndex);

			const FViewInfo& View = SceneRenderer.Views[ViewIndex];

			RenderVirtualShadowMapProjectionOnePass(
				GraphBuilder,
				SceneTextures,
				View, ViewIndex,
				VirtualShadowMapArray,
				EVirtualShadowMapProjectionInputType::GBuffer,
				VirtualShadowMapMaskBits);

			if (HairStrands::HasViewHairStrandsData(View))
			{
				// Shadow bits
				RenderVirtualShadowMapProjectionOnePass(
					GraphBuilder,
					SceneTextures,
					View, ViewIndex,
					VirtualShadowMapArray,
					EVirtualShadowMapProjectionInputType::HairStrands,
					VirtualShadowMapMaskBitsHairStrands);

				// Transmittance bits
				HairTransmittanceMaskBits.Add(RenderHairStrandsOnePassTransmittanceMask(GraphBuilder, View, ViewIndex, VirtualShadowMapMaskBitsHairStrands, VirtualShadowMapArray).TransmittanceMask);
			}
		}
	}
	else
	{
		VirtualShadowMapMaskBits = nullptr;//Dummy;
		VirtualShadowMapMaskBitsHairStrands = nullptr;//Dummy;
	}
}

void FShadowSceneRenderer::RenderVirtualShadowMapProjection(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FLightSceneId LightId,
	const FViewInfo& View, int32 ViewIndex,
	const FIntRect ScissorRect,
	EVirtualShadowMapProjectionInputType InputType,
	bool bModulateRGB,
	FTiledVSMProjection* TiledVSMProjection,
	FRDGTextureRef OutputShadowMaskTexture)
{
	FShadowSceneRenderer::FDirectionalLightShadowFrameSetup* DirSetup = FindDirectional(LightId, ViewIndex);
	if (DirSetup != nullptr)
	{
		::RenderVirtualShadowMapProjection(
			GraphBuilder,
			SceneTextures,
			View, ViewIndex,
			VirtualShadowMapArray,
			ScissorRect,
			EVirtualShadowMapProjectionInputType::GBuffer,
			DirSetup->ClipmapInfos[EVirtualShadowTypeId::Regular].Clipmap,
			true, // bModulateRGB
			TiledVSMProjection,
			OutputShadowMaskTexture,
			DirSetup->ClipmapInfos[EVirtualShadowTypeId::FirstPerson].Clipmap);
	}
}

TArray<int32, SceneRenderingAllocator> FShadowSceneRenderer::GatherClipmapIds(int32 ViewIndex) const
{
	TArray<int32, SceneRenderingAllocator> Result;
	Result.Reserve(DirectionalLights.Num() * EVirtualShadowTypeId::Max);

	for (const FDirectionalLightShadowFrameSetup& DirSetup : DirectionalLights)
	{
		if (DirSetup.ViewMask & (1u << ViewIndex))
		{
			for (const auto& Info: DirSetup.ClipmapInfos)
			{
				if (Info.Clipmap)
				{
					Result.Emplace(Info.Clipmap->GetVirtualShadowMapId());
				}
			}
		}
	}
	return Result;
}

void FShadowSceneRenderer::ApplyVirtualShadowMapProjectionForLight(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FLightSceneInfo* LightSceneInfo,
	const EVirtualShadowMapProjectionInputType InputType,
	FRDGTextureRef OutputScreenShadowMaskTexture)
{
	if (!VirtualShadowMapArray.HasAnyShadowData())
	{
		return;
	}

	// Some lights can elide the screen shadow mask entirely, in which case they will be sampled directly in the lighting shader
	if (!OutputScreenShadowMaskTexture)
	{
		return;
	}

	FLightSceneId LightId = LightSceneInfo->Id;
	const FLightCommonFrameSetup& CommonSetup = CommonSetups[LightId];

	// No VSM set up this frame
	if (!CommonSetup.bHasVirtualShadowMap)
	{
		return;
	}
	
	for (int32 ViewIndex = 0; ViewIndex < SceneRenderer.Views.Num(); ++ViewIndex)
	{
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, SceneRenderer.Views.Num() > 1, "View%d", ViewIndex);

		FViewInfo& View = SceneRenderer.Views[ViewIndex];

		FIntRect ScissorRect;
		if (!LightSceneInfo->Proxy->GetScissorRect(ScissorRect, View, View.ViewRect))
		{
			ScissorRect = View.ViewRect;
		}

		if (ScissorRect.Area() > 0)
		{
			if (InputType == EVirtualShadowMapProjectionInputType::HairStrands && !HairStrands::HasViewHairStrandsData(View))
			{
				continue;
			}

			if (CommonSetup.bIsDirectional)
			{
				// remap to use the primary view index for stereo rendering.
				int32 ShadowViewIndex = View.GetPrimaryView()->SceneRendererPrimaryViewId;
				const FDirectionalLightShadowFrameSetup& DirSetup = DirectionalLights[CommonSetup.SetupIndex + ShadowViewIndex];

				INC_DWORD_STAT(STAT_VSMDirectionalProjectionFull);

				// Project directional light virtual shadow map
				::RenderVirtualShadowMapProjection(
					GraphBuilder,
					SceneTextures,
					View, ViewIndex,
					VirtualShadowMapArray,
					ScissorRect,
					InputType,
					DirSetup.ClipmapInfos[EVirtualShadowTypeId::Regular].Clipmap,
					false, // bModulateRGB
					nullptr, // TiledVSMProjection
					OutputScreenShadowMaskTexture,
					DirSetup.ClipmapInfos[EVirtualShadowTypeId::FirstPerson].Clipmap);
			}
			else 
			{
				const FLocalLightShadowFrameSetup& LocalLightSetup = LocalLights[CommonSetup.SetupIndex];
				if (bShouldUseVirtualShadowMapOnePassProjection)
				{
					INC_DWORD_STAT(STAT_VSMLocalProjectionOnePassCopy);


					// Copy local light from one pass projection output
					CompositeVirtualShadowMapFromMaskBits(
						GraphBuilder,
						SceneTextures,
						View, ViewIndex,
						ScissorRect,
						VirtualShadowMapArray,
						InputType,
						LocalLightSetup.PerLightCacheEntry->GetVirtualShadowMapId(),
						InputType == EVirtualShadowMapProjectionInputType::HairStrands ? VirtualShadowMapMaskBitsHairStrands : VirtualShadowMapMaskBits,
						OutputScreenShadowMaskTexture);
				}
				else
				{
					INC_DWORD_STAT(STAT_VSMLocalProjectionFull);

					// Project local light virtual shadow map
					::RenderVirtualShadowMapProjection(
						GraphBuilder,
						SceneTextures,
						View, ViewIndex,
						VirtualShadowMapArray,
						ScissorRect,
						InputType,
						*LightSceneInfo,
						LocalLightSetup.PerLightCacheEntry->GetVirtualShadowMapId(),
						OutputScreenShadowMaskTexture);
				}
			}			
		}
	}
}

struct FVSMRenderViewInfo
{
	FCullingVolume CullingVolume;
	uint32 NumPrimaryViews = 0u;
	uint32 MaxCullingViews = 0u;
};

static FVSMRenderViewInfo GetRenderViewInfo(const FProjectedShadowInfo* ProjectedShadowInfo)
{
	FVSMRenderViewInfo Info;
	Info.CullingVolume = ProjectedShadowInfo->GetCullingVolume();

	uint32 NumPrimaryViews = 0;
	if (ProjectedShadowInfo->VirtualShadowMapClipmap)
	{
		Info.NumPrimaryViews = ProjectedShadowInfo->VirtualShadowMapClipmap->GetLevelCount();
		Info.MaxCullingViews = Info.NumPrimaryViews;
	}
	else
	{
		Info.NumPrimaryViews = ProjectedShadowInfo->bOnePassPointLightShadow ? 6u : 1u;
		uint32 MinMipLevel = ProjectedShadowInfo->VirtualShadowMapPerLightCacheEntry->ShadowMapEntries[0].ProjectionData.MinMipLevel;
		Info.MaxCullingViews = Info.NumPrimaryViews * (FVirtualShadowMap::MaxMipLevels - MinMipLevel);
	}

	return Info;
}

void FShadowSceneRenderer::CreateNaniteViewsForPass(
	FRDGBuilder& GraphBuilder,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	TConstArrayView<FViewInfo> Views,
	float ShadowsLODScaleFactor,
	FNaniteVirtualShadowMapRenderPass& InOutRenderPass)
{
	InOutRenderPass.VirtualShadowMapViews = Nanite::FPackedViewArray::CreateWithSetupTask(
		GraphBuilder,
		InOutRenderPass.TotalPrimaryViews,
		[VirtualShadowMapArray, Views, InOutRenderPass, ShadowsLODScaleFactor] (Nanite::FPackedViewArray::ArrayType& VirtualShadowViews)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AddNaniteRenderViews);

			const bool bUseHzbOcclusion = VirtualShadowMapArray.UseHzbOcclusion();
			for (FProjectedShadowInfo* Shadow : InOutRenderPass.Shadows)
			{
				check(Shadow->bShouldRenderVSM);
				
				VirtualShadowMapArray.AddRenderViews(
					FVirtualShadowMapArray::ERenderPassType::Nanite,
					Shadow,
					Views,
					ShadowsLODScaleFactor,
					bUseHzbOcclusion,
					bUseHzbOcclusion,
					VirtualShadowViews);
			}
		});
}

void FShadowSceneRenderer::CreateNaniteRenderPasses(
	FRDGBuilder& GraphBuilder,
	TConstArrayView<FViewInfo> Views,
	TConstArrayView<FProjectedShadowInfo*> Shadows)
{
	// NOTE: We need to assume the worst case in terms of max mip views because of the way we pack the array
	// In practice almost all view sets will have the max # of mips unless there are no local lights anyways
	static constexpr uint32 MaxViews = NANITE_MAX_VIEWS_PER_CULL_RASTERIZE_PASS;
	static constexpr uint32 MaxPrimaryViews = MaxViews / FVirtualShadowMap::MaxMipLevels;

	// Don't want to run this more than once in a given frame.
	check(NaniteRenderPasses.IsEmpty());

	FSceneCullingRenderer* SceneCullingRenderer = SceneRenderer.GetSceneExtensionsRenderers().GetRendererPtr<FSceneCullingRenderer>();
	if (ensure(SceneCullingRenderer))
	{
		FRDGBufferRef ExplicitChunkDrawInstanceIdsRDG = !ExplicitChunkDrawInstanceIds.IsEmpty() ? CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.ExplicitChunkDrawInstanceIds"), ExplicitChunkDrawInstanceIds) : nullptr;
		TArray<FInstanceCullingGroupWork, SceneRenderingAllocator> ExplicitChunkDraws;
		
		const bool bAllowMultipass = CVarVirtualShadowMapNaniteAllowMultipassViews.GetValueOnRenderThread() != 0;		
		FNaniteVirtualShadowMapRenderPass RenderPass;
		auto Flush = [this, ExplicitChunkDrawInstanceIdsRDG, &ExplicitChunkDraws, &GraphBuilder](FNaniteVirtualShadowMapRenderPass& NaniteRenderPass)
		{
			if (NaniteRenderPass.MaxCullingViews > 0)
			{
				// Flush any previous render pass
				check(NaniteRenderPass.TotalPrimaryViews > 0);
				check(NaniteRenderPass.SceneInstanceCullingQuery);
				
				if (!ExplicitChunkDraws.IsEmpty())
				{
					check(ExplicitChunkDrawInstanceIdsRDG);
					Nanite::FExplicitChunkDrawInfo* ExplicitChunkDrawInfo = GraphBuilder.AllocObject<Nanite::FExplicitChunkDrawInfo>();
					ExplicitChunkDrawInfo->NumChunks = ExplicitChunkDraws.Num();
					ExplicitChunkDrawInfo->ExplicitChunkDraws = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.ExplicitChunkDraws"), MoveTemp(ExplicitChunkDraws));
					ExplicitChunkDrawInfo->InstanceIds = ExplicitChunkDrawInstanceIdsRDG;

					NaniteRenderPass.ExplicitChunkDrawInfo = ExplicitChunkDrawInfo;
				}

				NaniteRenderPasses.Add(NaniteRenderPass);
				NaniteRenderPass = FNaniteVirtualShadowMapRenderPass();
				ExplicitChunkDraws = {};
			}
		};

		for (FProjectedShadowInfo* ProjectedShadowInfo : Shadows)
		{
			const bool bIsFirstPersonShadowClipmap = ProjectedShadowInfo->VirtualShadowMapClipmap && ProjectedShadowInfo->VirtualShadowMapClipmap->IsFirstPersonShadow();
			const bool bHasExplicitInstanceChunks = ProjectedShadowInfo->ExplicitNaniteInstanceChunkStartOffset < ProjectedShadowInfo->ExplicitNaniteInstanceChunkEndOffset;
			if (ProjectedShadowInfo->bShouldRenderVSM && (!bIsFirstPersonShadowClipmap || bHasExplicitInstanceChunks))
			{
				FVSMRenderViewInfo Info = GetRenderViewInfo(ProjectedShadowInfo);

				// Space for the new views in the current pass?
				if (bAllowMultipass && (RenderPass.MaxCullingViews + Info.MaxCullingViews) > MaxViews)
				{
					Flush(RenderPass);
				}
				RenderPass.Shadows.Add(ProjectedShadowInfo);

				// Add a shadow thingo to be culled, need to know the primary view ranges.
				if (!RenderPass.SceneInstanceCullingQuery)
				{
					RenderPass.SceneInstanceCullingQuery = SceneCullingRenderer->CreateInstanceQuery(GraphBuilder);
				}

				if (bHasExplicitInstanceChunks)
				{
					const int32 ViewGroupIndex = RenderPass.SceneInstanceCullingQuery->AddViewDrawGroup(RenderPass.TotalPrimaryViews, Info.NumPrimaryViews);
					for (int32 ChunkIndex = ProjectedShadowInfo->ExplicitNaniteInstanceChunkStartOffset; ChunkIndex < ProjectedShadowInfo->ExplicitNaniteInstanceChunkEndOffset; ++ChunkIndex)
					{
						const bool bIsLastChunk = (ChunkIndex + 1) == ProjectedShadowInfo->ExplicitNaniteInstanceChunkEndOffset;
						const int32 CurrentChunkCount = bIsLastChunk ? ProjectedShadowInfo->ExplicitNaniteInstanceLastChunkSize : INSTANCE_HIERARCHY_MAX_CHUNK_SIZE;
						
						FInstanceCullingGroupWork& InstanceCullingGroupWork = ExplicitChunkDraws.AddDefaulted_GetRef();
						InstanceCullingGroupWork.ViewGroupId = static_cast<uint32>(ViewGroupIndex);
						InstanceCullingGroupWork.PackedItemChunkDesc = uint32(ChunkIndex) | (uint32(CurrentChunkCount) << INSTANCE_HIERARCHY_ITEM_CHUNK_COUNT_SHIFT);
						InstanceCullingGroupWork.ActiveViewMask = UINT32_MAX; // This mask will later be properly set on the GPU after view compaction.
					}
				}
				else
				{
					RenderPass.SceneInstanceCullingQuery->Add(RenderPass.TotalPrimaryViews, Info.NumPrimaryViews, Info.CullingVolume);
				}

				RenderPass.MaxCullingViews += Info.MaxCullingViews;
				RenderPass.TotalPrimaryViews += Info.NumPrimaryViews;
			}
		}
		Flush(RenderPass);
	}

	const float ShadowsLODScaleFactor = ComputeNaniteShadowsLODScaleFactor();
	for (FNaniteVirtualShadowMapRenderPass& RenderPass : NaniteRenderPasses)
	{
		CreateNaniteViewsForPass(GraphBuilder, VirtualShadowMapArray, Views, ShadowsLODScaleFactor, RenderPass);
	}
}
