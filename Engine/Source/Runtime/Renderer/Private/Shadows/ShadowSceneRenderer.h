// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/SparseArray.h"
#include "Containers/ArrayView.h"
#include "Containers/Array.h"
#include "Tasks/Task.h"
#include "LightSceneInfo.h"
#include "ShadowScene.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VirtualShadowMaps/VirtualShadowMapProjection.h"

class FProjectedShadowInfo;
class FDeferredShadingSceneRenderer;
class FWholeSceneProjectedShadowInitializer;
class FRDGBuilder;
class FVirtualShadowMapPerLightCacheEntry;
struct FNaniteVisibilityQuery;
enum class EVirtualShadowMapProjectionInputType;

namespace UE::Renderer::Private
{
	class IShadowInvalidatingInstances;
}

/**
 * Type Id used to differentiate when there are multiple clipmaps for the same light & view.
 */
namespace EVirtualShadowTypeId
{
	enum Type : uint32
	{
		// Regular clipmap
		Regular,
		// First-person (3rd person) shadow map,
		FirstPerson,
		
		Max
	};
}

/**
 * Transient scope for per-frame rendering resources for the shadow rendering.
 */
class FShadowSceneRenderer : public ISceneExtensionRenderer
{
public:
	DECLARE_SCENE_EXTENSION_RENDERER(FShadowSceneRenderer, FShadowScene);

	FShadowSceneRenderer(FDeferredShadingSceneRenderer& InDeferredShadingSceneRenderer, FShadowScene& InShadowScene);

	/**
	 * Multiply PackedView.LODScale by return value when rendering Nanite shadows.
	 */
	static float ComputeNaniteShadowsLODScaleFactor();

	/**
	 */
	virtual void PreInitViews(FRDGBuilder& GraphBuilder) override;

	/**
	 * Add a cube/spot light for processing this frame.
	 * TODO: Don't use legacy FProjectedShadowInfo or other params, instead info should flow from persistent setup & update.
	 * TODO: Return reference to FLocalLightShadowFrameSetup ?
	 */
	TSharedPtr<FVirtualShadowMapPerLightCacheEntry> AddLocalLightShadow(const FWholeSceneProjectedShadowInitializer& Initializer, FProjectedShadowInfo* ProjectedShadowInfo, FLightSceneInfo* LightSceneInfo, float MaxScreenRadius);
	/**
	 * Add a directional light for processing this frame.
	 */
	void AddDirectionalLightShadow(FLightSceneInfo& LightSceneInfo, FViewInfo& View, float MaxNonFarCascadeDistance, TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& OutShadowInfosThatNeedCulling);

	/**
	 * Call after view-dependent setup has been processed (InitView etc) but before any rendering activity has been kicked off.
	 */
	void PostInitDynamicShadowsSetup();

	/**
	 * Call to kick off culling tasks for VSMs & prepare views for rendering.
	 */
	void DispatchVirtualShadowMapViewAndCullingSetup(FRDGBuilder& GraphBuilder, TConstArrayView<FProjectedShadowInfo*> VirtualShadowMapShadows);

	void PostSetupDebugRender();

	/** 
	 * returns true if the given light has any VSM set up (for any view in the case of view dependent)
	 */
	bool HasVirtualShadowMap(FLightSceneId LightId) const { return CommonSetups[LightId].bHasVirtualShadowMap; }

	/** 
	 * returns true if the given light has a clipmap VSM set up for any view
	 */
	bool HasVirtualClipMap(FLightSceneId LightId) const { return CommonSetups[LightId].bHasVirtualShadowMap && CommonSetups[LightId].bIsDirectional; }

	/**
	 */
	void BeginMarkVirtualShadowMapPages(
		FRDGBuilder& GraphBuilder,
		const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult,
		const FFrontLayerTranslucencyData& FrontLayerTranslucencyData,
		const Froxel::FRenderer& FroxelRenderer);

	/**
	 */
	void RenderVirtualShadowMaps(FRDGBuilder& GraphBuilder, bool bNaniteEnabled);

	/* Does any one pass shadow projection and generates screen space shadow mask bits
	 * Call before beginning light loop/shadow projection, but after shadow map rendering
	 */
	void RenderVirtualShadowMapProjectionMaskBits(
		FRDGBuilder& GraphBuilder,
		FMinimalSceneTextures& SceneTextures);

	void RenderVirtualShadowMapProjection(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		FLightSceneId LightId,
		const FViewInfo& View, int32 ViewIndex,
		const FIntRect ScissorRect,
		EVirtualShadowMapProjectionInputType InputType,
		bool bModulateRGB,
		FTiledVSMProjection* TiledVSMProjection,
		FRDGTextureRef OutputShadowMaskTexture);


	/**
	 * Get the clipmap IDs for the view (for all lights and features), which in case of stereo maps to the primary view internally.
	 */
	TArray<int32, SceneRenderingAllocator> GatherClipmapIds(int32 ViewIndex) const;

	/**
	 * Renders virtual shadow map projection for a given light into the shadow mask.
	 * If one pass projection is enabled, this may be a simple composite from the shadow mask bits.
	 */
	void ApplyVirtualShadowMapProjectionForLight(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		const FLightSceneInfo* LightSceneInfo,
		const EVirtualShadowMapProjectionInputType InputType,
		FRDGTextureRef OutputScreenShadowMaskTexture);

	// One pass projection stuff. Set up in RenderVitualShadowMapProjectionMaskBits
	FRDGTextureRef VirtualShadowMapMaskBits = nullptr;
	FRDGTextureRef VirtualShadowMapMaskBitsHairStrands = nullptr;
	TArray<FRDGBufferRef> HairTransmittanceMaskBits;

	bool UsePackedShadowMaskBits() const
	{
		return VirtualShadowMapMaskBits != nullptr;
	}
	
	UE::Tasks::FTask GetRendererSetupTask() const
	{
		return RendererSetupTask;
	}

	bool AreAnyLightsUsingMegaLightsVSM() const
	{
		return bNeedMegaLightsProjection;
	}

	bool AreAnyLocalLightsPresent() const
	{
		return LocalLights.Num() > 0;
	}

	bool HasNaniteVisibilityQuery() const
	{
		return NaniteVisibilityQuery != nullptr;
	}

	UE::Renderer::Private::IShadowInvalidatingInstances *GetInvalidatingInstancesInterface(const FSceneView *SceneView);

	FVirtualShadowMapArray& GetVirtualShadowMapArray() { return VirtualShadowMapArray; }
	const FVirtualShadowMapArray& GetVirtualShadowMapArray() const { return VirtualShadowMapArray; }

private:
	/**
	 */
	void RenderVirtualShadowMaps(FRDGBuilder& GraphBuilder, bool bNaniteEnabled, bool bUpdateNaniteStreaming);

	struct FViewData
	{
		float ClipToViewSizeScale = 0.0f;
		float ClipToViewSizeBias = 0.0f;
	};

	// Generally only one pass, but we collect this to handle exceptional cases
	struct FNaniteRenderPass
	{
		FSceneInstanceCullingQuery *SceneInstanceCullingQuery = nullptr;
		TArray<FProjectedShadowInfo*, SceneRenderingAllocator> Shadows;
		uint32 TotalPrimaryViews = 0;
		uint32 MaxNumMips = 0;
		Nanite::FPackedViewArray* VirtualShadowMapViews = nullptr;
	};

	UE::Tasks::FTask RendererSetupTask;

	FVirtualShadowMapProjectionShaderData GetLocalLightProjectionShaderData(
		float ResolutionLODBiasLocal,
		const FProjectedShadowInfo* ProjectedShadowInfo,
		int32 MapIndex) const;

	 void UpdateLocalLightProjectionShaderDataMatrices(
		const FProjectedShadowInfo* ProjectedShadowInfo,
		int32 MapIndex,
		FVirtualShadowMapProjectionShaderData* OutProjectionShaderData) const;

	void CreateNaniteRenderPasses(
		FRDGBuilder& GraphBuilder,
		TConstArrayView<FViewInfo> Views,
		TConstArrayView<FProjectedShadowInfo*> Shadows);

	static void CreateNaniteViewsForPass(
		FRDGBuilder& GraphBuilder,
		const FVirtualShadowMapArray& VirtualShadowMapArray,
		TConstArrayView<FViewInfo> Views,
		float ShadowsLODScaleFactor,
		FNaniteVirtualShadowMapRenderPass& InOutRenderPass);

	void AllocateVirtualShadowMapIds();

	struct FLightCommonFrameSetup
	{
		uint32 bHasVirtualShadowMap : 1 = 0;
		uint32 bIsDirectional : 1 = 0;
		// index into respective setup array, for a directional light this is the offset to the setup for the first view index 
		uint32 SetupIndex : 30 = 0;
	};

	// Indexed by light scene ID
	TSceneRendererLightArray<FLightCommonFrameSetup> CommonSetups;

	struct FLocalLightShadowFrameSetup
	{
		TSharedPtr<FVirtualShadowMapPerLightCacheEntry> PerLightCacheEntry;
		// link to legacy system stuff, to be removed in due time
		FProjectedShadowInfo* ProjectedShadowInfo = nullptr;
		FLightSceneInfo* LightSceneInfo = nullptr;
	};

	// Indexed by order of allocation, represented in CommonSetups[LightId].SetupIndex
	TArray<FLocalLightShadowFrameSetup, SceneRenderingAllocator> LocalLights;

	struct FDirectionalLightShadowFrameSetup
	{
		// Search key
		FLightSceneId LightId = {};
		// A clipmap may belong to more than one view (in stereo mode, specifically)
		uint32 ViewMask = 0u;

		FDirectionalLightShadowFrameSetup(FLightSceneId InLightId, uint32 InViewMask) 
			: LightId(InLightId)
			, ViewMask(InViewMask)
		{
		}

		struct FClipmapInfo
		{
			TSharedPtr<FVirtualShadowMapClipmap> Clipmap;
			// for culling and other misc reasons.
			FProjectedShadowInfo* ProjectedShadowInfo = nullptr;
		};

		// clipmaps, Indexed by the EVirtualShadowTypeId.
		TStaticArray<FClipmapInfo, EVirtualShadowTypeId::Max> ClipmapInfos;
	};

	// Indexed by CommonSetups[LightId].SetupIndex + ViewIndex
	TArray<FDirectionalLightShadowFrameSetup, SceneRenderingAllocator> DirectionalLights;

	FDirectionalLightShadowFrameSetup* FindDirectional(const FLightSceneId LightId, int32 ViewIndex);

	// Links to other systems etc.
	FDeferredShadingSceneRenderer& SceneRenderer;
	FScene& Scene;
	FShadowScene& ShadowScene;
	FVirtualShadowMapArray& VirtualShadowMapArray;

	TArray<FNaniteVirtualShadowMapRenderPass, SceneRenderingAllocator> NaniteRenderPasses;

	FNaniteVisibilityQuery* NaniteVisibilityQuery = nullptr;
	TArray<FViewData, SceneRenderingAllocator> ViewDatas;

	// One pass projection stuff. Set up in RenderVitualShadowMapProjectionMaskBits
	bool bShouldUseVirtualShadowMapOnePassProjection = false;
		
	int32 DistantLightMode = 0;

	// Tracking for a given frame/render of which passes we need - clear in BeginRender
	bool bNeedVSMOnePassProjection = false;
	bool bNeedMegaLightsProjection = false;

	TArray<uint32, SceneRenderingAllocator> ExplicitChunkDrawInstanceIds;
};
