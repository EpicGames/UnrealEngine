#pragma once

// Copyright 2023 Sony Interactive Entertainment.
//
// Licensed under the Apache License, Version 2.0 (the "License");
// you may not use this file except in compliance with the License.
// You may obtain a copy of the License at
//
//     http://www.apache.org/licenses/LICENSE-2.0
//
// Unless required by applicable law or agreed to in writing, software
// distributed under the License is distributed on an "AS IS" BASIS,
// WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
// See the License for the specific language governing permissions and
// limitations under the License.

// If you have feedback, or found this code useful, we'd love to hear from you.
// https://www.bendstudio.com
// https://www.twitter.com/bendstudio
// 
// We are *always* looking for talented graphics and technical programmers!
// https://www.bendstudio.com/careers

// Common screen space shadow projection code (CPU):
//--------------------------------------------------------------

namespace Bend
{
	// Generating a screen-space-shadow requires a number of Compute Shader dispatches
	// The compute shader reads from a depth buffer, and writes a single-channel texture of the same dimensions
	// Each dispatch is of the same compute shader, (see bend_sss_gpu.h).
	// The number of dispatches required varies based on the on-screen location of the light.
	// Typically there will be just one or two dispatches when the light is off-screen, and 4 to 6 when the light is on-screen.
	// Syncing the GPU between individual dispatches is not required

	// These structures and function are used to generate the number of dispatches, the wave count of each dispatch (X/Y/Z) and shader parameters for each dispatch

	struct DispatchData
	{
		int WaveCount[3];					// Compute Shader Dispatch(X,Y,Z) wave counts X/Y/Z
		int WaveOffset_Shader[2];			// This value is passed in to shader. It will be different for each dispatch
	};

	struct DispatchList
	{
		float LightCoordinate_Shader[4];	// This value is passed in to shader, this will be the same value for all dispatches for this light

		DispatchData Dispatch[8];			// List of dispatches (max count is 8)
		int DispatchCount;					// Number of compute dispatches written to the list
	};


	// Helper functions
	inline int bend_min(const int a, const int b) { return a > b ? b : a; }
	inline int bend_max(const int a, const int b) { return a > b ? a : b; }


	// Call this function on the CPU to get a list of Compute Shader dispatches required to generate a screen-space shadow for a given light
	// Syncing the GPU between individual dispatches is not required
	// 
	// inLightProjection:		Homogeneous coordinate of the light, result of {light} * {ViewProjectionMatrix}, (without W divide)
	//							For infinite directional lights, use {light} = float4(normalized light direction, 0) and for point/spot lights use {light} = float4(light world position, 1)
	// 
	// inViewportSize:			width/height of the render target
	// 
	// inRenderBounds:			2D Screen Bounds of the light within the viewport, inclusive. [0,0], [width,height] for full-screen.
	//							Note; the shader will still read/write outside of these bounds (by a maximum of 2 * WAVE_SIZE pixels), due to how the wavefront projection works.
	// 
	// inExpandedDepthRange:	Set to true if the rendering API expects z/w coordinate output from a vertex shader to be a [-1,+1] expanded range, and becomes [0,1] range in the depth buffer. Typically this is false.
	// 
	// inWaveSize:				Wavefront size of the compiled compute shader (currently only tested with 64)
	//
	inline DispatchList BuildDispatchList(float inLightProjection[4], int inViewportSize[2], int inMinRenderBounds[2], int inMaxRenderBounds[2], bool inExpandedZRange = false, int inWaveSize = 64)
	{
		DispatchList result = {};

		// Floating point division in the shader has a practical limit for precision when the light is *very* far off screen (~1m pixels+)
		// So when computing the light XY coordinate, use an adjusted w value to handle these extreme values
		float xy_light_w = inLightProjection[3];
		float FP_limit = 0.000002f * (float)inWaveSize;

		if (xy_light_w >= 0 && xy_light_w < FP_limit) xy_light_w = FP_limit;
		else if (xy_light_w < 0 && xy_light_w > -FP_limit) xy_light_w = -FP_limit;

		// Need precise XY pixel coordinates of the light
		result.LightCoordinate_Shader[0] = ((inLightProjection[0] / xy_light_w) * +0.5f + 0.5f) * (float)inViewportSize[0];
		result.LightCoordinate_Shader[1] = ((inLightProjection[1] / xy_light_w) * -0.5f + 0.5f) * (float)inViewportSize[1];
		result.LightCoordinate_Shader[2] = inLightProjection[3] == 0 ? 0 : (inLightProjection[2] / inLightProjection[3]);
		result.LightCoordinate_Shader[3] = inLightProjection[3] > 0 ? 1 : -1;

		if (inExpandedZRange)
		{
			result.LightCoordinate_Shader[2] = result.LightCoordinate_Shader[2] * 0.5f + 0.5f;
		}

		int light_xy[2] = { (int)(result.LightCoordinate_Shader[0] + 0.5f), (int)(result.LightCoordinate_Shader[1] + 0.5f) };

		// Make the bounds inclusive, relative to the light
		const int biased_bounds[4] =
		{
			inMinRenderBounds[0] - light_xy[0],
			-(inMaxRenderBounds[1] - light_xy[1]),
			inMaxRenderBounds[0] - light_xy[0],
			-(inMinRenderBounds[1] - light_xy[1]),
		};

		// Process 4 quadrants around the light center,
		// They each form a rectangle with one corner on the light XY coordinate
		// If the rectangle isn't square, it will need breaking in two on the larger axis
		// 0 = bottom left, 1 = bottom right, 2 = top left, 2 = top right
		for (int q = 0; q < 4; q++)
		{
			// Quads 0 and 3 needs to be +1 vertically, 1 and 2 need to be +1 horizontally
			bool vertical = q == 0 || q == 3;

			// Bounds relative to the quadrant
			const int bounds[4] =
			{
				bend_max(0, ((q & 1) ? biased_bounds[0] : -biased_bounds[2])) / inWaveSize,
				bend_max(0, ((q & 2) ? biased_bounds[1] : -biased_bounds[3])) / inWaveSize,
				bend_max(0, (((q & 1) ? biased_bounds[2] : -biased_bounds[0]) + inWaveSize * (vertical ? 1 : 2) - 1)) / inWaveSize,
				bend_max(0, (((q & 2) ? biased_bounds[3] : -biased_bounds[1]) + inWaveSize * (vertical ? 2 : 1) - 1)) / inWaveSize,
			};

			if ((bounds[2] - bounds[0]) > 0 && (bounds[3] - bounds[1]) > 0)
			{
				int bias_x = (q == 2 || q == 3) ? 1 : 0;
				int bias_y = (q == 1 || q == 3) ? 1 : 0;

				DispatchData& disp = result.Dispatch[result.DispatchCount++];

				disp.WaveCount[0] = inWaveSize;
				disp.WaveCount[1] = bounds[2] - bounds[0];
				disp.WaveCount[2] = bounds[3] - bounds[1];
				disp.WaveOffset_Shader[0] = ((q & 1) ? bounds[0] : -bounds[2]) + bias_x;
				disp.WaveOffset_Shader[1] = ((q & 2) ? -bounds[3] : bounds[1]) + bias_y;

				// We want the far corner of this quadrant relative to the light,
				// as we need to know where the diagonal light ray intersects with the edge of the bounds
				int axis_delta = +biased_bounds[0] - biased_bounds[1];
				if (q == 1) axis_delta = +biased_bounds[2] + biased_bounds[1];
				if (q == 2) axis_delta = -biased_bounds[0] - biased_bounds[3];
				if (q == 3) axis_delta = -biased_bounds[2] + biased_bounds[3];

				axis_delta = (axis_delta + inWaveSize - 1) / inWaveSize;

				if (axis_delta > 0)
				{
					DispatchData& disp2 = result.Dispatch[result.DispatchCount++];

					// Take copy of current volume
					disp2 = disp;

					if (q == 0)
					{
						// Split on Y, split becomes -1 larger on x
						disp2.WaveCount[2] = bend_min(disp.WaveCount[2], axis_delta);
						disp.WaveCount[2] -= disp2.WaveCount[2];
						disp2.WaveOffset_Shader[1] = disp.WaveOffset_Shader[1] + disp.WaveCount[2];
						disp2.WaveOffset_Shader[0]--;
						disp2.WaveCount[1] ++;
					}
					if (q == 1)
					{
						// Split on X, split becomes +1 larger on y
						disp2.WaveCount[1] = bend_min(disp.WaveCount[1], axis_delta);
						disp.WaveCount[1] -= disp2.WaveCount[1];
						disp2.WaveOffset_Shader[0] = disp.WaveOffset_Shader[0] + disp.WaveCount[1];
						disp2.WaveCount[2] ++;
					}
					if (q == 2)
					{
						// Split on X, split becomes -1 larger on y
						disp2.WaveCount[1] = bend_min(disp.WaveCount[1], axis_delta);
						disp.WaveCount[1] -= disp2.WaveCount[1];
						disp.WaveOffset_Shader[0] += disp2.WaveCount[1];
						disp2.WaveCount[2] ++;
						disp2.WaveOffset_Shader[1]--;
					}
					if (q == 3)
					{
						// Split on Y, split becomes +1 larger on x
						disp2.WaveCount[2] = bend_min(disp.WaveCount[2], axis_delta);
						disp.WaveCount[2] -= disp2.WaveCount[2];
						disp.WaveOffset_Shader[1] += disp2.WaveCount[2];
						disp2.WaveCount[1] ++;
					}

					// Remove if too small
					if (disp2.WaveCount[1] <= 0 || disp2.WaveCount[2] <= 0)
					{
						disp2 = result.Dispatch[--result.DispatchCount];
					}
					if (disp.WaveCount[1] <= 0 || disp.WaveCount[2] <= 0)
					{
						disp = result.Dispatch[--result.DispatchCount];
					}
				}
			}
		}

		// Scale the shader values by the wave count, the shader expects this
		for (int i = 0; i < result.DispatchCount; i++)
		{
			result.Dispatch[i].WaveOffset_Shader[0] *= inWaveSize;
			result.Dispatch[i].WaveOffset_Shader[1] *= inWaveSize;
		}

		return result;
	}
}


/*
* Common Problems, and tips to solve them:
*
* The shader doesn't compile?
*  -	The shader is only tested with HLSL (DXC compiler) and PS5 using a HLSL->PS5 warpper
*		It should be possible to compile for DX11 with FXC, with features removed (early-out's use of wave intrinsics is not supported in DX11)
*		Other shader languages (e.g., glsl) are unsupported and will require manual conversion
*
* I have it compiled and running, but I'm seeing complete nonsense?
*  -	Start by enabling 'DebugOutputWaveIndex' to visualize the wavefront layout.
*		You should see wavefronts aligned and projected towards the light position / direction. If not, then 'inLightProjection' is probably wrong.
*
* Struggling to get 'inLightProjection' right?
*  -	Think of this as similar to what a vertex-shader would output for position export; that is, a 4-component transformed position (e.g., SV_POSITION output from the VS)
*		This usually means:
*		inLightProjection = float4(position,1) * ViewProjectionMatrix		- positional light
*		or:
*		inLightProjection = float4(direction,0) * ViewProjectionMatrix		- directional light
*
* Almost everything is in shadow?
* Is the background around an object casting a shadow on to it?
* Does everything look like a weird paper cut-out?
* There is a big shadow halo around the light source?
*  -	FarDepthValue / NearDepthValue may not be set correctly
*		These are the values you'd see in the depth buffer for objects at the near and far clip plane (typically far = 0, near = 1)
*		In very rare cases, the vertex shader may be expected to output a [-1,+1] z-range, which becomes [0,1] in the depth buffer. If this is the case, enable 'inExpandedZRange'.
*
* There are small glitchy lines all over the output?
*  -	Try with/without 'USE_HALF_PIXEL_OFFSET' defined in the shader. This is enabled by default for HLSL.
*
* Invalid shadows occasionally appear from offscreen / at the edge of the screen?
*  -	The 'PointBorderSampler' may not be setup correctly. The shader will intentionally read from offscreen, so the sampler must return an invalid value (e.g., FarDepthValue) in these cases by using clamp-to-border.
*
* Light is always coming from the same direction no matter how I rotate the camera?
*  -	You may be using just the ProjectionMatrix, not the ViewProjectionMatrix when computing 'inLightProjection'.
*
* Shadow is extremely thick, or very faded?
*  -	Try scaling 'SurfaceThickness' up and down, start with 0.005, and scale up/down in multiples of 2. Make sure to scale BilinearThreshold in a similar way.
*
* I see lots of striated patterns on flat surfaces?
*  -	'BilinearThreshold' may not be set to an ideal value (enable 'DebugOutputEdgeMask' to debug it), or try enabling 'IgnoreEdgePixels'
*  -	These issues can be more common in otherwise occluded areas when BilinearSamplingOffsetMode is false, and may be prevalent if visualizing the shadow - but not noticeable in a lit scene.
*/