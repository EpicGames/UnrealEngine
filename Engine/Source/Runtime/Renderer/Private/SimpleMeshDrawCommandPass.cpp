// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SimpleMeshDrawCommandPass.cpp: 
=============================================================================*/

#include "SimpleMeshDrawCommandPass.h"
#include "ScenePrivate.h"

FSimpleMeshDrawCommandPass::FSimpleMeshDrawCommandPass(const FSceneView& View, FInstanceCullingManager* InstanceCullingManager, bool bEnableStereo) :
	DynamicPassMeshDrawListContext(DynamicMeshDrawCommandStorage, VisibleMeshDrawCommands, GraphicsMinimalPipelineStateSet, bNeedsInitialization)
{
	check(View.bIsViewInfo);
	const FViewInfo* ViewInfo = static_cast<const FViewInfo*>(&View);
	
	TArray<int32, TFixedAllocator<2> > ViewIds;
	ViewIds.Add(ViewInfo->SceneRendererPrimaryViewId);
	bUsingStereo = bEnableStereo && ViewInfo->IsInstancedStereo();
	if (bUsingStereo)
	{
		check(ViewInfo->GetInstancedView() != nullptr);
		ViewIds.Add(ViewInfo->GetInstancedView()->SceneRendererPrimaryViewId);
	}

	InstanceCullingContext = FInstanceCullingContext(TEXT("SimpleMDCPass"), ViewInfo->GetShaderPlatform(), InstanceCullingManager, ViewIds, nullptr, bUsingStereo ? EInstanceCullingMode::Stereo : EInstanceCullingMode::Normal);

	InstanceFactor = ViewInfo->GetStereoPassInstanceFactor();
}

void FSimpleMeshDrawCommandPass::BuildRenderingCommands(FRDGBuilder& GraphBuilder, const FSceneView& View, const FGPUScene& GPUScene, FInstanceCullingDrawParams& OutInstanceCullingDrawParams)
{
	// NOTE: Everything up to InstanceCullingContext.BuildRenderingCommands could be peeled off into an async task.
	ApplyViewOverridesToMeshDrawCommands(View, VisibleMeshDrawCommands, DynamicMeshDrawCommandStorage, GraphicsMinimalPipelineStateSet, bNeedsInitialization);

	VisibleMeshDrawCommands.Sort(FCompareFMeshDrawCommands());
	if (GPUScene.IsEnabled())
	{
		int32 MaxInstances = 0;
		int32 VisibleMeshDrawCommandsNum = 0;
		int32 NewPassVisibleMeshDrawCommandsNum = 0;

		// 1. Run draw command setup, but only the first time.
		if (!bWasDrawCommandsSetup)
		{
			InstanceCullingContext.SetupDrawCommands(VisibleMeshDrawCommands, true, &GPUScene.GetScene(), MaxInstances, VisibleMeshDrawCommandsNum, NewPassVisibleMeshDrawCommandsNum);
			bWasDrawCommandsSetup = true;
		}

		// 2. Run finalize culling commands pass
		check(View.bIsViewInfo);
		const FViewInfo* ViewInfo = static_cast<const FViewInfo*>(&View);
		InstanceCullingContext.SetDynamicPrimitiveInstanceOffsets(ViewInfo->DynamicPrimitiveCollector.GetInstanceSceneDataOffset(), ViewInfo->DynamicPrimitiveCollector.NumInstances());
		InstanceCullingContext.BuildRenderingCommands(GraphBuilder, GPUScene, &OutInstanceCullingDrawParams);
		
		// Signal that scene primitives are supported, used for validation, the existence of a valid InstanceCullingResult is the required signal
		bSupportsScenePrimitives = true;
	}
}

void FSimpleMeshDrawCommandPass::BuildRenderingCommands(FRDGBuilder& GraphBuilder, const FSceneView& View, const FScene& Scene, FInstanceCullingDrawParams& OutInstanceCullingDrawParams)
{
	BuildRenderingCommands(GraphBuilder, View, Scene.GPUScene, OutInstanceCullingDrawParams);
}

void FSimpleMeshDrawCommandPass::SubmitDraw(FRHICommandList& RHICmdList, const FInstanceCullingDrawParams& InstanceCullingDrawParams) const
{
	if (VisibleMeshDrawCommands.Num() > 0)
	{
		if (bSupportsScenePrimitives)
		{
			InstanceCullingContext.SubmitDrawCommands(
				VisibleMeshDrawCommands,
				GraphicsMinimalPipelineStateSet,
				GetMeshDrawCommandOverrideArgs(InstanceCullingDrawParams),
				0,
				VisibleMeshDrawCommands.Num(),
				InstanceFactor,
				RHICmdList);
		}
		else
		{
			FMeshDrawCommandSceneArgs SceneArgs;
			if (InstanceCullingContext.bUsesUniformBufferView)
			{
				SceneArgs.BatchedPrimitiveSlot = InstanceCullingContext.InstanceCullingStaticSlot;
			}
			else
			{
				SceneArgs.PrimitiveIdsBuffer = GetMeshDrawCommandOverrideArgs(InstanceCullingDrawParams).InstanceBuffer;
			}
			SceneArgs.PrimitiveIdOffset = 0u;
			const uint32 PrimitiveIdBufferStride = FInstanceCullingContext::GetInstanceIdBufferStride(InstanceCullingContext.ShaderPlatform);

			SubmitMeshDrawCommandsRange(VisibleMeshDrawCommands, GraphicsMinimalPipelineStateSet, SceneArgs, PrimitiveIdBufferStride, false, 0, VisibleMeshDrawCommands.Num(), InstanceFactor, RHICmdList);
		}
	}
}
