// Copyright Epic Games, Inc. All Rights Reserved.

#include "SingleLayerWaterRendering.h"
#include "BasePassRendering.h"
#include "DeferredShadingRenderer.h"
#include "DistortionRendering.h"
#include "MeshPassProcessor.inl"
#include "PixelShaderUtils.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "PostProcess/SceneRenderTargets.h"
#include "PostProcess/TemporalAA.h"
#include "RayTracing/RaytracingOptions.h"
#include "VolumetricRenderTarget.h"
#include "RenderGraph.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "VelocityRendering.h"
#include "ScreenSpaceRayTracing.h"
#include "SceneTextureParameters.h"
#include "Substrate/Substrate.h"
#include "Shadows/ShadowSceneRenderer.h"
#include "Lumen/LumenSceneData.h"
#include "Lumen/LumenTracingUtils.h"
#include "Lumen/LumenReflections.h"
#include "RenderCore.h"
#include "UnrealEngine.h"
#include "CustomDepthRendering.h"
#include "HairStrands/HairStrandsData.h"
#include "HairStrands/HairStrandsComposition.h"
#include "ComponentRecreateRenderStateContext.h"
#include "SceneViewState.h"

#include "DepthCopy.h"
using namespace DepthCopy;

DECLARE_GPU_STAT_NAMED(RayTracingWaterReflections, TEXT("Ray Tracing Water Reflections"));

DECLARE_GPU_DRAWCALL_STAT(SingleLayerWaterDepthPrepass);
DECLARE_GPU_DRAWCALL_STAT(SingleLayerWater);

static TAutoConsoleVariable<int32> CVarWaterSingleLayer(
	TEXT("r.Water.SingleLayer"), 1,
	TEXT("Enable the single water rendering system."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<bool> CVarWaterSingleLayerWaveOps(
	TEXT("r.Water.SingleLayer.WaveOps"),
	true,
	TEXT("Whether Single Layer Water should use wave ops if supported."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

//
// Reflections

namespace ESingleLayerWaterReflections
{
	enum Type
	{
		Disabled			= 0, // No reflections on water at all.
		Enabled				= 1, // Same reflection technique as the rest of the scene.
		ReflectionCaptures	= 2, // Force using reflection captures and skylight (cubemaps) only.
		SSR					= 3, // Force using SSR (includes cubemaps). Will fall back to cubemaps only if SSR is not supported.
		MaxValue			= SSR
	};
}

static TAutoConsoleVariable<int32> CVarWaterSingleLayerReflection(
	TEXT("r.Water.SingleLayer.Reflection"), 1,
	TEXT("Reflection technique to use on single layer water. 0: Disabled, 1: Enabled (same as rest of scene), 2: Force Reflection Captures and Sky, 3: Force SSR"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerReflectionDownsampleFactor(
	TEXT("r.Water.SingleLayer.Reflection.DownsampleFactor"),
	1,
	TEXT("Downsample factor for Single Layer Water Reflection. Downsampling will introduce extra noise, so it's recommend to be used together with denoising (r.Water.SingleLayer.Reflection.Denoiser 1)."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerReflectionDownsampleCheckerboard(
	TEXT("r.Water.SingleLayer.Reflection.DownsampleCheckerboard"),
	0,
	TEXT("Whether to use checkerboard downsampling when DownsampleFactor is greater than one."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerReflectionScreenSpaceReconstruction(
	TEXT("r.Water.SingleLayer.Reflection.ScreenSpaceReconstruction"),
	0,
	TEXT("Whether to use screen space reconstruction for Single Layer Water reflection traces. Usually not needed, as water has mostly mirror reflections."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerReflectionDenoising(
	TEXT("r.Water.SingleLayer.Reflection.Denoising"),
	0,
	TEXT("Whether to use denoising for Single Layer Water reflection traces. Adds some cost and makes reflections softer, but removes noise and flickering."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerTiledComposite(
	TEXT("r.Water.SingleLayer.TiledComposite"), 1,
	TEXT("Enable tiled optimization of the single layer water reflection rendering system."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerSSRTAA(
	TEXT("r.Water.SingleLayer.SSRTAA"), 1,
	TEXT("Enable SSR denoising using TAA for the single layer water rendering system."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

//
// Shadows

static TAutoConsoleVariable<int32> CVarWaterSingleLayerShadersSupportDistanceFieldShadow(
	TEXT("r.Water.SingleLayer.ShadersSupportDistanceFieldShadow"), 1,
	TEXT("Whether or not the single layer water material shaders are compiled with support for distance field shadow, i.e. output main directional light luminance in a separate render target. This is preconditioned on using deferred shading and having distance field support enabled in the project."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerDistanceFieldShadow(
	TEXT("r.Water.SingleLayer.DistanceFieldShadow"), 1,
	TEXT("When using deferred, distance field shadow tracing is supported on single layer water. This cvar can be used to toggle it on/off at runtime."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarSupportCloudShadowOnSingleLayerWater(
	TEXT("r.Water.SingleLayerWater.SupportCloudShadow"), 0,
	TEXT("Enables cloud shadows on SingleLayerWater materials."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerShadersSupportVSMFiltering(
	TEXT("r.Water.SingleLayer.ShadersSupportVSMFiltering"), 0,
	TEXT("Whether or not the single layer water material shaders are compiled with support for virtual shadow map filter, i.e. output main directional light luminance in a separate render target. This is preconditioned on using deferred shading and having VSM support enabled in the project."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerVSMFiltering(
	TEXT("r.Water.SingleLayer.VSMFiltering"), 0,
	TEXT("When using deferred, virtual shadow map filtering is supported on single layer water. This cvar can be used to toggle it on/off at runtime."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

//
// Misc

static TAutoConsoleVariable<int32> CVarWaterSingleLayerRefractionDownsampleFactor(
	TEXT("r.Water.SingleLayer.RefractionDownsampleFactor"), 1,
	TEXT("Resolution divider for the water refraction buffer."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarParallelSingleLayerWaterPass(
	TEXT("r.ParallelSingleLayerWaterPass"), 1,
	TEXT("Toggles parallel single layer water pass rendering. Parallel rendering must be enabled for this to have an effect."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerDepthPrepass(
	TEXT("r.Water.SingleLayer.DepthPrepass"), 1,
	TEXT("Enable a depth prepass for single layer water. Necessary for proper Virtual Shadow Maps support."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSingleLayerWaterPassOptimizedClear(
	TEXT("r.Water.SingleLayer.OptimizedClear"), 1,
	TEXT("Toggles optimized depth clear"),
	ECVF_RenderThreadSafe);

// Deprecated: use r.Water.SingleLayer.VelocityOutputPass instead
static TAutoConsoleVariable<int32> CVarWaterSingleLayerForceVelocity(
	TEXT("r.Water.SingleLayer.ForceVelocity"), -1,
	TEXT("Deprecated. Use r.Water.SingleLayer.VelocityOutputPass instead."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerVelocityOutputPass(
	TEXT("r.Water.SingleLayer.VelocityOutputPass"), 1,
	TEXT("When to write velocity for Single Layer Water.\n")
	TEXT("  0: None - no velocity written for SLW.\n")
	TEXT("  1: Depth Prepass - Write during SLW depth prepass (default, supports Motion Vector World Offset)\n")
	TEXT("  2: Base Pass - Write during SLW base pass."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSingleLayerWaterRefractionCulling(
	TEXT("r.Water.SingleLayer.Refraction.Culling"), 0,
	TEXT("Enables refraction culling on water. This allows the renderer to skip rendering portions of the scene behind water which are unlikely to be visible through water."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSingleLayerWaterRefractionDistanceCulling(
	TEXT("r.Water.SingleLayer.Refraction.DistanceCulling"), -1.0f,
	TEXT("Distance at which to cull refractions."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSingleLayerWaterRefractionDistanceCullingFadeRange(
	TEXT("r.Water.SingleLayer.Refraction.DistanceCullingFadeRange"), 400.0f,
	TEXT("Range over which to fade out refractions."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSingleLayerWaterRefractionFresnelCulling(
	TEXT("r.Water.SingleLayer.Refraction.FresnelCulling"), -1.0f,
	TEXT("Fresnel value below which to cull refractions."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSingleLayerWaterRefractionFresnelCullingFadeRange(
	TEXT("r.Water.SingleLayer.Refraction.FresnelCullingFadeRange"), 0.2f,
	TEXT("Fresnel range over which to fade out refractions."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSingleLayerWaterRefractionDepthCulling(
	TEXT("r.Water.SingleLayer.Refraction.DepthCulling"), -1.0f,
	TEXT("Depth below which to cull refractions."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSingleLayerWaterRefractionDepthCullingFadeRange(
	TEXT("r.Water.SingleLayer.Refraction.DepthCullingFadeRange"), 100.0f,
	TEXT("Depth range over which to fade out refractions."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerTiledSceneColorCopy(
	TEXT("r.Water.SingleLayer.TiledSceneColorCopy"), 1,
	TEXT("Use indirect draws and a list of water pixel tiles to copy only relevant parts of the SceneColor texture for refraction behind the water surface."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarWaterSingleLayerMobileFallback(
	TEXT("r.Water.SingleLayer.MobileFallback"), 1,
	TEXT("Whether to use a translucent rendering fallback for a single layer water on mobile platforms"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable) 
		{ 
			FGlobalComponentRecreateRenderStateContext Context; 
		}),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static int32 GetSingleLayerWaterReflectionTechnique()
{
	const int32 Value = CVarWaterSingleLayerReflection.GetValueOnRenderThread();
	return FMath::Clamp(Value, 0, ESingleLayerWaterReflections::MaxValue);
}

static int32 GetSingleLayerWaterRefractionDownsampleFactor()
{
	const int32 RefractionDownsampleFactor = FMath::Clamp(CVarWaterSingleLayerRefractionDownsampleFactor.GetValueOnRenderThread(), 1, 2);
	return RefractionDownsampleFactor;
}


ESingleLayerWaterVelocityOutputPass GetSingleLayerWaterVelocityOutputPass(bool bIsGameThread)
{
	const int32 VelocityOutputPass = !bIsGameThread 
		? CVarWaterSingleLayerVelocityOutputPass.GetValueOnRenderThread() 
		: CVarWaterSingleLayerVelocityOutputPass.GetValueOnGameThread();

	return static_cast<ESingleLayerWaterVelocityOutputPass>(
		FMath::Clamp(VelocityOutputPass, 
			static_cast<int32>(ESingleLayerWaterVelocityOutputPass::None), 
			static_cast<int32>(ESingleLayerWaterVelocityOutputPass::BasePass)));
}

bool CanSingleLayerWaterDepthPrepassOutputVelocity(EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel, bool bIsGameThread)
{
	return GetSingleLayerWaterVelocityOutputPass(bIsGameThread) == ESingleLayerWaterVelocityOutputPass::DepthPrepass
		&& IsSingleLayerWaterDepthPrepassEnabled(ShaderPlatform, FeatureLevel)
		&& !IsForwardShadingEnabled(ShaderPlatform);
}

static EGBufferLayout GetSingleLayerWaterGBufferLayout(EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel, bool bIsGameThread = false)
{
	const ESingleLayerWaterVelocityOutputPass VelocityOutputPass = GetSingleLayerWaterVelocityOutputPass(bIsGameThread);

	if (VelocityOutputPass == ESingleLayerWaterVelocityOutputPass::None)
	{
		return GBL_Default;
	}

	if (CanSingleLayerWaterDepthPrepassOutputVelocity(ShaderPlatform, FeatureLevel, bIsGameThread))
	{
		return GBL_Default;
	}

	// BasePass mode, DepthPrepass mode with prepass disabled (fallback), or Forward Renderer (velocity RT does not match multrisampled depth)
	return GBL_ForceVelocity;
}

static FIntPoint GetWaterReflectionDownsampleFactor()
{
	FIntPoint DownsampleFactorXY = FMath::Clamp(CVarWaterSingleLayerReflectionDownsampleFactor.GetValueOnRenderThread(), 1, 2);
	if (CVarWaterSingleLayerReflectionDownsampleCheckerboard.GetValueOnRenderThread() != 0)
	{
		DownsampleFactorXY.Y = 1;
	}
	return DownsampleFactorXY;
}

// This is to have platforms use the simple single layer water shading similar to mobile: no dynamic lights, only sun and sky, no distortion, no colored transmittance on background, no custom depth read.
bool SingleLayerWaterUsesSimpleShading(EShaderPlatform ShaderPlatform)
{
	return FDataDrivenShaderPlatformInfo::GetWaterUsesSimpleForwardShading(ShaderPlatform) && IsForwardShadingEnabled(ShaderPlatform);
}

bool ShouldRenderSingleLayerWater(TArrayView<const FViewInfo> Views)
{
	if (CVarWaterSingleLayer.GetValueOnRenderThread() > 0)
	{
		for (const FViewInfo& View : Views)
		{
			if (View.bHasSingleLayerWaterMaterial && HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::SingleLayerWaterPass]))
			{
				return true;
			}
		}
	}
	return false;
}

bool ShouldRenderSingleLayerWaterSkippedRenderEditorNotification(TArrayView<const FViewInfo> Views)
{
	if (CVarWaterSingleLayer.GetValueOnRenderThread() <= 0)
	{
		for (const FViewInfo& View : Views)
		{
			if (View.bHasSingleLayerWaterMaterial)
			{
				return true;
			}
		}
	}
	return false;
}

bool ShouldRenderSingleLayerWaterDepthPrepass(TArrayView<const FViewInfo> Views)
{
	check(Views.Num() > 0);
	const bool bPrepassEnabled = IsSingleLayerWaterDepthPrepassEnabled(Views[0].GetShaderPlatform(), Views[0].GetFeatureLevel());
	const bool bShouldRenderWater = ShouldRenderSingleLayerWater(Views);
	
	return bPrepassEnabled && bShouldRenderWater;
}

ESingleLayerWaterPrepassLocation GetSingleLayerWaterDepthPrepassLocation(bool bFullDepthPrepass, ECustomDepthPassLocation CustomDepthPassLocation)
{
	if (bFullDepthPrepass && CustomDepthPassLocation == ECustomDepthPassLocation::BeforeBasePass)
	{
		return ESingleLayerWaterPrepassLocation::BeforeBasePass;
	}
	return ESingleLayerWaterPrepassLocation::AfterBasePass;
}

namespace ScreenSpaceRayTracing
{
bool ShouldRenderScreenSpaceReflectionsWater(const FViewInfo& View)
{
	const int32 ReflectionsMethod = GetSingleLayerWaterReflectionTechnique();
	const bool bSSROverride = ReflectionsMethod == ESingleLayerWaterReflections::SSR;
	// Note: intentionally allow falling back to SSR from other reflection methods, which may be disabled by scalability (see ShouldRenderScreenSpaceReflections())
	const bool bSSRDefault = ReflectionsMethod == ESingleLayerWaterReflections::Enabled && View.FinalPostProcessSettings.ReflectionMethod != EReflectionMethod::None;

	if (!View.Family->EngineShowFlags.ScreenSpaceReflections
		|| !View.Family->EngineShowFlags.Lighting
		|| (!bSSROverride && !bSSRDefault)
		|| HasRayTracedOverlay(*View.Family)
		|| !View.State /*no view state(e.g.thumbnail rendering ? ), no HZB(no screen space reflections or occlusion culling)*/
		|| View.bIsReflectionCapture)
	{
		return false;
	}

	static const auto SSRQualityCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SSR.Quality"));
	int SSRQuality = SSRQualityCVar ? SSRQualityCVar->GetValueOnRenderThread() : 0;
	if (SSRQuality <= 0 
		|| View.FinalPostProcessSettings.ScreenSpaceReflectionIntensity < 1.0f 
		|| IsForwardShadingEnabled(View.GetShaderPlatform()))
	{
		return false;
	}

	return true;
}
}

bool ShouldRenderLumenReflectionsWater(const FViewInfo& View, bool bSkipTracingDataCheck, bool bSkipProjectCheck)
{
	// This only returns true if using the default reflections method and having Lumen enabled in the scene. It can't be forced with r.Water.SingleLayer.Reflection.
	return !View.bIsReflectionCapture  && View.Family->EngineShowFlags.Lighting
		&& GetSingleLayerWaterReflectionTechnique() == ESingleLayerWaterReflections::Enabled
		&& ShouldRenderLumenReflections(View, bSkipTracingDataCheck, bSkipProjectCheck);
}

bool ShouldUseBilinearSamplerForDepthWithoutSingleLayerWater(EPixelFormat DepthTextureFormat)
{
	const bool bHasDownsampling = GetSingleLayerWaterRefractionDownsampleFactor() > 1;
	const bool bSupportsLinearSampling = !!(GPixelFormats[DepthTextureFormat].Capabilities & EPixelFormatCapabilities::TextureSample);
	
	// Linear sampling is only required if the depth texture has been downsampled.
	return bHasDownsampling && bSupportsLinearSampling;
}

bool UseSingleLayerWaterIndirectDraw(EShaderPlatform ShaderPlatform)
{
	return IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM5)
		// Vulkan gives error with WaterTileCatergorisationMarkCS usage of atomic, and Metal does not play nice, either.
		&& !IsVulkanMobilePlatform(ShaderPlatform)
		&& FDataDrivenShaderPlatformInfo::GetSupportsWaterIndirectDraw(ShaderPlatform);
}

bool IsWaterDistanceFieldShadowEnabled_Runtime(const FStaticShaderPlatform Platform)
{
	return IsWaterDistanceFieldShadowEnabled(Platform) && CVarWaterSingleLayerDistanceFieldShadow.GetValueOnAnyThread() > 0;
}

bool IsWaterVirtualShadowMapFilteringEnabled_Runtime(const FStaticShaderPlatform Platform)
{
	return IsWaterVirtualShadowMapFilteringEnabled(Platform) && UseVirtualShadowMaps(Platform, GetMaxSupportedFeatureLevel(Platform)) && CVarWaterSingleLayerVSMFiltering.GetValueOnRenderThread() > 0;
}

bool NeedsSeparatedMainDirectionalLightTexture_Runtime(const FStaticShaderPlatform Platform)
{
	return IsWaterDistanceFieldShadowEnabled_Runtime(Platform) || IsWaterVirtualShadowMapFilteringEnabled_Runtime(Platform);
}

BEGIN_UNIFORM_BUFFER_STRUCT(FSingleLayerWaterPassUniformParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorWithoutSingleLayerWaterTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorWithoutSingleLayerWaterSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthWithoutSingleLayerWaterTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthWithoutSingleLayerWaterSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CustomDepthTexture)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint2>, CustomStencilTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, CustomDepthSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RefractionMaskTexture)
	SHADER_PARAMETER(FVector4f, DistortionParams)
	SHADER_PARAMETER(FVector2f, SceneWithoutSingleLayerWaterTextureSize)
	SHADER_PARAMETER(FVector2f, SceneWithoutSingleLayerWaterInvTextureSize)
	SHADER_PARAMETER(uint32, bMainDirectionalLightVSMFiltering)
	SHADER_PARAMETER(uint32, bSeparateMainDirLightLuminance)
	SHADER_PARAMETER_STRUCT(FLightCloudTransmittanceParameters, ForwardDirLightCloudShadow)
	SHADER_PARAMETER_STRUCT(FBlueNoiseParameters, BlueNoise)
END_UNIFORM_BUFFER_STRUCT()

// At the moment we reuse the DeferredDecals static uniform buffer slot because it is currently unused in this pass.
// When we add support for decals on SLW in the future, we might need to find another solution.
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FSingleLayerWaterPassUniformParameters, "SingleLayerWater", DeferredDecals);

BEGIN_SHADER_PARAMETER_STRUCT(FSingleLayerWaterCommonShaderParameters, )
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, ScreenSpaceReflectionsTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ScreenSpaceReflectionsSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGF)
	SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneNoWaterDepthTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneNoWaterDepthSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SeparatedMainDirLightTexture)
	SHADER_PARAMETER(FVector2f, SceneNoWaterTextureSize)
	SHADER_PARAMETER(FVector2f, SceneNoWaterInvTextureSize)
	SHADER_PARAMETER(float, UseSeparatedMainDirLightTexture)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)	// Water scene texture
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_REF(FReflectionCaptureShaderData, ReflectionCaptureData)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FReflectionUniformParameters, ReflectionsParameters)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
END_SHADER_PARAMETER_STRUCT()

class FSingleLayerWaterCompositePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSingleLayerWaterCompositePS);
	SHADER_USE_PARAMETER_STRUCT(FSingleLayerWaterCompositePS, FGlobalShader)

	class FHasBoxCaptures : SHADER_PERMUTATION_BOOL("REFLECTION_COMPOSITE_HAS_BOX_CAPTURES");
	class FHasSphereCaptures : SHADER_PERMUTATION_BOOL("REFLECTION_COMPOSITE_HAS_SPHERE_CAPTURES");
	using FPermutationDomain = TShaderPermutationDomain<FHasBoxCaptures, FHasSphereCaptures>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSingleLayerWaterCommonShaderParameters, CommonParameters)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

    static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);//Support reflection captures
	}
};

IMPLEMENT_GLOBAL_SHADER(FSingleLayerWaterCompositePS, "/Engine/Private/SingleLayerWaterComposite.usf", "SingleLayerWaterCompositePS", SF_Pixel);

class FSingleLayerWaterRefractionMaskPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSingleLayerWaterRefractionMaskPS);
	SHADER_USE_PARAMETER_STRUCT(FSingleLayerWaterRefractionMaskPS, FGlobalShader)

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterDepthTexture)
		SHADER_PARAMETER(float, DistanceCullingRangeBegin)
		SHADER_PARAMETER(float, DistanceCullingRangeEnd)
		SHADER_PARAMETER(float, FresnelCullingRangeBegin)
		SHADER_PARAMETER(float, FresnelCullingRangeEnd)
		SHADER_PARAMETER(float, DepthCullingRangeBegin)
		SHADER_PARAMETER(float, DepthCullingRangeEnd)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_R8);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSingleLayerWaterRefractionMaskPS, "/Engine/Private/SingleLayerWaterRefractionCulling.usf", "SingleLayerWaterRefractionMaskPS", SF_Pixel);

class FSingleLayerWaterRefractionCullingPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSingleLayerWaterRefractionCullingPS);
	SHADER_USE_PARAMETER_STRUCT(FSingleLayerWaterRefractionCullingPS, FGlobalShader)

	class FNaniteShadingMaskExport : SHADER_PERMUTATION_BOOL("SHADING_MASK_EXPORT");
	using FPermutationDomain = TShaderPermutationDomain<FNaniteShadingMaskExport>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterRefractionCullingTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterDepthTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const FPermutationDomain Permutation(Parameters.PermutationId);
		if (Permutation.Get<FNaniteShadingMaskExport>())
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_R32_UINT);
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FSingleLayerWaterRefractionCullingPS, "/Engine/Private/SingleLayerWaterRefractionCulling.usf", "SingleLayerWaterRefractionCullingPS", SF_Pixel);

class FWaterTileCategorisationMarkCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FWaterTileCategorisationMarkCS);
	SHADER_USE_PARAMETER_STRUCT(FWaterTileCategorisationMarkCS, FGlobalShader)

	class FUsePrepassStencil : SHADER_PERMUTATION_BOOL("USE_WATER_PRE_PASS_STENCIL");
	class FBuildFroxels : SHADER_PERMUTATION_BOOL("GENERATE_FROXELS");
	using FPermutationDomain = TShaderPermutationDomain<FUsePrepassStencil, FBuildFroxels>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)	// Water scene texture
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, WaterDepthStencilTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, WaterDepthTexture)
		SHADER_PARAMETER(FIntPoint, TiledViewRes)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, TileMaskBufferOut)
		SHADER_PARAMETER_STRUCT_INCLUDE(Froxel::FBuilderParameters, FroxelBuilder)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		// No need for froxels on non-VSM platforms
		if (PermutationVector.Get<FBuildFroxels>() 
			// only compile if on a supported platform & we have depth stencil avaliable
			&& (!DoesPlatformSupportVirtualShadowMaps(Parameters.Platform) || !PermutationVector.Get<FUsePrepassStencil>()))
		{
			return false;
		}

		return UseSingleLayerWaterIndirectDraw(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TILE_CATERGORISATION_SHADER"), 1);
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FWaterTileCategorisationMarkCS, "/Engine/Private/SingleLayerWaterComposite.usf", "WaterTileCatergorisationMarkCS", SF_Compute);

class FWaterTileClassificationBuildListsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FWaterTileClassificationBuildListsCS);
	SHADER_USE_PARAMETER_STRUCT(FWaterTileClassificationBuildListsCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(uint32, VertexCountPerInstanceIndirect)
		SHADER_PARAMETER(FIntPoint, TiledViewRes)
		SHADER_PARAMETER(FIntPoint, FullTiledViewRes)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, DrawIndirectDataUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, DispatchIndirectDataUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, DispatchClearIndirectDataUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, WaterTileListDataUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, ClearTileListDataUAV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileMaskBuffer)
	END_SHADER_PARAMETER_STRUCT()

	class FWaveOps : SHADER_PERMUTATION_BOOL("WAVE_OPS");
	class FDownsampleFactorX : SHADER_PERMUTATION_RANGE_INT("DOWNSAMPLE_FACTOR_X", 1, 2);
	class FDownsampleFactorY : SHADER_PERMUTATION_RANGE_INT("DOWNSAMPLE_FACTOR_Y", 1, 2);
	class FOutputClearTiles : SHADER_PERMUTATION_BOOL("OUTPUT_CLEAR_TILES");
	using FPermutationDomain = TShaderPermutationDomain<FWaveOps, FDownsampleFactorX, FDownsampleFactorY, FOutputClearTiles>;

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FDownsampleFactorY>() == 2)
		{
			PermutationVector.Set<FDownsampleFactorX>(2);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}

		if (PermutationVector.Get<FWaveOps>() && !RHISupportsWaveOperations(Parameters.Platform))
		{
			return false;
		}

		return UseSingleLayerWaterIndirectDraw(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return 8;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("TILE_CATERGORISATION_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FWaveOps>())
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_WaveOperations);
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FWaterTileClassificationBuildListsCS, "/Engine/Private/SingleLayerWaterComposite.usf", "WaterTileClassificationBuildListsCS", SF_Compute);

bool FWaterTileVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return UseSingleLayerWaterIndirectDraw(Parameters.Platform);
}

void FWaterTileVS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("TILE_VERTEX_SHADER"), 1.0f);
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

IMPLEMENT_GLOBAL_SHADER(FWaterTileVS, "/Engine/Private/SingleLayerWaterComposite.usf", "WaterTileVS", SF_Vertex);

class FWaterRefractionCopyPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FWaterRefractionCopyPS);
	SHADER_USE_PARAMETER_STRUCT(FWaterRefractionCopyPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorCopyDownsampleTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorCopyDownsampleSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthCopyDownsampleTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthCopyDownsampleSampler)
		SHADER_PARAMETER(FVector2f, SVPositionToSourceTextureUV)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	class FDownsampleRefraction : SHADER_PERMUTATION_BOOL("DOWNSAMPLE_REFRACTION");
	class FCopyDepth : SHADER_PERMUTATION_BOOL("COPY_DEPTH");
	class FCopyColor : SHADER_PERMUTATION_BOOL("COPY_COLOR");

	using FPermutationDomain = TShaderPermutationDomain<FDownsampleRefraction, FCopyDepth, FCopyColor>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain Permutation(Parameters.PermutationId);
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && (Permutation.Get<FCopyDepth>() || Permutation.Get<FCopyColor>());
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const FPermutationDomain Permutation(Parameters.PermutationId);
		const bool bCopyDepth = Permutation.Get<FCopyDepth>();
		const bool bCopyColor = Permutation.Get<FCopyColor>();
		const EPixelFormat DepthFormat = PF_R32_FLOAT;
		const EPixelFormat ColorFormat = PF_FloatRGBA;
		OutEnvironment.SetRenderTargetOutputFormat(0, bCopyDepth ? DepthFormat : ColorFormat);
		if (bCopyDepth && bCopyColor)
		{
			OutEnvironment.SetRenderTargetOutputFormat(1, ColorFormat);
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FWaterRefractionCopyPS, "/Engine/Private/SingleLayerWaterComposite.usf", "WaterRefractionCopyPS", SF_Pixel);

bool IsVSMTranslucentHighQualityEnabled();

FSingleLayerWaterPassUniformParameters* CreateSingleLayerWaterPassUniformParameters(FRDGBuilder& GraphBuilder, const FScene* Scene, const FViewInfo& View, const FSceneTextures& SceneTextures, FRDGTextureRef SceneDepthWithoutWater, FRDGTextureRef SceneColorWithoutWater, FRDGTextureRef RefractionMaskTexture)
{
	FSingleLayerWaterPassUniformParameters* SLWUniformParameters = GraphBuilder.AllocParameters<FSingleLayerWaterPassUniformParameters>();
	const bool bShouldUseBilinearSamplerForDepth = ShouldUseBilinearSamplerForDepthWithoutSingleLayerWater(SceneDepthWithoutWater->Desc.Format);
	const bool bCustomDepthTextureProduced = HasBeenProduced(SceneTextures.CustomDepth.Depth);
	const FIntVector DepthTextureSize = SceneDepthWithoutWater->Desc.GetSize();
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	SLWUniformParameters->SceneColorWithoutSingleLayerWaterTexture = SceneColorWithoutWater ? SceneColorWithoutWater : SystemTextures.Black;
	SLWUniformParameters->SceneColorWithoutSingleLayerWaterSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	SLWUniformParameters->SceneDepthWithoutSingleLayerWaterTexture = SceneDepthWithoutWater;
	SLWUniformParameters->SceneDepthWithoutSingleLayerWaterSampler = bShouldUseBilinearSamplerForDepth ? TStaticSamplerState<SF_Bilinear>::GetRHI() : TStaticSamplerState<SF_Point>::GetRHI();
	SLWUniformParameters->CustomDepthTexture = bCustomDepthTextureProduced ? SceneTextures.CustomDepth.Depth : SystemTextures.DepthDummy;
	SLWUniformParameters->CustomStencilTexture = bCustomDepthTextureProduced ? SceneTextures.CustomDepth.Stencil : SystemTextures.StencilDummySRV;
	SLWUniformParameters->CustomDepthSampler = TStaticSamplerState<SF_Point>::GetRHI();
	SLWUniformParameters->RefractionMaskTexture = RefractionMaskTexture ? RefractionMaskTexture : SystemTextures.White;
	SetupDistortionParams(SLWUniformParameters->DistortionParams, View);
	SLWUniformParameters->SceneWithoutSingleLayerWaterTextureSize = FVector2f(DepthTextureSize.X, DepthTextureSize.Y);
	SLWUniformParameters->SceneWithoutSingleLayerWaterInvTextureSize = FVector2f(1.0f / DepthTextureSize.X, 1.0f / DepthTextureSize.Y);
	SLWUniformParameters->bMainDirectionalLightVSMFiltering = IsWaterVirtualShadowMapFilteringEnabled_Runtime(View.GetShaderPlatform());
	SLWUniformParameters->bSeparateMainDirLightLuminance = NeedsSeparatedMainDirectionalLightTexture_Runtime(View.GetShaderPlatform());
	// Only use blue noise resources if VSM quality is set to high
	if (IsVSMTranslucentHighQualityEnabled())
	{
		SLWUniformParameters->BlueNoise = GetBlueNoiseParameters();
	}
	else
	{
		SLWUniformParameters->BlueNoise = GetBlueNoiseDummyParameters();
	}

	const FLightSceneProxy* SelectedForwardDirectionalLightProxy = View.ForwardLightingResources.SelectedForwardDirectionalLightProxy;
	SetupLightCloudTransmittanceParameters(GraphBuilder, Scene, View, SelectedForwardDirectionalLightProxy ? SelectedForwardDirectionalLightProxy->GetLightSceneInfo() : nullptr, SLWUniformParameters->ForwardDirLightCloudShadow);

	return SLWUniformParameters;
}

BEGIN_SHADER_PARAMETER_STRUCT(FSingleLayerWaterDepthPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSingleLayerWaterPassUniformParameters, SingleLayerWater)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

static FSingleLayerWaterDepthPassParameters* GetSingleLayerWaterDepthPassParameters(FRDGBuilder& GraphBuilder, const FScene* Scene, const FViewInfo& View, const FSceneTextures& SceneTextures, FRDGTextureRef SceneDepthWithoutWater, 
	FRDGTextureRef DepthTexture, FRDGTextureRef VelocityTexture = nullptr, FRDGTextureRef MotionVectorWorldOffsetTexture = nullptr)
{
	FSingleLayerWaterDepthPassParameters* PassParameters = GraphBuilder.AllocParameters<FSingleLayerWaterDepthPassParameters>();
	PassParameters->View = View.GetShaderParameters();
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);

	if (VelocityTexture)
	{
		// If the depth pass is the first to write to the texture, clear it instead of loading.
		const ERenderTargetLoadAction VelocityLoadAction = HasBeenProduced(VelocityTexture)
			? ERenderTargetLoadAction::ELoad
			: ERenderTargetLoadAction::EClear;

		PassParameters->RenderTargets[0] = FRenderTargetBinding(VelocityTexture, VelocityLoadAction);
		
		if (MotionVectorWorldOffsetTexture)
		{
			PassParameters->RenderTargets[1] = FRenderTargetBinding(MotionVectorWorldOffsetTexture, ERenderTargetLoadAction::ELoad);
		}
	}

	FSingleLayerWaterPassUniformParameters* SLWUniformParameters = CreateSingleLayerWaterPassUniformParameters(GraphBuilder, Scene, View, SceneTextures, SceneDepthWithoutWater, nullptr, nullptr);
	PassParameters->SingleLayerWater = GraphBuilder.CreateUniformBuffer(SLWUniformParameters);

	return PassParameters;
}

/**
 * Build lists of 8x8 tiles used by water pixels
 * Mark and build list steps are separated in order to build a more coherent list (z-ordered over a larger region), which is important for the performance of future passes like ray traced Lumen reflections
 */
static FSingleLayerWaterTileClassification ClassifyTiles(FRDGBuilder& GraphBuilder, const FViewInfo &View, const FSceneTextures& SceneTextures, FRDGTextureRef DepthPrepassTexture, const Froxel::FViewData* OutFroxelViewData, EReflectionsMethod ReflectionsMethod)
{
	FSingleLayerWaterTileClassification Result;
	const bool bRunTiled = UseSingleLayerWaterIndirectDraw(View.GetShaderPlatform()) && CVarWaterSingleLayerTiledComposite.GetValueOnRenderThread();
	if (bRunTiled)
	{
		const bool bUseLumenReflections = ReflectionsMethod == EReflectionsMethod::Lumen
			&& View.Family->EngineShowFlags.Lighting
			&& GetSingleLayerWaterReflectionTechnique() != ESingleLayerWaterReflections::Disabled;

		FIntPoint ViewRes(View.ViewRect.Width(), View.ViewRect.Height());
		Result.TiledViewRes = FIntPoint::DivideAndRoundUp(ViewRes, SLW_TILE_SIZE_XY);

		const FIntPoint DownsampleFactor = bUseLumenReflections ? GetWaterReflectionDownsampleFactor() : FIntPoint(1, 1);
		const FIntPoint DownsampledViewResInTiles = FIntPoint::DivideAndRoundUp(ViewRes, DownsampleFactor * SLW_TILE_SIZE_XY);

		const bool bNeedDownsample = DownsampleFactor.X > 1;
		const bool bNeedClearTiles = bUseLumenReflections && (CVarWaterSingleLayerReflectionDenoising.GetValueOnRenderThread() != 0 || bNeedDownsample);

		Result.TiledReflection.DrawIndirectParametersBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndirectParameters>(), TEXT("SLW.WaterIndirectDrawParameters"));
		Result.TiledReflection.DispatchIndirectParametersBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("SLW.WaterIndirectDispatchParameters"));
		Result.TiledReflection.DispatchClearIndirectParametersBuffer = bNeedClearTiles ? GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("SLW.ClearIndirectDispatchParameters")) : nullptr;
		Result.TiledReflection.DispatchDownsampledIndirectParametersBuffer = bNeedDownsample ? GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("SLW.DownsampledIndirectDispatchParameters")) : Result.TiledReflection.DispatchIndirectParametersBuffer;
		Result.TiledReflection.DispatchDownsampledClearIndirectParametersBuffer = (bNeedDownsample && bNeedClearTiles) ? GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("SLW.DownsampledClearIndirectDispatchParameters")) : Result.TiledReflection.DispatchClearIndirectParametersBuffer;

		FRDGBufferRef TileListDataBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Result.TiledViewRes.X * Result.TiledViewRes.Y), TEXT("SLW.TileListDataBuffer"));
		Result.TiledReflection.TileListDataBufferSRV = GraphBuilder.CreateSRV(TileListDataBuffer, PF_R32_UINT);

		FRDGBufferRef ClearTileListDataBuffer = nullptr;
		Result.TiledReflection.ClearTileListDataBufferSRV = nullptr;
		if (bNeedClearTiles)
		{
			ClearTileListDataBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Result.TiledViewRes.X * Result.TiledViewRes.Y), TEXT("SLW.ClearTileListDataBuffer"));
			Result.TiledReflection.ClearTileListDataBufferSRV = GraphBuilder.CreateSRV(ClearTileListDataBuffer, PF_R32_UINT);
		}

		FRDGBufferRef DownsampledTileListDataBuffer = nullptr;
		Result.TiledReflection.DownsampledTileListDataBufferSRV = Result.TiledReflection.TileListDataBufferSRV;
		if (bNeedDownsample)
		{
			DownsampledTileListDataBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), DownsampledViewResInTiles.X * DownsampledViewResInTiles.Y), TEXT("SLW.DownsampledTileListDataBuffer"));
			Result.TiledReflection.DownsampledTileListDataBufferSRV = GraphBuilder.CreateSRV(DownsampledTileListDataBuffer, PF_R32_UINT);
		}

		FRDGBufferRef DownsampledClearTileListDataBuffer = nullptr;
		Result.TiledReflection.DownsampledClearTileListDataBufferSRV = Result.TiledReflection.ClearTileListDataBufferSRV;
		if (bNeedDownsample && bNeedClearTiles)
		{
			DownsampledClearTileListDataBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), DownsampledViewResInTiles.X * DownsampledViewResInTiles.Y), TEXT("SLW.DownsampledClearTileListDataBuffer"));
			Result.TiledReflection.DownsampledClearTileListDataBufferSRV = GraphBuilder.CreateSRV(DownsampledClearTileListDataBuffer, PF_R32_UINT);
		}

		FRDGBufferUAVRef DrawIndirectParametersBufferUAV = GraphBuilder.CreateUAV(Result.TiledReflection.DrawIndirectParametersBuffer);
		FRDGBufferUAVRef DispatchIndirectParametersBufferUAV = GraphBuilder.CreateUAV(Result.TiledReflection.DispatchIndirectParametersBuffer);
		FRDGBufferUAVRef DispatchClearIndirectParametersBufferUAV = bNeedClearTiles ? GraphBuilder.CreateUAV(Result.TiledReflection.DispatchClearIndirectParametersBuffer) : nullptr;
		FRDGBufferUAVRef DispatchDownsampledIndirectParametersBufferUAV = bNeedDownsample ? GraphBuilder.CreateUAV(Result.TiledReflection.DispatchDownsampledIndirectParametersBuffer) : nullptr;
		FRDGBufferUAVRef DispatchDownsampledClearIndirectParametersBufferUAV = bNeedDownsample && bNeedClearTiles ? GraphBuilder.CreateUAV(Result.TiledReflection.DispatchDownsampledClearIndirectParametersBuffer) : nullptr;

		// Allocate buffer with 1 bit / tile
		Result.TileMaskBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FMath::DivideAndRoundUp(Result.TiledViewRes.X * Result.TiledViewRes.Y, 32)), TEXT("SLW.TileMaskBuffer"));
		FRDGBufferUAVRef TileMaskBufferUAV = GraphBuilder.CreateUAV(Result.TileMaskBuffer);
		AddClearUAVPass(GraphBuilder, TileMaskBufferUAV, 0);

		// Clear DrawIndirectParametersBuffer
		AddClearUAVPass(GraphBuilder, DrawIndirectParametersBufferUAV, 0);
		AddClearUAVPass(GraphBuilder, DispatchIndirectParametersBufferUAV, 0);
		if (DispatchClearIndirectParametersBufferUAV)
		{
			AddClearUAVPass(GraphBuilder, DispatchClearIndirectParametersBufferUAV, 0);
		}
		if (DispatchDownsampledIndirectParametersBufferUAV)
		{
			AddClearUAVPass(GraphBuilder, DispatchDownsampledIndirectParametersBufferUAV, 0);
		}
		if (DispatchDownsampledClearIndirectParametersBufferUAV)
		{
			AddClearUAVPass(GraphBuilder, DispatchDownsampledClearIndirectParametersBufferUAV, 0);
		}

		// Can't produce froxels unless we have depth data
		bool bProduceFroxelData = OutFroxelViewData != nullptr && DepthPrepassTexture != nullptr;

		const bool bWaveOps = CVarWaterSingleLayerWaveOps.GetValueOnRenderThread()
				&& GRHISupportsWaveOperations
				&& RHISupportsWaveOperations(View.GetShaderPlatform());

		// Mark used tiles based on SHADING_MODEL_ID
		{
			bool bUsePrepassStencil = DepthPrepassTexture != nullptr;

			FWaterTileCategorisationMarkCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FWaterTileCategorisationMarkCS::FUsePrepassStencil>(bUsePrepassStencil);
			PermutationVector.Set<FWaterTileCategorisationMarkCS::FBuildFroxels>(bProduceFroxelData);

			TShaderMapRef<FWaterTileCategorisationMarkCS> ComputeShader(View.ShaderMap, PermutationVector);

			FWaterTileCategorisationMarkCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterTileCategorisationMarkCS::FParameters>();

			PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
			PassParameters->View = View.GetShaderParameters();
			PassParameters->Substrate = !bUsePrepassStencil ? Substrate::BindSubstrateGlobalUniformParameters(View) : nullptr;
			PassParameters->TiledViewRes = Result.TiledViewRes;
			PassParameters->WaterDepthStencilTexture = DepthPrepassTexture ? GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(DepthPrepassTexture, PF_X24_G8)) : nullptr;
			PassParameters->WaterDepthTexture = DepthPrepassTexture ? DepthPrepassTexture : nullptr;
			PassParameters->TileMaskBufferOut = TileMaskBufferUAV;
			if (bProduceFroxelData)
			{
				PassParameters->FroxelBuilder = OutFroxelViewData->GetBuilderParameters(GraphBuilder); //-V522
			}
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SLW::TileCategorisationMarkTiles"),
				ComputeShader,
				PassParameters,
				FIntVector(Result.TiledViewRes.X, Result.TiledViewRes.Y, 1)
			);
		}

		// Build compacted and coherent light tiles from bit-marked tiles
		{
			FWaterTileClassificationBuildListsCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FWaterTileClassificationBuildListsCS::FWaveOps>(bWaveOps);
			PermutationVector.Set<FWaterTileClassificationBuildListsCS::FDownsampleFactorX>(1);
			PermutationVector.Set<FWaterTileClassificationBuildListsCS::FDownsampleFactorY>(1);
			PermutationVector.Set<FWaterTileClassificationBuildListsCS::FOutputClearTiles>(bNeedClearTiles);
			TShaderMapRef<FWaterTileClassificationBuildListsCS> ComputeShader(View.ShaderMap, PermutationVector);

			FWaterTileClassificationBuildListsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterTileClassificationBuildListsCS::FParameters>();

			PassParameters->View = View.GetShaderParameters();
			PassParameters->TiledViewRes = Result.TiledViewRes;
			PassParameters->FullTiledViewRes = Result.TiledViewRes;
			PassParameters->VertexCountPerInstanceIndirect = GRHISupportsRectTopology ? 3 : 6;
			PassParameters->DrawIndirectDataUAV = DrawIndirectParametersBufferUAV;
			PassParameters->DispatchIndirectDataUAV = DispatchIndirectParametersBufferUAV;
			PassParameters->DispatchClearIndirectDataUAV = DispatchClearIndirectParametersBufferUAV;
			PassParameters->WaterTileListDataUAV = GraphBuilder.CreateUAV(TileListDataBuffer, PF_R32_UINT);
			PassParameters->ClearTileListDataUAV = bNeedClearTiles ? GraphBuilder.CreateUAV(ClearTileListDataBuffer, PF_R32_UINT) : nullptr;
			PassParameters->TileMaskBuffer = GraphBuilder.CreateSRV(Result.TileMaskBuffer);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SLW::TileCategorisationBuildList"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(Result.TiledViewRes, FWaterTileClassificationBuildListsCS::GetGroupSize())
			);
		}

		if (bNeedDownsample)
		{
			FWaterTileClassificationBuildListsCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FWaterTileClassificationBuildListsCS::FWaveOps>(bWaveOps);
			PermutationVector.Set<FWaterTileClassificationBuildListsCS::FDownsampleFactorX>(DownsampleFactor.X);
			PermutationVector.Set<FWaterTileClassificationBuildListsCS::FDownsampleFactorY>(DownsampleFactor.Y);
			PermutationVector.Set<FWaterTileClassificationBuildListsCS::FOutputClearTiles>(bNeedClearTiles);

			TShaderMapRef<FWaterTileClassificationBuildListsCS> ComputeShader(View.ShaderMap, PermutationVector);

			FWaterTileClassificationBuildListsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterTileClassificationBuildListsCS::FParameters>();

			PassParameters->View = View.GetShaderParameters();
			PassParameters->TiledViewRes = DownsampledViewResInTiles;
			PassParameters->FullTiledViewRes = Result.TiledViewRes;
			PassParameters->VertexCountPerInstanceIndirect = GRHISupportsRectTopology ? 3 : 6;
			PassParameters->DrawIndirectDataUAV = nullptr;
			PassParameters->DispatchIndirectDataUAV = DispatchDownsampledIndirectParametersBufferUAV;
			PassParameters->DispatchClearIndirectDataUAV = DispatchDownsampledClearIndirectParametersBufferUAV;
			PassParameters->WaterTileListDataUAV = GraphBuilder.CreateUAV(DownsampledTileListDataBuffer, PF_R32_UINT);
			PassParameters->ClearTileListDataUAV = bNeedClearTiles ? GraphBuilder.CreateUAV(DownsampledClearTileListDataBuffer, PF_R32_UINT) : nullptr;;
			PassParameters->TileMaskBuffer = GraphBuilder.CreateSRV(Result.TileMaskBuffer);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SLW::TileCategorisationBuildList DownsampleFactor=%dx%d", DownsampleFactor.X, DownsampleFactor.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(DownsampledViewResInTiles, FWaterTileClassificationBuildListsCS::GetGroupSize())
			);
		}
	}
	return Result;
}

BEGIN_SHADER_PARAMETER_STRUCT(FWaterRefractionCopyParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FWaterTileVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FWaterRefractionCopyPS::FParameters, PS)
	RDG_BUFFER_ACCESS(IndirectDrawParameter, ERHIAccess::IndirectArgs)
END_SHADER_PARAMETER_STRUCT()

static void AddCopySceneWithoutWaterPass_Internal(
	FRDGBuilder& GraphBuilder,
	const FSceneViewFamily& ViewFamily,
	TArrayView<const FViewInfo> Views,
	FRDGTextureRef DstDepthTexture,
	FRDGTextureRef SrcDepthTexture,
	FRDGTextureRef DstColorTexture,
	FRDGTextureRef SrcColorTexture,
	int32 RefractionDownsampleFactor,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrepassResult
)
{
	const FIntPoint SceneTextureExtent = Views[0].GetSceneTextures().Config.Extent;
	const bool bCopyDepth = DstDepthTexture && SrcDepthTexture;
	const bool bCopyColor = DstColorTexture && SrcColorTexture;
	check(bCopyDepth || bCopyColor);
	const bool bDoTiledCopy = !bCopyDepth 
		&& RefractionDownsampleFactor == 1 
		&& SingleLayerWaterPrepassResult 
		&& !SingleLayerWaterPrepassResult->ViewTileClassification.IsEmpty()
		&& HasBeenProduced(SingleLayerWaterPrepassResult->RefractionMaskTexture)
		&& CVarWaterSingleLayerTiledSceneColorCopy.GetValueOnRenderThread() != 0;

	ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ENoAction;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (!View.ShouldRenderView())
		{
			continue;
		}

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		// Under ISR the secondary view is in Views but skipped by ShouldRenderView() above.
		// Pick it up here so the copy runs once per eye with that eye's own viewport, view UB
		// and tile list. The tile VS computes NDC using View.ViewSizeAndInvSize (single eye),
		// so a single draw covering the union of both eye rects would stretch each tile 2x.
		TArray<TPair<int32, const FViewInfo*>, TInlineAllocator<2>> ViewsToDraw;
		ViewsToDraw.Emplace(ViewIndex, &View);
		if (View.IsInstancedStereo())
		{
			if (const FViewInfo* InstancedView = View.GetInstancedView())
			{
				const int32 InstancedViewIndex = Views.IndexOfByPredicate(
					[InstancedView](const FViewInfo& V) { return &V == InstancedView; });
				if (ensure(InstancedViewIndex != INDEX_NONE))
				{
					ViewsToDraw.Emplace(InstancedViewIndex, InstancedView);
				}
			}
		}

		for (const TPair<int32, const FViewInfo*>& DrawViewEntry : ViewsToDraw)
		{
			const int32 DrawViewIndex = DrawViewEntry.Key;
			const FViewInfo& DrawView = *DrawViewEntry.Value;

			FWaterRefractionCopyParameters* PassParameters = GraphBuilder.AllocParameters<FWaterRefractionCopyParameters>();
			PassParameters->PS.SceneColorCopyDownsampleTexture = SrcColorTexture;
			PassParameters->PS.SceneColorCopyDownsampleSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->PS.SceneDepthCopyDownsampleTexture = SrcDepthTexture;
			PassParameters->PS.SceneDepthCopyDownsampleSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->PS.SVPositionToSourceTextureUV = FVector2f(RefractionDownsampleFactor / float(SceneTextureExtent.X), RefractionDownsampleFactor / float(SceneTextureExtent.Y));
			PassParameters->PS.RenderTargets[0] = FRenderTargetBinding(bCopyDepth ? DstDepthTexture : DstColorTexture, LoadAction);
			if (bCopyDepth && bCopyColor)
			{
				PassParameters->PS.RenderTargets[1] = FRenderTargetBinding(DstColorTexture, LoadAction);
			}

			if (!DrawView.Family->bMultiGPUForkAndJoin)
			{
				LoadAction = ERenderTargetLoadAction::ELoad;
			}

			FWaterRefractionCopyPS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FWaterRefractionCopyPS::FDownsampleRefraction>(RefractionDownsampleFactor > 1);
			PermutationVector.Set<FWaterRefractionCopyPS::FCopyDepth>(bCopyDepth);
			PermutationVector.Set<FWaterRefractionCopyPS::FCopyColor>(bCopyColor);
			auto PixelShader = DrawView.ShaderMap->GetShader<FWaterRefractionCopyPS>(PermutationVector);

			const FIntRect RefractionViewRect = FIntRect(FIntPoint::DivideAndRoundDown(DrawView.ViewRect.Min, RefractionDownsampleFactor), FIntPoint::DivideAndRoundDown(DrawView.ViewRect.Max, RefractionDownsampleFactor));
			if (bDoTiledCopy)
			{
				const FTiledReflection* TiledReflection = &SingleLayerWaterPrepassResult->ViewTileClassification[DrawViewIndex].TiledReflection;
				SingleLayerWaterAddTiledFullscreenPass(GraphBuilder, DrawView.ShaderMap, RDG_EVENT_NAME("SLW::Copy"), PixelShader, PassParameters, DrawView.ViewUniformBuffer, RefractionViewRect, TiledReflection);
			}
			else
			{
				FPixelShaderUtils::AddFullscreenPass(GraphBuilder, DrawView.ShaderMap, RDG_EVENT_NAME("SLW::Copy"), PixelShader, &PassParameters->PS, RefractionViewRect);
			}
		}
	}
}

FSingleLayerWaterPrePassResult* FDeferredShadingSceneRenderer::RenderSingleLayerWaterDepthPrepass(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, const FSceneTextures& SceneTextures, ESingleLayerWaterPrepassLocation Location, TConstArrayView<Nanite::FRasterResults> NaniteRasterResults)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, Water);
	SCOPED_NAMED_EVENT(FDeferredShadingSceneRenderer_RenderSingleLayerWaterDepthPrepass, FColor::Emerald);
	SCOPE_CYCLE_COUNTER(STAT_WaterPassDrawTime);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, SingleLayerWaterDepthPrepass, "SingleLayerWaterDepthPrepass");

#if !UE_BUILD_SHIPPING
	{
		static bool bValidated = false;
		if (!bValidated)
		{
			const int32 LegacyValue = CVarWaterSingleLayerForceVelocity.GetValueOnRenderThread();
			ensureMsgf(LegacyValue == -1,
				TEXT("r.Water.SingleLayer.ForceVelocity is deprecated and ignored. ")
				TEXT("Use r.Water.SingleLayer.VelocityOutputPass instead (0=Disabled, 1=DepthPrepass, 2=BasePass)."));

			bValidated = true;
		}
	}
#endif

	FSingleLayerWaterPrePassResult* Result = GraphBuilder.AllocObject<FSingleLayerWaterPrePassResult>();
	Result->ViewTileClassification.SetNum(InViews.Num());

	FRDGTextureMSAA &OutDepthPrepassTexture = Result->DepthPrepassTexture;
	// Create an identical copy of the main depth buffer
	{
		const FRDGTextureDesc& DepthPrepassTextureDesc = SceneTextures.Depth.Target->Desc;
		OutDepthPrepassTexture = GraphBuilder.CreateTexture(DepthPrepassTextureDesc, TEXT("SLW.DepthPrepassOutput"));
		if (DepthPrepassTextureDesc.NumSamples > 1)
		{
			FRDGTextureDesc DepthPrepassResolveTextureDesc = DepthPrepassTextureDesc;
			DepthPrepassResolveTextureDesc.NumSamples = 1;
			OutDepthPrepassTexture.Resolve = GraphBuilder.CreateTexture(DepthPrepassResolveTextureDesc, TEXT("SLW.DepthPrepassOutputResolve"));
		}

		//AddCopyTexturePass(GraphBuilder, SceneTextures.Depth.Target, OutDepthPrepassTexture.Target);
		//AddClearDepthStencilPass(GraphBuilder, OutDepthPrepassTexture.Target, false, 0.0f, true, 0);

		// Copy main depth buffer content to our prepass depth buffer and clear stencil to 0
		// TODO: replace with AddCopyTexturePass() and AddClearDepthStencilPass() once CopyTexture() supports depth buffer copies on all platforms.

		const bool bOptimizedClear = CVarSingleLayerWaterPassOptimizedClear.GetValueOnRenderThread() == 1;
		if (false)//bOptimizedClear && GRHISupportsDepthUAV && GRHISupportsExplicitHTile)
		{
			// TODO: Implement optimized copy path
		}
		else
		{
			FCopyDepthPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyDepthPS::FParameters>();
			if (DepthPrepassTextureDesc.NumSamples > 1)
			{
				PassParameters->DepthTextureMS = SceneTextures.Depth.Target;
			}
			else
			{
				PassParameters->DepthTexture = SceneTextures.Depth.Target;
			}
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(OutDepthPrepassTexture.Target, ERenderTargetLoadAction::ENoAction, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);

			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);

			FCopyDepthPS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCopyDepthPS::FMSAASampleCount>(DepthPrepassTextureDesc.NumSamples);
			TShaderMapRef<FCopyDepthPS> PixelShader(ShaderMap, PermutationVector);

			FIntRect Viewport(0, 0, DepthPrepassTextureDesc.Extent.X, DepthPrepassTextureDesc.Extent.Y);
			if (bOptimizedClear && InViews.Num() == 1)
			{
				Viewport = InViews[0].ViewRect;
			}

			// Set depth test to always pass and stencil test to replace all pixels with zero, essentially also clearing stencil while doing the depth copy.
			FRHIDepthStencilState* DepthStencilState = TStaticDepthStencilState<
				true, CF_Always,								// depth
				true, CF_Always, SO_Zero, SO_Zero, SO_Zero,		// frontface stencil
				true, CF_Always, SO_Zero, SO_Zero, SO_Zero		// backface stencil
			>::GetRHI();

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				ShaderMap,
				RDG_EVENT_NAME("SLW::DepthBufferCopy"),
				PixelShader,
				PassParameters,
				Viewport,
				nullptr, /*BlendState*/
				nullptr, /*RasterizerState*/
				DepthStencilState);

			// The above copy technique loses HTILE data during the copy, so until AddCopyTexturePass() supports depth buffer copies on all platforms,
			// this is the best we can do.
			AddResummarizeHTilePass(GraphBuilder, OutDepthPrepassTexture.Target);
		}
	}

	// Create SceneDepthWithoutWater texture
	{
		const int32 RefractionDownsampleFactor = GetSingleLayerWaterRefractionDownsampleFactor();
		const FIntPoint RefractionResolution = FIntPoint::DivideAndRoundDown(SceneTextures.Config.Extent, RefractionDownsampleFactor);
		Result->SceneDepthWithoutWater = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(RefractionResolution, PF_R32_FLOAT, FClearValueBinding::DepthFar, TexCreate_ShaderResource | TexCreate_RenderTargetable), TEXT("SLW.SceneDepthWithout"));

		AddCopySceneWithoutWaterPass_Internal(GraphBuilder, ViewFamily, Views, Result->SceneDepthWithoutWater, SceneTextures.Depth.Resolve, nullptr, nullptr, RefractionDownsampleFactor, nullptr);
	}

	const bool bRenderInParallel = GRHICommandList.UseParallelAlgorithms() && CVarParallelSingleLayerWaterPass.GetValueOnRenderThread() == 1;

	// Check if any view has SLW materials using motion vector world offset
	bool bAnySLWUsesMVWO = false;
	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
	{
		bAnySLWUsesMVWO |= InViews[ViewIndex].bSLWUsesMotionVectorWorldOffset;
	}

	// Conditionally create velocity and MVWO textures when needed.
	FRDGTextureRef VelocityPrepassTexture = nullptr;
	FRDGTextureRef MotionVectorWorldOffsetPrepassTexture = nullptr;
	if (CanSingleLayerWaterDepthPrepassOutputVelocity(ShaderPlatform, FeatureLevel))
	{
		const bool bPlatformSupportsMVWO = SupportsPixelShaderMotionVectorWorldOffset(ShaderPlatform);
		if (bAnySLWUsesMVWO && bPlatformSupportsMVWO)
		{
			const FRDGTextureDesc VelocityDesc = FVelocityRendering::GetRenderTargetDesc(ShaderPlatform, SceneTextures.Config.Extent);
			VelocityPrepassTexture = GraphBuilder.CreateTexture(VelocityDesc, TEXT("SLW.VelocityPrepassTexture"));
			MotionVectorWorldOffsetPrepassTexture = GraphBuilder.CreateTexture(VelocityDesc, TEXT("SLW.MotionVectorWorldOffsetPrepassTexture"));
			AddClearRenderTargetPass(GraphBuilder, VelocityPrepassTexture);
			AddClearRenderTargetPass(GraphBuilder, MotionVectorWorldOffsetPrepassTexture);
		}
		else if (bPlatformSupportsMVWO)
		{
			VelocityPrepassTexture = SceneTextures.Velocity;
			// No MVWO materials, still need to allocate as velocity shaders still declare SV_Target1 on this platform.
			const FRDGTextureDesc VelocityDesc = FVelocityRendering::GetRenderTargetDesc(ShaderPlatform, SceneTextures.Config.Extent);
			MotionVectorWorldOffsetPrepassTexture = GraphBuilder.CreateTexture(VelocityDesc, TEXT("SLW.MotionVectorWorldOffsetDummy"));
		}
		else
		{
			// No MVWO: write velocity directly to scene velocity textures
			VelocityPrepassTexture = SceneTextures.Velocity;
		}
	}

	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
	{
		FViewInfo& View = InViews[ViewIndex];

		auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::SingleLayerWaterDepthPrepass];

		if (!Pass || !View.ShouldRenderView())
		{
			continue;
		}

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1, "View%d", ViewIndex);
		View.BeginRenderView();

		FSingleLayerWaterDepthPassParameters* PassParameters = GetSingleLayerWaterDepthPassParameters(GraphBuilder, Scene, View, SceneTextures, Result->SceneDepthWithoutWater, OutDepthPrepassTexture.Target, VelocityPrepassTexture, MotionVectorWorldOffsetPrepassTexture);

		Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

		if (bRenderInParallel)
		{
			GraphBuilder.AddDispatchPass(
				RDG_EVENT_NAME("SingleLayerWaterDepthPrepassParallel"),
				PassParameters,
				ERDGPassFlags::Raster,
				[Pass, PassParameters](FRDGDispatchPassBuilder& DispatchPassBuilder)
			{
				Pass->Dispatch(DispatchPassBuilder, &PassParameters->InstanceCullingDrawParams);
			});
		}
		else
		{
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("SingleLayerWaterDepthPrepass"),
				PassParameters,
				ERDGPassFlags::Raster,
				[&View, Pass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				SetStereoViewport(RHICmdList, View, 1.0f);
				Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
			});
		}
	}

	AddResolveSceneDepthPass(GraphBuilder, InViews, OutDepthPrepassTexture);

	if (bAnySLWUsesMVWO && MotionVectorWorldOffsetPrepassTexture)
	{
		// Save them into intermediate texture so we can resolve later after SLW pass.
		Result->VelocityPrepassTexture = VelocityPrepassTexture;
		Result->MotionVectorWorldOffsetPrepassTexture = MotionVectorWorldOffsetPrepassTexture;
	}

	// Run classification pass.
	if (UseSingleLayerWaterIndirectDraw(ShaderPlatform) && (CVarWaterSingleLayerTiledComposite.GetValueOnRenderThread() || CVarWaterSingleLayerTiledSceneColorCopy.GetValueOnRenderThread()))
	{
		Result->Froxels = Froxel::FRenderer(DoesVSMWantFroxels(ShaderPlatform), GraphBuilder, Views);

		for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
		{
			FViewInfo& View = InViews[ViewIndex];
			const EReflectionsMethod ReflectionsMethod = GetViewPipelineState(View).ReflectionsMethodWater;
			Result->ViewTileClassification[ViewIndex] = ClassifyTiles(GraphBuilder, View, SceneTextures, OutDepthPrepassTexture.Resolve, Result->Froxels.GetView(ViewIndex), ReflectionsMethod);
		}
	}

	Result->RefractionMaskTexture = nullptr;
	const float RefractionDistanceCulling = CVarSingleLayerWaterRefractionDistanceCulling.GetValueOnRenderThread();
	const float RefractionFresnelCulling = CVarSingleLayerWaterRefractionFresnelCulling.GetValueOnRenderThread();
	const float RefractionDepthCulling = CVarSingleLayerWaterRefractionDepthCulling.GetValueOnRenderThread();
	const bool bDoRefractionCulling = CVarSingleLayerWaterRefractionCulling.GetValueOnRenderThread() != 0
		&& (RefractionDistanceCulling > 0.0f || RefractionFresnelCulling > 0.0f || RefractionDepthCulling > 0.0f)
		&& Location == ESingleLayerWaterPrepassLocation::BeforeBasePass
		&& GetRendererOutput() != ERendererOutput::DepthPrepassOnly
		&& !IsForwardShadingEnabled(ShaderPlatform);
	
	if (bDoRefractionCulling)
	{
		Result->RefractionMaskTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PF_R8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource), TEXT("SLW.RefractionMask"));

		AddClearRenderTargetPass(GraphBuilder, Result->RefractionMaskTexture);

		// Create refraction mask texture
		for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
		{
			FViewInfo& View = InViews[ViewIndex];

			FSingleLayerWaterRefractionMaskPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSingleLayerWaterRefractionMaskPS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->SceneDepthTexture = Result->SceneDepthWithoutWater;
			PassParameters->WaterDepthTexture = Result->DepthPrepassTexture.Resolve;
			PassParameters->DistanceCullingRangeEnd = RefractionDistanceCulling;
			PassParameters->DistanceCullingRangeBegin = PassParameters->DistanceCullingRangeEnd - FMath::Max(CVarSingleLayerWaterRefractionDistanceCullingFadeRange.GetValueOnRenderThread(), 0.01f);
			PassParameters->FresnelCullingRangeEnd = RefractionFresnelCulling;
			PassParameters->FresnelCullingRangeBegin = PassParameters->FresnelCullingRangeEnd + FMath::Max(CVarSingleLayerWaterRefractionFresnelCullingFadeRange.GetValueOnRenderThread(), 0.01f);
			PassParameters->DepthCullingRangeEnd = RefractionDepthCulling;
			PassParameters->DepthCullingRangeBegin = PassParameters->DepthCullingRangeEnd - FMath::Max(CVarSingleLayerWaterRefractionDepthCullingFadeRange.GetValueOnRenderThread(), 0.01f);
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Result->DepthPrepassTexture.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(Result->RefractionMaskTexture, ERenderTargetLoadAction::ELoad);

			TShaderMapRef<FSingleLayerWaterRefractionMaskPS> PixelShader(View.ShaderMap);

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				View.ShaderMap,
				RDG_EVENT_NAME("SLW::RefractionMask (View: %i)", ViewIndex),
				PixelShader,
				PassParameters,
				View.ViewRect,
				nullptr /*BlendState*/,
				nullptr /*RasterizerState*/,
				TStaticDepthStencilState<false, CF_Always, true, CF_Equal>::GetRHI(),
				STENCIL_SANDBOX_MASK);
		}

		// Write near plane depth for all water pixels where we want to skip all shading for the scene behind them
		for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
		{
			FViewInfo& View = InViews[ViewIndex];

			const bool bNaniteShadingMaskExport = NaniteRasterResults.IsValidIndex(ViewIndex) && HasBeenProduced(NaniteRasterResults[ViewIndex].ShadingMask);

			FSingleLayerWaterRefractionCullingPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSingleLayerWaterRefractionCullingPS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->WaterRefractionCullingTexture = Result->RefractionMaskTexture;
			PassParameters->WaterDepthTexture = Result->DepthPrepassTexture.Resolve;
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilRead);
			if (bNaniteShadingMaskExport)
			{
				PassParameters->RenderTargets[0] = FRenderTargetBinding(NaniteRasterResults[ViewIndex].ShadingMask, ERenderTargetLoadAction::ELoad);
			}

			FSingleLayerWaterRefractionCullingPS::FPermutationDomain PermutationDomain;
			PermutationDomain.Set<FSingleLayerWaterRefractionCullingPS::FNaniteShadingMaskExport>(bNaniteShadingMaskExport);
			TShaderMapRef<FSingleLayerWaterRefractionCullingPS> PixelShader(View.ShaderMap, PermutationDomain);

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				View.ShaderMap,
				RDG_EVENT_NAME("SLW::RefractionCulling (View: %i)", ViewIndex),
				PixelShader,
				PassParameters,
				View.ViewRect,
				nullptr /*BlendState*/,
				nullptr /*RasterizerState*/,
				TStaticDepthStencilState<true, CF_Always>::GetRHI(),
				STENCIL_SANDBOX_MASK);
		}
	}

	return Result;
}

static FSceneWithoutWaterTextures AddCopySceneWithoutWaterPass(
	FRDGBuilder& GraphBuilder,
	const FSceneViewFamily& ViewFamily, 
	TArrayView<const FViewInfo> Views,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult)
{
	RDG_EVENT_SCOPE(GraphBuilder, "SLW::CopySceneWithoutWater");

	check(Views.Num() > 0);
	check(SceneColorTexture);
	check(SceneDepthTexture);

	const EShaderPlatform ShaderPlatform = Views[0].GetShaderPlatform();
	const bool bCopyDepth = !SingleLayerWaterPrePassResult || !HasBeenProduced(SingleLayerWaterPrePassResult->SceneDepthWithoutWater);
	const bool bCopyColor = !SingleLayerWaterUsesSimpleShading(ShaderPlatform);

	const FRDGTextureDesc& SceneDepthDesc = SceneColorTexture->Desc;
	const FRDGTextureDesc& SceneColorDesc = SceneColorTexture->Desc;

	const int32 RefractionDownsampleFactor = GetSingleLayerWaterRefractionDownsampleFactor();
	const FIntPoint RefractionResolution = FIntPoint::DivideAndRoundDown(SceneColorDesc.Extent, RefractionDownsampleFactor);
	FRDGTextureRef SceneColorWithoutSingleLayerWaterTexture = nullptr;
	FRDGTextureRef SceneDepthWithoutSingleLayerWaterTexture = nullptr;
	if (bCopyDepth)
	{
		// Note: if changing format, also update FWaterRefractionCopyPS::ModifyCompilationEnvironment accordingly
		const FRDGTextureDesc DepthDesc(FRDGTextureDesc::Create2D(RefractionResolution, PF_R32_FLOAT, SceneDepthDesc.ClearValue, TexCreate_ShaderResource | TexCreate_RenderTargetable));
		SceneDepthWithoutSingleLayerWaterTexture = GraphBuilder.CreateTexture(DepthDesc, TEXT("SLW.SceneDepthWithout"));
	}
	if (bCopyColor)
	{
		const FRDGTextureDesc ColorDesc = FRDGTextureDesc::Create2D(RefractionResolution, SceneColorDesc.Format, SceneColorDesc.ClearValue, TexCreate_ShaderResource | TexCreate_RenderTargetable);
		SceneColorWithoutSingleLayerWaterTexture = GraphBuilder.CreateTexture(ColorDesc, TEXT("SLW.SceneColorWithout"));
	}

	const FRDGTextureDesc SeparatedMainDirLightDesc(FRDGTextureDesc::Create2D(SceneColorDesc.Extent, PF_FloatR11G11B10, FClearValueBinding(FLinearColor::White), TexCreate_ShaderResource | TexCreate_RenderTargetable));
	FRDGTextureRef SeparatedMainDirLightTexture = GraphBuilder.CreateTexture(SeparatedMainDirLightDesc, TEXT("SLW.SeparatedMainDirLight"));

	FSceneWithoutWaterTextures Textures;
	Textures.RefractionDownsampleFactor = float(RefractionDownsampleFactor);
	Textures.Views.SetNum(Views.Num());

	// Compose hair pixels below water into the main scene color in order to be affected by water transmittance
	if (SingleLayerWaterPrePassResult && HairStrands::HasHairStrandsVisible(Views))
	{
		RenderHairCompositionForSLW(
			GraphBuilder,
			Views,
			SingleLayerWaterPrePassResult->DepthPrepassTexture.Target,
			SceneColorTexture,
			SceneDepthTexture);
	}

	AddCopySceneWithoutWaterPass_Internal(GraphBuilder, ViewFamily, Views, 
		SceneDepthWithoutSingleLayerWaterTexture, SceneDepthTexture, 
		SceneColorWithoutSingleLayerWaterTexture, SceneColorTexture, 
		RefractionDownsampleFactor,
		SingleLayerWaterPrePassResult);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (!View.ShouldRenderView())
		{
			continue;
		}

		// if we have a particular case of ISR where two views are laid out in side by side, we should copy both views at once
		const bool bIsInstancedStereo = View.IsInstancedStereo();
		FIntRect RectToCopy = View.ViewRect;
		if (bIsInstancedStereo)
		{
			const FViewInfo* NeighboringStereoView = View.GetInstancedView();
			if (ensure(NeighboringStereoView))
			{
				RectToCopy.Union(NeighboringStereoView->ViewRect);
			}
		}

		const FIntRect RefractionViewRect = FIntRect(FIntPoint::DivideAndRoundDown(RectToCopy.Min, RefractionDownsampleFactor), FIntPoint::DivideAndRoundDown(RectToCopy.Max, RefractionDownsampleFactor));
		Textures.Views[ViewIndex].ViewRect = RefractionViewRect;

		// This is usually half a pixel. But it seems that when using Gather4, 0.5 is not conservative enough and can return pixel outside the guard band. 
		// That is why it is a tiny bit higher than 0.5: for Gathre4 to always return pixels within the valid side of UVs (see EvaluateWaterVolumeLighting).
		const float PixelSafeGuardBand = 0.55f;
		Textures.Views[ViewIndex].MinMaxUV.X = (RefractionViewRect.Min.X + PixelSafeGuardBand) / RefractionResolution.X;
		Textures.Views[ViewIndex].MinMaxUV.Y = (RefractionViewRect.Min.Y + PixelSafeGuardBand) / RefractionResolution.Y;
		Textures.Views[ViewIndex].MinMaxUV.Z = (RefractionViewRect.Max.X - PixelSafeGuardBand) / RefractionResolution.X;
		Textures.Views[ViewIndex].MinMaxUV.W = (RefractionViewRect.Max.Y - PixelSafeGuardBand) / RefractionResolution.Y;
	}

	Textures.DepthTexture = bCopyDepth ? SceneDepthWithoutSingleLayerWaterTexture : SingleLayerWaterPrePassResult->SceneDepthWithoutWater;
	Textures.ColorTexture = bCopyColor ? SceneColorWithoutSingleLayerWaterTexture : GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
	Textures.SeparatedMainDirLightTexture = SeparatedMainDirLightTexture;

	check(HasBeenProduced(Textures.DepthTexture));
	check(HasBeenProduced(Textures.ColorTexture));

	return MoveTemp(Textures);
}

BEGIN_SHADER_PARAMETER_STRUCT(FWaterCompositeParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FWaterTileVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSingleLayerWaterCompositePS::FParameters, PS)
	RDG_BUFFER_ACCESS(IndirectDrawParameter, ERHIAccess::IndirectArgs)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FDeferredShadingSceneRenderer::RenderSingleLayerWaterReflections(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> InViews,
	const FSceneTextures& SceneTextures,
	const FSceneWithoutWaterTextures& SceneWithoutWaterTextures,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult,
	FLumenSceneFrameTemporaries& LumenFrameTemporaries)
{
	if (CVarWaterSingleLayer.GetValueOnRenderThread() <= 0)
	{
		return;
	}

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
	FRDGTextureRef SceneColorTexture = SceneTextures.Color.Resolve;

	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ViewIndex++)
	{
		FViewInfo& View = InViews[ViewIndex];

		// Unfortunately, reflections cannot handle two views at once (yet?) - because of that, allow the secondary pass here.
		// Note: not completely removing ShouldRenderView in case some other reason to not render it is valid.
		if (!View.ShouldRenderView() && !IStereoRendering::IsASecondaryPass(View.StereoPass))
		{
			continue;
		}

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1, "View%d", ViewIndex);

		FRDGTextureRef ReflectionsColor = nullptr;
		FRDGTextureRef BlackDummyTexture = SystemTextures.Black;
		FRDGTextureRef WhiteDummyTexture = SystemTextures.White;
		const FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, SceneTextures);

		auto SetCommonParameters = [&](FSingleLayerWaterCommonShaderParameters& Parameters)
		{
			FIntVector DepthTextureSize = SceneWithoutWaterTextures.DepthTexture ? SceneWithoutWaterTextures.DepthTexture->Desc.GetSize() : FIntVector::ZeroValue;
			const bool bShouldUseBilinearSamplerForDepth = SceneWithoutWaterTextures.DepthTexture && ShouldUseBilinearSamplerForDepthWithoutSingleLayerWater(SceneWithoutWaterTextures.DepthTexture->Desc.Format);

			FRDGTextureRef ScreenSpaceReflectionsTexture = ReflectionsColor ? ReflectionsColor : BlackDummyTexture;
			if (ReflectionsColor && ReflectionsColor->Desc.Dimension == ETextureDimension::Texture2DArray)
			{
				Parameters.ScreenSpaceReflectionsTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForSlice(ScreenSpaceReflectionsTexture, 0));
			}
			else
			{
				Parameters.ScreenSpaceReflectionsTexture = GraphBuilder.CreateSRV(ScreenSpaceReflectionsTexture);
			}

			Parameters.ScreenSpaceReflectionsSampler = TStaticSamplerState<SF_Point>::GetRHI();
			Parameters.PreIntegratedGF = GSystemTextures.PreintegratedGF->GetRHI();
			Parameters.PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters.SceneNoWaterDepthTexture = SceneWithoutWaterTextures.DepthTexture ? SceneWithoutWaterTextures.DepthTexture : BlackDummyTexture;
			Parameters.SceneNoWaterDepthSampler = bShouldUseBilinearSamplerForDepth ? TStaticSamplerState<SF_Bilinear>::GetRHI() : TStaticSamplerState<SF_Point>::GetRHI();
			Parameters.SceneNoWaterTextureSize = SceneWithoutWaterTextures.DepthTexture ? FVector2f(DepthTextureSize.X, DepthTextureSize.Y) : FVector2f();
			Parameters.SceneNoWaterInvTextureSize = SceneWithoutWaterTextures.DepthTexture ? FVector2f(1.0f / DepthTextureSize.X, 1.0f / DepthTextureSize.Y) : FVector2f();
			Parameters.SeparatedMainDirLightTexture = BlackDummyTexture;
			Parameters.UseSeparatedMainDirLightTexture = 0.0f;
			Parameters.SceneTextures = SceneTextureParameters;
			Parameters.View = View.GetShaderParameters();
			Parameters.ReflectionCaptureData = View.ReflectionCaptureUniformBuffer;
			Parameters.ReflectionsParameters = CreateReflectionUniformBuffer(GraphBuilder, View);
			Parameters.ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
			Parameters.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		};

		const bool bRunTiled = UseSingleLayerWaterIndirectDraw(View.GetShaderPlatform()) && CVarWaterSingleLayerTiledComposite.GetValueOnRenderThread();
		const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);

		FSingleLayerWaterTileClassification SingleLayerWaterTileClassification;
		if (bRunTiled)
		{
			if (SingleLayerWaterPrePassResult)
			{
				SingleLayerWaterTileClassification = SingleLayerWaterPrePassResult->ViewTileClassification[ViewIndex];
			}
			else
			{
				SingleLayerWaterTileClassification = ClassifyTiles(GraphBuilder, View, SceneTextures, nullptr, nullptr, ViewPipelineState.ReflectionsMethodWater);
			}
		}
		FTiledReflection& TiledScreenSpaceReflection = SingleLayerWaterTileClassification.TiledReflection;
		const FStaticShaderPlatform StaticShaderPlatform = View.GetShaderPlatform();
		const bool bWaterVSMFiltering = IsWaterVirtualShadowMapFilteringEnabled_Runtime(StaticShaderPlatform);
		const bool bWaterDistanceFieldShadow = IsWaterDistanceFieldShadowEnabled_Runtime(StaticShaderPlatform);

		if (bWaterVSMFiltering || bWaterDistanceFieldShadow)
		{
			const FLightSceneProxy* SelectedForwardDirectionalLightProxy = View.ForwardLightingResources.SelectedForwardDirectionalLightProxy;

			if (bWaterVSMFiltering && SelectedForwardDirectionalLightProxy)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "SLW::VirtualShadowMaps");

				FIntRect ScissorRect;
				if (!SelectedForwardDirectionalLightProxy->GetScissorRect(ScissorRect, View, View.ViewRect))
				{
					ScissorRect = View.ViewRect;
				}

				FLightSceneId LightId = SelectedForwardDirectionalLightProxy->GetLightSceneInfo()->Id;
				const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[SelectedForwardDirectionalLightProxy->GetLightSceneInfo()->Id];

				if (VisibleLightInfo.VirtualShadowMapClipmaps.Num() > 0)
				{
					FTiledVSMProjection TiledVSMProjection{};
					if (bRunTiled)
					{
						TiledVSMProjection.DrawIndirectParametersBuffer = TiledScreenSpaceReflection.DrawIndirectParametersBuffer;
						TiledVSMProjection.DispatchIndirectParametersBuffer = TiledScreenSpaceReflection.DispatchIndirectParametersBuffer;
						TiledVSMProjection.TileListDataBufferSRV = TiledScreenSpaceReflection.TileListDataBufferSRV;
						TiledVSMProjection.TileSize = TiledScreenSpaceReflection.TileSize;
					}

					GetSceneExtensionsRenderers().GetRenderer<FShadowSceneRenderer>().RenderVirtualShadowMapProjection(
						GraphBuilder,
						SceneTextures,
						LightId,
						View, ViewIndex,
						ScissorRect,
						EVirtualShadowMapProjectionInputType::GBuffer,
						true, // bModulateRGB
						bRunTiled  ? &TiledVSMProjection : nullptr,
						SceneWithoutWaterTextures.SeparatedMainDirLightTexture);
				}
			}

			if (bWaterDistanceFieldShadow)
			{
				FProjectedShadowInfo* DistanceFieldShadowInfo = nullptr;

				// Try to find the ProjectedShadowInfo corresponding to ray trace shadow info for the main directional light.
				if (SelectedForwardDirectionalLightProxy)
				{
					FLightSceneInfo* LightSceneInfo = SelectedForwardDirectionalLightProxy->GetLightSceneInfo();
					FVisibleLightInfo& VisibleLightViewInfo = VisibleLightInfos[LightSceneInfo->Id];

					for (int32 ShadowIndex = 0; ShadowIndex < VisibleLightViewInfo.ShadowsToProject.Num(); ShadowIndex++)
					{
						FProjectedShadowInfo* ProjectedShadowInfo = VisibleLightViewInfo.ShadowsToProject[ShadowIndex];
						if (ProjectedShadowInfo->bRayTracedDistanceField)
						{
							DistanceFieldShadowInfo = ProjectedShadowInfo;
						}
					}
				}

				// If DFShadow data has been found, then combine it with the separate main directional light luminance texture.
				FRDGTextureRef ScreenShadowMaskTexture = SystemTextures.White;
				if (DistanceFieldShadowInfo)
				{
					RDG_EVENT_SCOPE(GraphBuilder, "SLW::DistanceFieldShadow");

					FIntRect ScissorRect;
					if (!SelectedForwardDirectionalLightProxy->GetScissorRect(ScissorRect, View, View.ViewRect))
					{
						ScissorRect = View.ViewRect;
					}

					// Reset the cached texture to create a new one mapping to the water depth buffer
					DistanceFieldShadowInfo->ResetRayTracedDistanceFieldShadow(&View);

					FTiledShadowRendering TiledShadowRendering;
					if (bRunTiled)
					{
						TiledShadowRendering.DrawIndirectParametersBuffer = TiledScreenSpaceReflection.DrawIndirectParametersBuffer;
						TiledShadowRendering.TileListDataBufferSRV = TiledScreenSpaceReflection.TileListDataBufferSRV;
						TiledShadowRendering.TileSize = TiledScreenSpaceReflection.TileSize;
						TiledShadowRendering.TileType = FTiledShadowRendering::ETileType::Tile12bits;
					}

					const bool bProjectingForForwardShading = false;
					const bool bForceRGBModulation = true;
					DistanceFieldShadowInfo->RenderRayTracedDistanceFieldProjection(
						GraphBuilder,
						SceneTextures,
						SceneWithoutWaterTextures.SeparatedMainDirLightTexture,
						View,
						ViewIndex,
						ScissorRect,
						bProjectingForForwardShading,
						bForceRGBModulation,
						bRunTiled ? &TiledShadowRendering : nullptr);
				}
			}
		}

		// ReflectionsMethodWater can also be Disabled when only reflection captures are requested, so check CVarWaterSingleLayerReflection directly before early exiting.
		if (GetSingleLayerWaterReflectionTechnique() == ESingleLayerWaterReflections::Disabled || !View.Family->EngineShowFlags.Lighting)
		{
			continue;
		}

		if (ViewPipelineState.ReflectionsMethodWater == EReflectionsMethod::Lumen)
		{
			check(ShouldRenderLumenReflectionsWater(View));
			RDG_EVENT_SCOPE(GraphBuilder, "SLW::LumenReflections");

			FLumenMeshSDFGridParameters MeshSDFGridParameters;
			LumenRadianceCache::FRadianceCacheInterpolationParameters RadianceCacheParameters;

			FLumenReflectionsConfig LumenReflectionsConfig;
			LumenReflectionsConfig.TiledReflection = &TiledScreenSpaceReflection;
			LumenReflectionsConfig.DownsampleFactorXY = GetWaterReflectionDownsampleFactor();
			LumenReflectionsConfig.bScreenSpaceReconstruction = CVarWaterSingleLayerReflectionScreenSpaceReconstruction.GetValueOnRenderThread() != 0;
			LumenReflectionsConfig.bDenoising = CVarWaterSingleLayerReflectionDenoising.GetValueOnRenderThread() != 0;

			ReflectionsColor = RenderLumenReflections(
				GraphBuilder,
				View,
				SceneTextures,
				LumenFrameTemporaries,
				MeshSDFGridParameters,
				RadianceCacheParameters,
				ELumenReflectionPass::SingleLayerWater,
				LumenReflectionsConfig,
				ERDGPassFlags::Compute);
		}
		else if (ViewPipelineState.ReflectionsMethodWater == EReflectionsMethod::SSR)
		{
			check(ScreenSpaceRayTracing::ShouldRenderScreenSpaceReflectionsWater(View));
			// RUN SSR
			// Uses the water GBuffer (depth, ABCDEF) to know how to start tracing.
			// The water scene depth is used to know where to start tracing.
			// Then it uses the scene HZB for the ray casting process.

			IScreenSpaceDenoiser::FReflectionsInputs DenoiserInputs;
			IScreenSpaceDenoiser::FReflectionsRayTracingConfig RayTracingConfig;
			ESSRQuality SSRQuality;
			ScreenSpaceRayTracing::GetSSRQualityForView(View, &SSRQuality, &RayTracingConfig);

			RDG_EVENT_SCOPE(GraphBuilder, "SLW::ScreenSpaceReflections(Quality=%d)", int32(SSRQuality));

			const bool bDenoise = false;
			const bool bSingleLayerWater = true;
			ScreenSpaceRayTracing::RenderScreenSpaceReflections(
				GraphBuilder, SceneTextureParameters, SceneTextures.Color.Resolve, View, SSRQuality, bDenoise, &DenoiserInputs, bSingleLayerWater, bRunTiled ? &TiledScreenSpaceReflection : nullptr);

			ReflectionsColor = DenoiserInputs.Color;

			if (CVarWaterSingleLayerSSRTAA.GetValueOnRenderThread() && ScreenSpaceRayTracing::IsSSRTemporalPassRequired(View)) // TAA pass is an option
			{
				check(View.ViewState);
				FTAAPassParameters TAASettings(View);
				TAASettings.SceneDepthTexture = SceneTextureParameters.SceneDepthTexture;
				TAASettings.SceneVelocityTexture = SceneTextureParameters.GBufferVelocityTexture;
				TAASettings.Pass = ETAAPassConfig::ScreenSpaceReflections;
				TAASettings.SceneColorInput = DenoiserInputs.Color;
				TAASettings.bOutputRenderTargetable = true;

				FTAAOutputs TAAOutputs = AddTemporalAAPass(
					GraphBuilder,
					View,
					TAASettings,
					View.PrevViewInfo.WaterSSRHistory,
					&View.ViewState->PrevFrameViewInfo.WaterSSRHistory);

				ReflectionsColor = TAAOutputs.SceneColor;
			}
		}

		// Composite reflections on water
		{
			const bool bHasBoxCaptures = (View.NumBoxReflectionCaptures > 0);
			const bool bHasSphereCaptures = (View.NumSphereReflectionCaptures > 0);

			FSingleLayerWaterCompositePS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSingleLayerWaterCompositePS::FHasBoxCaptures>(bHasBoxCaptures);
			PermutationVector.Set<FSingleLayerWaterCompositePS::FHasSphereCaptures>(bHasSphereCaptures);
			TShaderMapRef<FSingleLayerWaterCompositePS> PixelShader(View.ShaderMap, PermutationVector);

			FWaterCompositeParameters* PassParameters = GraphBuilder.AllocParameters<FWaterCompositeParameters>();

			PassParameters->VS.ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->VS.TileListData = TiledScreenSpaceReflection.TileListDataBufferSRV;

			SetCommonParameters(PassParameters->PS.CommonParameters);
			if (NeedsSeparatedMainDirectionalLightTexture_Runtime(Scene->GetShaderPlatform()))
			{
				PassParameters->PS.CommonParameters.SeparatedMainDirLightTexture = SceneWithoutWaterTextures.SeparatedMainDirLightTexture;
				PassParameters->PS.CommonParameters.UseSeparatedMainDirLightTexture = 1.0f;
			}

			PassParameters->IndirectDrawParameter = TiledScreenSpaceReflection.DrawIndirectParametersBuffer;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);

			ValidateShaderParameters(PixelShader, PassParameters->PS);
			ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

			if (bRunTiled)
			{
				TShaderMapRef<FWaterTileVS> VertexShader(View.ShaderMap);
				ValidateShaderParameters(VertexShader, PassParameters->VS);
				ClearUnusedGraphResources(VertexShader, &PassParameters->VS);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("SLW::Composite %dx%d", View.ViewRect.Width(), View.ViewRect.Height()),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, &View, TiledScreenSpaceReflection, VertexShader, PixelShader, bRunTiled](FRDGAsyncTask, FRHICommandList& InRHICmdList)
				{
					InRHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					InRHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					GraphicsPSOInit.PrimitiveType = GRHISupportsRectTopology ? PT_RectList : PT_TriangleList;
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, 0);

					SetShaderParameters(InRHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
					SetShaderParameters(InRHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

					InRHICmdList.DrawPrimitiveIndirect(PassParameters->IndirectDrawParameter->GetIndirectRHICallBuffer(), 0);
				});
			}
			else
			{
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("SLW::Composite %dx%d", View.ViewRect.Width(), View.ViewRect.Height()),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, &View, TiledScreenSpaceReflection, PixelShader, bRunTiled](FRDGAsyncTask, FRHICommandList& InRHICmdList)
				{
					InRHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					FPixelShaderUtils::InitFullscreenPipelineState(InRHICmdList, View.ShaderMap, PixelShader, GraphicsPSOInit);

					// Premultiplied alpha where alpha is transmittance.
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha>::GetRHI();

					SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, 0);
					SetShaderParameters(InRHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);
					FPixelShaderUtils::DrawFullscreenTriangle(InRHICmdList);
				});
			}
		}
	}
}

void FDeferredShadingSceneRenderer::RenderSingleLayerWater(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> InViews,
	const FSceneTextures& SceneTextures,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult,
	bool bShouldRenderVolumetricCloud,
	FSceneWithoutWaterTextures& SceneWithoutWaterTextures,
	FLumenSceneFrameTemporaries& LumenFrameTemporaries,
	bool bIsCameraUnderWater)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, SingleLayerWater, "SingleLayerWater");
	SCOPED_NAMED_EVENT(SingleLayerWater, FColor::Emerald);

	// Copy the texture to be available for the water surface to refract
	SceneWithoutWaterTextures = AddCopySceneWithoutWaterPass(GraphBuilder, ViewFamily, InViews, SceneTextures.Color.Resolve, SceneTextures.Depth.Resolve, SingleLayerWaterPrePassResult);

	// Check if this is depth or base pass only renderer, where the final scene color isn't relevant, and we don't need fog, clouds, or reflections
	bool bFinalSceneColor = !InViews[0].CustomRenderPass && GetRendererOutput() == ERendererOutput::FinalSceneColor;

	if (bFinalSceneColor)
	{
		// Render height fog over the color buffer if it is allocated, e.g. SingleLayerWaterUsesSimpleShading is true.
		if (!bIsCameraUnderWater && SceneWithoutWaterTextures.ColorTexture && ShouldRenderFog(ViewFamily))
		{
			RenderUnderWaterFog(GraphBuilder, SceneWithoutWaterTextures, SceneTextures.UniformBuffer);
		}
		if (!bIsCameraUnderWater && SceneWithoutWaterTextures.ColorTexture && bShouldRenderVolumetricCloud)
		{
			// This path is only taken when rendering the clouds in a render target that can be composited and when the view is possibly intersecting the water surface.
			// The !bIsCameraUnderWater check is a bit misleading: In this case, volumetrics (including clouds) are rendered after water, but since bIsCameraUnderWater is
			// somewhat imprecise, it's possible for the camera to be fully or partially below the water surface and bIsCameraUnderWater being false. Without this call,
			// clouds would not be visible when looking up from under the water surface in such a case.
			ComposeVolumetricRenderTargetOverSceneUnderWater(GraphBuilder, InViews, SceneWithoutWaterTextures, SceneTextures);
		}
	}

	RenderSingleLayerWaterInner(GraphBuilder, InViews, SceneTextures, SceneWithoutWaterTextures, SingleLayerWaterPrePassResult);

	if (SingleLayerWaterPrePassResult &&
		SingleLayerWaterPrePassResult->MotionVectorWorldOffsetPrepassTexture)
	{
		// SceneTextures.Velocity is not produced and needs to be cleared before resolve.
		if (!HasBeenProduced(SceneTextures.Velocity))
		{
			AddClearRenderTargetPass(GraphBuilder, SceneTextures.Velocity);
		}

		ResolveSLWMotionVectorWorldOffset(
			GraphBuilder,
			InViews,
			SingleLayerWaterPrePassResult->DepthPrepassTexture.Resolve,
			SingleLayerWaterPrePassResult->VelocityPrepassTexture,
			SingleLayerWaterPrePassResult->MotionVectorWorldOffsetPrepassTexture,
			SceneTextures.Velocity,
			ShaderPlatform
		);
	}

	// No SSR or composite needed in Forward. Reflections are applied in the WaterGBuffer pass.
	if (!IsForwardShadingEnabled(ShaderPlatform) && bFinalSceneColor)
	{
		// Reflection composite expects the depth buffer in FSceneTextures to contain water but the swap of the main depth buffer with the water prepass depth buffer
		// is only done at the call site after this function returns (for visibility and to keep SceneTextures const), so we need to swap the depth buffers on an internal copy.
		FSceneTextures SceneTexturesInternal = SceneTextures;
		if (SingleLayerWaterPrePassResult)
		{
			SceneTexturesInternal.Depth = SingleLayerWaterPrePassResult->DepthPrepassTexture;
			// Rebuild scene textures uniform buffer to include new depth buffer.
			SceneTexturesInternal.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTexturesInternal, FeatureLevel, SceneTexturesInternal.SetupMode);
		}

		// If supported render SSR, the composite pass in non deferred and/or under water effect.
		RenderSingleLayerWaterReflections(GraphBuilder, InViews, SceneTexturesInternal, SceneWithoutWaterTextures, SingleLayerWaterPrePassResult, LumenFrameTemporaries);
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FSingleLayerWaterPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_REF(FReflectionCaptureShaderData, ReflectionCapture)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FOpaqueBasePassUniformParameters, BasePass)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSingleLayerWaterPassUniformParameters, SingleLayerWater)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FDeferredShadingSceneRenderer::RenderSingleLayerWaterInner(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> InViews,
	const FSceneTextures& SceneTextures,
	const FSceneWithoutWaterTextures& SceneWithoutWaterTextures,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, Water);
	SCOPED_NAMED_EVENT(FDeferredShadingSceneRenderer_RenderSingleLayerWaterPass, FColor::Emerald);
	SCOPE_CYCLE_COUNTER(STAT_WaterPassDrawTime);
	RDG_EVENT_SCOPE(GraphBuilder, "SLW::Draw");

	const bool bRenderInParallel = GRHICommandList.UseParallelAlgorithms() && CVarParallelSingleLayerWaterPass.GetValueOnRenderThread() == 1;

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	const EGBufferLayout GBufferLayout = GetSingleLayerWaterGBufferLayout(Scene->GetShaderPlatform(),FeatureLevel);
	const FGBufferBindings& GBufferBindings = SceneTextures.Config.GBufferBindings[GBufferLayout];
	TStaticArray<FTextureRenderTargetBinding, MaxSimultaneousRenderTargets> BasePassTextures;
	uint32 BasePassTextureCount = SceneTextures.GetGBufferRenderTargets(BasePassTextures, GBufferLayout);
	if(IsWaterSeparateMainDirLightEnabled(Scene->GetShaderPlatform()))
	{
		const bool bUsesSubstrateAdaptiveGBufferFormat = Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(ShaderPlatform);
		int32 SeparateMainDirLightIndex = INDEX_NONE;
		if (bUsesSubstrateAdaptiveGBufferFormat)
		{
			SeparateMainDirLightIndex = BasePassTextureCount++;
		}
		else
		{
			// SLW never uses GBufferD (CustomData) and we want to tightly pack RTVs so there's space for UAVs after the RTV range (D3D11.0 only supports a total of 8 RTVs and UAVs and they must have non-overlapping ranges).
			check(GBufferBindings.GBufferD.Index >= 0 && GBufferBindings.GBufferD.Index < MaxSimultaneousRenderTargets);
			SeparateMainDirLightIndex = GBufferBindings.GBufferD.Index;
		}
		
		const bool bNeverClear = true;
		BasePassTextures[SeparateMainDirLightIndex] = FTextureRenderTargetBinding(SceneWithoutWaterTextures.SeparatedMainDirLightTexture, bNeverClear);
	}
	Substrate::AppendSubstrateMRTs(*this, BasePassTextureCount, BasePassTextures);
	TArrayView<FTextureRenderTargetBinding> BasePassTexturesView = MakeArrayView(BasePassTextures.GetData(), BasePassTextureCount);

	FRDGTextureRef WhiteForwardScreenSpaceShadowMask = SystemTextures.White;

	const bool bHasDepthPrepass = SingleLayerWaterPrePassResult != nullptr;
	FDepthStencilBinding DepthStencilBinding;
	if (bHasDepthPrepass)
	{
		DepthStencilBinding = FDepthStencilBinding(SingleLayerWaterPrePassResult->DepthPrepassTexture.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);
	}
	else
	{
		DepthStencilBinding = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilNop);
	}

	const bool bEnableSkylight = Scene->ShouldRenderSkylightInBasePass(/*bIsTranslucent*/true);

	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
	{
		FViewInfo& View = InViews[ViewIndex];

		auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::SingleLayerWaterPass];

		if (!Pass || !View.ShouldRenderView())
		{
			continue;
		}

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1, "View%d", ViewIndex);
		View.BeginRenderView();

		FRDGTextureRef RefractionMaskTexture = SingleLayerWaterPrePassResult && SingleLayerWaterPrePassResult->RefractionMaskTexture ? SingleLayerWaterPrePassResult->RefractionMaskTexture : SystemTextures.White;
		FSingleLayerWaterPassUniformParameters* SLWUniformParameters = CreateSingleLayerWaterPassUniformParameters(GraphBuilder, Scene, View, SceneTextures, SceneWithoutWaterTextures.DepthTexture, SceneWithoutWaterTextures.ColorTexture, RefractionMaskTexture);

		FSingleLayerWaterPassParameters* PassParameters = GraphBuilder.AllocParameters<FSingleLayerWaterPassParameters>();
		PassParameters->View = View.GetShaderParameters();
		PassParameters->ReflectionCapture = View.ReflectionCaptureUniformBuffer;
		PassParameters->BasePass = CreateOpaqueBasePassUniformBuffer(GraphBuilder, View, ViewIndex, {}, {}, bEnableSkylight);
		PassParameters->VirtualShadowMapSamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
		PassParameters->SingleLayerWater = GraphBuilder.CreateUniformBuffer(SLWUniformParameters);
		PassParameters->RenderTargets = GetRenderTargetBindings(ERenderTargetLoadAction::ELoad, BasePassTexturesView);
		PassParameters->RenderTargets.DepthStencil = DepthStencilBinding;

		// Make sure to clear the velocity texture if it wasn't already written to. This can be the case if the velocity pass is set to "Write after base pass".
		if (GBufferBindings.GBufferVelocity.Index > 0 && GBufferBindings.GBufferVelocity.Index < (int32)BasePassTextureCount && !HasBeenProduced(SceneTextures.Velocity))
		{
			PassParameters->RenderTargets[GBufferBindings.GBufferVelocity.Index].SetLoadAction(ERenderTargetLoadAction::EClear);
		}

		Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

		if (bRenderInParallel)
		{
			GraphBuilder.AddDispatchPass(
				RDG_EVENT_NAME("SingleLayerWaterParallel"),
				PassParameters,
				ERDGPassFlags::Raster,
				[Pass, PassParameters](FRDGDispatchPassBuilder& DispatchPassBuilder)
			{
				Pass->Dispatch(DispatchPassBuilder, &PassParameters->InstanceCullingDrawParams);
			});
		}
		else
		{
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("SingleLayerWater"),
				PassParameters,
				ERDGPassFlags::Raster,
				[&View, Pass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				SetStereoViewport(RHICmdList, View, 1.0f);
				Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
			});
		}
	}

	if (bHasDepthPrepass)
	{
		AddClearStencilPass(GraphBuilder, SingleLayerWaterPrePassResult->DepthPrepassTexture.Target);
	}

	if (!bHasDepthPrepass)
	{
		AddResolveSceneDepthPass(GraphBuilder, InViews, SceneTextures.Depth);
	}
}

class FSingleLayerWaterPassMeshProcessor : public FSceneRenderingAllocatorObject<FSingleLayerWaterPassMeshProcessor>, public FMeshPassProcessor
{
public:
	FSingleLayerWaterPassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	FMeshPassProcessorRenderState PassDrawRenderState;
};

FSingleLayerWaterPassMeshProcessor::FSingleLayerWaterPassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::SingleLayerWaterPass, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InPassDrawRenderState)
{
	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	if (SingleLayerWaterUsesSimpleShading(ShaderPlatform))
	{
		// Force non opaque, pre multiplied alpha, transparent blend mode because water is going to be blended against scene color (no distortion from texture scene color).
		FRHIBlendState* ForwardSimpleWaterBlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI();
		PassDrawRenderState.SetBlendState(ForwardSimpleWaterBlendState);
	}
}

void FSingleLayerWaterPassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

bool FSingleLayerWaterPassMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	if (Material.GetShadingModels().HasShadingModel(MSM_SingleLayerWater))
	{
		// Determine the mesh's material and blend mode.
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
		return Process(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, MeshFillMode, MeshCullMode);
	}

	return true;
}

bool FSingleLayerWaterPassMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	FUniformLightMapPolicy NoLightmapPolicy(LMP_NO_LIGHTMAP);
	typedef FUniformLightMapPolicy LightMapPolicyType;
	TMeshProcessorShaders<
		TBasePassVertexShaderPolicyParamType<LightMapPolicyType>,
		TBasePassPixelShaderPolicyParamType<LightMapPolicyType>> WaterPassShaders;

	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;
	if (!GetBasePassShaders<LightMapPolicyType>(
		MaterialResource,
		VertexFactory->GetType(),
		NoLightmapPolicy,
		FeatureLevel,
		false, // 128bit
		false, // bIsDebug
		GetSingleLayerWaterGBufferLayout(GetFeatureLevelShaderPlatform(FeatureLevel), FeatureLevel),
		nullptr, // Mesh Shader
		&WaterPassShaders.VertexShader,
		&WaterPassShaders.PixelShader
		))
	{ 
		return false;
	}

	TBasePassShaderElementData<LightMapPolicyType> ShaderElementData(nullptr);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(WaterPassShaders.VertexShader, WaterPassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		WaterPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

void FSingleLayerWaterPassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	if (Material.GetShadingModels().HasShadingModel(MSM_SingleLayerWater))
	{
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

		FUniformLightMapPolicy NoLightmapPolicy(LMP_NO_LIGHTMAP);
		typedef FUniformLightMapPolicy LightMapPolicyType;
		TMeshProcessorShaders<
			TBasePassVertexShaderPolicyParamType<LightMapPolicyType>,
			TBasePassPixelShaderPolicyParamType<LightMapPolicyType>> WaterPassShaders;

		const EGBufferLayout GBufferLayout = GetSingleLayerWaterGBufferLayout(GetFeatureLevelShaderPlatform(FeatureLevel), FeatureLevel, true /*bIsGameThread*/);

		if (!GetBasePassShaders<LightMapPolicyType>(
			Material,
			VertexFactoryData.VertexFactoryType,
			NoLightmapPolicy,
			FeatureLevel,
			false, // 128bit
			false, // bIsDebug
			GBufferLayout,
			nullptr, // Mesh Shader
			&WaterPassShaders.VertexShader,
			&WaterPassShaders.PixelShader
			))
		{
			return;
		}

		FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
		SetupGBufferRenderTargetInfo(SceneTexturesConfig, RenderTargetsInfo, true /*bSetupDepthStencil*/, GBufferLayout);
		if (IsWaterSeparateMainDirLightEnabled(GMaxRHIShaderPlatform))
		{
			const bool bUsesSubstrateAdaptiveGBufferFormat = Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(GMaxRHIShaderPlatform);
			int32 SeparateMainDirLightIndex = INDEX_NONE;
			if (bUsesSubstrateAdaptiveGBufferFormat)
			{
				SeparateMainDirLightIndex = RenderTargetsInfo.RenderTargetsEnabled++;
			}
			else
			{
				const FGBufferBindings& GBufferBindings = SceneTexturesConfig.GBufferBindings[GBufferLayout];
				// SLW never uses GBufferD (CustomData) and we want to tightly pack RTVs so there's space for UAVs after the RTV range (D3D11.0 only supports a total of 8 RTVs and UAVs and they must have non-overlapping ranges).
				check(GBufferBindings.GBufferD.Index >= 0 && GBufferBindings.GBufferD.Index < MaxSimultaneousRenderTargets);
				SeparateMainDirLightIndex = GBufferBindings.GBufferD.Index;
			}
			RenderTargetsInfo.RenderTargetFormats[SeparateMainDirLightIndex] = PF_FloatR11G11B10;
			RenderTargetsInfo.RenderTargetFlags[SeparateMainDirLightIndex] = TexCreate_ShaderResource | TexCreate_RenderTargetable;
		}

		const bool bHasDepthPrepass = IsSingleLayerWaterDepthPrepassEnabled(GetFeatureLevelShaderPlatform(FeatureLevel), FeatureLevel);
		RenderTargetsInfo.DepthStencilAccess = bHasDepthPrepass ? FExclusiveDepthStencil::DepthRead_StencilRead : FExclusiveDepthStencil::DepthWrite_StencilNop;

		FBasePassMeshProcessor::AddBasePassGraphicsPipelineStateInitializer(
			FeatureLevel,
			VertexFactoryData,
			Material,
			PassDrawRenderState,
			RenderTargetsInfo,
			WaterPassShaders,
			MeshFillMode,
			MeshCullMode,
			(EPrimitiveType)PreCacheParams.PrimitiveType,
			true /*bPrecacheAlphaColorChannel*/,
			PSOCollectorIndex,
			PSOInitializers);
	}
}

FMeshPassProcessor* CreateSingleLayerWaterPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	const bool bHasDepthPrepass = IsSingleLayerWaterDepthPrepassEnabled(GetFeatureLevelShaderPlatform(FeatureLevel), FeatureLevel);
	const FExclusiveDepthStencil::Type SceneBasePassDepthStencilAccess = FScene::GetDefaultBasePassDepthStencilAccess(FeatureLevel);

	FMeshPassProcessorRenderState DrawRenderState;

	// Make sure depth write is enabled if no prepass is used.
	FExclusiveDepthStencil::Type BasePassDepthStencilAccess_DepthWrite = FExclusiveDepthStencil::Type(bHasDepthPrepass ? FExclusiveDepthStencil::DepthRead_StencilRead : SceneBasePassDepthStencilAccess | FExclusiveDepthStencil::DepthWrite_StencilNop);
	SetupBasePassState(BasePassDepthStencilAccess_DepthWrite, false, DrawRenderState);
	if (bHasDepthPrepass)
	{
		// Set depth stencil test to only pass if depth and stencil are equal to the values written by the prepass
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<
			false, CF_Equal,							// Depth test
			true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,	// Front face stencil 
			true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,	// Back face stencil
			STENCIL_SANDBOX_MASK, 0x0 					// Stencil read/write masks
		>::GetRHI());
		DrawRenderState.SetStencilRef(STENCIL_SANDBOX_MASK);
	}

	return new FSingleLayerWaterPassMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, DrawRenderState, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(SingleLayerWater, CreateSingleLayerWaterPassProcessor, EShadingPath::Deferred, EMeshPass::SingleLayerWaterPass, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);


class FSingleLayerWaterDepthPrepassMeshProcessor : public FSceneRenderingAllocatorObject<FSingleLayerWaterDepthPrepassMeshProcessor>, public FMeshPassProcessor
{
public:
	FSingleLayerWaterDepthPrepassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	template<bool bPositionOnly>
	bool Process(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	bool ProcessVelocity(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	template<bool bPositionOnly>
	void CollectPSOInitializersInternal(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		const FPSOPrecacheParams& PreCacheParams, 
		TArray<FPSOPrecacheData>& PSOInitializers);

	FMeshPassProcessorRenderState PassDrawRenderState;
};

FSingleLayerWaterDepthPrepassMeshProcessor::FSingleLayerWaterDepthPrepassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::SingleLayerWaterDepthPrepass, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InPassDrawRenderState)
{

}

void FSingleLayerWaterDepthPrepassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	// Early out if the depth prepass for water is disabled
	if (!IsSingleLayerWaterDepthPrepassEnabled(GetFeatureLevelShaderPlatform(FeatureLevel), FeatureLevel))
	{
		return;
	}

	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

bool FSingleLayerWaterDepthPrepassMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	if (Material.GetShadingModels().HasShadingModel(MSM_SingleLayerWater))
	{
		// Determine the mesh's material and blend mode.
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
		const bool bVFTypeSupportsNullPixelShader = MeshBatch.VertexFactory->SupportsNullPixelShader();
		const bool bModifiesMeshPosition = DoMaterialAndPrimitiveModifyMeshPosition(Material, PrimitiveSceneProxy);

		// When SLW velocity is enabled, all materials route through velocity shaders.
		// When disabled, only material using MVWO routes through velocity shaders instead of depth-only.
		const EShaderPlatform PrepassShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
		const bool bSLWPrepassVelocity = CanSingleLayerWaterDepthPrepassOutputVelocity(PrepassShaderPlatform, FeatureLevel);
		if (bSLWPrepassVelocity)
		{
			const bool bMaterialNeedsVelocity =
				!Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader)
				|| bModifiesMeshPosition
				|| (Material.MaterialUsesMotionVectorWorldOffset_RenderThread() && SupportsPixelShaderMotionVectorWorldOffset(PrepassShaderPlatform))
				|| (Material.MaterialUsesTemporalResponsiveness_RenderThread() && SupportsTemporalResponsiveness(PrepassShaderPlatform));

			if (bMaterialNeedsVelocity)
			{
				return ProcessVelocity(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, MeshFillMode, MeshCullMode);
			}
		}

		if (IsOpaqueBlendMode(Material)
			&& MeshBatch.VertexFactory->SupportsPositionOnlyStream()
			&& !bModifiesMeshPosition
			&& Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader))
		{
			const FMaterialRenderProxy& DefaultProxy = *UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			const FMaterial& DefaultMaterial = *DefaultProxy.GetMaterialNoFallback(FeatureLevel);
			return Process<true>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, DefaultProxy, DefaultMaterial, MeshFillMode, MeshCullMode);
		}
		else
		{
			const bool bMaterialMasked = !Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader);
			const FMaterialRenderProxy* EffectiveMaterialRenderProxy = &MaterialRenderProxy;
			const FMaterial* EffectiveMaterial = &Material;

			if (!bMaterialMasked && !bModifiesMeshPosition)
			{
				// Override with the default material for opaque materials that are not two sided
				EffectiveMaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
				EffectiveMaterial = EffectiveMaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
				check(EffectiveMaterial);
			}

			return Process<false>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *EffectiveMaterialRenderProxy, *EffectiveMaterial, MeshFillMode, MeshCullMode);
		}
	}

	return true;
}

template<bool bPositionOnly>
bool FSingleLayerWaterDepthPrepassMeshProcessor::Process(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	TMeshProcessorShaders<TDepthOnlyVS<bPositionOnly>, FDepthOnlyPS> DepthPassShaders;
	FShaderPipelineRef ShaderPipeline;

	if (!GetDepthPassShaders<bPositionOnly>(
		MaterialResource,
		MeshBatch.VertexFactory->GetType(),
		FeatureLevel,
		MaterialResource.MaterialUsesPixelDepthOffset_GameThread(),
		DepthPassShaders.VertexShader,
		DepthPassShaders.PixelShader,
		ShaderPipeline))
	{
		return false;
	}

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, true);

	const bool bIsMasked = IsMaskedBlendMode(MaterialResource);
	FMeshDrawCommandSortKey SortKey = CalculateDepthPassMeshStaticSortKey(bIsMasked, DepthPassShaders.VertexShader.GetShader(), DepthPassShaders.PixelShader.GetShader());

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		DepthPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		bPositionOnly ? EMeshPassFeatures::PositionOnly : EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

bool FSingleLayerWaterDepthPrepassMeshProcessor::ProcessVelocity(
	const FMeshBatch& RESTRICT MeshBatch, 
	uint64 BatchElementMask, 
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, 
	int32 StaticMeshId, const FMaterialRenderProxy& RESTRICT MaterialRenderProxy, 
	const FMaterial& RESTRICT MaterialResource, 
	ERasterizerFillMode MeshFillMode, 
	ERasterizerCullMode MeshCullMode)
{
	TMeshProcessorShaders<FVelocityVS, FVelocityPS> VelocityPassShaders;

	if (!GetVelocityPassShaders(
		MaterialResource,
		MeshBatch.VertexFactory->GetType(),
		FeatureLevel,
		EVelocityPassMode::Velocity_Standard,
		VelocityPassShaders.VertexShader,
		VelocityPassShaders.PixelShader))
	{
		return false;
	}

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, true);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(VelocityPassShaders.VertexShader, VelocityPassShaders.PixelShader);

	// Override blend state to use the default: the depth prepass uses CW_NONE, but MVWO velocity
	// shaders need to write to RT0 (velocity) and RT1 (MVWO offset).
	FMeshPassProcessorRenderState VelocityDrawRenderState(PassDrawRenderState);
	VelocityDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		VelocityDrawRenderState,
		VelocityPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

void FSingleLayerWaterDepthPrepassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	if (Material.GetShadingModels().HasShadingModel(MSM_SingleLayerWater) && IsSingleLayerWaterDepthPrepassEnabled(ShaderPlatform, FeatureLevel))
	{
		const bool bSLWPrepassVelocity = CanSingleLayerWaterDepthPrepassOutputVelocity(ShaderPlatform, FeatureLevel, /*bIsGameThread*/true);
		
		if (bSLWPrepassVelocity)
		{
			const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
			const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
			const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

			const bool bVFTypeSupportsNullPixelShader = VertexFactoryData.VertexFactoryType->SupportsNullPixelShader();
			const bool bMaterialNeedsVelocity = 
				!Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader)
				|| Material.MaterialModifiesMeshPosition_GameThread()
				|| (Material.MaterialUsesMotionVectorWorldOffset_GameThread() && SupportsPixelShaderMotionVectorWorldOffset(ShaderPlatform))
				|| (Material.MaterialUsesTemporalResponsiveness_GameThread() && SupportsTemporalResponsiveness(ShaderPlatform));

			if (bMaterialNeedsVelocity)
			{
				TMeshProcessorShaders<FVelocityVS, FVelocityPS> VelocityPassShaders;
				if (GetVelocityPassShaders(
					Material,
					VertexFactoryData.VertexFactoryType,
					FeatureLevel,
					EVelocityPassMode::Velocity_Standard,
					VelocityPassShaders.VertexShader,
					VelocityPassShaders.PixelShader))
				{
					FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
					RenderTargetsInfo.NumSamples = SceneTexturesConfig.NumSamples;

					// Two velocity render targets (velocity + MVWO offset)
					AddRenderTargetInfo(FVelocityRendering::GetFormat(ShaderPlatform), FVelocityRendering::GetCreateFlags(ShaderPlatform), RenderTargetsInfo);
					AddRenderTargetInfo(FVelocityRendering::GetFormat(ShaderPlatform), FVelocityRendering::GetCreateFlags(ShaderPlatform), RenderTargetsInfo);

					ETextureCreateFlags DepthStencilCreateFlags = SceneTexturesConfig.DepthCreateFlags;
					SetupDepthStencilInfo(PF_DepthStencil, DepthStencilCreateFlags, ERenderTargetLoadAction::ELoad,
						ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite, RenderTargetsInfo);

					// Must match ProcessVelocity's blend state override — depth prepass
					// has CW_NONE but MVWO velocity shaders need to write to RT0/RT1.
					FMeshPassProcessorRenderState VelocityDrawRenderState(PassDrawRenderState);
					VelocityDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());

					AddGraphicsPipelineStateInitializer(
						VertexFactoryData,
						Material,
						VelocityDrawRenderState,
						RenderTargetsInfo,
						VelocityPassShaders,
						MeshFillMode,
						MeshCullMode,
						PT_TriangleList,
						EMeshPassFeatures::Default,
						true /*bRequired*/,
						PSOInitializers);
				}
				return;
			}
		}
		
		// Determine the mesh's material and blend mode.
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
		const bool bSupportPositionOnlyStream = VertexFactoryData.VertexFactoryType->SupportsPositionOnly();
		const bool bVFTypeSupportsNullPixelShader = VertexFactoryData.VertexFactoryType->SupportsNullPixelShader();

		if (IsOpaqueBlendMode(Material)
			&& bSupportPositionOnlyStream
			&& !Material.MaterialModifiesMeshPosition_GameThread()
			&& Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader))
		{
			EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
			const FMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface)->GetMaterialResource(SceneTexturesConfig.ShaderPlatform, ActiveQualityLevel);
			check(DefaultMaterial);

			CollectPSOInitializersInternal<true>(SceneTexturesConfig, VertexFactoryData, *DefaultMaterial, MeshFillMode, MeshCullMode, PreCacheParams, PSOInitializers);
		}
		else
		{
			const bool bMaterialMasked = !Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader);
			const FMaterial* EffectiveMaterial = &Material;

			if (!bMaterialMasked && !Material.MaterialModifiesMeshPosition_GameThread())
			{
				// Override with the default material for opaque materials that are not two sided
				EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
				EffectiveMaterial = UMaterial::GetDefaultMaterial(MD_Surface)->GetMaterialResource(SceneTexturesConfig.ShaderPlatform, ActiveQualityLevel);
				check(EffectiveMaterial);
			}

			CollectPSOInitializersInternal<false>(SceneTexturesConfig, VertexFactoryData, *EffectiveMaterial, MeshFillMode, MeshCullMode, PreCacheParams, PSOInitializers);
		}
	}
}

template<bool bPositionOnly>
void FSingleLayerWaterDepthPrepassMeshProcessor::CollectPSOInitializersInternal(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	const FPSOPrecacheParams& PreCacheParams,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	TMeshProcessorShaders<TDepthOnlyVS<bPositionOnly>, FDepthOnlyPS> DepthPassShaders;
	FShaderPipelineRef ShaderPipeline;

	if (!GetDepthPassShaders<bPositionOnly>(
		MaterialResource,
		VertexFactoryData.VertexFactoryType,
		FeatureLevel,
		MaterialResource.MaterialUsesPixelDepthOffset_GameThread(),
		DepthPassShaders.VertexShader,
		DepthPassShaders.PixelShader,
		ShaderPipeline))
	{
		return;
	}

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = SceneTexturesConfig.NumSamples;

	ETextureCreateFlags DepthStencilCreateFlags = SceneTexturesConfig.DepthCreateFlags;
	SetupDepthStencilInfo(PF_DepthStencil, DepthStencilCreateFlags, ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite, RenderTargetsInfo);

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		MaterialResource,
		PassDrawRenderState,
		RenderTargetsInfo,
		DepthPassShaders,
		MeshFillMode,
		MeshCullMode,
		(EPrimitiveType)PreCacheParams.PrimitiveType,
		bPositionOnly ? EMeshPassFeatures::PositionOnly : EMeshPassFeatures::Default,
		true /*bRequired*/,
		PSOInitializers);
}

FMeshPassProcessor* CreateSingleLayerWaterDepthPrepassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	const FExclusiveDepthStencil::Type SceneBasePassDepthStencilAccess = FScene::GetDefaultBasePassDepthStencilAccess(FeatureLevel);

	FMeshPassProcessorRenderState DrawRenderState;

	// Make sure depth write is enabled.
	FExclusiveDepthStencil::Type BasePassDepthStencilAccess_DepthWrite = FExclusiveDepthStencil::Type(SceneBasePassDepthStencilAccess | FExclusiveDepthStencil::DepthWrite_StencilWrite);
	
	// Disable color writes, enable depth tests and writes.
	DrawRenderState.SetBlendState(TStaticBlendState<CW_NONE>::GetRHI());
	DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<
		true, CF_DepthNearOrEqual,						// Depth test
		true, CF_Always, SO_Keep, SO_Keep, SO_Replace,	// Front face stencil 
		true, CF_Always, SO_Keep, SO_Keep, SO_Replace	// Back face stencil
	>::GetRHI());
	DrawRenderState.SetDepthStencilAccess(BasePassDepthStencilAccess_DepthWrite);
	DrawRenderState.SetStencilRef(STENCIL_SANDBOX_MASK);

	return new FSingleLayerWaterDepthPrepassMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, DrawRenderState, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(SingleLayerWaterDepthPrepass, CreateSingleLayerWaterDepthPrepassProcessor, EShadingPath::Deferred, EMeshPass::SingleLayerWaterDepthPrepass, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
