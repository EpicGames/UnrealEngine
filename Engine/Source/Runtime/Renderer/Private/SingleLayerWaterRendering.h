// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraph.h"
#include "PixelShaderUtils.h"
#include "ScreenSpaceRayTracing.h"
#include "SingleLayerWaterDefinitions.h"
#include "Froxel/Froxel.h"

class FViewInfo;
enum class ECustomDepthPassLocation : uint32;

struct FSingleLayerWaterTileClassification
{
	FTiledReflection TiledReflection = FTiledReflection{ nullptr, nullptr, nullptr, nullptr, nullptr, nullptr, nullptr, nullptr, nullptr, SLW_TILE_SIZE_XY };
	FRDGBufferRef TileMaskBuffer = nullptr;
	FIntPoint TiledViewRes = FIntPoint{0, 0};
};

struct FSingleLayerWaterPrePassResult
{
	Froxel::FRenderer Froxels;
	FRDGTextureMSAA DepthPrepassTexture;
	FRDGTextureRef RefractionMaskTexture;
	FRDGTextureRef SceneDepthWithoutWater;
	TArray<FSingleLayerWaterTileClassification> ViewTileClassification;
	FRDGTextureRef VelocityPrepassTexture = nullptr;
	FRDGTextureRef MotionVectorWorldOffsetPrepassTexture = nullptr;
};


struct FSceneWithoutWaterTextures
{
	struct FView
	{
		FIntRect ViewRect;
		FVector4f MinMaxUV;
	};

	FRDGTextureRef SeparatedMainDirLightTexture = nullptr;
	FRDGTextureRef ColorTexture = nullptr;
	FRDGTextureRef DepthTexture = nullptr;
	TArray<FView> Views;
	float RefractionDownsampleFactor = 1.0f;
};

// Location relative to the base pass when to run the water depth prepass. If a full depth prepass is available,
// the water depth prepass can run before the base pass, allowing certain optimizations to save work in the base pass.
enum class ESingleLayerWaterPrepassLocation : uint8
{
	// The water depth prepass (if enabled) runs after the regular depth prepass and before the base pass.
	BeforeBasePass,
	// The water depth prepass (if enabled) runs after the base pass.
	AfterBasePass
};

enum class ESingleLayerWaterVelocityOutputPass : uint8 
{ 
	None = 0, 
	DepthPrepass = 1, 
	BasePass = 2
};

bool ShouldRenderSingleLayerWater(TArrayView<const FViewInfo> Views);
bool ShouldRenderSingleLayerWaterSkippedRenderEditorNotification(TArrayView<const FViewInfo> Views);
bool ShouldRenderSingleLayerWaterDepthPrepass(TArrayView<const FViewInfo> Views);
ESingleLayerWaterPrepassLocation GetSingleLayerWaterDepthPrepassLocation(bool bFullDepthPrepass, ECustomDepthPassLocation CustomDepthPassLocation);
bool CanSingleLayerWaterDepthPrepassOutputVelocity(EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel, bool bIsGameThread = false);
ESingleLayerWaterVelocityOutputPass GetSingleLayerWaterVelocityOutputPass(bool bIsGameThread = false);
bool ShouldUseBilinearSamplerForDepthWithoutSingleLayerWater(EPixelFormat DepthTextureFormat);

class FWaterTileVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FWaterTileVS);
	SHADER_USE_PARAMETER_STRUCT(FWaterTileVS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileListData)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

template<typename TPixelShaderClass, typename TPassParameters>
void SingleLayerWaterAddTiledFullscreenPass(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* GlobalShaderMap,
	FRDGEventName&& PassName,
	TShaderRefBase<TPixelShaderClass, FShaderMapPointerTable> PixelShader,
	TPassParameters* PassParameters,
	const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
	const FIntRect& Viewport,
	const FTiledReflection* TiledScreenSpaceReflection = nullptr,
	FRHIBlendState* BlendState = nullptr,
	FRHIRasterizerState* RasterizerState = nullptr,
	FRHIDepthStencilState* DepthStencilState = nullptr,
	uint32 StencilRef = 0)
{
	PassParameters->IndirectDrawParameter = TiledScreenSpaceReflection ? TiledScreenSpaceReflection->DrawIndirectParametersBuffer : nullptr;

	PassParameters->VS.ViewUniformBuffer = ViewUniformBuffer;
	PassParameters->VS.TileListData = TiledScreenSpaceReflection ? TiledScreenSpaceReflection->TileListDataBufferSRV : nullptr;

	ValidateShaderParameters(PixelShader, PassParameters->PS);
	ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

	const bool bRunTiled = TiledScreenSpaceReflection != nullptr;
	if (bRunTiled)
	{
		FWaterTileVS::FPermutationDomain PermutationVector;
		TShaderMapRef<FWaterTileVS> VertexShader(GlobalShaderMap, PermutationVector);

		ValidateShaderParameters(VertexShader, PassParameters->VS);
		ClearUnusedGraphResources(VertexShader, &PassParameters->VS);

		GraphBuilder.AddPass(
			Forward<FRDGEventName>(PassName),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, GlobalShaderMap, Viewport, TiledScreenSpaceReflection, VertexShader, PixelShader, BlendState, RasterizerState, DepthStencilState, StencilRef](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			FPixelShaderUtils::InitFullscreenPipelineState(RHICmdList, GlobalShaderMap, PixelShader, GraphicsPSOInit);

			GraphicsPSOInit.PrimitiveType = GRHISupportsRectTopology ? PT_RectList : PT_TriangleList;
			GraphicsPSOInit.BlendState = BlendState ? BlendState : GraphicsPSOInit.BlendState;
			GraphicsPSOInit.RasterizerState = RasterizerState ? RasterizerState : GraphicsPSOInit.RasterizerState;
			GraphicsPSOInit.DepthStencilState = DepthStencilState ? DepthStencilState : GraphicsPSOInit.DepthStencilState;
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

			RHICmdList.DrawPrimitiveIndirect(PassParameters->IndirectDrawParameter->GetIndirectRHICallBuffer(), 0);
		});
	}
	else
	{
		GraphBuilder.AddPass(
			Forward<FRDGEventName>(PassName),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, GlobalShaderMap, Viewport, PixelShader, BlendState, RasterizerState, DepthStencilState, StencilRef](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			FPixelShaderUtils::InitFullscreenPipelineState(RHICmdList, GlobalShaderMap, PixelShader, GraphicsPSOInit);

			GraphicsPSOInit.BlendState = BlendState ? BlendState : GraphicsPSOInit.BlendState;
			GraphicsPSOInit.RasterizerState = RasterizerState ? RasterizerState : GraphicsPSOInit.RasterizerState;
			GraphicsPSOInit.DepthStencilState = DepthStencilState ? DepthStencilState : GraphicsPSOInit.DepthStencilState;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

			FPixelShaderUtils::DrawFullscreenTriangle(RHICmdList);
		});
	}
}