// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimBankTransformProvider.h"
#include "Animation/AnimBoneCompressionCodec.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "Animation/Skeleton.h"
#include "AnimationRuntime.h"
#include "AnimEncoding.h"
#include "ComponentRecreateRenderStateContext.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneView.h"
#include "SkeletalRenderPublic.h"
#include "SkinningDefinitions.h"


static FGuid AnimBankGPUProviderId(ANIM_BANK_GPU_TRANSFORM_PROVIDER_GUID);
static FGuid AnimBankCPUProviderId(ANIM_BANK_CPU_TRANSFORM_PROVIDER_GUID);

IMPLEMENT_SCENE_EXTENSION(FAnimBankTransformProvider);

static TAutoConsoleVariable<bool> CVarAnimBankInterp(
	TEXT("r.AnimBank.Interpolation"),
	true,
	TEXT(""),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarAnimBankTimeScale(
	TEXT("r.AnimBank.TimeScale"),
	1.0f,
	TEXT(""),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarAnimBankGPU(
	TEXT("r.AnimBank.GPU"),
	true,
	TEXT(""),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

class FAnimBankEvaluateCS : public FGlobalShader
{
public:
	static constexpr uint32 BonesPerGroup = 64u;

private:
	DECLARE_GLOBAL_SHADER(FAnimBankEvaluateCS);
	SHADER_USE_PARAMETER_STRUCT(FAnimBankEvaluateCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("BONES_PER_GROUP"), BonesPerGroup);

		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, SequenceTransformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BankBlockBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BoneMapBuffer)
		SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_UAV(TransformBuffer)
		SHADER_PARAMETER(uint32, bInterpolating)
		SHADER_PARAMETER(float, SampleInterval)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FAnimBankEvaluateCS, "/Engine/Private/Skinning/AnimBankEval.usf", "BankEvaluateCS", SF_Compute);

FAnimBankTransformProvider::FProxyData::FProxyData(FAnimBankTransformProviderProxy* Proxy)
	: Tracks(Proxy->GetTracks())
	, SceneExtensionProxy(*Proxy->GetSceneExtensionProxy())
	, TransformStorageMode(SceneExtensionProxy.GetBoneTransformStorageMode())
{}

FAnimBankTransformProvider::FAnimBankTransformProvider(FScene& InScene)
	: ISceneExtension(InScene)
	, SequenceTransformPersistentBuffer(0, TEXT("AnimBank.SequenceTransformBuffer"))
{}

bool FAnimBankTransformProvider::ShouldCreateExtension(FScene& InScene)
{
	return IsGPUSkinSceneExtensionEnabled();
}

void FAnimBankTransformProvider::InitExtension(FScene& InScene)
{
	if (auto TransformProvider = InScene.GetExtensionPtr<FSkinningTransformProvider>())
	{
		// Register GPU animation bank transform provider
		TransformProvider->RegisterProvider(
			AnimBankGPUProviderId,
			FSkinningTransformProvider::FOnProvideTransforms::CreateRaw(this, &FAnimBankTransformProvider::ProvideGPUBankTransforms),
			false /* Use skeleton batching */
		);

		// Register CPU animation bank transform provider
		TransformProvider->RegisterProvider(
			AnimBankCPUProviderId,
			FSkinningTransformProvider::FOnProvideTransforms::CreateRaw(this, &FAnimBankTransformProvider::ProvideCPUBankTransforms),
			false /* Use skeleton batching */
		);
	}
}

void FAnimBankTransformProvider::RegisterAnimBank(FAnimBankTransformProviderProxy* Proxy)
{
	TConstArrayView<FAnimBankItem> AnimBankItems = Proxy->GetAnimBankItems();

	FProxyData& ProxyData = ProxyDataMap.Emplace(Proxy->GetSceneExtensionProxy(), Proxy);
	ProxyData.Sequences.Reserve(AnimBankItems.Num());
	check(ProxyData.Sequences.IsEmpty());

	for (const FAnimBankItem& Item : Proxy->GetAnimBankItems())
	{
		const FAnimBankData& AnimBankData = Item.BankAsset->GetData();
		const FSequenceRegistryKey SequenceRegistryKey(Item);

		TUniquePtr<FMapping>& Mapping = MappingRegistry.FindOrAdd(SequenceRegistryKey.BankAsset);
		if (!Mapping)
		{
			Mapping = MakeUnique<FMapping>();

			const FSkinnedAssetMapping& AnimBankMapping = AnimBankData.Mapping;

			Mapping->RegistryKey   = SequenceRegistryKey.BankAsset;
			Mapping->Keys.Position = AnimBankMapping.PositionKeys;
			Mapping->Keys.Rotation = AnimBankMapping.RotationKeys;
			Mapping->BoneCount     = AnimBankMapping.BoneCount;

			check(!MappingsToUpload.Contains(Mapping.Get()));
			MappingsToUpload.AddUnique(Mapping.Get());
		}
		++Mapping->RefCount;

		TUniquePtr<FSequence>& Sequence = SequenceRegistry.FindOrAdd(SequenceRegistryKey);
		if (!Sequence)
		{
			Sequence = MakeUnique<FSequence>();

			const FAnimBankEntry& AnimBankEntry = AnimBankData.Entries[SequenceRegistryKey.SequenceIndex];

			Sequence->RegistryKey   = SequenceRegistryKey;
			Sequence->Keys.Position = AnimBankEntry.PositionKeys;
			Sequence->Keys.Rotation = AnimBankEntry.RotationKeys;
			Sequence->PlayLength    = AnimBankEntry.GetPlayLength();
			Sequence->FrameCount    = AnimBankEntry.FrameCount;
			Sequence->KeyCount      = AnimBankEntry.KeyCount;
			Sequence->BoneCount     = Mapping->BoneCount;
			Sequence->Mapping       = Mapping.Get();
			Sequence->bLooping      = AnimBankEntry.IsLooping();

			check(!SequencesToUpload.Contains(Sequence.Get()));
			SequencesToUpload.AddUnique(Sequence.Get());
		}
		++Sequence->RefCount;

		ProxyData.Sequences.Emplace(SequenceRegistryKey, *Mapping, *Sequence);
	}
}

void FAnimBankTransformProvider::UnregisterAnimBank(FAnimBankTransformProviderProxy* Proxy)
{
	const FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy = Proxy->GetSceneExtensionProxy();
	const uint32 SceneExtensionProxyHash = GetTypeHash(SceneExtensionProxy);

	FProxyData& ProxyData = ProxyDataMap.FindByHashChecked(SceneExtensionProxyHash, SceneExtensionProxy);

	for (const FSequenceView& SequenceView : ProxyData.Sequences)
	{
		const uint32 BankAssetHash = GetTypeHash(SequenceView.Key.BankAsset);
		TUniquePtr<FMapping>& Mapping = MappingRegistry.FindByHashChecked(BankAssetHash, SequenceView.Key.BankAsset);
		check(Mapping && Mapping->RefCount > 0);
		--Mapping->RefCount;

		if (Mapping->RefCount == 0)
		{
			if (Mapping->BufferAllocation.IsValid())
			{
				SequenceTransformAllocator.Free(Mapping->BufferAllocation.Offset, Mapping->BufferAllocation.Count);
				Mapping->BufferAllocation = {};
			}

			MappingsToUpload.RemoveSingleSwap(Mapping.Get(), EAllowShrinking::No);
			MappingRegistry.RemoveByHash(BankAssetHash, SequenceView.Key.BankAsset);
		}

		const uint32 SequenceRegistryKeyHash = GetTypeHash(SequenceView.Key);
		TUniquePtr<FSequence>& Sequence = SequenceRegistry.FindByHashChecked(SequenceRegistryKeyHash, SequenceView.Key);
		check(Sequence && Sequence->RefCount > 0);
		--Sequence->RefCount;

		if (Sequence->RefCount == 0)
		{
			if (Sequence->BufferAllocation.IsValid())
			{
				SequenceTransformAllocator.Free(Sequence->BufferAllocation.Offset, Mapping->BufferAllocation.Count);
				Sequence->BufferAllocation = {};
			}

			SequencesToUpload.RemoveSingleSwap(Sequence.Get(), EAllowShrinking::No);
			SequenceRegistry.RemoveByHash(SequenceRegistryKeyHash, SequenceView.Key);
		}
	}

	ProxyDataMap.RemoveByHash(SceneExtensionProxyHash, SceneExtensionProxy);
}

void FAnimBankTransformProvider::ProvideGPUBankTransforms(FSkinningTransformProvider::FProviderContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankTransformProvider::ProvideGPUBankTransforms);
	FRDGBuilder& GraphBuilder = Context.GraphBuilder;

	int32 TotalSequenceTransformCount = 0;
	SequenceTransformAllocator.Consolidate();

	static_assert(sizeof(FAnimBankTransformProvider::FSequenceTransform) == 7 * sizeof(float));

	for (FMapping* Mapping : MappingsToUpload)
	{
		const int32 TransformCount = Mapping->BoneCount;
		Mapping->BufferAllocation.Count  = TransformCount;
		Mapping->BufferAllocation.Offset = SequenceTransformAllocator.Allocate(TransformCount);
		TotalSequenceTransformCount += TransformCount;
	}

	for (FSequence* Sequence : SequencesToUpload)
	{
		const int32 TransformCount = Sequence->KeyCount;
		Sequence->BufferAllocation.Count  = TransformCount;
		Sequence->BufferAllocation.Offset = SequenceTransformAllocator.Allocate(TransformCount);
		TotalSequenceTransformCount += TransformCount;
	}

	FRDGBuffer* SequenceTransformBuffer = SequenceTransformPersistentBuffer.ResizeBufferIfNeeded(GraphBuilder, SequenceTransformAllocator.GetMaxSize());

	if (TotalSequenceTransformCount > 0)
	{
		Context.ScatterUploadBuilder.AddPass(GraphBuilder, SequenceTransformUploadBuffer, SequenceTransformBuffer, TotalSequenceTransformCount, sizeof(FSequenceTransform), TEXT("AnimBank.SequenceTransforms"),
			[MappingsToUpload = MoveTemp(MappingsToUpload), SequencesToUpload = MoveTemp(SequencesToUpload)] (FRDGScatterUploader& ScatterUploader)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankTransformProvider::UploadSequences);

			for (const FMapping* Mapping : MappingsToUpload)
			{
				TArrayView<FSequenceTransform> DstTransforms = ScatterUploader.Add_GetRef<FSequenceTransform>(Mapping->BufferAllocation.Offset, Mapping->BoneCount);

				for (uint32 Index = 0; Index < Mapping->BoneCount; ++Index)
				{
					DstTransforms[Index].SetRotation(Mapping->Keys.Rotation[Index]);
					DstTransforms[Index].SetPosition(Mapping->Keys.Position[Index]);
				}
			}

			for (const FSequence* Sequence : SequencesToUpload)
			{
				TArrayView<FSequenceTransform> DstTransforms = ScatterUploader.Add_GetRef<FSequenceTransform>(Sequence->BufferAllocation.Offset, Sequence->KeyCount);

				for (uint32 Index = 0; Index < Sequence->KeyCount; ++Index)
				{
					DstTransforms[Index].SetRotation(Sequence->Keys.Rotation[Index]);
					DstTransforms[Index].SetPosition(Sequence->Keys.Position[Index]);
				}
			}
		});

		Context.ScatterUploadBuilder.Execute(Context.GraphBuilder);
	}

	uint32 BlockCountMax = 0;
	const uint32 BonesPerGroup = FAnimBankEvaluateCS::BonesPerGroup;

	TArrayView<FProxyData*> ProxyDatas = GraphBuilder.AllocPODArrayView<FProxyData*>(Context.Proxies.Num());

	for (const FSkinningTransformProvider::FProviderIndirection Indirection : Context.Indirections)
	{
		FProxyData& ProxyData = ProxyDataMap.FindChecked(Context.Proxies[Indirection.Index]);
		ProxyDatas[Indirection.Index] = &ProxyData;

		const uint32 ActiveTrackCount  = ProxyData.Tracks.GetNumActiveTracks();
		const uint32 LocalBlockCount   = FMath::DivideAndRoundUp(ProxyData.SceneExtensionProxy.GetMaxBoneTransformCount(), BonesPerGroup);

		BlockCountMax += ActiveTrackCount * LocalBlockCount * 2;
	}

	if (!BlockCountMax)
	{
		return;
	}

	TArrayView<UE::HLSL::FAnimBankBlockHeader> BlockHeaders = GraphBuilder.AllocPODArrayView<UE::HLSL::FAnimBankBlockHeader>(BlockCountMax);
	int32& BlockCount = *GraphBuilder.AllocPOD<int32>();
	BlockCount = 0;

	GraphBuilder.AddSetupTask([this, ProxyDatas, BlockHeaders, CurrentTime = Context.CurrentTime, Indirections = Context.Indirections, &BlockCount, BonesPerGroup]
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimBankTransformProvider::BuildBoneBlocks);
		const float GlobalTimeScale = CVarAnimBankTimeScale.GetValueOnRenderThread();
		const float DeltaTime = CurrentTime.GetDeltaWorldTimeSeconds() * GlobalTimeScale;

		for (const FSkinningTransformProvider::FProviderIndirection Indirection : Indirections)
		{
			FProxyData& ProxyData = *ProxyDatas[Indirection.Index];
			FAnimBankTrackPool& Tracks = ProxyData.Tracks;
			ProxyData.TrackHistories.SetNum(Tracks.GetNumTracks());

			const uint32 MaxTransformCount     = ProxyData.SceneExtensionProxy.GetMaxBoneTransformCount();
			const uint32 FullBlockCount        = MaxTransformCount / BonesPerGroup;
			const uint32 PartialTransformCount = MaxTransformCount - (FullBlockCount * BonesPerGroup);
			const uint32 BoneMapOffset         = ProxyData.TransformStorageMode != EBoneTransformStorageMode::Direct ? Indirection.BoneMapOffset : ~uint32(0);

			const uint32 CurrentTransformIndex  = Indirection.CurrentTransformOffset  / sizeof(FCompressedBoneTransform);
			const uint32 PreviousTransformIndex = Indirection.PreviousTransformOffset / sizeof(FCompressedBoneTransform);
			const uint32 CurrentTransformSlot   = CurrentTransformIndex < PreviousTransformIndex ? 0 : 1;

			Tracks.EnumerateActiveTracks([&] (int32 TrackIndex)
			{
				FAnimBankTrackPackedData& Data      = Tracks.GetData(TrackIndex);
				const FSequenceView& SequenceView   = ProxyData.GetSequenceView(Data);
				const FSequence& Sequence           = SequenceView.Sequence;
				const uint32 SequenceTransformIndex = Sequence.BufferAllocation.Offset;
				const uint32 MappingTransformIndex  = SequenceView.Mapping.BufferAllocation.Offset;
				const uint32 LocalTransformIndex    = TrackIndex * MaxTransformCount * 2;

				bool bNeedsCurrentUpdate, bNeedsPreviousUpdate;
				ProxyData.TrackHistories[TrackIndex].Update(Data, CurrentTransformSlot, bNeedsCurrentUpdate, bNeedsPreviousUpdate);

				// Dirty previous transforms means the whole allocation is dirty. Rebuild everything.
				const bool bDirtyTransformAllocation = EnumHasAnyFlags(Indirection.DirtyBoneTransforms, EDirtyBoneTransforms::Previous);
				bNeedsCurrentUpdate  |= bDirtyTransformAllocation;
				bNeedsPreviousUpdate |= bDirtyTransformAllocation;

				const float Position              = Data.GetPosition();
				const float PreviousPosition      = Data.GetPreviousPosition();

				EPreviousBoneTransformUpdateMode UpdateMode = EPreviousBoneTransformUpdateMode::None;

				if (bNeedsPreviousUpdate)
				{
					UpdateMode = bNeedsCurrentUpdate && bNeedsPreviousUpdate && Position == PreviousPosition
						? EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious
						: EPreviousBoneTransformUpdateMode::UpdatePrevious;
				}

				const auto AddBlockHeaders = [&] (float Time, uint32 TransformIndex, uint32 OutputTransformCopyIndexOffset)
				{
					for (uint32 BlockIndex = 0; BlockIndex < FullBlockCount; ++BlockIndex)
					{
						UE::HLSL::FAnimBankBlockHeader& BlockHeader = BlockHeaders[BlockCount++];

						BlockHeader.Time                      = Time;
						BlockHeader.BoneMapOffset             = BoneMapOffset;
						BlockHeader.SequenceTransformIndex    = SequenceTransformIndex;
						BlockHeader.MappingTransformIndex     = MappingTransformIndex;
						BlockHeader.OutputTransformIndex      = TransformIndex;
						BlockHeader.FrameCount                = Sequence.FrameCount;
						BlockHeader.BoneCount                 = Sequence.BoneCount;
						BlockHeader.OutputTransformCopyIndexOffset = OutputTransformCopyIndexOffset;
						BlockHeader.BlockLocalIndex           = BlockIndex;
						BlockHeader.BlockTransformCount       = BonesPerGroup;
					}

					if (PartialTransformCount > 0)
					{
						UE::HLSL::FAnimBankBlockHeader& BlockHeader = BlockHeaders[BlockCount++];
						
						BlockHeader.Time                      = Time;
						BlockHeader.BoneMapOffset             = BoneMapOffset;
						BlockHeader.SequenceTransformIndex    = SequenceTransformIndex;
						BlockHeader.MappingTransformIndex     = MappingTransformIndex;
						BlockHeader.OutputTransformIndex      = TransformIndex;
						BlockHeader.FrameCount                = Sequence.FrameCount;
						BlockHeader.BoneCount                 = Sequence.BoneCount;
						BlockHeader.OutputTransformCopyIndexOffset = OutputTransformCopyIndexOffset;
						BlockHeader.BlockLocalIndex           = FullBlockCount;
						BlockHeader.BlockTransformCount       = PartialTransformCount;
					}
				};

				if (bNeedsCurrentUpdate)
				{
					uint32 PrimaryOutputIndex = CurrentTransformIndex;
					uint32 SecondaryOutputIndexOffset = 0;

					if (UpdateMode == EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious)
					{
						PrimaryOutputIndex = CurrentTransformSlot == 0 ? CurrentTransformIndex : PreviousTransformIndex;
						SecondaryOutputIndexOffset = MaxTransformCount;
					}

					AddBlockHeaders(Position, PrimaryOutputIndex + LocalTransformIndex, SecondaryOutputIndexOffset);
				}

				if (UpdateMode == EPreviousBoneTransformUpdateMode::UpdatePrevious)
				{
					AddBlockHeaders(PreviousPosition, PreviousTransformIndex + LocalTransformIndex, 0);
				}

				Data.Update(Sequence.PlayLength, Sequence.bLooping, DeltaTime);
			});
		}
	});

	FRDGBuffer* BankBlockBuffer = CreateByteAddressBuffer(
		GraphBuilder,
		TEXT("AnimBank.BlockHeaders"),
		// NumElements
		[&BlockCount] { return BlockCount * (sizeof(UE::HLSL::FAnimBankBlockHeader) / 4); },
		// InitialData
		[Data = BlockHeaders.GetData()] { return Data; },
		// InitialDataSize
		[&BlockCount] { return BlockCount * sizeof(UE::HLSL::FAnimBankBlockHeader); }
	);

	const bool bInterpolating = CVarAnimBankInterp.GetValueOnRenderThread();

	FAnimBankEvaluateCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAnimBankEvaluateCS::FParameters>();
	PassParameters->SequenceTransformBuffer = GraphBuilder.CreateSRV(SequenceTransformBuffer);
	PassParameters->BankBlockBuffer = GraphBuilder.CreateSRV(BankBlockBuffer);
	PassParameters->BoneMapBuffer = Context.BoneMapBufferSRV;
	PassParameters->TransformBuffer = GetCompressedBoneTransformUAV(GraphBuilder, Context.TransformBuffer);
	PassParameters->bInterpolating = bInterpolating;
	PassParameters->SampleInterval = 1.0f / ANIM_BANK_SAMPLE_RATE;

	auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FAnimBankEvaluateCS>();
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("AnimBankEvaluate"),
		ComputeShader,
		PassParameters,
		[&BlockCount] { return FIntVector(BlockCount, 1, 1); }
	);
}

void FAnimBankTransformProvider::ProvideCPUBankTransforms(FSkinningTransformProvider::FProviderContext& Context)
{
	FRDGBuilder& GraphBuilder = Context.GraphBuilder;

	uint32 GlobalTransformCount = 0;

	for (const FSkinningTransformProvider::FProviderIndirection Indirection : Context.Indirections)
	{
		const FInstancedSkinningSceneExtensionProxy* Proxy = static_cast<FInstancedSkinningSceneExtensionProxy*>(Context.Proxies[Indirection.Index]);
		FAnimBankTransformProviderProxy* AnimBankProxy = static_cast<FAnimBankTransformProviderProxy*>(Proxy->GetTransformProviderProxy());

		const uint32 TransformCount = Proxy->GetMaxBoneTransformCount();
		const uint32 ActiveAnimationCount = AnimBankProxy->GetTracks().GetNumActiveTracks();
		GlobalTransformCount += ActiveAnimationCount * TransformCount * 2; // Current and Previous
	}

	if (GlobalTransformCount == 0)
	{
		return;
	}

	Context.ScatterUploadBuilder.AddPass(GraphBuilder, SkinningTransformUploadBuffer, Context.TransformBuffer, GlobalTransformCount, sizeof(FCompressedBoneTransform), TEXT("Skinning.AnimTransforms"),
		[this, Indirections = Context.Indirections, Proxies = Context.Proxies, CurrentTime = Context.CurrentTime, GlobalTransformCount] (FRDGScatterUploader& ScatterUploader)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSkinningSceneExtension::ProvideAnimBankRuntimeTransformsTask);
		TArray<FCompressedBoneTransform, SceneRenderingAllocator> SourceTransforms;
		TBitArray<SceneRenderingBitArrayAllocator> SourceTransformMap;
		const bool bInterpolating = CVarAnimBankInterp.GetValueOnRenderThread();

		const auto SampleAnimBank = [&SourceTransforms, &SourceTransformMap, bInterpolating] (
			const FInstancedSkinningSceneExtensionProxy* Proxy,
			TArrayView<FCompressedBoneTransform> Transforms,
			TArrayView<FCompressedBoneTransform> DuplicatePreviousTransforms,
			const FSequence& Sequence,
			const FMapping& Mapping,
			uint32 TransformCount,
			float TimeOffset)
		{
			int32 KeyIndex0, KeyIndex1;
			float Alpha;
			FAnimationRuntime::GetKeyIndicesFromTime(KeyIndex0, KeyIndex1, Alpha, TimeOffset, Sequence.FrameCount, Sequence.PlayLength);

			if (!bInterpolating)
			{
				// Forcing alpha to zero disables pose interpolation (interpolation method is "step")
				Alpha = 0.0f;
			}

			const auto SampleTransform = [&] (int32 TransformIndex) -> FMatrix44f
			{
				const FTransform InvGlobalRefPoseXform = FTransform(FQuat(Mapping.Keys.Rotation[TransformIndex]), FVector(Mapping.Keys.Position[TransformIndex]));

				FTransform MeshGlobalAnimPoseXform;

				if (Alpha <= 0.f)
				{
					MeshGlobalAnimPoseXform = FTransform(FQuat(Sequence.Keys.Rotation[(KeyIndex0 * Mapping.BoneCount) + TransformIndex]), FVector(Sequence.Keys.Position[(KeyIndex0 * Mapping.BoneCount) + TransformIndex]));
					MeshGlobalAnimPoseXform.NormalizeRotation();
				}
				else if (Alpha >= 1.f)
				{
					MeshGlobalAnimPoseXform = FTransform(FQuat(Sequence.Keys.Rotation[(KeyIndex1 * Mapping.BoneCount) + TransformIndex]), FVector(Sequence.Keys.Position[(KeyIndex1 * Mapping.BoneCount) + TransformIndex]));
					MeshGlobalAnimPoseXform.NormalizeRotation();
				}
				else
				{
					FTransform MeshGlobalXformA = FTransform(FQuat(Sequence.Keys.Rotation[(KeyIndex0 * Mapping.BoneCount) + TransformIndex]), FVector(Sequence.Keys.Position[(KeyIndex0 * Mapping.BoneCount) + TransformIndex]));
					FTransform MeshGlobalXformB = FTransform(FQuat(Sequence.Keys.Rotation[(KeyIndex1 * Mapping.BoneCount) + TransformIndex]), FVector(Sequence.Keys.Position[(KeyIndex1 * Mapping.BoneCount) + TransformIndex]));

					// Ensure rotations are normalized
					MeshGlobalXformA.NormalizeRotation();
					MeshGlobalXformB.NormalizeRotation();

					MeshGlobalAnimPoseXform.Blend(MeshGlobalXformA, MeshGlobalXformB, Alpha);
					MeshGlobalAnimPoseXform.NormalizeRotation();
				}

				MeshGlobalAnimPoseXform = InvGlobalRefPoseXform * MeshGlobalAnimPoseXform;

				return (FMatrix44f)MeshGlobalAnimPoseXform.ToMatrixNoScale();
			};

			if (Proxy->GetBoneTransformStorageMode() == EBoneTransformStorageMode::Direct)
			{
				for (uint32 TransformIndex = 0; TransformIndex < TransformCount; ++TransformIndex)
				{
					StoreCompressedBoneTransform(&Transforms[TransformIndex], SampleTransform(TransformIndex));

					if (!DuplicatePreviousTransforms.IsEmpty())
					{
						StoreCompressedBoneTransform(&DuplicatePreviousTransforms[TransformIndex], SampleTransform(TransformIndex));
					}
				}
			}
			else
			{
				// Mapping.BoneCount is always the number of mesh skeleton bones.
				SourceTransformMap.Init(false, Mapping.BoneCount);
				SourceTransforms.SetNumUninitialized(Mapping.BoneCount, EAllowShrinking::No);
				int32 DstTransformIndex = 0;

				for (uint32 SrcTransformIndex : Proxy->GetBoneMap())
				{
					if (!SourceTransformMap[SrcTransformIndex])
					{
						SourceTransformMap[SrcTransformIndex] = true;
						StoreCompressedBoneTransform(&SourceTransforms[SrcTransformIndex], SampleTransform(SrcTransformIndex));
					}

					Transforms[DstTransformIndex] = SourceTransforms[SrcTransformIndex];

					if (!DuplicatePreviousTransforms.IsEmpty())
					{
						DuplicatePreviousTransforms[DstTransformIndex] = SourceTransforms[SrcTransformIndex];
					}

					DstTransformIndex++;
				}
			}
		};

		const float GlobalTimeScale = CVarAnimBankTimeScale.GetValueOnRenderThread();
		const float DeltaTime = CurrentTime.GetDeltaWorldTimeSeconds() * GlobalTimeScale;

		for (const FSkinningTransformProvider::FProviderIndirection Indirection : Indirections)
		{
			const FInstancedSkinningSceneExtensionProxy* Proxy = static_cast<FInstancedSkinningSceneExtensionProxy*>(Proxies[Indirection.Index]);
			FProxyData& ProxyData = ProxyDataMap.FindChecked(Proxy);
			FAnimBankTrackPool& Tracks = ProxyData.Tracks;
			ProxyData.TrackHistories.SetNum(Tracks.GetNumTracks());

			const uint32 MaxTransformCount = Proxy->GetMaxBoneTransformCount();
			const uint32 UniqueAnimationCount = Proxy->GetUniqueAnimationCount();
			const uint32 MaxTotalTransformCount = MaxTransformCount * 2u; // Current and Previous

			const uint32 CurrentTransformIndex  = Indirection.CurrentTransformOffset  / sizeof(FCompressedBoneTransform);
			const uint32 PreviousTransformIndex = Indirection.PreviousTransformOffset / sizeof(FCompressedBoneTransform);
			const uint32 CurrentTransformSlot   = CurrentTransformIndex < PreviousTransformIndex ? 0 : 1;

			Tracks.EnumerateActiveTracks([&] (int32 TrackIndex)
			{
				FAnimBankTrackPackedData& Data    = Tracks.GetData(TrackIndex);
				const FSequenceView& SequenceView = ProxyData.GetSequenceView(Data);
	
				bool bNeedsCurrentUpdate, bNeedsPreviousUpdate;
				ProxyData.TrackHistories[TrackIndex].Update(Data, CurrentTransformSlot, bNeedsCurrentUpdate, bNeedsPreviousUpdate);

				// Dirty previous transforms means the whole allocation is dirty. Rebuild everything.
				const bool bDirtyTransformAllocation = EnumHasAnyFlags(Indirection.DirtyBoneTransforms, EDirtyBoneTransforms::Previous);
				bNeedsCurrentUpdate  |= bDirtyTransformAllocation;
				bNeedsPreviousUpdate |= bDirtyTransformAllocation;

				const float Position              = Data.GetPosition();
				const float PreviousPosition      = Data.GetPreviousPosition();

				EPreviousBoneTransformUpdateMode UpdateMode = EPreviousBoneTransformUpdateMode::None;
				TArrayView<FCompressedBoneTransform> DstTransformsPrevious;

				if (bNeedsPreviousUpdate)
				{
					UpdateMode = bNeedsCurrentUpdate && bNeedsPreviousUpdate && Position == PreviousPosition
						? EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious
						: EPreviousBoneTransformUpdateMode::UpdatePrevious;

					DstTransformsPrevious = ScatterUploader.Add_GetRef<FCompressedBoneTransform>(PreviousTransformIndex + TrackIndex * MaxTotalTransformCount, MaxTransformCount);
				}

				if (bNeedsCurrentUpdate)
				{
					TArrayView<FCompressedBoneTransform> DstTransforms = ScatterUploader.Add_GetRef<FCompressedBoneTransform>(CurrentTransformIndex + TrackIndex * MaxTotalTransformCount, MaxTransformCount);
					SampleAnimBank(Proxy, DstTransforms, UpdateMode == EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious ? DstTransformsPrevious : TArrayView<FCompressedBoneTransform>{}, SequenceView.Sequence, SequenceView.Mapping, MaxTransformCount,  Position);
				}

				if (UpdateMode == EPreviousBoneTransformUpdateMode::UpdatePrevious)
				{
					SampleAnimBank(Proxy, DstTransformsPrevious, {}, SequenceView.Sequence, SequenceView.Mapping, MaxTransformCount, PreviousPosition);
				}

				Data.Update(SequenceView.Sequence.PlayLength, SequenceView.Sequence.bLooping, DeltaTime);
			});
		}
	});
}