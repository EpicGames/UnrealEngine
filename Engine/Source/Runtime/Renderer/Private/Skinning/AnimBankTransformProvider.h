// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimBank.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "RendererPrivateUtils.h"
#include "SceneExtensions.h"
#include "SkinningTransformProvider.h"
#include "SkinningSceneExtensionProxy.h"
#include "UnifiedBuffer.h"
#include "SpanAllocator.h"

class FAnimBankTransformProvider
	: public ISceneExtension
	, public IAnimBankTransformProvider
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FAnimBankTransformProvider);

public:
	FAnimBankTransformProvider(FScene& InScene);

	static bool ShouldCreateExtension(FScene& InScene);

	virtual void InitExtension(FScene& InScene) override;

	void RegisterAnimBank(FAnimBankTransformProviderProxy* Proxy) override;
	void UnregisterAnimBank(FAnimBankTransformProviderProxy* Proxy) override;

private:
	void ProvideGPUBankTransforms(FSkinningTransformProvider::FProviderContext& Context);
	void ProvideCPUBankTransforms(FSkinningTransformProvider::FProviderContext& Context);

	struct FBufferAllocation
	{
		bool IsValid() const
		{
			return Count != 0;
		}

		uint32 Offset = 0;
		uint32 Count  = 0;
	};

	struct FSequenceTransform
	{
		void SetRotation(FQuat4f Q)
		{
			Rotation[0] = Q.X;
			Rotation[1] = Q.Y;
			Rotation[2] = Q.Z;
			Rotation[3] = Q.W;
		}

		void SetPosition(FVector3f P)
		{
			Position[0] = P.X;
			Position[1] = P.Y;
			Position[2] = P.Z;
		}

		float Rotation[4];
		float Position[3];
	};
	
	struct FSequenceTransformKeys
	{
		TConstArrayView<FVector3f> Position;
		TConstArrayView<FQuat4f>   Rotation;
	};

	struct FSequenceRegistryKey
	{
		FSequenceRegistryKey() = default;

		FSequenceRegistryKey(const FAnimBankItem& Item)
			: BankAsset(Item.BankAsset.Get())
			, SequenceIndex(Item.SequenceIndex)
		{}

		const UAnimBank* BankAsset = nullptr;
		int32 SequenceIndex = 0;

		bool operator==(const FSequenceRegistryKey& Other) const
		{
			return BankAsset == Other.BankAsset && SequenceIndex == Other.SequenceIndex;
		}

		friend uint32 GetTypeHash(const FSequenceRegistryKey& Key)
		{
			return PointerHash(Key.BankAsset, Key.SequenceIndex);
		}
	};
	
	using FMappingRegistryKey = const UAnimBank*;

	struct FMapping
	{
		FMappingRegistryKey RegistryKey = nullptr;
		FSequenceTransformKeys Keys;
		FBufferAllocation BufferAllocation;

		uint32 RefCount = 0;
		uint32 BoneCount = 0;
	};

	struct FSequence
	{
		FSequenceRegistryKey RegistryKey;
		FSequenceTransformKeys Keys;
		FBufferAllocation BufferAllocation;
		FMapping* Mapping = nullptr;

		float PlayLength = 0.0f;
		uint32 RefCount = 0;
		uint32 BoneCount = 0;
		uint32 FrameCount = 0;
		uint32 KeyCount = 0;
		bool bLooping = false;
	};

	struct FSequenceView
	{
		FSequenceView(const FSequenceRegistryKey& InKey, const FMapping& InMapping, const FSequence& InSequence)
			: Key(InKey)
			, Mapping(InMapping)
			, Sequence(InSequence)
		{}

		FSequenceRegistryKey Key;
		const FMapping& Mapping;
		const FSequence& Sequence;
	};

	struct FTrackHistory
	{
		static constexpr float Invalid = UE_MAX_FLT;

		void Update(const FAnimBankTrackPackedData& Data, int32 InCurrentSlot, bool& bUpdateCurrent, bool& bUpdatePrevious)
		{
			if (ItemIndex != Data.GetItemIndex())
			{
				ItemIndex = Data.GetItemIndex();
				Slots = FVector2f{ Invalid, Invalid };
			}

			bUpdateCurrent  = Slots[ InCurrentSlot] != Data.GetPosition();
			bUpdatePrevious = Slots[!InCurrentSlot] != Data.GetPreviousPosition() && Data.HasPreviousPosition();
			Slots[InCurrentSlot] = Data.GetPosition();

			if (Data.HasPreviousPosition())
			{
				Slots[!InCurrentSlot] = Data.GetPreviousPosition();
			}
		}

		int32 ItemIndex = INDEX_NONE;
		FVector2f Slots{ Invalid, Invalid };
	};

	struct FProxyData
	{
		FProxyData(FAnimBankTransformProviderProxy* Proxy);

		const FSequenceView& GetSequenceView(const FAnimBankTrackPackedData& Data) const
		{
			return Sequences[FMath::Clamp(Data.GetItemIndex(), 0, Sequences.Num() - 1)];
		}

		FAnimBankTrackPool& Tracks;
		TArray<FTrackHistory> TrackHistories;
		const FInstancedSkinningSceneExtensionProxy& SceneExtensionProxy;
		TArray<FSequenceView> Sequences;
		const EBoneTransformStorageMode TransformStorageMode;
		bool bHasEmptyTracks = false;
	};

	TMap<FMappingRegistryKey, TUniquePtr<FMapping>> MappingRegistry;
	TMap<FSequenceRegistryKey, TUniquePtr<FSequence>> SequenceRegistry;
	TMap<const FSkinningSceneExtensionProxy*, FProxyData> ProxyDataMap;

	TArray<FSequence*> SequencesToUpload;
	TArray<FMapping*> MappingsToUpload;

	FSpanAllocator SequenceTransformAllocator;
	FRDGAsyncScatterUploadBuffer SequenceTransformUploadBuffer;
	TPersistentByteAddressBuffer<FSequenceTransform> SequenceTransformPersistentBuffer;
	
	FRDGAsyncScatterUploadBuffer SkinningTransformUploadBuffer;
};