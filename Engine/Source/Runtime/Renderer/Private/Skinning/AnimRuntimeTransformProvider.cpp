// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimRuntimeTransformProvider.h"
#include "ScenePrivate.h"

IMPLEMENT_SCENE_EXTENSION(FAnimRuntimeTransformProvider);

static FGuid AnimRuntimeProviderId(ANIM_RUNTIME_TRANSFORM_PROVIDER_GUID);

FAnimRuntimeTransformProvider::FAnimRuntimeTransformProvider(FScene& InScene)
	: ISceneExtension(InScene)
{}

bool FAnimRuntimeTransformProvider::ShouldCreateExtension(FScene& InScene)
{
	return IsGPUSkinSceneExtensionEnabled() || (NaniteSkinnedMeshesSupported() && DoesRuntimeSupportNanite(GetFeatureLevelShaderPlatform(InScene.GetFeatureLevel()), true, true));
}

void FAnimRuntimeTransformProvider::InitExtension(FScene& InScene)
{
	if (auto TransformProvider = InScene.GetExtensionPtr<FSkinningTransformProvider>())
	{
		TransformProvider->RegisterProvider(
			AnimRuntimeProviderId,
			FSkinningTransformProvider::FOnProvideTransforms::CreateRaw(this, &FAnimRuntimeTransformProvider::ProvideAnimRuntimeTransforms),
			false /* Use skeleton batching */
		);
	}
}

void FAnimRuntimeTransformProvider::RegisterProxy(FAnimRuntimeTransformProviderProxy* Proxy)
{
	ProxyDataMap.Emplace(Proxy->GetSceneExtensionProxy(), Proxy);
}

void FAnimRuntimeTransformProvider::UnregisterProxy(FAnimRuntimeTransformProviderProxy* Proxy)
{
	ProxyDataMap.Remove(Proxy->GetSceneExtensionProxy());
}

void FAnimRuntimeTransformProvider::ProvideAnimRuntimeTransforms(FSkinningTransformProvider::FProviderContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimRuntimeTransformProvider::ProvideAnimRuntimeTransforms);
	RDG_EVENT_SCOPE(Context.GraphBuilder, "ProvideAnimRuntimeTransforms");

	uint32 GlobalTransformCount = 0;

	for (const FSkinningTransformProvider::FProviderIndirection Indirection : Context.Indirections)
	{
		const FInstancedSkinningSceneExtensionProxy* Proxy = static_cast<FInstancedSkinningSceneExtensionProxy*>(Context.Proxies[Indirection.Index]);
		FAnimRuntimeTransformProviderProxy* AnimRuntimeProxy = static_cast<FAnimRuntimeTransformProviderProxy*>(Proxy->GetTransformProviderProxy());

		const uint32 TransformCount = Proxy->GetMaxBoneTransformCount();
		const uint32 ActiveAnimationCount = AnimRuntimeProxy->GetTracks().GetNumActiveTracks();
		GlobalTransformCount += ActiveAnimationCount * TransformCount * 2; // Current and Previous
	}

	if (GlobalTransformCount == 0)
	{
		return;
	}

	FRDGBuilder& GraphBuilder = Context.GraphBuilder;
	FRDGAsyncScatterUploadBuffer& TransformUploadBuffer = *GraphBuilder.AllocObject<FRDGAsyncScatterUploadBuffer>();

	Context.ScatterUploadBuilder.AddPass(GraphBuilder, TransformUploadBuffer, Context.TransformBuffer, GlobalTransformCount, sizeof(FCompressedBoneTransform), TEXT("Skinning.AnimRuntimeTransforms"),
		[this, Indirections = Context.Indirections, Proxies = Context.Proxies, CurrentTime = Context.CurrentTime, GlobalTransformCount] (FRDGScatterUploader& ScatterUploader)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAnimRuntimeTransformProvider::ProvideAnimRuntimeTransformsTask);

		for (const FSkinningTransformProvider::FProviderIndirection Indirection : Indirections)
		{
			const FInstancedSkinningSceneExtensionProxy* Proxy = static_cast<FInstancedSkinningSceneExtensionProxy*>(Proxies[Indirection.Index]);
			FProxyData& ProxyData = ProxyDataMap.FindChecked(Proxy);
			FAnimRuntimeTrackPool& Tracks = ProxyData.Tracks;
			ProxyData.TrackHistories.SetNum(Tracks.GetNumTracks());

			const EBoneTransformStorageMode StorageMode = Proxy->GetBoneTransformStorageMode();
			const uint32 MaxTransformCount = Proxy->GetMaxBoneTransformCount();
			const uint32 UniqueAnimationCount = Proxy->GetUniqueAnimationCount();
			const uint32 MaxTotalTransformCount = MaxTransformCount * 2u; // Current and Previous

			const uint32 CurrentTransformIndex  = Indirection.CurrentTransformOffset  / sizeof(FCompressedBoneTransform);
			const uint32 PreviousTransformIndex = Indirection.PreviousTransformOffset / sizeof(FCompressedBoneTransform);
			const uint32 CurrentTransformSlot   = CurrentTransformIndex < PreviousTransformIndex ? 0 : 1;

			Tracks.EnumerateActiveTracks([&] (int32 TrackIndex)
			{
				FAnimRuntimeTrackData& Data = Tracks.GetData(TrackIndex);

				bool bNeedsCurrentUpdate, bNeedsPreviousUpdate;
				ProxyData.TrackHistories[TrackIndex].Update(Data, CurrentTransformSlot, bNeedsCurrentUpdate, bNeedsPreviousUpdate);

				// Dirty previous transforms means the whole allocation is dirty. Rebuild everything.
				const bool bDirtyTransformAllocation = EnumHasAnyFlags(Indirection.DirtyBoneTransforms, EDirtyBoneTransforms::Previous);
				bNeedsCurrentUpdate  |= bDirtyTransformAllocation;
				bNeedsPreviousUpdate |= bDirtyTransformAllocation;

				const auto FillTransforms = [&] (TArrayView<FCompressedBoneTransform> DstTransforms, const FAnimRuntimeTrackTransformData* SrcTransformData)
				{
					if (SrcTransformData)
					{
						TConstArrayView<FCompressedBoneTransform> SrcTransforms = SrcTransformData->GetTransforms();

						if (StorageMode == EBoneTransformStorageMode::BoneMap)
						{
							uint32 DstTransformIndex = 0;

							for (uint32 SrcTransformIndex : Proxy->GetBoneMap())
							{
								DstTransforms[DstTransformIndex] = SrcTransforms[SrcTransformIndex];
								DstTransformIndex++;
							}
						}
						else
						{
							for (int32 DstTransformIndex = 0; DstTransformIndex < DstTransforms.Num(); ++DstTransformIndex)
							{
								DstTransforms[DstTransformIndex] = SrcTransforms[DstTransformIndex];
							}
						}
					}
					else
					{
						if (StorageMode == EBoneTransformStorageMode::BoneMap)
						{
							uint32 DstTransformIndex = 0;

							for (uint32 SrcTransformIndex : Proxy->GetBoneMap())
							{
								SetCompressedBoneTransformIdentity(DstTransforms[DstTransformIndex]);
								DstTransformIndex++;
							}
						}
						else
						{
							for (int32 DstTransformIndex = 0; DstTransformIndex < DstTransforms.Num(); ++DstTransformIndex)
							{
								SetCompressedBoneTransformIdentity(DstTransforms[DstTransformIndex]);
							}
						}
					}
				};

				if (bNeedsCurrentUpdate)
				{
					TArrayView<FCompressedBoneTransform> DstCurrentTransforms = ScatterUploader.Add_GetRef<FCompressedBoneTransform>(CurrentTransformIndex + TrackIndex * MaxTotalTransformCount, MaxTransformCount);
					FillTransforms(DstCurrentTransforms, Data.Current);
				}

				if (bNeedsPreviousUpdate)
				{
					TArrayView<FCompressedBoneTransform> DstPreviousTransforms = ScatterUploader.Add_GetRef<FCompressedBoneTransform>(PreviousTransformIndex + TrackIndex * MaxTotalTransformCount, MaxTransformCount);
					FillTransforms(DstPreviousTransforms, Data.Previous ? Data.Previous : Data.Current);
				}
			});
		}
	});
}