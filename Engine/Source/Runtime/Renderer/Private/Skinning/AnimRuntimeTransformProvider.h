// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimRuntimeTransformProviderData.h"
#include "SkinningTransformProvider.h"
#include "SkinningSceneExtensionProxy.h"
#include "UnifiedBuffer.h"
#include "Math/IntVector.h"

class FAnimRuntimeTransformProvider
	: public ISceneExtension
	, public IAnimRuntimeTransformProvider
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FAnimRuntimeTransformProvider);

public:
	FAnimRuntimeTransformProvider(FScene& InScene);

	static bool ShouldCreateExtension(FScene& InScene);

	virtual void InitExtension(FScene& InScene) override;

	void RegisterProxy(FAnimRuntimeTransformProviderProxy* Proxy) override;
	void UnregisterProxy(FAnimRuntimeTransformProviderProxy* Proxy) override;

private:
	void ProvideAnimRuntimeTransforms(FSkinningTransformProvider::FProviderContext& Context);

	struct FTrackHistory
	{
		void Update(const FAnimRuntimeTrackData& Data, int32 InCurrentSlot, bool& bUpdateCurrent, bool& bUpdatePrevious)
		{
			bUpdateCurrent  = Data.Current  && Slots[ InCurrentSlot] != Data.Current->GetRevisionNumber();
			bUpdatePrevious = Data.Previous && Slots[!InCurrentSlot] != Data.Previous->GetRevisionNumber();

			if (bUpdateCurrent)
			{
				Slots[InCurrentSlot] = Data.Current->GetRevisionNumber();
			}

			if (bUpdatePrevious)
			{
				Slots[!InCurrentSlot] = Data.Previous->GetRevisionNumber();
			}
		}

		FUintVector2 Slots = FUintVector2::NoneValue;
	};

	struct FProxyData
	{
		FProxyData(FAnimRuntimeTransformProviderProxy* Proxy)
			: Tracks(Proxy->GetTracks())
		{}

		FAnimRuntimeTrackPool& Tracks;
		TArray<FTrackHistory> TrackHistories;
	};

	TMap<const FSkinningSceneExtensionProxy*, FProxyData> ProxyDataMap;
	FRDGAsyncScatterUploadBuffer SkinningTransformUploadBuffer;
};