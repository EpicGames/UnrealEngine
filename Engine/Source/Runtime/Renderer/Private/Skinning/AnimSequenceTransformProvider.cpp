// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceTransformProvider.h"
#include "Animation/AnimBoneCompressionCodec.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/Skeleton.h"
#include "Animation/BlendProfile.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "AnimationRuntime.h"
#include "AnimEncoding.h"
#include "ComponentRecreateRenderStateContext.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "RenderUtils.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneView.h"
#include "SkeletalRenderPublic.h"
#include "SkinningDefinitions.h"

IMPLEMENT_SCENE_EXTENSION(FAnimSequenceTransformProvider);

DECLARE_STATS_GROUP(TEXT("AnimSequenceTransformProvider"), STATGROUP_AnimSequenceTransformProvider, STATCAT_Advanced);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Sequences"), STAT_AnimSequenceTransformProvider_Sequences, STATGROUP_AnimSequenceTransformProvider);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Mappings"), STAT_AnimSequenceTransformProvider_Mappings, STATGROUP_AnimSequenceTransformProvider);

DECLARE_MEMORY_STAT(TEXT("Sequence Transform Buffer"), STAT_AnimSequenceTransformProvider_SequenceTransformMemory, STATGROUP_AnimSequenceTransformProvider);
DECLARE_MEMORY_STAT(TEXT("Retargeting Data Buffer"), STAT_AnimSequenceTransformProvider_RetargetingDataMemory, STATGROUP_AnimSequenceTransformProvider);
DECLARE_MEMORY_STAT(TEXT("Proxy Data"), STAT_AnimSequenceTransformProvider_ProxyDataMemory, STATGROUP_AnimSequenceTransformProvider);
DECLARE_MEMORY_STAT(TEXT("Mapping Data"), STAT_AnimSequenceTransformProvider_MappingDataMemory, STATGROUP_AnimSequenceTransformProvider);
DECLARE_MEMORY_STAT(TEXT("Sequence Data"), STAT_AnimSequenceTransformProvider_SequenceDataMemory, STATGROUP_AnimSequenceTransformProvider);

DECLARE_DWORD_COUNTER_STAT(TEXT("Active Tracks"), STAT_AnimSequenceTransformProvider_ActiveTracks, STATGROUP_AnimSequenceTransformProvider);
DECLARE_DWORD_COUNTER_STAT(TEXT("Evaluate Blocks"), STAT_AnimSequenceTransformProvider_EvaluateBlocks, STATGROUP_AnimSequenceTransformProvider);
DECLARE_DWORD_COUNTER_STAT(TEXT("Evaluate Blend Blocks"), STAT_AnimSequenceTransformProvider_EvaluateBlendBlocks, STATGROUP_AnimSequenceTransformProvider);
DECLARE_DWORD_COUNTER_STAT(TEXT("Current Transform Updates"), STAT_AnimSequenceTransformProvider_CurrentTransformUpdates, STATGROUP_AnimSequenceTransformProvider);
DECLARE_DWORD_COUNTER_STAT(TEXT("Previous Transform Updates"), STAT_AnimSequenceTransformProvider_PreviousTransformUpdates, STATGROUP_AnimSequenceTransformProvider);
DECLARE_DWORD_COUNTER_STAT(TEXT("Sequence Frame Uploads"), STAT_AnimSequenceTransformProvider_SequenceFrameUploads, STATGROUP_AnimSequenceTransformProvider);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Registered Proxies"), STAT_AnimSequenceTransformProvider_RegisteredProxies, STATGROUP_AnimSequenceTransformProvider);

DECLARE_GPU_STAT(AnimSequenceTransformProvider);

static FGuid AnimSequenceProviderId(ANIM_SEQUENCE_GPU_TRANSFORM_PROVIDER_GUID);

static TAutoConsoleVariable<float> CVarAnimSequenceTransformProviderTimeScale(
	TEXT("r.AnimSequenceTransformProvider.TimeScale"),
	1.0f,
	TEXT(""),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static constexpr uint32 Float4sPerTransformFull = sizeof(FAnimSequenceTransformProvider::FBoneTransformWithUniformScale) / sizeof(FVector4f);
static_assert(Float4sPerTransformFull == 2, "FBoneTransformWithUniformScale must be exactly 2 float4s");

static TAutoConsoleVariable<int32> CVarAnimSequenceTransformProviderUploadBudget(
	TEXT("r.AnimSequenceTransformProvider.UploadBudget"),
	32,
	TEXT("Maximum number of sequence frames to upload per frame across all sequences.\n")
	TEXT("Higher values reduce time to full quality but increase per-frame cost."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAnimSequenceTransformProviderMaxSampleRateLevels(
	TEXT("r.AnimSequenceTransformProvider.MaxSampleRateLevels"),
	3,
	TEXT("Number of sample rate levels for progressive sequence upload.\n")
	TEXT("1 = no progressive upload (full rate immediately)\n")
	TEXT("2 = two levels (half rate, then full)\n")
	TEXT("3 = three levels (quarter, half, full) [default]"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAnimSequenceTransformProviderForceSampleRateLevel(
	TEXT("r.AnimSequenceTransformProvider.ForceSampleRateLevel"),
	-1,
	TEXT("Force a minimum sample rate level for debugging.\n")
	TEXT("-1 = disabled (use actual completed level)\n")
	TEXT("0 = full rate\n")
	TEXT("1 = half rate\n")
	TEXT("2 = quarter rate"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAnimSequenceTransformProviderForceSampleRateMax(
	TEXT("r.AnimSequenceTransformProvider.ForceSampleRateMax"),
	30,
	TEXT("Force a maximum sample rate for every anim sequence in frames. Clamped from 0 to the sequence sample rate.\n"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarAnimSequenceTransformProviderHalfPrecision(
	TEXT("r.AnimSequenceTransformProvider.HalfPrecision"),
	false,
	TEXT("Pack sequence transforms as 16-bit floats. Halves sequence buffer memory with minor precision loss.\n")
	TEXT("Read-only: must be set at startup. Requires shader recompile in editor, cook for client builds."),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<bool> CVarAnimSequenceTransformProviderAnimCompactBones(
	TEXT("r.AnimSequenceTransformProvider.AnimCompactBones"),
	true,
	TEXT("Only upload animation bone tracks that are referenced by registered proxies.\n")
	TEXT("Reduces sequence buffer memory when the skeleton has fewer bones than the source animation."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

FFrameRate GetAnimSequenceSampleRate(const UAnimSequence* AnimSequence)
{
	const int32 ForcedSampleRate = CVarAnimSequenceTransformProviderForceSampleRateMax.GetValueOnRenderThread();
	if (ForcedSampleRate > 0)
	{
		FFrameRate SampleRate(ForcedSampleRate, 1);
		SampleRate.Numerator = FMath::Min(SampleRate.Numerator, AnimSequence->GetSamplingFrameRate().Numerator);
		return SampleRate;
	}
	return AnimSequence->GetSamplingFrameRate();
}

uint32 GetMaxSampleRateLevels(uint32 NumFrames)
{
	if (NumFrames <= 1)
	{
		return 1;
	}

	const uint32 MaxLevels = FMath::Max(1, CVarAnimSequenceTransformProviderMaxSampleRateLevels.GetValueOnRenderThread());

	return FMath::Min(MaxLevels, FMath::FloorLog2(NumFrames - 1) + 1);
}

uint32 FAnimSequenceTransformProvider::FSequence::GetSampleStep() const
{
	const uint32 ForcedSampleRateLevel = CVarAnimSequenceTransformProviderForceSampleRateLevel.GetValueOnRenderThread();
	uint32 SampleRateLevel = ForcedSampleRateLevel != INDEX_NONE ? ForcedSampleRateLevel : CurrentSampleRateLevel;
	return 1u << FMath::Clamp<uint32>(SampleRateLevel, CurrentSampleRateLevel, NumSampleRateLevels - 1);
}

void FAnimSequenceTransformProvider::FSequence::GetKeyIndicesFromTime(int32& OutKey0, int32& OutKey1, float& OutAlpha, float Time, EAnimSequenceTrackLoopMode LoopMode) const
{
	const auto AlignDown = [] (int32 Value, int32 Alignment)
	{
		return (Value / Alignment) * Alignment;
	};

	const int32 Step = GetSampleStep();
	const float ExactFrame = Time * NumFrames / PlayLength;
	const int32 LastAlignedFrame = AlignDown((int32)NumFrames - 1, Step);

	OutKey0 = FMath::Min(AlignDown((int32)ExactFrame, Step), LastAlignedFrame);
	OutKey1 = OutKey0 + Step;

	if (OutKey1 > LastAlignedFrame)
	{
		OutKey1 = (LoopMode == EAnimSequenceTrackLoopMode::Loop) ? 0 : LastAlignedFrame;
	}

	OutAlpha = (OutKey1 != OutKey0) ? (ExactFrame - (float)OutKey0) / (float)Step : 0.0f;
}

class FEvaluateConcatenateScatterCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FEvaluateConcatenateScatterCS);
	SHADER_USE_PARAMETER_STRUCT(FEvaluateConcatenateScatterCS, FGlobalShader);

	class FBonesPerGroupDim : SHADER_PERMUTATION_SPARSE_INT("BONES_PER_GROUP", 64, 128, 256, 512, 1024);
	class FSingleLayerDim : SHADER_PERMUTATION_BOOL("SINGLE_LAYER");
	class FMeshSpaceBlend : SHADER_PERMUTATION_BOOL("HAS_MESH_SPACE_LAYERS");
	using FPermutationDomain = TShaderPermutationDomain<FBonesPerGroupDim, FSingleLayerDim, FMeshSpaceBlend>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		const int32 BonesPerGroup = PermutationVector.Get<FBonesPerGroupDim>();
		const bool bSingleLayer = PermutationVector.Get<FSingleLayerDim>();
		const bool bMeshSpace = PermutationVector.Get<FMeshSpaceBlend>();

		// Single-layer can't have mesh-space (layer 0 is always local-space override)
		if (bSingleLayer && bMeshSpace)
		{
			return false;
		}

		// Mesh-space requires 2x groupshared
		if (bMeshSpace && BonesPerGroup > SKINNING_MAX_BONES_PER_GROUP_MESHSPACE)
		{
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
		OutEnvironment.SetDefine(TEXT("HALF_PRECISION_SEQUENCES"), CVarAnimSequenceTransformProviderHalfPrecision.GetValueOnAnyThread() ? 1 : 0);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumTotalHeaders)
		SHADER_PARAMETER(uint32, HeaderBaseOffset)
		SHADER_PARAMETER(uint32, SlotsPerInstance)
		SHADER_PARAMETER(uint32, NumCompactBones)
		SHADER_PARAMETER(uint32, NumLayers)
		SHADER_PARAMETER(uint32, MeshSpaceMask)
		SHADER_PARAMETER(uint32, LayerModeMask)

		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, RetargetingDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector4f>, SequenceTransformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FEvaluateLayerHeader>, LayerHeaderBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FSamplePoseData>, SamplePoseBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BoneMapBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BoneMaskBuffer)
		SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_UAV(SkinningTransformBufferUAV)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FEvaluateConcatenateScatterCS, "/Engine/Private/Skinning/AnimSequenceTransformProvider.usf", "EvaluateConcatenateScatterCS", SF_Compute);

FAnimSequenceTransformProvider::FProxyUserData::FProxyUserData(FAnimSequenceTransformProviderProxy* Proxy)
	: RenderData(Proxy->GetRenderData())
	, SceneExtensionProxy(*Proxy->GetSceneExtensionProxy())
{
	check(RenderData);
}

FAnimSequenceTransformProvider::FAnimSequenceTransformProvider(FScene& InScene)
	: ISceneExtension(InScene)
	, SequenceTransformPersistentBuffer(0, TEXT("AnimSequenceTransformProvider.SequenceTransformBuffer"))
	, RetargetingDataPersistentBuffer(0, TEXT("AnimSequenceTransformProvider.RetargetingDataBuffer"))
	, BoneHierarchyPersistentBuffer(0, TEXT("AnimSequenceTransformProvider.BoneHierarchyBuffer"))
	, BoneMaskPersistentBuffer(1024u, TEXT("AnimSequenceTransformProvider.BoneMaskBuffer"))
	, bHalfPrecisionSequences(CVarAnimSequenceTransformProviderHalfPrecision.GetValueOnAnyThread())
{}

void FAnimSequenceTransformProvider::WriteBoneTransform(TArrayView<FVector4f>& DstFloat4s, uint32 Index, const FBoneTransformWithUniformScale& Transform) const
{
	if (bHalfPrecisionSequences)
	{
		FVector4f Packed;
		Packed.X = BitCast<float>(uint32(FFloat16(Transform.Rotation[0]).Encoded) | (uint32(FFloat16(Transform.Rotation[1]).Encoded) << 16));
		Packed.Y = BitCast<float>(uint32(FFloat16(Transform.Rotation[2]).Encoded) | (uint32(FFloat16(Transform.Rotation[3]).Encoded) << 16));
		Packed.Z = BitCast<float>(uint32(FFloat16(Transform.Translation[0]).Encoded) | (uint32(FFloat16(Transform.Translation[1]).Encoded) << 16));
		Packed.W = BitCast<float>(uint32(FFloat16(Transform.Translation[2]).Encoded) | (uint32(FFloat16(Transform.Scale).Encoded) << 16));
		DstFloat4s[Index] = Packed;
	}
	else
	{
		DstFloat4s[Index * 2]     = FVector4f(Transform.Rotation[0], Transform.Rotation[1], Transform.Rotation[2], Transform.Rotation[3]);
		DstFloat4s[Index * 2 + 1] = FVector4f(Transform.Translation[0], Transform.Translation[1], Transform.Translation[2], Transform.Scale);
	}
}

bool FAnimSequenceTransformProvider::ShouldCreateExtension(FScene& InScene)
{
	return IsGPUSkinSceneExtensionEnabled();
}

void FAnimSequenceTransformProvider::InitExtension(FScene& InScene)
{
	if (auto TransformProvider = InScene.GetExtensionPtr<FSkinningTransformProvider>())
	{
		TransformProvider->RegisterProvider(
			AnimSequenceProviderId,
			FSkinningTransformProvider::FOnProvideTransforms::CreateRaw(this, &FAnimSequenceTransformProvider::ProvideTransforms),
			false /* Use skeleton batching */
		);
	}
}

void FAnimSequenceTransformProvider::BuildBoneTables(
	FBoneTables& BoneTables,
	const USkinnedAsset* SkinnedMesh,
	const FSkeletalMeshLODRenderData& LODRenderData,
	TConstArrayView<FBoneIndexType> BoneMap,
	bool bHasSocketsInBoneMap)
{
	const USkeleton* TargetSkeleton = SkinnedMesh->GetSkeleton();
	const FReferenceSkeleton& MeshRefSkeleton = SkinnedMesh->GetRefSkeleton();
	const FReferenceSkeleton& TargetSkeletonRefSkeleton = TargetSkeleton->GetReferenceSkeleton();
	const uint32 NumMeshBones = MeshRefSkeleton.GetNum();

	// Start with the LOD's RequiredBones. Optionally inject force-animated socket bones + parent chains.
	TArray<FBoneIndexType> ExtendedRequiredBones;
	bool bUseExtended = false;

	if (bHasSocketsInBoneMap)
	{
		TArray<FBoneIndexType> SocketBones;
		FSkinningSceneExtensionProxy::GetSocketBoneIndices(SkinnedMesh, SocketBones);

		if (SocketBones.Num() > 0)
		{
			ExtendedRequiredBones = LODRenderData.RequiredBones;
			for (const FBoneIndexType BoneIndex : SocketBones)
			{
				ExtendedRequiredBones.AddUnique(BoneIndex);
			}
			ExtendedRequiredBones.Sort();
			FAnimationRuntime::EnsureParentsPresent(ExtendedRequiredBones, MeshRefSkeleton);
			ExtendedRequiredBones.Sort();
			bUseExtended = true;
		}
	}

	TConstArrayView<FBoneIndexType> RequiredBones = bUseExtended ? TConstArrayView<FBoneIndexType>(ExtendedRequiredBones) : TConstArrayView<FBoneIndexType>(LODRenderData.RequiredBones);
	const uint32 NumCompactBones = RequiredBones.Num();

	BoneTables.MeshToCompact.Init(INVALID_BONE_INDEX, NumMeshBones);
	BoneTables.CompactToMesh.SetNumUninitialized(NumCompactBones);
	BoneTables.CompactToParent.SetNumUninitialized(NumCompactBones);
	BoneTables.CompactToTargetSkeleton.SetNumUninitialized(NumCompactBones);
	BoneTables.NumMeshBones = NumMeshBones;
	BoneTables.NumCompactBones = NumCompactBones;

	for (uint32 CompactBoneIndex = 0; CompactBoneIndex < NumCompactBones; ++CompactBoneIndex)
	{
		BoneTables.MeshToCompact[RequiredBones[CompactBoneIndex]] = CompactBoneIndex;
	}

	for (uint32 CompactBoneIndex = 0; CompactBoneIndex < NumCompactBones; ++CompactBoneIndex)
	{
		const uint16 MeshBoneIndex = RequiredBones[CompactBoneIndex];
		BoneTables.CompactToMesh[CompactBoneIndex] = MeshBoneIndex;

		const uint16 MeshParentBoneIndex = MeshRefSkeleton.GetParentIndex(MeshBoneIndex);
		const uint16 CompactParentBoneIndex = MeshParentBoneIndex != INVALID_BONE_INDEX
			? BoneTables.MeshToCompact[MeshParentBoneIndex]
			: INVALID_BONE_INDEX;
		BoneTables.CompactToParent[CompactBoneIndex] = CompactParentBoneIndex;

		const FName BoneName = MeshRefSkeleton.GetBoneName(MeshBoneIndex);
		const uint16 TargetSkeletonBoneIndex = TargetSkeletonRefSkeleton.FindBoneIndex(BoneName);
		BoneTables.CompactToTargetSkeleton[CompactBoneIndex] = TargetSkeletonBoneIndex;
	}

#if DO_CHECK
	// Validate that every bone in the BoneMap has a corresponding compact bone.
	for (int32 BoneMapIndex = 0; BoneMapIndex < BoneMap.Num(); ++BoneMapIndex)
	{
		const uint16 MeshBoneIndex = BoneMap[BoneMapIndex];

		if (MeshBoneIndex < NumMeshBones && BoneTables.MeshToCompact[MeshBoneIndex] == INVALID_BONE_INDEX)
		{
			UE_LOG(LogAnimation, Error,
				TEXT("AnimSequenceTransformProvider: BoneMap[%d] references mesh bone %u (%s) which is not in RequiredBones. "
					 "Mesh: %s. This bone's transform will not be written, causing visual artifacts."),
				BoneMapIndex, MeshBoneIndex,
				*MeshRefSkeleton.GetBoneName(MeshBoneIndex).ToString(),
				*SkinnedMesh->GetPathName());
		}
	}
#endif
}

void FAnimSequenceTransformProvider::BuildRetargetingTables(
	FRetargetingTables& RetargetingTables,
	USkeleton* AnimSkeleton,
	const USkinnedAsset* MeshAsset,
	const FBoneTables& BoneTables,
	FAnimCompactBoneTable& AnimCompactBoneTable)
{
	const USkeleton* TargetSkeleton = MeshAsset->GetSkeleton();
	const FReferenceSkeleton& MeshRefSkeleton = MeshAsset->GetRefSkeleton();
	const FReferenceSkeleton& TargetSkeletonRefSkeleton = TargetSkeleton->GetReferenceSkeleton();

	const TArray<FTransform>& AnimRefPose = AnimSkeleton->GetRefLocalPoses();
	const TArray<FTransform>& MeshRefPose = MeshRefSkeleton.GetRefBonePose();
	const bool bUseSourceRetargetModes = TargetSkeleton->GetUseRetargetModesFromCompatibleSkeleton();

	const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(AnimSkeleton, TargetSkeleton);
	const bool bRequiresRefPoseRetarget = SkeletonRemapping.IsValid() && SkeletonRemapping.RequiresReferencePoseRetarget();
	const uint32 NumAnimBones = AnimSkeleton->GetReferenceSkeleton().GetNum();

	RetargetingTables.Mode.Reset(BoneTables.NumCompactBones);
	RetargetingTables.Transform.Reset(BoneTables.NumCompactBones);
	RetargetingTables.AnimToCompact.Init(INVALID_BONE_INDEX, NumAnimBones);
	RetargetingTables.bRequiresRefPoseRetarget = bRequiresRefPoseRetarget;

	if (bRequiresRefPoseRetarget)
	{
		RetargetingTables.RefPoseRetargetQuats.Reset(BoneTables.NumCompactBones);
	}

	for (uint32 CompactBoneIndex = 0; CompactBoneIndex < BoneTables.NumCompactBones; ++CompactBoneIndex)
	{
		EBoneTranslationRetargetingMode::Type Mode = EBoneTranslationRetargetingMode::Animation;

		const uint16 MeshBoneIndex = BoneTables.CompactToMesh[CompactBoneIndex];
		const uint16 TargetSkeletonBoneIndex = BoneTables.CompactToTargetSkeleton[CompactBoneIndex];

		const uint16 AnimBoneIndex = TargetSkeletonBoneIndex != INVALID_BONE_INDEX && SkeletonRemapping.IsValid()
			? SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex)
			: TargetSkeletonBoneIndex;

		const FTransform& MeshRefTransform = MeshRefPose[MeshBoneIndex];
		TTuple<FQuat, FQuat> RefPoseRetargetQuats(FQuat::Identity, FQuat::Identity);
		FTransform3f Transform;

		if (AnimBoneIndex != INVALID_BONE_INDEX)
		{
			if (bRequiresRefPoseRetarget)
			{
				RefPoseRetargetQuats = SkeletonRemapping.GetRetargetingQuaternions(TargetSkeletonBoneIndex);
			}

			Mode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
				bUseSourceRetargetModes,
				AnimBoneIndex,
				TargetSkeletonBoneIndex,
				AnimSkeleton,
				TargetSkeleton,
				false);

			if (MeshBoneIndex == 0)
			{
				// Lock root bone translation to disable root motion.
				Mode = EBoneTranslationRetargetingMode::Skeleton;
			}

			const FTransform& AnimRefTransform = AnimRefPose[AnimBoneIndex];

			switch (Mode)
			{
				case EBoneTranslationRetargetingMode::Animation:
					break;
				case EBoneTranslationRetargetingMode::Skeleton:
					Transform.SetTranslation(FVector3f(MeshRefTransform.GetTranslation()));
					if (MeshBoneIndex == 0)
					{
						Transform.SetRotation(FQuat4f(MeshRefTransform.GetRotation()));
					}
					break;
				case EBoneTranslationRetargetingMode::AnimationScaled:
				{
					const float AnimTranslationLength = AnimRefTransform.GetTranslation().Size();
					const float MeshTranslationLength = MeshRefTransform.GetTranslation().Size();
					if (AnimTranslationLength > UE_KINDA_SMALL_NUMBER)
					{
						Transform.SetScale3D(FVector3f(MeshTranslationLength / AnimTranslationLength));
					}
				}
				break;
				case EBoneTranslationRetargetingMode::AnimationRelative:
				{
					FTransform RetargetedAnimRefTransform = AnimRefTransform;

					if (bRequiresRefPoseRetarget)
					{
						FQuat RetargetedRotation = RefPoseRetargetQuats.Get<0>() * AnimRefTransform.GetRotation() * RefPoseRetargetQuats.Get<1>();
						RetargetedRotation.Normalize();
						RetargetedAnimRefTransform.SetRotation(RetargetedRotation);
						RetargetedAnimRefTransform.SetTranslation(RefPoseRetargetQuats.Get<0>().RotateVector(AnimRefTransform.GetTranslation()));
					}

					FQuat DeltaRotation = RetargetedAnimRefTransform.GetRotation().Inverse() * MeshRefTransform.GetRotation();
					DeltaRotation.Normalize();

					const float ReciprocalScale = FMath::Abs(RetargetedAnimRefTransform.GetScale3D().X) < UE_SMALL_NUMBER ? 0.0f : RetargetedAnimRefTransform.GetScale3D().X;

					Transform.SetRotation(FQuat4f(DeltaRotation));
					Transform.SetTranslation(FVector3f(MeshRefTransform.GetTranslation() - RetargetedAnimRefTransform.GetTranslation()));
					Transform.SetScale3D(FVector3f(MeshRefTransform.GetScale3D().X * ReciprocalScale));
				}
				break;
				case EBoneTranslationRetargetingMode::OrientAndScale:
				{
					const FVector AnimTranslation = AnimRefTransform.GetTranslation();
					const FVector MeshTranslation = MeshRefTransform.GetTranslation();

					if (!AnimTranslation.Equals(MeshTranslation, BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION))
					{
						const float AnimTranslationLength = AnimTranslation.Size();
						const float MeshTranslationLength = MeshTranslation.Size();

						if (!FMath::IsNearlyZero(AnimTranslationLength * MeshTranslationLength))
						{
							const FVector AnimDir = AnimTranslation / AnimTranslationLength;
							const FVector MeshDir = MeshTranslation / MeshTranslationLength;
							const FQuat DeltaRotation = FQuat::FindBetweenNormals(AnimDir, MeshDir);
							const float Scale = MeshTranslationLength / AnimTranslationLength;

							Transform.SetRotation(FQuat4f(DeltaRotation));
							Transform.SetScale3D(FVector3f(Scale));
						}
					}
				}
				break;
			}

			RetargetingTables.AnimToCompact[AnimBoneIndex] = CompactBoneIndex;
		}
		else
		{
			Transform = FTransform3f(MeshRefTransform);
		}

		if (bRequiresRefPoseRetarget)
		{
			RetargetingTables.RefPoseRetargetQuats.Emplace(TTuple<FQuat4f, FQuat4f>(RefPoseRetargetQuats.Get<0>(), RefPoseRetargetQuats.Get<1>()));
		}

		RetargetingTables.Mode.Emplace(Mode);
		RetargetingTables.Transform.Emplace(Transform);
		RetargetingTables.CompactToAnim.Emplace(AnimBoneIndex);
	}

	if (AccumulateAnimCompactBones(RetargetingTables, BoneTables.NumCompactBones, AnimCompactBoneTable))
	{
		AnimCompactBoneTablesToUpload.Add(&AnimCompactBoneTable);
	}

	RetargetingTables.AnimCompactBoneTable = &AnimCompactBoneTable;
}

bool FAnimSequenceTransformProvider::AccumulateAnimCompactBones(const FRetargetingTables& RetargetingTables, uint32 NumCompactBones, FAnimCompactBoneTable& Remap)
{
	if (!CVarAnimSequenceTransformProviderAnimCompactBones.GetValueOnRenderThread())
	{
		const bool bWasAlreadyFull = (Remap.NumAnimCompactBones == Remap.NumAnimBones);
		if (!bWasAlreadyFull)
		{
			Remap.UsedAnimBones.Init(true, Remap.NumAnimBones);
			Remap.bDirty = true;
			return true;
		}
		return false;
	}

	bool bAddedNewBones = false;
	for (uint32 CompactBoneIndex = 0; CompactBoneIndex < NumCompactBones; ++CompactBoneIndex)
	{
		const uint16 AnimBone = RetargetingTables.CompactToAnim[CompactBoneIndex];
		if (AnimBone != INVALID_BONE_INDEX && !Remap.UsedAnimBones[AnimBone])
		{
			Remap.UsedAnimBones[AnimBone] = true;
			bAddedNewBones = true;
		}
	}

	if (bAddedNewBones)
	{
		Remap.bDirty = true;
	}

	return bAddedNewBones;
}

void FAnimSequenceTransformProvider::RegisterProxy(FAnimSequenceTransformProviderProxy* Proxy)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProvider::RegisterProxy);
	TConstArrayView<FAnimSequenceTransformProviderSequence> ProviderSequences = Proxy->GetSequences();
	FInstancedSkinningSceneExtensionProxy* ExtensionProxy = Proxy->GetSceneExtensionProxy();
	const USkinnedAsset* SkinnedAsset = ExtensionProxy->GetSkinnedAsset();
	const FSkeletalMeshRenderData* MeshRenderData = SkinnedAsset->GetResourceForRendering();
	const FSkeletalMeshObject* MeshObject = ExtensionProxy->GetMeshObject();
	TConstArrayView<FMatrix44f> InvGlobalRefPoseTransforms = SkinnedAsset->GetRefBasesInvMatrix();

	FProxyUserData* ProxyUserData = new FProxyUserData(Proxy);
	check(Proxy->GetNumLayers() >= 1);
	ProxyUserData->Sequences.Reserve(ProviderSequences.Num());
	ProxyUserData->NumMeshBones = InvGlobalRefPoseTransforms.Num();

	// Nanite performs its own per-cluster LOD selection at render time and always reads LOD0
	// bone transforms. For Nanite proxies, build only LOD0 and treat MinLODLevel as 0 — evaluating
	// at any other LOD would leave bones that LOD0 needs but the active LOD doesn't process stuck
	// at stale values, causing visual artifacts.
	const bool bIsNaniteMesh = MeshObject->IsNaniteMesh();
	ProxyUserData->bIsNaniteMesh = bIsNaniteMesh;

	const int32 NumLODs = bIsNaniteMesh ? 1 : MeshRenderData->LODRenderData.Num();
	const int32 MinLODIndex = bIsNaniteMesh ? 0 : (int32)MeshObject->MinLODLevel;
	ProxyUserData->MinLODLevel = MinLODIndex;

	ProxyUserData->LODs.SetNum(NumLODs);
	ProxyUserData->SequenceToMappingTable.Reserve(ProviderSequences.Num() * NumLODs);

	for (int32 LODIndex = MinLODIndex; LODIndex < NumLODs; ++LODIndex)
	{
		FProxyUserData::FLOD& LOD = ProxyUserData->LODs[LODIndex];
		BuildBoneTables(LOD.BoneTables, SkinnedAsset, MeshRenderData->LODRenderData[LODIndex], ExtensionProxy->GetBoneMap(), ExtensionProxy->HasSocketsInBoneMap());
		ComputeGroupParams(LOD.BoneTables.NumCompactBones, LOD.GroupSize, LOD.InstancesPerGroup, LOD.SlotsPerInstance);
	}

	for (const FAnimSequenceTransformProviderSequence& ProviderSequence : ProviderSequences)
	{
		const UAnimSequence* AnimSequence = ProviderSequence.Sequence.Get();
		USkeleton* SourceSkeleton = AnimSequence->GetSkeleton();

		const FSequenceRegistryKey SequenceRegistryKey(AnimSequence);

		TUniquePtr<FSequence>& Sequence = SequenceRegistry.FindOrAdd(SequenceRegistryKey);
		const bool bIsNewSequence = !Sequence;
		if (bIsNewSequence)
		{
			Sequence = MakeUnique<FSequence>();

			Sequence->RegistryKey            = SequenceRegistryKey;
			Sequence->Sequence               = AnimSequence;
			Sequence->Skeleton               = SourceSkeleton;
			Sequence->PlayLength             = AnimSequence->GetPlayLength();
			Sequence->SampleRate             = GetAnimSequenceSampleRate(AnimSequence);
			Sequence->NumFrames              = FMath::Min(Sequence->SampleRate.AsFrameTime(Sequence->PlayLength).RoundToFrame().Value, SKINNING_MAX_KEY_INDEX);
			Sequence->NumAnimBones           = SourceSkeleton->GetReferenceSkeleton().GetNum();
			Sequence->NumSampleRateLevels    = GetMaxSampleRateLevels(Sequence->NumFrames);
			Sequence->InitUploadState();
		}

		TUniquePtr<FAnimCompactBoneTable>& AnimCompactBoneTable = AnimCompactBoneTableRegistry.FindOrAdd(SourceSkeleton);
		if (!AnimCompactBoneTable)
		{
			AnimCompactBoneTable = MakeUnique<FAnimCompactBoneTable>();
			AnimCompactBoneTable->Init(Sequence->NumAnimBones);
		}

		if (!Sequence->AnimCompactBoneTable)
		{
			Sequence->AnimCompactBoneTable = AnimCompactBoneTable.Get();
			AnimCompactBoneTable->ReferencingSequences.Add(Sequence.Get());
		}

		++AnimCompactBoneTable->RefCount;
		++Sequence->RefCount;

		if (bIsNewSequence)
		{
			check(!SequencesToUpload.Contains(Sequence.Get()));
			SequencesToUpload.Add(Sequence.Get());

			TotalSequenceMemory += Sequence->GetAllocatedSize();
			INC_DWORD_STAT(STAT_AnimSequenceTransformProvider_Sequences);
		}

		for (int32 LODIndex = 0; LODIndex < MinLODIndex; ++LODIndex)
		{
			ProxyUserData->SequenceToMappingTable.Emplace(nullptr);
		}

		for (int32 LODIndex = MinLODIndex; LODIndex < NumLODs; ++LODIndex)
		{
			const FMappingRegistryKey MappingRegistryKey(SourceSkeleton, SkinnedAsset, LODIndex, ExtensionProxy->HasSocketsInBoneMap());
			TUniquePtr<FMapping>& Mapping = MappingRegistry.FindOrAdd(MappingRegistryKey);
			if (!Mapping)
			{
				Mapping = MakeUnique<FMapping>();

				const FProxyUserData::FLOD& LOD = ProxyUserData->LODs[LODIndex];
				Mapping->SourceSkeleton  = SourceSkeleton;
				Mapping->SkinnedAsset    = SkinnedAsset;
				Mapping->NumMeshBones    = ProxyUserData->NumMeshBones;
				Mapping->LODIndex        = LODIndex;
				Mapping->bHasSocketsInBoneMap = ExtensionProxy->HasSocketsInBoneMap();
				Mapping->NumCompactBones = LOD.BoneTables.NumCompactBones;
				BuildRetargetingTables(Mapping->RetargetingTables, SourceSkeleton, SkinnedAsset, LOD.BoneTables, *Sequence->AnimCompactBoneTable);
				Sequence->AnimCompactBoneTable->ReferencingMappings.Add(Mapping.Get());

				Mapping->InvGlobalRefPoseTransforms.Reserve(Mapping->NumCompactBones);
				for (uint32 CompactBoneIndex = 0; CompactBoneIndex < Mapping->NumCompactBones; ++CompactBoneIndex)
				{
					const uint32 MeshBoneIndex = LOD.BoneTables.CompactToMesh[CompactBoneIndex];
					const FMatrix44f& InvGlobalRefPose = InvGlobalRefPoseTransforms[MeshBoneIndex];
					Mapping->InvGlobalRefPoseTransforms.Emplace(InvGlobalRefPose);
				}

				MappingsToUpload.Add(Mapping.Get());

				TotalMappingMemory += Mapping->GetAllocatedSize();
				INC_DWORD_STAT(STAT_AnimSequenceTransformProvider_Mappings);
			}

			if (AccumulateAnimCompactBones(Mapping->RetargetingTables, Mapping->NumCompactBones, *Sequence->AnimCompactBoneTable))
			{
				AnimCompactBoneTablesToUpload.Add(Sequence->AnimCompactBoneTable);
			}
			Sequence->AnimCompactBoneTable->ReferencingMappings.Add(Mapping.Get());

			++Mapping->RefCount;
			ProxyUserData->Mappings.AddUnique(Mapping.Get());
			ProxyUserData->SequenceToMappingTable.Emplace(Mapping.Get());
		}

		ProxyUserData->Sequences.Emplace(Sequence.Get());
	}

	const USkeleton* Skeleton = SkinnedAsset->GetSkeleton();
	FAnimSequenceTransformProviderRenderData& RenderData = *ProxyUserData->GetRenderData();
	const int32 NumLayers = RenderData.GetNumLayers();

	for (FProxyUserData::FLOD& LOD : ProxyUserData->LODs)
	{
		LOD.BoneMaskBufferOffsets.SetNum(NumLayers);
		for (int32 LayerIndex = 0; LayerIndex < NumLayers; LayerIndex++)
		{
			LOD.BoneMaskBufferOffsets[LayerIndex] = FProxyUserData::InvalidBoneMaskOffset;
		}
	}

	ProxyUserData->BlendProfiles.SetNum(NumLayers);

	for (int32 LayerIndex = 0; LayerIndex < NumLayers; LayerIndex++)
	{
		const FAnimSequenceTransformProviderLayer& Layer = RenderData.GetLayers()[LayerIndex].Layer;
		if (Layer.BoneMaskProfileName.IsNone())
		{
			continue;
		}

		const UBlendProfile* BlendProfile = Skeleton ? const_cast<USkeleton*>(Skeleton)->GetBlendProfile(Layer.BoneMaskProfileName) : nullptr;
		if (!BlendProfile || BlendProfile->GetMode() != EBlendProfileMode::BlendMask)
		{
			continue;
		}

		ProxyUserData->BlendProfiles[LayerIndex] = BlendProfile;

		for (int32 LODIndex = ProxyUserData->MinLODLevel; LODIndex < ProxyUserData->LODs.Num(); LODIndex++)
		{
			FProxyUserData::FLOD& LOD = ProxyUserData->LODs[LODIndex];
			if (!LOD.IsValid())
			{
				continue;
			}

			const uint32 NumCompactBones = LOD.BoneTables.NumCompactBones;
			const uint32 AllocationOffset = BoneMaskAllocator.Allocate(NumCompactBones);
			LOD.BoneMaskBufferOffsets[LayerIndex] = AllocationOffset * sizeof(float);
		}

		ProxyUserDatasToUpload.Add(ProxyUserData);
	}

	ExtensionProxy->SetUserData(ProxyUserData);

	TotalProxyMemory += ProxyUserData->GetAllocatedSize();
	INC_DWORD_STAT(STAT_AnimSequenceTransformProvider_RegisteredProxies);
}

void FAnimSequenceTransformProvider::UnregisterProxy(FAnimSequenceTransformProviderProxy* Proxy)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProvider::UnregisterProxy);
	FInstancedSkinningSceneExtensionProxy* ExtensionProxy = Proxy->GetSceneExtensionProxy();

	FProxyUserData* ProxyUserData = static_cast<FProxyUserData*>(ExtensionProxy->GetUserData());
	check(ProxyUserData);
	ExtensionProxy->SetUserData(nullptr);

	TotalProxyMemory -= ProxyUserData->GetAllocatedSize();
	DEC_DWORD_STAT(STAT_AnimSequenceTransformProvider_RegisteredProxies);

	for (FProxyUserData::FLOD& LOD : ProxyUserData->LODs)
	{
		for (int32 LayerIndex = 0; LayerIndex < LOD.BoneMaskBufferOffsets.Num(); LayerIndex++)
		{
			if (LOD.BoneMaskBufferOffsets[LayerIndex] != FProxyUserData::InvalidBoneMaskOffset)
			{
				BoneMaskAllocator.Free(LOD.BoneMaskBufferOffsets[LayerIndex] / sizeof(float), LOD.BoneTables.NumCompactBones);
			}
		}
	}

	for (FProxyUserData::FLOD& LOD : ProxyUserData->LODs)
	{
		if (LOD.BoneMapBufferAllocation.IsValid())
		{
			BoneHierarchyAllocator.Free(LOD.BoneMapBufferAllocation.Offset, LOD.BoneMapBufferAllocation.Count);
			LOD.BoneMapBufferAllocation = {};
		}
	}

	for (FMapping* Mapping : ProxyUserData->SequenceToMappingTable)
	{
		if (!Mapping)
		{
			continue;
		}

		check(Mapping && Mapping->RefCount > 0);
		--Mapping->RefCount;

		if (Mapping->RefCount == 0)
		{
			if (Mapping->MappingTransformBufferAllocation.IsValid())
			{
				SequenceTransformAllocator.Free(Mapping->MappingTransformBufferAllocation.Offset, Mapping->MappingTransformBufferAllocation.Count);
				Mapping->MappingTransformBufferAllocation = {};
				Mapping->InvGlobalRefPoseTransformBufferOffset = 0;
			}

			if (Mapping->RetargetingDataBufferAllocation.IsValid())
			{
				RetargetingDataAllocator.Free(Mapping->RetargetingDataBufferAllocation.Offset, Mapping->RetargetingDataBufferAllocation.Count);
				Mapping->RetargetingDataBufferAllocation = {};
			}

			if (Mapping->RetargetingTables.AnimCompactBoneTable)
			{
				const_cast<FAnimCompactBoneTable*>(Mapping->RetargetingTables.AnimCompactBoneTable)->ReferencingMappings.Remove(Mapping);
			}

			TotalMappingMemory -= Mapping->GetAllocatedSize();
			MappingsToUpload.Remove(Mapping);
			MappingRegistry.Remove(FMappingRegistryKey(Mapping->SourceSkeleton, Mapping->SkinnedAsset, Mapping->LODIndex, Mapping->bHasSocketsInBoneMap));
			Mapping = nullptr;

			DEC_DWORD_STAT(STAT_AnimSequenceTransformProvider_Mappings);
		}
	}

	for (FSequence* Sequence : ProxyUserData->Sequences)
	{
		check(Sequence && Sequence->RefCount > 0);
		--Sequence->RefCount;

		FAnimCompactBoneTable* AnimCompactBoneTable = Sequence->AnimCompactBoneTable;
		const USkeleton* Skeleton = Sequence->Skeleton;

		if (Sequence->RefCount == 0)
		{
			if (Sequence->BufferAllocation.IsValid())
			{
				SequenceTransformAllocator.Free(Sequence->BufferAllocation.Offset, Sequence->BufferAllocation.Count);
				Sequence->BufferAllocation = {};
			}

			if (AnimCompactBoneTable)
			{
				AnimCompactBoneTable->ReferencingSequences.Remove(Sequence);
			}

			TotalSequenceMemory -= Sequence->GetAllocatedSize();
			SequencesToUpload.Remove(Sequence);
			SequenceRegistry.Remove(Sequence->RegistryKey);
			Sequence = nullptr;

			DEC_DWORD_STAT(STAT_AnimSequenceTransformProvider_Sequences);
		}

		// Decrement the bone table refcount in lockstep with RegisterProxy's increment
		// (one per proxy registration, regardless of whether the sequence was new).
		if (AnimCompactBoneTable)
		{
			check(AnimCompactBoneTable->RefCount > 0);
			if (--AnimCompactBoneTable->RefCount == 0)
			{
				AnimCompactBoneTablesToUpload.Remove(AnimCompactBoneTable);
				AnimCompactBoneTableRegistry.Remove(Skeleton);
			}
		}
	}

	ProxyUserDatasToUpload.Remove(ProxyUserData);
	delete ProxyUserData;
}

template <typename T>
FRDGBuffer* CreateByteAddressBuffer(FRDGBuilder& GraphBuilder, uint32 MaxNumElements, const TCHAR* Name)
{
	if (MaxNumElements > 0)
	{
		FRDGBuffer* Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc(MaxNumElements * sizeof(T)), Name);
		GraphBuilder.ConvertToExternalBuffer(Buffer);
		return Buffer;
	}
	return nullptr;
}

template <typename T>
TArrayView<T> Lock(FRHICommandList& RHICmdList, FRDGBuffer* Buffer)
{
	if (Buffer)
	{
		return TArrayView<T>(reinterpret_cast<T*>(RHICmdList.LockBuffer(Buffer->GetRHIUnchecked(), 0, Buffer->Desc.GetSize(), RLM_WriteOnly)), Buffer->Desc.GetSize() / sizeof(T));
	}
	return {};
}

void Unlock(FRHICommandList& RHICmdList, FRDGBuffer* Buffer)
{
	if (Buffer)
	{
		RHICmdList.UnlockBuffer(Buffer->GetRHIUnchecked());
	}
}

struct FAnimSequenceTransformProvider::FTrackUpdateRequest
{
	uint8 bNeedsCurrent : 1;
	uint8 UpdateMode    : 7;

	void SetUpdateMode(EPreviousBoneTransformUpdateMode InUpdateMode)
	{
		UpdateMode = static_cast<uint8>(InUpdateMode);
	}

	EPreviousBoneTransformUpdateMode GetUpdateMode() const
	{
		return static_cast<EPreviousBoneTransformUpdateMode>(UpdateMode);
	}
};

struct FAnimSequenceTransformProvider::FDispatchGroupKey
{
	uint32 NumCompactBones  = 0;
	uint32 NumLayers        = 0;
	uint32 MeshSpaceMask    = 0;
	uint32 LayerModeMask    = 0;

	friend bool operator==(const FDispatchGroupKey& A, const FDispatchGroupKey& B)
	{
		return A.NumCompactBones == B.NumCompactBones && A.NumLayers == B.NumLayers
			&& A.MeshSpaceMask == B.MeshSpaceMask && A.LayerModeMask == B.LayerModeMask;
	}

	friend uint32 GetTypeHash(const FDispatchGroupKey& Key)
	{
		uint32 Hash = GetTypeHash(Key.NumCompactBones);
		Hash = HashCombine(Hash, GetTypeHash(Key.NumLayers));
		Hash = HashCombine(Hash, GetTypeHash(Key.MeshSpaceMask));
		Hash = HashCombine(Hash, GetTypeHash(Key.LayerModeMask));
		return Hash;
	}
};

struct FAnimSequenceTransformProvider::FDispatchGroup
{
	uint32 GroupSize                      = 0;
	uint32 InstancesPerGroup              = 1;
	uint32 SlotsPerInstance               = 0;
	uint32 NumCompactBones                = 0;
	uint32 NumLayers                      = 0;
	uint32 MeshSpaceMask                  = 0;
	uint32 LayerModeMask                  = 0;

	uint32 NumHeaders                     = 0;
	uint32 NumSamples                     = 0;
	uint32 HeaderBaseOffset               = 0;
	uint32 SampleBaseOffset               = 0;
	uint32 NextHeaderWriteIndex           = 0;
	uint32 NextSampleWriteIndex           = 0;
};

void FAnimSequenceTransformProvider::UploadSequences(FRDGScatterUploader& ScatterUploader, uint32 NumTotalFramesRequested)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProvider::UploadSequences);

	for (const FMapping* Mapping : MappingsToUpload)
	{
		TArrayView<FVector4f> DstFloat4s = ScatterUploader.Add_GetRef<FVector4f>(Mapping->MappingTransformBufferAllocation.Offset, Mapping->MappingTransformBufferAllocation.Count);

		for (uint32 CompactBoneIndex = 0; CompactBoneIndex < Mapping->NumCompactBones; ++CompactBoneIndex)
		{
			FBoneTransformWithUniformScale InvRefPose;
			InvRefPose.SetTransform(Mapping->InvGlobalRefPoseTransforms[CompactBoneIndex]);
			WriteBoneTransform(DstFloat4s, CompactBoneIndex, InvRefPose);

			const uint32 RetargetTransformIndex = Mapping->NumCompactBones + CompactBoneIndex * (Mapping->RetargetingTables.bRequiresRefPoseRetarget ? 2 : 1);
			FBoneTransformWithUniformScale RetargetTransform;
			RetargetTransform.SetTransform(Mapping->RetargetingTables.Transform[CompactBoneIndex]);
			WriteBoneTransform(DstFloat4s, RetargetTransformIndex, RetargetTransform);

			if (Mapping->RetargetingTables.bRequiresRefPoseRetarget)
			{
				const FQuat4f& Q0 = Mapping->RetargetingTables.RefPoseRetargetQuats[CompactBoneIndex].Get<0>();
				const FQuat4f& Q1 = Mapping->RetargetingTables.RefPoseRetargetQuats[CompactBoneIndex].Get<1>();

				FBoneTransformWithUniformScale RefPoseTransform;
				RefPoseTransform.Rotation[0] = Q0.X;
				RefPoseTransform.Rotation[1] = Q0.Y;
				RefPoseTransform.Rotation[2] = Q0.Z;
				RefPoseTransform.Rotation[3] = Q0.W;
				RefPoseTransform.Translation[0] = Q1.X;
				RefPoseTransform.Translation[1] = Q1.Y;
				RefPoseTransform.Translation[2] = Q1.Z;
				RefPoseTransform.Scale = Q1.W;
				WriteBoneTransform(DstFloat4s, RetargetTransformIndex + 1, RefPoseTransform);
			}
		}
	}

	BoneTrackArray TrackToBoneIndexMap;
	FMemMark Mark(FMemStack::Get());
	TArray<FTransform, TMemStackAllocator<>> AnimLocalPoses;
	uint32 NumSequenceFrameUploads = 0;
	int32 SequenceIndex = 0;
	int32 NumActiveSequences = 0;

	const int32 UploadBudget = FMath::Max(0, CVarAnimSequenceTransformProviderUploadBudget.GetValueOnRenderThread() - (int32)NumTotalFramesRequested);

	for (const FSequence* Sequence : SequencesToUpload)
	{
		if (Sequence->NumFramesUploaded > 0)
		{
			NumActiveSequences++;
		}
	}

	const int32 UploadBudgetDivisor = FMath::Max(1, NumActiveSequences);
	const int32 FramesPerSequence   = UploadBudget / UploadBudgetDivisor;
	int32 NumFramesLeftover         = UploadBudget % UploadBudgetDivisor;

	TArray<FSequence*> CompletedSequences;

	for (FSequence* SequencePtr : SequencesToUpload)
	{
		FSequence& Sequence = *SequencePtr;

		// Budget-limited progressive upload. Only for sequences that have been touched by at least one track.
		if (Sequence.NumFramesUploaded > 0)
		{
			int32 NumBudgetFramesToUpload = FramesPerSequence + (NumFramesLeftover > 0 ? 1 : 0);
			NumFramesLeftover = FMath::Max(NumFramesLeftover - 1, 0);

			while (Sequence.NextFrameToUpload < Sequence.NumFrames && NumBudgetFramesToUpload > 0)
			{
				const int32 FrameStep = 1u << Sequence.TargetSampleRateLevel;

				if (Sequence.FramesUploaded[Sequence.NextFrameToUpload] || Sequence.FramesRequested[Sequence.NextFrameToUpload])
				{
					Sequence.NextFrameToUpload += FrameStep;
					continue;
				}

				Sequence.FramesRequested[Sequence.NextFrameToUpload] = true;
				Sequence.NumFramesRequested++;
				Sequence.NextFrameToUpload += FrameStep;
				NumBudgetFramesToUpload--;
			}
		}

		if (Sequence.NextFrameToUpload >= Sequence.NumFrames && Sequence.TargetSampleRateLevel > 0)
		{
			Sequence.TargetSampleRateLevel--;
			Sequence.CurrentSampleRateLevel--;
			Sequence.NextFrameToUpload = 1u << Sequence.TargetSampleRateLevel;
		}

		if (Sequence.NumFramesRequested > 0)
		{
			const UAnimSequence* AnimSequence = Sequence.Sequence;
			UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSequence->GetCompressedData();
			const FCompressedAnimSequence& PlatformCompressedData = CompressedAnimSequence.Get();

			const TArray<FTransform>& AnimRefLocalPoses = AnimSequence->GetSkeleton()->GetRefLocalPoses();
			check(Sequence.NumAnimBones <= (uint16)AnimRefLocalPoses.Num());

			const int32 NumTracks = PlatformCompressedData.CompressedTrackToSkeletonMapTable.Num();
			TrackToBoneIndexMap.Reset(NumTracks);

			for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
			{
				const uint16 BoneIndex = PlatformCompressedData.GetSkeletonIndexFromTrackIndex(TrackIndex);

				if (BoneIndex < (uint16)AnimRefLocalPoses.Num())
				{
					TrackToBoneIndexMap.Add(BoneTrackPair(BoneIndex, TrackIndex));
				}
			}

			FAnimSequenceDecompressionContext DecompressionContext(
				  Sequence.SampleRate
				, Sequence.NumFrames
				, AnimSequence->Interpolation
				, AnimSequence->GetRetargetTransformsSourceName()
				, *PlatformCompressedData.CompressedDataStructure
				, AnimRefLocalPoses
				, PlatformCompressedData.CompressedTrackToSkeletonMapTable
				, AnimSequence->GetSkeleton()
				, AnimSequence->IsValidAdditive()
				, AnimSequence->GetAdditiveAnimType()
			);

			AnimLocalPoses.Reset(AnimRefLocalPoses.Num());
			AnimLocalPoses.Append(AnimRefLocalPoses);

			for (TConstSetBitIterator<> BitIt(Sequence.FramesRequested); BitIt; ++BitIt)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProvider::DecompressPose);
				const int32 FrameIndex = BitIt.GetIndex();

				const double SeekTime = Sequence.SampleRate.AsSeconds(FFrameTime(FrameIndex));
				DecompressionContext.Seek(SeekTime);

				if (PlatformCompressedData.BoneCompressionCodec)
				{
					TArrayView<FTransform> AnimLocalPosesView(AnimLocalPoses);
					PlatformCompressedData.BoneCompressionCodec->DecompressPose(DecompressionContext, TrackToBoneIndexMap, TrackToBoneIndexMap, TrackToBoneIndexMap, AnimLocalPosesView);
				}

				const uint32 Float4sPerBone = GetFloat4sPerBone();
				const uint32 FrameFloat4Offset = Sequence.BufferAllocation.Offset + Sequence.GetNumAnimCompactBones() * FrameIndex * Float4sPerBone;
				TArrayView<FVector4f> DstFloat4s = ScatterUploader.Add_GetRef<FVector4f>(FrameFloat4Offset, Sequence.GetNumAnimCompactBones() * Float4sPerBone);

				for (uint16 AnimCompactBoneIndex = 0; AnimCompactBoneIndex < Sequence.GetNumAnimCompactBones(); ++AnimCompactBoneIndex)
				{
					const uint16 AnimBoneIndex = Sequence.AnimCompactBoneTable->AnimCompactToAnimBone[AnimCompactBoneIndex];
					FBoneTransformWithUniformScale Transform;
					Transform.SetTransform(FTransform3f(AnimLocalPoses[AnimBoneIndex]));
					WriteBoneTransform(DstFloat4s, AnimCompactBoneIndex, Transform);
				}

				Sequence.NumFramesUploaded++;
				Sequence.FramesUploaded[FrameIndex] = true;
				Sequence.FramesRequested[FrameIndex] = false;
				NumSequenceFrameUploads++;
			}

			Sequence.NumFramesRequested = 0;
		}

		if (Sequence.NumFramesUploaded == Sequence.NumFrames)
		{
			CompletedSequences.Add(SequencePtr);
		}
	}

	for (FSequence* Completed : CompletedSequences)
	{
		SequencesToUpload.Remove(Completed);
	}

	INC_DWORD_STAT_BY(STAT_AnimSequenceTransformProvider_SequenceFrameUploads, NumSequenceFrameUploads);
}

void FAnimSequenceTransformProvider::UploadBlockHeaders(
	FRHICommandList& RHICmdList,
	double WorldTime,
	float DeltaTime,
	TConstArrayView<FSkinningTransformProvider::FProviderIndirection> Indirections,
	TConstArrayView<FSkinningSceneExtensionProxy*> Proxies,
	uint32& NumTotalFramesRequested,
	uint32& NumTotalSequenceTransforms,
	TConstArrayView<FTrackUpdateRequest> TrackUpdateRequests,
	FRDGBuffer* LayerHeaderBuffer,
	FRDGBuffer* SamplePoseBuffer,
	uint32 NumTotalEntries,
	uint32 NumTotalSamples,
	TMap<FDispatchGroupKey, FDispatchGroup>& DispatchGroups)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProvider::UploadBlockHeaders);

	UE::HLSL::FEvaluateLayerHeader* HeaderUpload = reinterpret_cast<UE::HLSL::FEvaluateLayerHeader*>(
		RHICmdList.LockBuffer(LayerHeaderBuffer->GetRHIUnchecked(), 0, NumTotalEntries * sizeof(UE::HLSL::FEvaluateLayerHeader), RLM_WriteOnly));

	UE::HLSL::FSamplePoseData* SampleUpload = reinterpret_cast<UE::HLSL::FSamplePoseData*>(
		RHICmdList.LockBuffer(SamplePoseBuffer->GetRHIUnchecked(), 0, NumTotalSamples * sizeof(UE::HLSL::FSamplePoseData), RLM_WriteOnly));

	uint32 NumActiveTracks = 0;
	uint32 NumCurrentTransformUpdates = 0;
	uint32 NumPreviousTransformUpdates = 0;

	struct FWriteSamplesResult
	{
		uint32 NumSamples = 0;
		uint32 InvGlobalRefPoseTransformBufferOffset = 0;
	};

	const auto WriteSamples = [&](
		const FProxyUserData& ProxyUserData,
		const FAnimSequenceTrackPackedData& PackedData,
		const FAnimSequenceTrackRenderData& RenderData,
		uint32 LODIndex,
		FDispatchGroup& Group,
		bool bPrevious) -> FWriteSamplesResult
	{
		FWriteSamplesResult Result;

		const double TimePosition = bPrevious ? RenderData.GetPreviousTimePosition() : PackedData.GetTimePosition(WorldTime);

		if (PackedData.IsBlendSpace())
		{
			Result.NumSamples = PackedData.GetNumSamples();

			if (Result.NumSamples == 0)
			{
				return Result;
			}

			for (uint32 SampleIndex = 0; SampleIndex < Result.NumSamples; SampleIndex++)
			{
				const uint16 SequenceIndex = PackedData.GetSampleSequenceIndex(SampleIndex);
				const FTrackData TrackData = ProxyUserData.GetTrackDataForSequence(SequenceIndex, TimePosition, LODIndex, PackedData.GetLoopMode());

				int32 KeyIndex0, KeyIndex1;
				float Alpha;
				TrackData.Sequence.GetKeyIndicesFromTime(KeyIndex0, KeyIndex1, Alpha, TrackData.Position, TrackData.LoopMode);
				TrackData.Sequence.RequestFrame(KeyIndex0, NumTotalFramesRequested, NumTotalSequenceTransforms);
				TrackData.Sequence.RequestFrame(KeyIndex1, NumTotalFramesRequested, NumTotalSequenceTransforms);

				UE::HLSL::FSamplePoseData& Sample = SampleUpload[Group.SampleBaseOffset + Group.NextSampleWriteIndex++];
				Sample.Alpha                            = Alpha;
				Sample.KeyIndex0                        = KeyIndex0;
				Sample.KeyIndex1                        = KeyIndex1;
				Sample.SequenceTransformBufferOffset    = TrackData.Sequence.BufferAllocation.Offset;
				Sample.RetargetingTransformBufferOffset = TrackData.Mapping.RetargetingTransformBufferOffset;
				Sample.RetargetingDataBufferOffset      = TrackData.Mapping.RetargetingDataBufferAllocation.Offset * sizeof(UE::HLSL::FBoneRetargetingData);
				Sample.NumAnimBones                     = TrackData.NumAnimCompactBones;
				Sample.Weight                           = PackedData.GetSampleWeight(SampleIndex).GetFloat();

				if (SampleIndex == 0)
				{
					Result.InvGlobalRefPoseTransformBufferOffset = TrackData.Mapping.InvGlobalRefPoseTransformBufferOffset;
				}
			}
		}
		else
		{
			const uint16 SequenceIndex = static_cast<uint16>(PackedData.GetSequenceIndex());
			const FTrackData TrackData = ProxyUserData.GetTrackDataForSequence(SequenceIndex, TimePosition, LODIndex, PackedData.GetLoopMode());

			int32 KeyIndex0, KeyIndex1;
			float Alpha;
			TrackData.Sequence.GetKeyIndicesFromTime(KeyIndex0, KeyIndex1, Alpha, TrackData.Position, TrackData.LoopMode);
			TrackData.Sequence.RequestFrame(KeyIndex0, NumTotalFramesRequested, NumTotalSequenceTransforms);
			TrackData.Sequence.RequestFrame(KeyIndex1, NumTotalFramesRequested, NumTotalSequenceTransforms);

			UE::HLSL::FSamplePoseData& Sample = SampleUpload[Group.SampleBaseOffset + Group.NextSampleWriteIndex++];
			Sample.Alpha                            = Alpha;
			Sample.KeyIndex0                        = KeyIndex0;
			Sample.KeyIndex1                        = KeyIndex1;
			Sample.SequenceTransformBufferOffset    = TrackData.Sequence.BufferAllocation.Offset;
			Sample.RetargetingTransformBufferOffset = TrackData.Mapping.RetargetingTransformBufferOffset;
			Sample.RetargetingDataBufferOffset      = TrackData.Mapping.RetargetingDataBufferAllocation.Offset * sizeof(UE::HLSL::FBoneRetargetingData);
			Sample.NumAnimBones                     = TrackData.NumAnimCompactBones;
			Sample.Weight                           = 1.0f;

			Result.NumSamples = 1;
			Result.InvGlobalRefPoseTransformBufferOffset = TrackData.Mapping.InvGlobalRefPoseTransformBufferOffset;
		}

		return Result;
	};

	const auto WriteTrackLayers = [&](
		const FProxyUserData& ProxyUserData,
		const FProxyUserData::FLOD& LOD,
		FAnimSequenceTransformProviderRenderData& RenderData,
		TConstArrayView<uint32> BoneMaskOffsets,
		FDispatchGroup& Group,
		int32 LODIndex,
		int32 TrackIndex,
		uint32 DstOffset1,
		uint32 DstOffset2,
		bool bPrevious,
		bool bTick)
	{
		const uint32 BoneMapOffset = LOD.BoneMapBufferOffset * sizeof(uint32);

		for (int32 LayerIndex = 0; LayerIndex < RenderData.GetNumLayers(); LayerIndex++)
		{
			const uint32 CurrentHeaderIndex = Group.HeaderBaseOffset + Group.NextHeaderWriteIndex++;
			UE::HLSL::FEvaluateLayerHeader& Header = HeaderUpload[CurrentHeaderIndex];

			Header.BoneMapBufferOffset       = BoneMapOffset;
			Header.DstTransformBufferOffset1 = DstOffset1;
			Header.DstTransformBufferOffset2 = DstOffset2;

			if (RenderData.IsTrackActive(TrackIndex, LayerIndex))
			{
				FAnimSequenceTransformProviderRenderLayer& Layer = RenderData.GetLayers()[LayerIndex];
				FAnimSequenceTrackRenderData& Data = Layer.Tracks.GetData(TrackIndex);

				const uint32 SampleOffset = Group.SampleBaseOffset + Group.NextSampleWriteIndex;
				const float BlendWeight = Data.GetBlendWeight(bPrevious);

				const FWriteSamplesResult SourceResult = WriteSamples(ProxyUserData, Data.GetSource(), Data, LODIndex, Group, bPrevious);
				FWriteSamplesResult TargetResult;

				if (BlendWeight > 0.0f)
				{
					TargetResult = WriteSamples(ProxyUserData, Data.GetTarget(), Data, LODIndex, Group, bPrevious);
				}

				Header.SampleDataOffset = SampleOffset;
				Header.BlendWeight      = BlendWeight;
				Header.LayerWeight      = Layer.Weight * Data.GetTarget().GetLayerWeight();
				Header.InvGlobalRefPoseTransformBufferOffset = SourceResult.InvGlobalRefPoseTransformBufferOffset;
				SetBoneMaskAndSampleCounts(Header, BoneMaskOffsets[LayerIndex], SourceResult.NumSamples, TargetResult.NumSamples);

				if (bTick)
				{
					Data.Tick(DeltaTime, WorldTime);
				}
			}
			else
			{
				Header.SampleDataOffset                      = 0;
				Header.BlendWeight                           = 0.0f;
				Header.LayerWeight                           = 0.0f;
				Header.InvGlobalRefPoseTransformBufferOffset = 0;
				SetBoneMaskAndSampleCounts(Header, SKINNING_INVALID_BONE_MASK_OFFSET, 0, 0);
			}
		}
	};

	uint32 UpdateResultIndex = 0;

	for (const FSkinningTransformProvider::FProviderIndirection Indirection : Indirections)
	{
		FInstancedSkinningSceneExtensionProxy* ExtensionProxy = static_cast<FInstancedSkinningSceneExtensionProxy*>(Proxies[Indirection.Index]);
		FProxyUserData& ProxyUserData = static_cast<FProxyUserData&>(ExtensionProxy->GetUserDataChecked());
		FAnimSequenceTransformProviderRenderData& RenderData = *ProxyUserData.GetRenderData();
		const uint32 LODIndex = ProxyUserData.bIsNaniteMesh ? 0u : ExtensionProxy->GetLOD();
		const uint32 NumTransforms = ExtensionProxy->GetMaxBoneTransformCount();
		FProxyUserData::FLOD& LOD = ProxyUserData.LODs[LODIndex];
		FAnimSequenceTrackRenderPool& Tracks = RenderData.GetLayers()[0].Tracks;
		TConstArrayView<uint32> BoneMaskOffsets = LOD.BoneMaskBufferOffsets;

		FDispatchGroupKey GroupKey;
		GroupKey.NumCompactBones = LOD.BoneTables.NumCompactBones;
		GroupKey.NumLayers       = RenderData.GetNumLayers();
		GroupKey.MeshSpaceMask   = RenderData.GetMeshSpaceMask();
		GroupKey.LayerModeMask   = RenderData.GetLayerModeMask();
		FDispatchGroup& DispatchGroup = DispatchGroups.FindChecked(GroupKey);

		const uint32 CurrentTransformOffset  = Indirection.CurrentTransformOffset;
		const uint32 PreviousTransformOffset = Indirection.PreviousTransformOffset;

		check(Tracks.GetNumActiveTracks() <= (int32)ExtensionProxy->GetUniqueAnimationCount());

		Tracks.EnumerateActiveTracks([&] (int32 TrackIndex)
		{
			const FTrackUpdateRequest UpdateRequest = TrackUpdateRequests[UpdateResultIndex++];
			const uint32 TrackTransformOffset         = TrackIndex * NumTransforms * 2 * sizeof(FCompressedBoneTransform);
			const uint32 TrackCurrentTransformOffset  = TrackTransformOffset + CurrentTransformOffset;
			const uint32 TrackPreviousTransformOffset = TrackTransformOffset + PreviousTransformOffset;
			const bool bNeedsPrevious                 = UpdateRequest.GetUpdateMode() == EPreviousBoneTransformUpdateMode::UpdatePrevious;

			if (UpdateRequest.bNeedsCurrent)
			{
				const bool bTick = !bNeedsPrevious;
				uint32 OutputTransformOffset1 = TrackCurrentTransformOffset;
				uint32 OutputTransformOffset2 = UpdateRequest.GetUpdateMode() == EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious ? TrackPreviousTransformOffset : TrackCurrentTransformOffset;
				WriteTrackLayers(ProxyUserData, LOD, RenderData, BoneMaskOffsets, DispatchGroup, LODIndex, TrackIndex, OutputTransformOffset1, OutputTransformOffset2, false, bTick);
				NumCurrentTransformUpdates++;
			}

			if (bNeedsPrevious)
			{
				const bool bTick = true;
				WriteTrackLayers(ProxyUserData, LOD, RenderData, BoneMaskOffsets, DispatchGroup, LODIndex, TrackIndex, TrackPreviousTransformOffset, TrackPreviousTransformOffset, true, bTick);
				NumPreviousTransformUpdates++;
			}
			NumActiveTracks++;
		});
	}

	RHICmdList.UnlockBuffer(LayerHeaderBuffer->GetRHIUnchecked());
	RHICmdList.UnlockBuffer(SamplePoseBuffer->GetRHIUnchecked());

	INC_DWORD_STAT_BY(STAT_AnimSequenceTransformProvider_ActiveTracks, NumActiveTracks);
	INC_DWORD_STAT_BY(STAT_AnimSequenceTransformProvider_CurrentTransformUpdates, NumCurrentTransformUpdates);
	INC_DWORD_STAT_BY(STAT_AnimSequenceTransformProvider_PreviousTransformUpdates, NumPreviousTransformUpdates);
}

void FAnimSequenceTransformProvider::ProvideTransforms(FSkinningTransformProvider::FProviderContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProvider::ProvideTransforms);
	FRDGBuilder& GraphBuilder = Context.GraphBuilder;

	SequenceTransformAllocator.Consolidate();
	RetargetingDataAllocator.Consolidate();
	BoneHierarchyAllocator.Consolidate();
	BoneMaskAllocator.Consolidate();

	uint32 MaxTotalActiveTracks = 0;

	for (const FSkinningTransformProvider::FProviderIndirection Indirection : Context.Indirections)
	{
		FInstancedSkinningSceneExtensionProxy* ExtensionProxy = static_cast<FInstancedSkinningSceneExtensionProxy*>(Context.Proxies[Indirection.Index]);
		FProxyUserData& ProxyUserData = static_cast<FProxyUserData&>(ExtensionProxy->GetUserDataChecked());
		FAnimSequenceTransformProviderRenderData& RenderData = *ProxyUserData.GetRenderData();
		MaxTotalActiveTracks += RenderData.GetLayers()[0].Tracks.GetNumActiveTracks();
	}

	if (!MaxTotalActiveTracks)
	{
		return;
	}

	TArray<FTrackUpdateRequest, SceneRenderingAllocator> TrackUpdateRequests;
	TrackUpdateRequests.Reserve(MaxTotalActiveTracks);

	struct FLODUploadRequest
	{
		FProxyUserData::FLOD& LOD;
		const FInstancedSkinningSceneExtensionProxy& ExtensionProxy;
	};

	TArray<FLODUploadRequest, SceneRenderingAllocator> LODUploadRequests;

	const float GlobalTimeScale = CVarAnimSequenceTransformProviderTimeScale.GetValueOnRenderThread();
	const double WorldTime = Context.CurrentTime.GetWorldTimeSeconds() * GlobalTimeScale;
	const float DeltaTime = Context.CurrentTime.GetDeltaWorldTimeSeconds() * GlobalTimeScale;

	uint32 NumTotalSubTracks = 0;

	auto& DispatchGroups = *GraphBuilder.AllocObject<TMap<FDispatchGroupKey, FDispatchGroup>>();

	for (const FSkinningTransformProvider::FProviderIndirection Indirection : Context.Indirections)
	{
		FInstancedSkinningSceneExtensionProxy* ExtensionProxy = static_cast<FInstancedSkinningSceneExtensionProxy*>(Context.Proxies[Indirection.Index]);
		FProxyUserData& ProxyUserData = static_cast<FProxyUserData&>(ExtensionProxy->GetUserDataChecked());
		const uint32 LODIndex = ProxyUserData.bIsNaniteMesh ? 0u : ExtensionProxy->GetLOD();
		FProxyUserData::FLOD& LOD = ProxyUserData.LODs[LODIndex];
		FAnimSequenceTransformProviderRenderData& RenderData = *ProxyUserData.GetRenderData();
		FAnimSequenceTrackRenderPool& Tracks = RenderData.GetLayers()[0].Tracks;

		const uint32 CurrentTransformSlot = Indirection.CurrentTransformOffset < Indirection.PreviousTransformOffset ? 0 : 1;
		const uint32 NumLayers = RenderData.GetNumLayers();
		const uint32 MeshSpaceMask = RenderData.GetMeshSpaceMask();
		const uint32 LayerModeMask = RenderData.GetLayerModeMask();

		FDispatchGroupKey GroupKey;
		GroupKey.NumCompactBones = LOD.BoneTables.NumCompactBones;
		GroupKey.NumLayers       = NumLayers;
		GroupKey.MeshSpaceMask   = MeshSpaceMask;
		GroupKey.LayerModeMask   = LayerModeMask;

		FDispatchGroup& DispatchGroup = DispatchGroups.FindOrAdd(GroupKey);
		if (DispatchGroup.GroupSize == 0)
		{
			DispatchGroup.NumCompactBones   = LOD.BoneTables.NumCompactBones;
			DispatchGroup.NumLayers         = NumLayers;
			DispatchGroup.MeshSpaceMask     = MeshSpaceMask;
			DispatchGroup.LayerModeMask     = LayerModeMask;
			DispatchGroup.GroupSize         = LOD.GroupSize;
			DispatchGroup.InstancesPerGroup = LOD.InstancesPerGroup;
			DispatchGroup.SlotsPerInstance  = LOD.SlotsPerInstance;
		}

		uint32 NumSubTracks = 0;

		Tracks.EnumerateActiveTracks([&] (int32 TrackIndex)
		{
			FTrackUpdateRequest UpdateRequest;
			UpdateRequest.SetUpdateMode(EPreviousBoneTransformUpdateMode::None);
			const bool bForceUpdateAll = EnumHasAnyFlags(Indirection.DirtyBoneTransforms, EDirtyBoneTransforms::Previous);

			UpdateRequest.bNeedsCurrent = true;

			if (bForceUpdateAll)
			{
				UpdateRequest.SetUpdateMode(EPreviousBoneTransformUpdateMode::UpdatePrevious);
			}

			for (int32 LayerIndex = 0; LayerIndex < (int32)NumLayers; LayerIndex++)
			{
				if (RenderData.IsTrackActive(TrackIndex, LayerIndex))
				{
					FAnimSequenceTransformProviderRenderLayer& Layer = RenderData.GetLayers()[LayerIndex];
					FAnimSequenceTrackRenderData& Data = Layer.Tracks.GetData(TrackIndex);

					if (!Data.HasPreviousPosition())
					{
						UpdateRequest.SetUpdateMode(EPreviousBoneTransformUpdateMode::DuplicateCurrentToPrevious);
					}

					DispatchGroup.NumSamples += Data.GetTotalSampleCount(UpdateRequest.GetUpdateMode());
				}
			}

			const int32 NumLocalSubTracks = UpdateRequest.bNeedsCurrent + (UpdateRequest.GetUpdateMode() == EPreviousBoneTransformUpdateMode::UpdatePrevious ? 1 : 0);

			NumSubTracks += NumLocalSubTracks;
			DispatchGroup.NumHeaders += NumLocalSubTracks;

			TrackUpdateRequests.Emplace(UpdateRequest);
		});

		NumTotalSubTracks += NumSubTracks;

		if (!LOD.bUploaded && NumSubTracks > 0)
		{
			LOD.bUploaded = true;

			if (LODUploadRequests.IsEmpty())
			{
				LODUploadRequests.Reserve(Context.Indirections.Num());
			}

			LODUploadRequests.Add(FLODUploadRequest
			{
				  .LOD = LOD
				, .ExtensionProxy = *ExtensionProxy
			});
		}
	}

	if (!NumTotalSubTracks)
	{
		return;
	}

	uint32& NumTotalFramesRequested = *GraphBuilder.AllocPOD<uint32>();
	NumTotalFramesRequested = 0;

	uint32& NumTotalSequenceTransforms = *GraphBuilder.AllocPOD<uint32>();
	NumTotalSequenceTransforms = 0;

	uint32 MaxTotalSequenceTransforms = 0;
	uint32 NumTotalRetargetingDatas = 0;
	uint32 NumTotalBoneHierarchyElements = 0;

	for (FAnimCompactBoneTable* Table : AnimCompactBoneTablesToUpload)
	{
		const uint16 PreviousNumAnimCompactBones = Table->NumAnimCompactBones;
		Table->Rebuild();
		Table->bDirty = false;

		MappingsToUpload.Append(Table->ReferencingMappings);

		if (PreviousNumAnimCompactBones != Table->NumAnimCompactBones)
		{
			for (FSequence* Sequence : Table->ReferencingSequences)
			{
				if (Sequence->BufferAllocation.IsValid())
				{
					SequenceTransformAllocator.Free(Sequence->BufferAllocation.Offset, Sequence->BufferAllocation.Count);
					Sequence->BufferAllocation = FBufferAllocation{};
					Sequence->InitUploadState();
				}
				SequencesToUpload.Add(Sequence);
			}
		}
	}
	AnimCompactBoneTablesToUpload.Empty();

	SequenceTransformAllocator.Consolidate();

	const uint32 Float4sPerBone = GetFloat4sPerBone();
	uint32 MaxAnimCompactBones = 0;
	uint32 NumTotalRemainingFrames = 0;

	for (FSequence* Sequence : SequencesToUpload)
	{
		const int32 NumFloat4s = Sequence->GetNumAnimCompactBones() * Sequence->NumFrames * Float4sPerBone;

		if (!Sequence->BufferAllocation.IsValid())
		{
			Sequence->BufferAllocation.Count  = NumFloat4s;
			Sequence->BufferAllocation.Offset = SequenceTransformAllocator.Allocate(NumFloat4s);
		}

		MaxTotalSequenceTransforms += Sequence->GetFirstMipUploadTransforms();
		MaxAnimCompactBones = FMath::Max(MaxAnimCompactBones, (uint32)Sequence->GetNumAnimCompactBones());
		NumTotalRemainingFrames += Sequence->NumFrames - Sequence->NumFramesUploaded;
	}

	if (!SequencesToUpload.IsEmpty())
	{
		const int32 UploadBudget = FMath::Clamp(CVarAnimSequenceTransformProviderUploadBudget.GetValueOnRenderThread(), 0, (int32)NumTotalRemainingFrames);
		const int32 NumBudgetTransforms = UploadBudget * MaxAnimCompactBones;
		MaxTotalSequenceTransforms += NumBudgetTransforms;
		NumTotalSequenceTransforms += NumBudgetTransforms;
	}

	for (FMapping* Mapping : MappingsToUpload)
	{
		if (Mapping->MappingTransformBufferAllocation.IsValid())
		{
			SequenceTransformAllocator.Free(Mapping->MappingTransformBufferAllocation.Offset, Mapping->MappingTransformBufferAllocation.Count);
		}

		if (Mapping->RetargetingDataBufferAllocation.IsValid())
		{
			RetargetingDataAllocator.Free(Mapping->RetargetingDataBufferAllocation.Offset, Mapping->RetargetingDataBufferAllocation.Count);
		}

		const uint32 NumInvGlobalRefPoses     = Mapping->NumCompactBones;
		const uint32 NumRetargetingTransforms = (Mapping->RetargetingTables.bRequiresRefPoseRetarget ? 2 : 1);
		const uint32 NumTransforms            = Mapping->NumCompactBones * NumRetargetingTransforms + NumInvGlobalRefPoses;
		const uint32 NumRetargetingDatas      = Mapping->NumCompactBones;

		Mapping->MappingTransformBufferAllocation.Count  = NumTransforms * Float4sPerBone;
		Mapping->MappingTransformBufferAllocation.Offset = SequenceTransformAllocator.Allocate(NumTransforms * Float4sPerBone);

		Mapping->InvGlobalRefPoseTransformBufferOffset = Mapping->MappingTransformBufferAllocation.Offset;
		Mapping->RetargetingTransformBufferOffset      = Mapping->MappingTransformBufferAllocation.Offset + NumInvGlobalRefPoses * Float4sPerBone;

		Mapping->RetargetingDataBufferAllocation.Count = NumRetargetingDatas;
		Mapping->RetargetingDataBufferAllocation.Offset = RetargetingDataAllocator.Allocate(NumRetargetingDatas);

		MaxTotalSequenceTransforms += NumTransforms;
		NumTotalSequenceTransforms += NumTransforms;
		NumTotalRetargetingDatas  += NumRetargetingDatas;
	}

	if (!MappingsToUpload.IsEmpty() || !ProxyUserDatasToUpload.IsEmpty())
	{
		GraphBuilder.AddPostExecuteCallback([this]
		{
			MappingsToUpload.Empty();
			ProxyUserDatasToUpload.Empty();
		});
	}

	for (const FLODUploadRequest& Request : LODUploadRequests)
	{
		FProxyUserData::FLOD& LOD = Request.LOD;
		const uint32 NumBoneMapEntries = LOD.BoneTables.NumCompactBones;
		LOD.BoneMapBufferAllocation.Count = NumBoneMapEntries;
		LOD.BoneMapBufferAllocation.Offset = BoneHierarchyAllocator.Allocate(NumBoneMapEntries);
		LOD.BoneMapBufferOffset = LOD.BoneMapBufferAllocation.Offset;
		NumTotalBoneHierarchyElements += NumBoneMapEntries;
	}

	uint32 NumTotalEntries = 0;
	uint32 NumTotalSamples = 0;

	for (auto& [Key, Group] : DispatchGroups)
	{
		Group.HeaderBaseOffset = NumTotalEntries;
		Group.SampleBaseOffset = NumTotalSamples;
		NumTotalEntries += Group.NumHeaders * Group.NumLayers;
		NumTotalSamples += Group.NumSamples;
	}

	FRDGBuffer* LayerHeaderBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(UE::HLSL::FEvaluateLayerHeader), NumTotalEntries),
		TEXT("AnimSequenceTransformProvider.LayerHeaders"));
	GraphBuilder.ConvertToExternalBuffer(LayerHeaderBuffer);

	FRDGBuffer* SamplePoseBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(UE::HLSL::FSamplePoseData), NumTotalSamples),
		TEXT("AnimSequenceTransformProvider.SamplePoseData"));
	GraphBuilder.ConvertToExternalBuffer(SamplePoseBuffer);

	UE::Tasks::FTask ProcessTask = GraphBuilder.AddCommandListSetupTask(
		[
			  WorldTime
			, DeltaTime
			, Indirections = Context.Indirections
			, Proxies = Context.Proxies
			, &NumTotalFramesRequested
			, &NumTotalSequenceTransforms
			, TrackUpdateRequests = MoveTemp(TrackUpdateRequests)
			, LayerHeaderBuffer
			, SamplePoseBuffer
			, NumTotalEntries
			, NumTotalSamples
			, &DispatchGroups
		] (FRHICommandList& RHICmdList) mutable
	{
		UploadBlockHeaders(RHICmdList, WorldTime, DeltaTime, Indirections, Proxies, NumTotalFramesRequested, NumTotalSequenceTransforms, TrackUpdateRequests, LayerHeaderBuffer, SamplePoseBuffer, NumTotalEntries, NumTotalSamples, DispatchGroups);
	});

	FRDGBuffer* SequenceTransformBuffer = SequenceTransformPersistentBuffer.ResizeBufferIfNeeded(GraphBuilder, SequenceTransformAllocator.GetMaxSize());
	FRDGBuffer* RetargetingDataBuffer = RetargetingDataPersistentBuffer.ResizeBufferIfNeeded(GraphBuilder, RetargetingDataAllocator.GetMaxSize());
	FRDGBuffer* BoneHierarchyBuffer = BoneHierarchyPersistentBuffer.ResizeBufferIfNeeded(GraphBuilder, BoneHierarchyAllocator.GetMaxSize());
	FRDGBuffer* BoneMaskBuffer = BoneMaskPersistentBuffer.ResizeBufferIfNeeded(GraphBuilder, BoneMaskAllocator.GetMaxSize());
	SET_MEMORY_STAT(STAT_AnimSequenceTransformProvider_SequenceTransformMemory, SequenceTransformBuffer->Desc.GetSize());
	SET_MEMORY_STAT(STAT_AnimSequenceTransformProvider_RetargetingDataMemory, RetargetingDataBuffer->Desc.GetSize());

	SET_MEMORY_STAT(STAT_AnimSequenceTransformProvider_ProxyDataMemory, TotalProxyMemory);
	SET_MEMORY_STAT(STAT_AnimSequenceTransformProvider_MappingDataMemory, TotalMappingMemory);
	SET_MEMORY_STAT(STAT_AnimSequenceTransformProvider_SequenceDataMemory, TotalSequenceMemory);

	bool bExecuteScatterUpload = false;

	if (MaxTotalSequenceTransforms > 0)
	{
		Context.ScatterUploadBuilder.AddPrerequisiteTask(ProcessTask);

		Context.ScatterUploadBuilder.AddPass(GraphBuilder, SequenceTransformUploadBuffer, SequenceTransformBuffer, MaxTotalSequenceTransforms * Float4sPerBone, sizeof(FVector4f), TEXT("AnimSequenceTransformProvider.Sequences"),
			[&NumTotalSequenceTransforms, Float4sPerBone] () -> uint32
			{
				return NumTotalSequenceTransforms * Float4sPerBone;
			},
			[this, &NumTotalFramesRequested] (FRDGScatterUploader& ScatterUploader)
			{
				UploadSequences(ScatterUploader, NumTotalFramesRequested);
			}
		);

		bExecuteScatterUpload = true;
	}

	if (NumTotalRetargetingDatas > 0)
	{
		Context.ScatterUploadBuilder.AddPass(GraphBuilder, RetargetingDataUploadBuffer, RetargetingDataBuffer, NumTotalRetargetingDatas, sizeof(UE::HLSL::FBoneRetargetingData), TEXT("AnimSequenceTransformProvider.RetargetingData"),
			[this] (FRDGScatterUploader& ScatterUploader)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAnimSequenceTransformProvider::UploadRetargetingData);

			for (const FMapping* Mapping : MappingsToUpload)
			{
				TArrayView<UE::HLSL::FBoneRetargetingData> DstRetargetingDatas = ScatterUploader.Add_GetRef<UE::HLSL::FBoneRetargetingData>(Mapping->RetargetingDataBufferAllocation.Offset, Mapping->RetargetingDataBufferAllocation.Count);

				for (uint32 CompactBoneIndex = 0; CompactBoneIndex < Mapping->NumCompactBones; ++CompactBoneIndex)
				{
					UE::HLSL::FBoneRetargetingData& RetargetingData = DstRetargetingDatas[CompactBoneIndex];
					RetargetingData.RetargetMode  = Mapping->RetargetingTables.Mode[CompactBoneIndex];
					const uint16 AnimBone = Mapping->RetargetingTables.CompactToAnim[CompactBoneIndex];
					const FAnimCompactBoneTable* Table = Mapping->RetargetingTables.AnimCompactBoneTable;
					RetargetingData.AnimBoneIndex = (AnimBone != INVALID_BONE_INDEX && Table && Table->AnimBoneToAnimCompact.IsValidIndex(AnimBone))
						? Table->AnimBoneToAnimCompact[AnimBone]
						: INVALID_BONE_INDEX;
					RetargetingData.bRequiresRefPoseRetarget = Mapping->RetargetingTables.bRequiresRefPoseRetarget;
				}
			}
		});

		bExecuteScatterUpload = true;
	}

	if (!ProxyUserDatasToUpload.IsEmpty() && BoneMaskBuffer)
	{
		uint32 NumTotalBoneMaskElements = 0;
		for (const FProxyUserData* UserData : ProxyUserDatasToUpload)
		{
			for (const FProxyUserData::FLOD& LOD : UserData->LODs)
			{
				for (int32 LayerIndex = 0; LayerIndex < LOD.BoneMaskBufferOffsets.Num(); LayerIndex++)
				{
					if (LOD.BoneMaskBufferOffsets[LayerIndex] != FProxyUserData::InvalidBoneMaskOffset)
					{
						NumTotalBoneMaskElements += LOD.BoneTables.NumCompactBones;
					}
				}
			}
		}

		if (NumTotalBoneMaskElements > 0)
		{
			Context.ScatterUploadBuilder.AddPass(GraphBuilder, BoneMaskUploadBuffer, BoneMaskBuffer, NumTotalBoneMaskElements, sizeof(float), TEXT("AnimSequenceTransformProvider.BoneMasks"),
				[this](FRDGScatterUploader& ScatterUploader)
			{
				for (const FProxyUserData* UserData : ProxyUserDatasToUpload)
				{
					for (const FProxyUserData::FLOD& LOD : UserData->LODs)
					{
						const uint32 NumCompactBones = LOD.BoneTables.NumCompactBones;

						for (int32 LayerIndex = 0; LayerIndex < LOD.BoneMaskBufferOffsets.Num(); LayerIndex++)
						{
							const uint32 MaskOffset = LOD.BoneMaskBufferOffsets[LayerIndex];
							if (MaskOffset == FProxyUserData::InvalidBoneMaskOffset)
							{
								continue;
							}

							const UBlendProfile* BlendMask = UserData->BlendProfiles[LayerIndex];
							check(BlendMask);

							const uint32 AllocationIndex = MaskOffset / sizeof(float);
							TArrayView<float> DstWeights = ScatterUploader.Add_GetRef<float>(AllocationIndex, NumCompactBones);

							for (uint32 CompactBoneIndex = 0; CompactBoneIndex < NumCompactBones; CompactBoneIndex++)
							{
								const uint16 TargetSkeletonBoneIndex = LOD.BoneTables.CompactToTargetSkeleton[CompactBoneIndex];
								DstWeights[CompactBoneIndex] = FMath::Clamp(BlendMask->GetBoneBlendScale(TargetSkeletonBoneIndex), 0.0f, 1.0f);
							}
						}
					}
				}
			});

			bExecuteScatterUpload = true;
		}
	}

	if (!LODUploadRequests.IsEmpty())
	{
		Context.ScatterUploadBuilder.AddPass(GraphBuilder, BoneHierarchyUploadBuffer, BoneHierarchyBuffer, NumTotalBoneHierarchyElements, sizeof(UE::HLSL::FBoneHierarchyData), TEXT("AnimSequenceTransformProvider.BoneHierarchy"),
			[this, LODUploadRequests = MoveTemp(LODUploadRequests)](FRDGScatterUploader& ScatterUploader)
		{
			TArray<uint16, SceneRenderingAllocator> CompactToTransform;

			for (const FLODUploadRequest& Request : LODUploadRequests)
			{
				FProxyUserData::FLOD& LOD = Request.LOD;
				TConstArrayView<FBoneIndexType> BoneMap = Request.ExtensionProxy.GetBoneMap();
				const FBoneTables& BoneTables = LOD.BoneTables;

				TArrayView<UE::HLSL::FBoneHierarchyData> DstBuffer = ScatterUploader.Add_GetRef<UE::HLSL::FBoneHierarchyData>(LOD.BoneMapBufferAllocation.Offset, LOD.BoneMapBufferAllocation.Count);

				CompactToTransform.Init(INVALID_BONE_INDEX, BoneTables.NumCompactBones);

				for (uint32 TransformIndex = 0; TransformIndex < (uint32)BoneMap.Num(); ++TransformIndex)
				{
					const uint16 CompactBoneIndex = BoneTables.MeshToCompact[BoneMap[TransformIndex]];

					if (CompactBoneIndex != INVALID_BONE_INDEX)
					{
						CompactToTransform[CompactBoneIndex] = TransformIndex;
					}
				}

				for (uint32 CompactBoneIndex = 0; CompactBoneIndex < BoneTables.NumCompactBones; ++CompactBoneIndex)
				{
					DstBuffer[CompactBoneIndex] = UE::HLSL::FBoneHierarchyData
					{
						  .ParentIndex    = (uint16)BoneTables.CompactToParent[CompactBoneIndex]
						, .TransformIndex = (uint16)CompactToTransform[CompactBoneIndex]
					};
				}
			}
		});

		bExecuteScatterUpload = true;
	}

	if (bExecuteScatterUpload)
	{
		Context.ScatterUploadBuilder.Execute(Context.GraphBuilder);
	}

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, AnimSequenceTransformProvider, "AnimSequenceTransformProvider");

	{
		FEvaluateConcatenateScatterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FEvaluateConcatenateScatterCS::FParameters>();
		PassParameters->RetargetingDataBuffer        = GraphBuilder.CreateSRV(RetargetingDataBuffer);
		PassParameters->SequenceTransformBuffer      = GraphBuilder.CreateSRV(SequenceTransformBuffer);
		PassParameters->LayerHeaderBuffer            = GraphBuilder.CreateSRV(LayerHeaderBuffer);
		PassParameters->SamplePoseBuffer             = GraphBuilder.CreateSRV(SamplePoseBuffer);
		PassParameters->BoneMapBuffer                = GraphBuilder.CreateSRV(BoneHierarchyBuffer);
		PassParameters->BoneMaskBuffer               = GraphBuilder.CreateSRV(BoneMaskBuffer);
		PassParameters->SkinningTransformBufferUAV   = GetCompressedBoneTransformUAV(GraphBuilder, Context.TransformBuffer);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("EvaluateConcatenateScatter"),
			PassParameters,
			ERDGPassFlags::Compute,
			[ShaderMap, PassParameters, &DispatchGroups](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			RHICmdList.BeginUAVOverlap();

			for (const auto& [Key, Group] : DispatchGroups)
			{
				FEvaluateConcatenateScatterCS::FPermutationDomain Permutation;
				Permutation.Set<FEvaluateConcatenateScatterCS::FBonesPerGroupDim>(Group.GroupSize);
				Permutation.Set<FEvaluateConcatenateScatterCS::FSingleLayerDim>(Group.NumLayers == 1);
				Permutation.Set<FEvaluateConcatenateScatterCS::FMeshSpaceBlend>(Group.MeshSpaceMask != 0);
				TShaderMapRef<FEvaluateConcatenateScatterCS> ComputeShader(ShaderMap, Permutation);

				PassParameters->NumTotalHeaders  = Group.NumHeaders;
				PassParameters->HeaderBaseOffset = Group.HeaderBaseOffset;
				PassParameters->SlotsPerInstance  = Group.SlotsPerInstance;
				PassParameters->NumCompactBones  = Group.NumCompactBones;
				PassParameters->NumLayers        = Group.NumLayers;
				PassParameters->MeshSpaceMask    = Group.MeshSpaceMask;
				PassParameters->LayerModeMask    = Group.LayerModeMask;

				const uint32 NumGroups = FMath::DivideAndRoundUp(Group.NumHeaders, Group.InstancesPerGroup);
				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *PassParameters, FIntVector(NumGroups, 1, 1));
			}

			RHICmdList.EndUAVOverlap();
		});
	}
}