// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimSequenceTransformProviderData.h"
#include "Animation/Skeleton.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "RendererPrivateUtils.h"
#include "SceneExtensions.h"
#include "SkinningTransformProvider.h"
#include "SkinningSceneExtensionProxy.h"
#include "UnifiedBuffer.h"
#include "SpanAllocator.h"

class FAnimSequenceTransformProvider
	: public ISceneExtension
	, public IAnimSequenceTransformProvider
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FAnimSequenceTransformProvider);

	struct FDispatchGroupKey;
	struct FDispatchGroup;
	struct FTrackUpdateRequest;
public:
	FAnimSequenceTransformProvider(FScene& InScene);

	static bool ShouldCreateExtension(FScene& InScene);

	virtual void InitExtension(FScene& InScene) override;

	void RegisterProxy(FAnimSequenceTransformProviderProxy* Proxy) override;
	void UnregisterProxy(FAnimSequenceTransformProviderProxy* Proxy) override;

	struct FBoneTransformWithUniformScale
	{
		float Rotation[4];
		float Translation[3];
		float Scale;

		void SetTransform(const FTransform3f& Xfm)
		{
			SetRotation(Xfm.GetRotation());
			SetTranslation(Xfm.GetTranslation());
			SetScale(Xfm.GetScale3D().X);
		}

		void SetRotation(FQuat4f Q)
		{
			Rotation[0] = Q.X;
			Rotation[1] = Q.Y;
			Rotation[2] = Q.Z;
			Rotation[3] = Q.W;
		}

		void SetTranslation(FVector3f P)
		{
			Translation[0] = P.X;
			Translation[1] = P.Y;
			Translation[2] = P.Z;
		}

		void SetScale(float S)
		{
			Scale = S;
		}
	};

private:
	struct FMapping;
	struct FSequence;

	void ProvideTransforms(FSkinningTransformProvider::FProviderContext& Context);
	void UploadSequences(FRDGScatterUploader& ScatterUploader, uint32 NumTotalFramesRequested);

	static void UploadBlockHeaders(
		FRHICommandList& RHICmdList,
		double WorldTime,
		float DeltaTime,
		TConstArrayView<FSkinningTransformProvider::FProviderIndirection> Indirections,
		TConstArrayView<FSkinningSceneExtensionProxy*> Proxies,
		uint32& NumTotalFramesRequested,
		uint32& NumTotalSequenceTransforms,
		TConstArrayView<FTrackUpdateRequest> TrackUpdateRequests,
		FRDGBuffer* LayerHeaderBuffer,
		FRDGBuffer* SamplePoseBuffer,
		uint32 NumTotalEntries,
		uint32 NumTotalSamples,
		TMap<FDispatchGroupKey, FDispatchGroup>& DispatchGroups);

	static void ComputeGroupParams(
		uint32 NumCompactBones,
		uint32& OutGroupSize,
		uint32& OutInstancesPerGroup,
		uint32& OutSlotsPerInstance)
	{
		OutSlotsPerInstance = FMath::RoundUpToPowerOfTwo(NumCompactBones);

		if (NumCompactBones <= 64)
		{
			OutGroupSize = 64;
		}
		else if (NumCompactBones <= 128)
		{
			OutGroupSize = 128;
		}
		else if (NumCompactBones <= 256)
		{
			OutGroupSize = 256;
		}
		else if (NumCompactBones <= 512)
		{
			OutGroupSize = 512;
		}
		else
		{
			OutGroupSize = 1024;
		}

		OutInstancesPerGroup = OutGroupSize / OutSlotsPerInstance;
	}

	struct FBoneTables
	{
		TArray<uint16> MeshToCompact;
		TArray<uint16> CompactToMesh;
		TArray<uint16> CompactToParent;
		TArray<uint16> CompactToTargetSkeleton;
		uint32 NumMeshBones = 0;
		uint32 NumCompactBones = 0;
	};

	void BuildBoneTables(
		FBoneTables& BoneTables,
		const USkinnedAsset* SkinnedMesh,
		const FSkeletalMeshLODRenderData& LODRenderData,
		TConstArrayView<FBoneIndexType> BoneMap,
		bool bHasSocketsInBoneMap);

	struct FAnimCompactBoneTable
	{
		TBitArray<> UsedAnimBones;
		TArray<uint16> AnimCompactToAnimBone;
		TArray<uint16> AnimBoneToAnimCompact;
		uint16 NumAnimCompactBones = 0;
		uint16 NumAnimBones = 0;
		bool bDirty = false;
		uint32 RefCount = 0;
		TSet<FMapping*> ReferencingMappings;
		TSet<FSequence*> ReferencingSequences;

		void Init(uint32 InNumAnimBones)
		{
			NumAnimBones = static_cast<uint16>(InNumAnimBones);
			UsedAnimBones.Init(false, InNumAnimBones);
			AnimBoneToAnimCompact.SetNumUninitialized(InNumAnimBones);
			FMemory::Memset(AnimBoneToAnimCompact.GetData(), 0xFF, InNumAnimBones * sizeof(uint16));
		}

		void Rebuild()
		{
			AnimCompactToAnimBone.Reset();
			FMemory::Memset(AnimBoneToAnimCompact.GetData(), 0xFF, AnimBoneToAnimCompact.Num() * sizeof(uint16));

			for (TConstSetBitIterator<> It(UsedAnimBones); It; ++It)
			{
				const uint16 AnimBone = static_cast<uint16>(It.GetIndex());
				AnimBoneToAnimCompact[AnimBone] = static_cast<uint16>(AnimCompactToAnimBone.Num());
				AnimCompactToAnimBone.Add(AnimBone);
			}

			NumAnimCompactBones = static_cast<uint16>(AnimCompactToAnimBone.Num());
		}
	};

	struct FRetargetingTables
	{
		TArray<EBoneTranslationRetargetingMode::Type> Mode;
		TArray<FTransform3f> Transform;
		TArray<TTuple<FQuat4f, FQuat4f>> RefPoseRetargetQuats;
		TArray<uint16> CompactToAnim;
		TArray<uint16> AnimToCompact;
		bool bRequiresRefPoseRetarget = false;

		const FAnimCompactBoneTable* AnimCompactBoneTable = nullptr;
	};

	void BuildRetargetingTables(
		FRetargetingTables& RetargetingTables,
		USkeleton* AnimSkeleton,
		const USkinnedAsset* MeshAsset,
		const FBoneTables& BoneTables,
		FAnimCompactBoneTable& AnimCompactBoneTable);

	bool AccumulateAnimCompactBones(const FRetargetingTables& RetargetingTables, uint32 NumCompactBones, FAnimCompactBoneTable& AnimCompactBoneTable);

	struct FBufferAllocation
	{
		bool IsValid() const
		{
			return Count != 0;
		}

		uint32 Offset = 0;
		uint32 Count  = 0;
	};

	using FSequenceRegistryKey = const UAnimSequence*;

	struct FMappingRegistryKey
	{
		FMappingRegistryKey() = default;

		FMappingRegistryKey(const USkeleton* InSourceSkeleton, const USkinnedAsset* InSkinnedAsset, uint32 InLODIndex, bool bInHasSocketsInBoneMap = false)
			: SourceSkeleton(InSourceSkeleton)
			, SkinnedAsset(InSkinnedAsset)
			, LODIndex(InLODIndex)
			, bHasSocketsInBoneMap(bInHasSocketsInBoneMap)
		{}

		const USkeleton* SourceSkeleton = nullptr;
		const USkinnedAsset* SkinnedAsset = nullptr;
		uint32 LODIndex = 0;
		bool bHasSocketsInBoneMap = false;

		bool operator==(const FMappingRegistryKey& Other) const
		{
			return SourceSkeleton == Other.SourceSkeleton && SkinnedAsset == Other.SkinnedAsset && LODIndex == Other.LODIndex && bHasSocketsInBoneMap == Other.bHasSocketsInBoneMap;
		}

		friend uint32 GetTypeHash(const FMappingRegistryKey& Key)
		{
			return PointerHash(Key.SourceSkeleton, PointerHash(Key.SkinnedAsset, HashCombine(Key.LODIndex, uint32(Key.bHasSocketsInBoneMap))));
		}
	};

	struct FSequence
	{
		FSequenceRegistryKey RegistryKey;
		FBufferAllocation BufferAllocation;
		const UAnimSequence* Sequence = nullptr;
		FAnimCompactBoneTable* AnimCompactBoneTable = nullptr;
		const USkeleton* Skeleton = nullptr;
		FFrameRate SampleRate;
		TBitArray<> FramesRequested;
		TBitArray<> FramesUploaded;
		uint32 NumFramesRequested = 0;
		uint32 NumFramesUploaded = 0;
		float PlayLength = 0.0f;
		uint32 RefCount = 0;
		uint32 NumFrames = 0;
		uint32 NumAnimBones = 0;
		uint32 NumSampleRateLevels = 0;
		uint32 CurrentSampleRateLevel = 0;
		uint32 TargetSampleRateLevel = 0;
		uint32 NextFrameToUpload = 0;

		uint32 GetFloat4sPerBone() const;

		uint64 GetAllocatedSize() const
		{
			return FramesRequested.GetAllocatedSize() + FramesUploaded.GetAllocatedSize();
		}

		void InitUploadState()
		{
			FramesUploaded.Init(false, NumFrames);
			FramesRequested.Init(false, NumFrames);
			NumFramesUploaded = 0;
			NumFramesRequested = 0;
			CurrentSampleRateLevel = NumSampleRateLevels;
			TargetSampleRateLevel = (NumSampleRateLevels > 0) ? NumSampleRateLevels - 1 : 0;
			NextFrameToUpload = 0;
		}

		uint32 GetNumAnimCompactBones() const
		{
			return AnimCompactBoneTable ? AnimCompactBoneTable->NumAnimCompactBones : NumAnimBones;
		}

		uint32 GetFirstMipUploadTransforms() const
		{
			if (CurrentSampleRateLevel != NumSampleRateLevels)
			{
				return 0;
			}

			const uint32 Step = GetSampleStep();
			const uint32 FirstMipFrames = (NumFrames + Step - 1) / Step;
			return FirstMipFrames * GetNumAnimCompactBones();
		}

		int32 RequestFrame(int32 Index, uint32& OutTotalFramesRequested, uint32& OutTotalSequenceTransforms)
		{
			if (!FramesUploaded[Index] && CurrentSampleRateLevel == NumSampleRateLevels)
			{
				auto Bit = FramesRequested[Index];
				if (!Bit)
				{
					Bit = true;
					NumFramesRequested++;
					OutTotalFramesRequested++;
					OutTotalSequenceTransforms += GetNumAnimCompactBones();
					return 1;
				}
			}
			return 0;
		}

		uint32 GetSampleStep() const;

		void GetKeyIndicesFromTime(int32& OutKey0, int32& OutKey1, float& OutAlpha, float Time, EAnimSequenceTrackLoopMode LoopMode) const;
	};

	struct FMapping
	{
		FBufferAllocation MappingTransformBufferAllocation;
		FBufferAllocation RetargetingDataBufferAllocation;
		TArray<FTransform3f> InvGlobalRefPoseTransforms;
		uint32 RetargetingTransformBufferOffset = 0;
		uint32 InvGlobalRefPoseTransformBufferOffset = 0;
		FRetargetingTables RetargetingTables;
		const USkeleton* SourceSkeleton = nullptr;
		const USkinnedAsset* SkinnedAsset = nullptr;
		uint32 LODIndex = 0;
		uint32 NumMeshBones = 0;
		uint32 NumCompactBones = 0;
		uint32 RefCount = 0;
		bool bHasSocketsInBoneMap = false;

		uint64 GetAllocatedSize() const
		{
			return InvGlobalRefPoseTransforms.GetAllocatedSize()
				+ RetargetingTables.Mode.GetAllocatedSize()
				+ RetargetingTables.Transform.GetAllocatedSize()
				+ RetargetingTables.RefPoseRetargetQuats.GetAllocatedSize()
				+ RetargetingTables.CompactToAnim.GetAllocatedSize()
				+ RetargetingTables.AnimToCompact.GetAllocatedSize();
		}
	};

	struct FTrackData
	{
		const FMapping& Mapping;
		FSequence& Sequence;
		uint16 SequenceIndex;
		float Position;
		EAnimSequenceTrackLoopMode LoopMode;
		uint32 NumAnimCompactBones;
	};

	class FProxyUserData : public FInstancedSkinningSceneExtensionProxyUserData
	{
	public:
		static constexpr uint32 InvalidBoneMaskOffset = SKINNING_INVALID_BONE_MASK_OFFSET;

		FProxyUserData(FAnimSequenceTransformProviderProxy* Proxy);

		FAnimSequenceTransformProviderRenderData* GetRenderData() const
		{
			return RenderData;
		}

		FSequence& GetSequence(uint16 InSequenceIndex) const
		{
			return *Sequences[Sequences.IsValidIndex(InSequenceIndex) ? InSequenceIndex : 0];
		}

		const FMapping& GetMapping(uint16 InSequenceIndex, uint32 LODIndex) const
		{
			check(LODIndex >= MinLODLevel && LODIndex < (uint32)LODs.Num());
			const int32 SeqIndex = Sequences.IsValidIndex(InSequenceIndex) ? InSequenceIndex : 0;
			return *SequenceToMappingTable[SeqIndex * LODs.Num() + LODIndex];
		}

		FTrackData GetTrackDataForSequence(uint16 InSequenceIndex, double Position, uint32 LODIndex, EAnimSequenceTrackLoopMode LoopMode) const
		{
			FSequence& Sequence = GetSequence(InSequenceIndex);
			const float WrappedPosition = FAnimSequenceTrackPackedData::WrapPosition(LoopMode, static_cast<float>(Position), Sequence.PlayLength);

			return FTrackData
			{
				  .Mapping  = GetMapping(InSequenceIndex, LODIndex)
				, .Sequence = Sequence
				, .SequenceIndex = InSequenceIndex
				, .Position = WrappedPosition
				, .LoopMode = LoopMode
				, .NumAnimCompactBones = Sequence.GetNumAnimCompactBones()
			};
		}

		struct FLOD
		{
			FBoneTables BoneTables;
			FBufferAllocation BoneMapBufferAllocation;
			int32 BoneMapBufferOffset = 0;
			bool bUploaded = false;

			uint32 GroupSize         = 0;
			uint32 InstancesPerGroup = 1;
			uint32 SlotsPerInstance  = 0;
			FDispatchGroup* DispatchGroup = nullptr;

			TArray<uint32> BoneMaskBufferOffsets;

			bool IsValid() const
			{
				return BoneTables.NumCompactBones != 0;
			}
		};

		FAnimSequenceTransformProviderRenderData* RenderData = nullptr;
		TArray<const UBlendProfile*> BlendProfiles;
		const FInstancedSkinningSceneExtensionProxy& SceneExtensionProxy;
		TArray<FSequence*> Sequences;
		TArray<FMapping*> Mappings;
		TArray<FMapping*> SequenceToMappingTable;
		TArray<FLOD> LODs;
		uint32 MinLODLevel = 0;
		uint32 NumMeshBones = 0;
		bool bIsNaniteMesh = false;

		uint64 GetAllocatedSize() const
		{
			uint64 Size = Sequences.GetAllocatedSize() + Mappings.GetAllocatedSize()
				+ SequenceToMappingTable.GetAllocatedSize() + BlendProfiles.GetAllocatedSize();
			for (const FLOD& LOD : LODs)
			{
				Size += LOD.BoneTables.MeshToCompact.GetAllocatedSize()
					+ LOD.BoneTables.CompactToMesh.GetAllocatedSize()
					+ LOD.BoneTables.CompactToParent.GetAllocatedSize()
					+ LOD.BoneTables.CompactToTargetSkeleton.GetAllocatedSize()
					+ LOD.BoneMaskBufferOffsets.GetAllocatedSize();
			}
			return Size;
		}
	};

	TMap<FMappingRegistryKey, TUniquePtr<FMapping>> MappingRegistry;
	TMap<FSequenceRegistryKey, TUniquePtr<FSequence>> SequenceRegistry;
	TMap<const USkeleton*, TUniquePtr<FAnimCompactBoneTable>> AnimCompactBoneTableRegistry;

	TSet<FSequence*> SequencesToUpload;
	TSet<FMapping*> MappingsToUpload;
	TSet<FAnimCompactBoneTable*> AnimCompactBoneTablesToUpload;
	TSet<FProxyUserData*> ProxyUserDatasToUpload;

	FSpanAllocator SequenceTransformAllocator;
	FRDGAsyncScatterUploadBuffer SequenceTransformUploadBuffer;
	TPersistentStructuredBuffer<FVector4f> SequenceTransformPersistentBuffer;

	FSpanAllocator RetargetingDataAllocator;
	FRDGAsyncScatterUploadBuffer RetargetingDataUploadBuffer;
	TPersistentByteAddressBuffer<UE::HLSL::FBoneRetargetingData> RetargetingDataPersistentBuffer;

	FSpanAllocator BoneHierarchyAllocator;
	FRDGAsyncScatterUploadBuffer BoneHierarchyUploadBuffer;
	TPersistentByteAddressBuffer<UE::HLSL::FBoneHierarchyData> BoneHierarchyPersistentBuffer;

	FSpanAllocator BoneMaskAllocator;
	TPersistentByteAddressBuffer<float> BoneMaskPersistentBuffer;
	FRDGAsyncScatterUploadBuffer BoneMaskUploadBuffer;

	uint64 TotalProxyMemory = 0;
	uint64 TotalMappingMemory = 0;
	uint64 TotalSequenceMemory = 0;
	bool bHalfPrecisionSequences = false;

	uint32 GetFloat4sPerBone() const
	{
		return bHalfPrecisionSequences ? 1 : 2;
	}

	void WriteBoneTransform(TArrayView<FVector4f>& DstFloat4s, uint32 Index, const FBoneTransformWithUniformScale& Transform) const;
};