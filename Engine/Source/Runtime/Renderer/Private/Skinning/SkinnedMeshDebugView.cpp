// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinnedMeshDebugView.h"
#include "Components/InstancedSkinnedMeshComponent.h"

#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "TextureResource.h"

#include "SceneView.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"

#include "PostProcess/PostProcessing.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "Skinning/SkinningSceneExtension.h"
#include "SkinningSceneExtensionProxy.h"

#include "AnimationRuntime.h"

TAutoConsoleVariable<bool> CVarSkinnedMeshDebugDraw(
	TEXT("r.InstancedSkinnedMeshes.DebugDraw"), false,
	TEXT("Whether to enable instanced skinned mesh debug draw."),
	ECVF_RenderThreadSafe
);

class FSkinnedMeshDrawLineVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSkinnedMeshDrawLineVS);
	SHADER_USE_PARAMETER_STRUCT(FSkinnedMeshDrawLineVS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FBatchedPrimitiveParameters, BatchedPrimitive)
		SHADER_PARAMETER(uint32, bPreviousTransform)
		SHADER_PARAMETER(uint32, PersistentPrimitiveIndex)
	END_SHADER_PARAMETER_STRUCT()
};

class FSkinnedMeshDrawLinePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSkinnedMeshDrawLinePS);
	SHADER_USE_PARAMETER_STRUCT(FSkinnedMeshDrawLinePS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

BEGIN_SHADER_PARAMETER_STRUCT(FSkinnedMeshDrawLineParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FSkinnedMeshDrawLineVS::FParameters, VSParameters)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSkinnedMeshDrawLinePS::FParameters, PSParameters)
END_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER(FSkinnedMeshDrawLineVS, "/Engine/Private/Skinning/SkinnedMeshDebug.usf", "LineVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FSkinnedMeshDrawLinePS, "/Engine/Private/Skinning/SkinnedMeshDebug.usf", "LinePS", SF_Pixel);

#if UE_ENABLE_DEBUG_DRAWING

FSkinnedMeshDebugViewExtension::FSkinnedMeshDebugViewExtension(const FAutoRegister& AutoRegister)
: FSceneViewExtensionBase(AutoRegister)
{
	FCoreDelegates::OnEnginePreExit.AddLambda([]()
		{
			Instance = nullptr;
		});
}

void FSkinnedMeshDebugViewExtension::SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled)
{
	if (PassId == EPostProcessingPass::VisualizeDepthOfField)
	{
		if (CVarSkinnedMeshDebugDraw.GetValueOnAnyThread())
		{
			InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FSkinnedMeshDebugViewExtension::PostProcessPass_RenderThread));
		}
	}
}

FScreenPassTexture FSkinnedMeshDebugViewExtension::PostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs)
{
	FScreenPassTextureSlice SceneColorSlice = InOutInputs.GetInput(EPostProcessMaterialInput::SceneColor);
	check(SceneColorSlice.IsValid());

	FScreenPassTexture SceneColor(SceneColorSlice);

	FScreenPassRenderTarget Output = InOutInputs.OverrideOutput;

	// If the override output is provided, it means that this is the last pass in post processing.
	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget(SceneColor.Texture, SceneColor.ViewRect, View.GetOverwriteLoadAction());
	}
	else
	{
		AddDrawTexturePass(GraphBuilder, View, SceneColorSlice, Output);
	}

	RenderSkeletons(GraphBuilder, View, Output);

	return MoveTemp(Output);
}

void FSkinnedMeshDebugViewExtension::RenderSkeletons(FRDGBuilder& GraphBuilder, const FSceneView& View, const FScreenPassRenderTarget& Output)
{
	if (View.Family == nullptr || View.Family->Scene == nullptr)
	{
		return;
	}

	const FScene* Scene = View.Family->Scene->GetRenderScene();
	if (Scene == nullptr)
	{
		return;
	}

	const FSkinningSceneExtension* SkinningExtension = Scene->GetExtensionPtr<FSkinningSceneExtension>();
	if (SkinningExtension == nullptr)
	{
		return;
	}

	TArray<FPrimitiveSceneInfo*> SkinnedPrimitives;
	SkinningExtension->GetSkinnedPrimitives(SkinnedPrimitives);

	TArray<FSkinnedMeshPrimitive> Primitives;
	Primitives.Reserve(SkinnedPrimitives.Num());

	for (const FPrimitiveSceneInfo* SceneInfo : SkinnedPrimitives)
	{
		if (SceneInfo == nullptr || !SceneInfo->GetPersistentIndex().IsValid() || SceneInfo->Proxy == nullptr)
		{
			continue;
		}

		FSkinningSceneExtensionProxy* Proxy = SceneInfo->Proxy->GetSkinningSceneExtensionProxy();

		if (!Proxy || Proxy->GetMaxBoneHierarchyCount() == 0)
		{
			continue;
		}

		FSkinnedMeshPrimitive& Primitive = Primitives.AddDefaulted_GetRef();
		Primitive.Index = SceneInfo->GetPersistentIndex();
		Primitive.BoneCount = Proxy->GetMaxBoneHierarchyCount();
		Primitive.InstanceCount = SceneInfo->GetNumInstanceSceneDataEntries();
	}

	if (Primitives.Num() == 0)
	{
		return;
	}

	const FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(View.GetFeatureLevel());

	TShaderMapRef<FSkinnedMeshDrawLineVS> VertexShader(ShaderMap);
	TShaderMapRef<FSkinnedMeshDrawLinePS> PixelShader(ShaderMap);

	if (VertexShader.IsNull() || PixelShader.IsNull())
	{
		return;
	}

	const FIntRect ViewRect = Output.ViewRect;

	TRDGUniformBufferRef<FSceneUniformParameters> SceneUniformParameters = GetSceneUniformBufferRef(GraphBuilder, View);

	FSkinnedMeshDrawLineParameters* PassParameters = GraphBuilder.AllocParameters<FSkinnedMeshDrawLineParameters>();
	PassParameters->VSParameters.View = View.ViewUniformBuffer;
	PassParameters->VSParameters.Scene = SceneUniformParameters;
	PassParameters->PSParameters.RenderTargets[0] = Output.GetRenderTargetBinding();

	{
		FBatchedPrimitiveParameters* UniformParameters = GraphBuilder.AllocParameters<FBatchedPrimitiveParameters>();
		UniformParameters->Data = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 4));
		PassParameters->VSParameters.BatchedPrimitive = GraphBuilder.CreateUniformBuffer(UniformParameters);
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("RenderSkinnedMeshDebug"),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, PassParameters, ViewRect = ViewRect, Primitives = MoveTemp(Primitives)](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

			GraphicsPSOInit.DepthStencilState	= TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BlendState			= TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_One, BF_Zero>::GetRHI();
			GraphicsPSOInit.RasterizerState		= TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.PrimitiveType		= PT_LineList;

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI	= GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI		= VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI			= PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			const auto DrawPrimitives = [&]
			{
				for (const FSkinnedMeshPrimitive& Primitive : Primitives)
				{
					const uint32 BoneCount = Primitive.BoneCount;
					const uint32 InstanceCount = Primitive.InstanceCount;

					check(Primitive.Index.Index != INDEX_NONE);
					PassParameters->VSParameters.PersistentPrimitiveIndex = uint32(Primitive.Index.Index);

					SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VSParameters);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PSParameters);
					RHICmdList.DrawPrimitive(0, BoneCount, InstanceCount);
				}
			};

			PassParameters->VSParameters.bPreviousTransform = true;
			DrawPrimitives();

			PassParameters->VSParameters.bPreviousTransform = false;
			DrawPrimitives();
		}
	);
}

#endif
