// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneViewExtension.h"
#include "RenderGraphFwd.h"
#include "MeshPassProcessor.h"

#if UE_ENABLE_DEBUG_DRAWING

struct FSkinnedMeshPrimitive
{
	FPersistentPrimitiveIndex Index;
	uint32 BoneCount = 0;
	uint32 InstanceCount = 0;
};

class FSkinnedMeshDebugViewExtension : public FSceneViewExtensionBase
{
public:
	FSkinnedMeshDebugViewExtension(const FAutoRegister& AutoRegister);

	virtual void SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) override;

	FScreenPassTexture PostProcessPass_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const struct FPostProcessMaterialInputs& InOutInputs);

	void RenderSkeletons(class FRDGBuilder& GraphBuilder, const class FSceneView& View, const struct FScreenPassRenderTarget& Output);

	static void Init()
	{
		Instance = FSceneViewExtensions::NewExtension<FSkinnedMeshDebugViewExtension>();
	}

private:
	inline static TSharedPtr<FSkinnedMeshDebugViewExtension, ESPMode::ThreadSafe> Instance;
};

#endif

static void InitSkinnedMeshDebugViewExtension()
{
#if UE_ENABLE_DEBUG_DRAWING
	FSkinnedMeshDebugViewExtension::Init();
#endif
}