// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinningSceneExtension.h"
#include "ViewDefinitions.h"
#include "ScenePrivate.h"
#include "RenderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SkeletalRenderPublic.h"
#include "SkinningDefinitions.h"
#include "ViewData.h"
#include "SceneCulling/SceneCullingRenderer.h"
#include "UnifiedBuffer.h"
#include "Animation/Skeleton.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "SkinningDefinitions.h"
#include "SkinningSceneExtensionProxy.h"
#include "GPUSceneWriter.h"
#include "GPUScene.h"

DECLARE_STATS_GROUP(TEXT("SkinningSceneExtension"), STATGROUP_SkinningSceneExtension, STATCAT_Advanced);

DECLARE_DWORD_COUNTER_STAT(TEXT("Num Primitives"), STAT_SkinningSceneExtension_NumPrimitives, STATGROUP_SkinningSceneExtension);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Allocations"), STAT_SkinningSceneExtension_NumAllocations, STATGROUP_SkinningSceneExtension);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Updates"), STAT_SkinningSceneExtension_NumUpdates, STATGROUP_SkinningSceneExtension);

#if STATS
#define DECLARE_SKINNING_MEMORY_STAT(CounterName,StatId,GroupId) \
	DECLARE_STAT(CounterName,StatId,GroupId,EStatDataType::ST_int64, EStatFlags::ClearEveryFrame, FPlatformMemory::MCR_Physical); \
	static DEFINE_STAT(StatId)

#else
#define DECLARE_SKINNING_MEMORY_STAT(CounterName,StatId,GroupId)
#endif

DECLARE_SKINNING_MEMORY_STAT(TEXT("Header Buffer Size"), STAT_SkinningSceneExtension_HeaderBufferSize, STATGROUP_SkinningSceneExtension);
DECLARE_SKINNING_MEMORY_STAT(TEXT("Bone Transform Buffer Size"), STAT_SkinningSceneExtension_BoneTransformBufferSize, STATGROUP_SkinningSceneExtension);
DECLARE_SKINNING_MEMORY_STAT(TEXT("Bone Map Buffer Size"), STAT_SkinningSceneExtension_BoneMapBufferSize, STATGROUP_SkinningSceneExtension);
DECLARE_SKINNING_MEMORY_STAT(TEXT("Bone Hierarchy Buffer Size"), STAT_SkinningSceneExtension_BoneHierarchyBufferSize, STATGROUP_SkinningSceneExtension);
DECLARE_SKINNING_MEMORY_STAT(TEXT("Bone Object Space Buffer Size"), STAT_SkinningSceneExtension_BoneObjectSpaceBufferSize, STATGROUP_SkinningSceneExtension);
DECLARE_SKINNING_MEMORY_STAT(TEXT("Bone Transform Upload Size"), STAT_SkinningSceneExtension_BoneTransformUploadSize, STATGROUP_SkinningSceneExtension);
DECLARE_SKINNING_MEMORY_STAT(TEXT("Bone Transform Allocator Size"), STAT_SkinningSceneExtension_BoneTransformAllocatorSize, STATGROUP_SkinningSceneExtension);

static int32 GSkinningBuffersTransformDataMinSizeBytes = 4 * 1024;
static FAutoConsoleVariableRef CVarSkinningBuffersTransformDataMinSizeBytes(
	TEXT("r.Skinning.Buffers.TransformDataMinSizeBytes"),
	GSkinningBuffersTransformDataMinSizeBytes,
	TEXT("The smallest size (in bytes) of the bone transform data buffer."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static int32 GSkinningBuffersHeaderDataMinSizeBytes = 4 * 1024;
static FAutoConsoleVariableRef CVarSkinningBuffersHeaderDataMinSizeBytes(
	TEXT("r.Skinning.Buffers.HeaderDataMinSizeBytes"),
	GSkinningBuffersHeaderDataMinSizeBytes,
	TEXT("The smallest size (in bytes) of the per-primitive skinning header data buffer."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static bool GSkinningBuffersAsyncUpdate = true;
static FAutoConsoleVariableRef CVarSkinningBuffersAsyncUpdates(
	TEXT("r.Skinning.Buffers.AsyncUpdate"),
	GSkinningBuffersAsyncUpdate,
	TEXT("When non-zero, skinning data buffer updates are updated asynchronously."),
	ECVF_RenderThreadSafe
);

static int32 GSkinningBuffersForceFullUpload = 0;
static FAutoConsoleVariableRef CVarSkinningBuffersForceFullUpload(
	TEXT("r.Skinning.Buffers.ForceFullUpload"),
	GSkinningBuffersForceFullUpload,
	TEXT("0: Do not force a full upload.\n")
	TEXT("1: Force one full upload on the next update.\n")
	TEXT("2: Force a full upload every frame."),
	ECVF_RenderThreadSafe
);

static bool GSkinningBuffersDefrag = true;
static FAutoConsoleVariableRef CVarSkinningBuffersDefrag(
	TEXT("r.Skinning.Buffers.Defrag"),
	GSkinningBuffersDefrag,
	TEXT("Whether or not to allow defragmentation of the skinning buffers."),
	ECVF_RenderThreadSafe
);

static int32 GSkinningBuffersForceDefrag = 0;
static FAutoConsoleVariableRef CVarSkinningBuffersForceDefrag(
	TEXT("r.Skinning.Buffers.Defrag.Force"),
	GSkinningBuffersForceDefrag,
	TEXT("0: Do not force a full defrag.\n")
	TEXT("1: Force one full defrag on the next update.\n")
	TEXT("2: Force a full defrag every frame."),
	ECVF_RenderThreadSafe
);

static float GSkinningBuffersDefragLowWatermark = 0.375f;
static FAutoConsoleVariableRef CVarSkinningBuffersDefragLowWatermark(
	TEXT("r.Skinning.Buffers.Defrag.LowWatermark"),
	GSkinningBuffersDefragLowWatermark,
	TEXT("Ratio of used to allocated memory at which to decide to defrag the skinning buffers."),
	ECVF_RenderThreadSafe
);

static bool GSkinningTransformProviders = true;
static FAutoConsoleVariableRef CVarSkinningTransformProviders(
	TEXT("r.Skinning.TransformProviders"),
	GSkinningTransformProviders,
	TEXT("When set, transform providers are enabled (if registered)."),
	ECVF_RenderThreadSafe
);

static float GSkinningDefaultAnimationMinScreenSize = 0.1f;
static FAutoConsoleVariableRef CVarSkinningDefaultAnimationMinScreenSize(
	TEXT("r.Skinning.DefaultAnimationMinScreenSize"),
	GSkinningDefaultAnimationMinScreenSize,
	TEXT("Default animation screen size to stop animating at, applies when the per-component value is 0.0."),
	ECVF_RenderThreadSafe
);

BEGIN_UNIFORM_BUFFER_STRUCT(FSkinningSceneParameters, RENDERER_API)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, Headers)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BoneMap)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BoneHierarchy)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BoneObjectSpace)
	SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_SRV(BoneTransforms)
END_UNIFORM_BUFFER_STRUCT()

DECLARE_SCENE_UB_STRUCT(FSkinningSceneParameters, Skinning, RENDERER_API)

// Blackboard struct to cache the result of FinishSkinningBufferUpload.
// Avoids redundant buffer uploads when the function is called multiple
// times within the same RDG builder (PreGPUSceneUpdate + UpdateSceneUniformBuffer).
struct FSkinningBufferUploadResult
{
	FSkinningSceneParameters Parameters;
};
RDG_REGISTER_BLACKBOARD_STRUCT(FSkinningBufferUploadResult)

// Reference pose transform provider
struct FTransformBlockHeader
{
	uint32 BlockLocalIndex;
	uint32 BlockTransformCount;
	uint32 BlockTransformOffset;
};

class FRefPoseTransformProviderCS : public FGlobalShader
{
public:
	static constexpr uint32 TransformsPerGroup = 64u;

private:
	DECLARE_GLOBAL_SHADER(FRefPoseTransformProviderCS);
	SHADER_USE_PARAMETER_STRUCT(FRefPoseTransformProviderCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_UAV(TransformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FTransformBlockHeader>, HeaderBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);

		OutEnvironment.SetDefine(TEXT("TRANSFORMS_PER_GROUP"), TransformsPerGroup);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRefPoseTransformProviderCS, "/Engine/Private/Skinning/TransformProviders.usf", "RefPoseProviderCS", SF_Compute);

struct FCopyTransformsHeader
{
	uint32 SrcByteOffset;
	uint32 DstByteOffset;
	uint32 Count;
	uint32 _Pad0;
};

class FCopyPreviousTransformsCS : public FGlobalShader
{
public:
	static constexpr uint32 ThreadGroupSize = 64u;

	class FUseStagingBuffer : SHADER_PERMUTATION_BOOL("USE_STAGING_BUFFER");
	using FPermutationDomain = TShaderPermutationDomain<FUseStagingBuffer>;

private:
	DECLARE_GLOBAL_SHADER(FCopyPreviousTransformsCS);
	SHADER_USE_PARAMETER_STRUCT(FCopyPreviousTransformsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_UAV(DstTransformBuffer)
		SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_SRV(SrcTransformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FCopyTransformsHeader>, CopyHeaders)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		OutEnvironment.SetDefine(TEXT("COPY_THREAD_GROUP_SIZE"), ThreadGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCopyPreviousTransformsCS, "/Engine/Private/Skinning/CopyPreviousTransforms.usf", "CopyPreviousTransformsCS", SF_Compute);

static FGuid RefPoseProviderId(REF_POSE_TRANSFORM_PROVIDER_GUID);
static FGuid MeshObjectProviderId(ANIM_MESH_OBJECT_TRANSFORM_PROVIDER_GUID);

static void GetDefaultSkinningParameters(FSkinningSceneParameters& OutParameters, FRDGBuilder& GraphBuilder)
{
	auto DefaultBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, 4u));
	OutParameters.Headers			= DefaultBuffer;
	OutParameters.BoneMap			= DefaultBuffer;
	OutParameters.BoneHierarchy		= DefaultBuffer;
	OutParameters.BoneObjectSpace	= DefaultBuffer;
	OutParameters.BoneTransforms	= GetCompressedBoneTransformSRV(GraphBuilder, (GetDefaultCompressedBoneTransformBuffer(GraphBuilder)));
}

IMPLEMENT_SCENE_EXTENSION(FSkinningSceneExtension);

FSkeletonBatchKey FSkinningSceneExtension::FHeaderData::GetSkeletonBatchKey() const
{
	return FSkeletonBatchKey
	{
	#if ENABLE_SKELETON_DEBUG_NAME
		.SkeletonName = Proxy->GetSkinnedAsset()->GetSkeleton()->GetFName(),
	#endif
		.SkeletonGuid = Proxy->GetSkinnedAsset()->GetSkeleton()->GetGuid(),
		.TransformProviderId = Proxy->GetTransformProviderId(),
		.bNanite = bIsNanite != 0
	};
}

bool FSkinningSceneExtension::ShouldCreateExtension(FScene& InScene)
{
	return IsGPUSkinSceneExtensionEnabled() || (NaniteSkinnedMeshesSupported() && DoesRuntimeSupportNanite(GetFeatureLevelShaderPlatform(InScene.GetFeatureLevel()), true, true));
}

FSkinningSceneExtension::FSkinningSceneExtension(FScene& InScene)
:	ISceneExtension(InScene)
{
	WorldRef = InScene.GetWorld();
	UpdateTimerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FSkinningSceneExtension::Tick));
}

FSkinningSceneExtension::~FSkinningSceneExtension()
{
	FTSTicker::RemoveTicker(UpdateTimerHandle);
}

void FSkinningSceneExtension::InitExtension(FScene& InScene)
{
	// Register animation runtime and reference pose transform providers
	if (auto TransformProvider = Scene.GetExtensionPtr<FSkinningTransformProvider>())
	{
		TransformProvider->RegisterProvider(
			GetRefPoseProviderId(),
			FSkinningTransformProvider::FOnProvideTransforms::CreateStatic(&FSkinningSceneExtension::ProvideRefPoseTransforms),
			false /* Use skeleton batching */
		);

		TransformProvider->RegisterProvider(
			GetMeshObjectProviderId(),
			FSkinningTransformProvider::FOnProvideTransforms::CreateStatic(&FSkinningSceneExtension::ProvideMeshObjectTransforms),
			false /* Use skeleton batching */
		);

		const bool bNaniteEnabled = UseNanite(GetFeatureLevelShaderPlatform(InScene.GetFeatureLevel()));
		SetEnabled(IsGPUSkinSceneExtensionEnabled() || bNaniteEnabled);
	}
}

ISceneExtensionUpdater* FSkinningSceneExtension::CreateUpdater()
{
	return new FUpdater(*this);
}

ISceneExtensionRenderer* FSkinningSceneExtension::CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags)
{
	// We only need to create renderers when we're enabled
	if (!IsEnabled() || !InSceneRenderer.GetViewFamily())
	{
		return nullptr;
	}

	return new FRenderer(InSceneRenderer, *this);
}

void FSkinningSceneExtension::SetEnabled(bool bEnabled)
{
	if (bEnabled != IsEnabled())
	{
		if (bEnabled)
		{
			Buffers = MakeUnique<FBuffers>();
		}
		else
		{
			Buffers = nullptr;
			BoneMapAllocator.Reset();
			BoneHierarchyAllocator.Reset();
			ObjectSpaceAllocator.Reset();
			TransformAllocator.Reset();
			HeaderDatas.Reset();
			BatchHeaderDatas.Reset();
			AllocatedHeaderDataIndices.Reset();
			InstancedHeaderDataIndices.Reset();
		}
	}
}

void FSkinningSceneExtension::FinishSkinningBufferUpload(FRDGBuilder& GraphBuilder, FSkinningSceneParameters* OutParams, bool bUpdateStats)
{
	if (!IsEnabled())
	{
		return;
	}

	// If we've already uploaded this builder, just return the cached parameters.
	if (const FSkinningBufferUploadResult* Cached = GraphBuilder.Blackboard.Get<FSkinningBufferUploadResult>())
	{
		if (OutParams)
		{
			*OutParams = Cached->Parameters;
		}
		return;
	}

	FRDGBufferRef HeaderBuffer = nullptr;
	FRDGBufferRef BoneMapBuffer = nullptr;
	FRDGBufferRef BoneHierarchyBuffer = nullptr;
	FRDGBufferRef BoneObjectSpaceBuffer = nullptr;
	FRDGBufferRef TransformBuffer = nullptr;

	// Sync on upload tasks
	UE::Tasks::Wait(
		MakeArrayView(
			{
				TaskHandles[UploadHeaderDataTask],
				TaskHandles[UploadHierarchyDataTask],
				TaskHandles[UploadTransformDataTask]
			}
		)
	);

	const uint32 MinHeaderDataSize = (HeaderDatas.GetMaxIndex() + 1);
	const uint32 MinTransformDataSize = TransformAllocator.GetMaxSize();
	const uint32 MinBoneMapDataSize = BoneMapAllocator.GetMaxSize();
	const uint32 MinBoneHierarchyDataSize = BoneHierarchyAllocator.GetMaxSize();
	const uint32 MinObjectSpaceDataSize = ObjectSpaceAllocator.GetMaxSize();

	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	if (Uploader.IsValid())
	{
		HeaderBuffer = Uploader->HeaderDataUploader.ResizeAndUploadTo(
			GraphBuilder,
			Buffers->HeaderDataBuffer,
			MinHeaderDataSize
		);

		BoneMapBuffer = Uploader->BoneMapUploader.ResizeAndUploadTo(
			GraphBuilder,
			Buffers->BoneMapBuffer,
			MinBoneMapDataSize
		);

		BoneHierarchyBuffer = Uploader->BoneHierarchyUploader.ResizeAndUploadTo(
			GraphBuilder,
			Buffers->BoneHierarchyBuffer,
			MinBoneHierarchyDataSize
		);

		BoneObjectSpaceBuffer = Uploader->BoneObjectSpaceUploader.ResizeAndUploadTo(
			GraphBuilder,
			Buffers->BoneObjectSpaceBuffer,
			MinObjectSpaceDataSize
		);

		TransformBuffer = Uploader->TransformDataUploader.ResizeAndUploadTo(
			GraphBuilder,
			Buffers->TransformDataBuffer,
			MinTransformDataSize
		);

		Uploader = nullptr;
	}
	else
	{
		HeaderBuffer			= Buffers->HeaderDataBuffer.ResizeBufferIfNeeded(GraphBuilder, MinHeaderDataSize);
		BoneMapBuffer			= Buffers->BoneMapBuffer.ResizeBufferIfNeeded(GraphBuilder, MinBoneMapDataSize);
		BoneHierarchyBuffer		= Buffers->BoneHierarchyBuffer.ResizeBufferIfNeeded(GraphBuilder, MinBoneHierarchyDataSize);
		BoneObjectSpaceBuffer	= Buffers->BoneObjectSpaceBuffer.ResizeBufferIfNeeded(GraphBuilder, MinObjectSpaceDataSize);
		TransformBuffer			= Buffers->TransformDataBuffer.ResizeBufferIfNeeded(GraphBuilder, MinTransformDataSize);
	}

	if (bUpdateStats)
	{
		INC_DWORD_STAT_BY(STAT_SkinningSceneExtension_NumPrimitives, HeaderDatas.Num());
		INC_MEMORY_STAT_BY(STAT_SkinningSceneExtension_HeaderBufferSize, HeaderBuffer->Desc.GetSize());
		INC_MEMORY_STAT_BY(STAT_SkinningSceneExtension_BoneTransformBufferSize, TransformBuffer->Desc.GetSize());
		INC_MEMORY_STAT_BY(STAT_SkinningSceneExtension_BoneMapBufferSize, BoneMapBuffer->Desc.GetSize());
		INC_MEMORY_STAT_BY(STAT_SkinningSceneExtension_BoneHierarchyBufferSize, BoneHierarchyBuffer->Desc.GetSize());
		INC_MEMORY_STAT_BY(STAT_SkinningSceneExtension_BoneObjectSpaceBufferSize, BoneObjectSpaceBuffer->Desc.GetSize());
		INC_MEMORY_STAT_BY(STAT_SkinningSceneExtension_BoneTransformAllocatorSize, MinTransformDataSize * sizeof(FCompressedBoneTransform));
	}

	if (!Buffers->TransformDataBufferRHI || Buffers->TransformDataBufferRHI != TransformBuffer->GetRHIUnchecked())
	{
		Buffers->TransformDataBufferRHI = TransformBuffer->GetRHIUnchecked();
		Buffers->TransformDataBufferSRV = GraphBuilder.GetPooledBuffer(TransformBuffer)->GetOrCreateSRV(GraphBuilder.RHICmdList, FRHIBufferSRVCreateInfo(COMPRESSED_BONE_TRANSFORM_PIXEL_FORMAT));
		bVertexFactoryFullUpload = true;
	}

	// Cache the result in the blackboard so subsequent calls within the same RDG builder can skip the upload work entirely.
	if (OutParams != nullptr)
	{
		FSkinningBufferUploadResult& CachedResult = GraphBuilder.Blackboard.Create<FSkinningBufferUploadResult>();
		CachedResult.Parameters.Headers			= GraphBuilder.CreateSRV(HeaderBuffer);
		CachedResult.Parameters.BoneMap			= GraphBuilder.CreateSRV(BoneMapBuffer);
		CachedResult.Parameters.BoneHierarchy	= GraphBuilder.CreateSRV(BoneHierarchyBuffer);
		CachedResult.Parameters.BoneObjectSpace	= GraphBuilder.CreateSRV(BoneObjectSpaceBuffer);
		CachedResult.Parameters.BoneTransforms	= GetCompressedBoneTransformSRV(GraphBuilder, TransformBuffer);

		*OutParams = CachedResult.Parameters;
	}
}

void FSkinningSceneExtension::PerformSkinning(FSkinningSceneParameters& Parameters, FRDGBuilder& GraphBuilder, const FGameTime& CurrentTime)
{
	RDG_EVENT_SCOPE(GraphBuilder, "Skinning");

	FRDGScatterUploadBuilder& ScatterUploadBuilder = *FRDGScatterUploadBuilder::Create(GraphBuilder);

	ON_SCOPE_EXIT
	{
		// Vertex factories are updated prior to rendering regardless of whether the skinning is actually executed. They could
		// technically be part of the updater but a command list task is expensive, so it's batched together with other skinning work.
		if (!VertexFactoryUpdateList.IsEmpty() || bVertexFactoryFullUpload)
		{
			ScatterUploadBuilder.AddFunction([this, TaskVertexFactoryUpdateList = MoveTemp(VertexFactoryUpdateList)] (FRHICommandListBase& RHICmdList)
			{
				const auto UpdateVertexFactory = [&](FHeaderData& Data)
				{
					Data.MeshObject->UpdateSceneExtensionHeader(RHICmdList, Data.Pack(), Buffers->TransformDataBufferSRV);
					Data.bVertexFactoryDirty = false;
				};

				if (bVertexFactoryFullUpload)
				{
					for (FHeaderData& Data : HeaderDatas)
					{
						UpdateVertexFactory(Data);
					}
				}
				else
				{
					for (int32 HeaderDataIndex : TaskVertexFactoryUpdateList)
					{
						UpdateVertexFactory(HeaderDatas[HeaderDataIndex]);
					}
				}

				bVertexFactoryFullUpload = false;
			});
		}

		RDG_EVENT_SCOPE(GraphBuilder, "ScatterUpload");
		ScatterUploadBuilder.Execute(GraphBuilder);
	};

	if (!GSkinningTransformProviders)
	{
		return;
	}

	if (auto TransformProvider = Scene.GetExtensionPtr<FSkinningTransformProvider>())
	{
		if (HeaderDatas.Num() == 0)
		{
			return;
		}

		const TArray<FGuid> SkeletonProviderIds = TransformProvider->GetSkeletonProviderIds();
		const TArray<FGuid> PrimitiveProviderIds = TransformProvider->GetPrimitiveProviderIds();

		checkf((SkeletonProviderIds.Num() + PrimitiveProviderIds.Num()) < 256, TEXT("The number of provider ids exceeds storage capacity for PrimitivesToRangeIndex."));

		auto ResetRanges = [](const TArray<FGuid>& Providers, TArray<FSkinningTransformProvider::FProviderRange, TInlineAllocator<8>>& Ranges)
		{
			Ranges.Reset();
			for (const FGuid& ProviderId : Providers)
			{
				FSkinningTransformProvider::FProviderRange& Range = Ranges.Emplace_GetRef();
				Range.Id = ProviderId;
				Range.Count = 0;
				Range.Offset = 0;
			}
		};

		// TODO: Optimize further (incremental tracking of primitives within provider extension?)
		// The current assumption is that skinned primitive counts should be fairly low, and heavy
		// instancing would be used. If we need a ton of primitives, revisit this algorithm.

		struct FOffsets
		{
			uint32 CurrentTransformOffset;
			uint32 PreviousTransformOffset;
			uint32 BoneMapOffset;
			EDirtyBoneTransforms DirtyBoneTransforms;
		};

		// Skeleton
		if (BatchHeaderDatas.Num() > 0)
		{
			TArray<FSkinningTransformProvider::FProviderRange, TInlineAllocator<8>> SkeletonRanges;
			SkeletonRanges.Reserve(SkeletonProviderIds.Num());
			ResetRanges(SkeletonProviderIds, SkeletonRanges);

			TArrayView<FSkeletonBatch> Batches = GraphBuilder.AllocPODArrayView<FSkeletonBatch>(BatchHeaderDatas.Num());
			TArrayView<FOffsets> Offsets = GraphBuilder.AllocPODArrayView<FOffsets>(BatchHeaderDatas.Num());

			uint32 TotalOffset = 0;
			uint32 TotalBatches = 0;

			for (auto& [BatchKey, BatchEntry] : BatchHeaderDatas)
			{
				const FHeaderData& Header = BatchEntry.HeaderData;
				const FGuid ProviderId = BatchKey.TransformProviderId;
				for (FSkinningTransformProvider::FProviderRange& Range : SkeletonRanges)
				{
					if (ProviderId == Range.Id)
					{
						++Range.Count;
						break;
					}
				}

				Batches[TotalBatches] = FSkeletonBatch
				{
				#if ENABLE_SKELETON_DEBUG_NAME
					.SkeletonName = BatchKey.SkeletonName,
				#endif
					.SkeletonGuid = BatchKey.SkeletonGuid,
					.MaxBoneTransforms = Header.MaxTransformCount,
					.UniqueAnimationCount = Header.UniqueAnimationCount
				};

				Offsets[TotalBatches].CurrentTransformOffset  = Header.TransformBufferOffset + ( Header.CurrentTransformSlot * Header.MaxTransformCount);
				Offsets[TotalBatches].PreviousTransformOffset = Header.TransformBufferOffset + (!Header.CurrentTransformSlot * Header.MaxTransformCount);
				Offsets[TotalBatches].BoneMapOffset = Header.BoneMapBufferOffset;
				Offsets[TotalBatches].DirtyBoneTransforms = EDirtyBoneTransforms::All;

				++TotalBatches;
			}

			uint32 IndirectionCount = 0;

			for (FSkinningTransformProvider::FProviderRange& Range : SkeletonRanges)
			{
				Range.Offset = IndirectionCount;
				IndirectionCount += Range.Count;
				Range.Count = 0;
			}

			uint32 TotalBatchIndices = 0;

			TArrayView<FSkinningTransformProvider::FProviderIndirection> BatchIndices = GraphBuilder.AllocPODArrayView<FSkinningTransformProvider::FProviderIndirection>(IndirectionCount);
			for (auto& [HeaderDataCacheKey, BatchEntry] : BatchHeaderDatas)
			{
				const FGuid ProviderId = HeaderDataCacheKey.TransformProviderId;

				for (FSkinningTransformProvider::FProviderRange& Range : SkeletonRanges)
				{
					if (ProviderId == Range.Id)
					{
						BatchIndices[Range.Offset + Range.Count] = FSkinningTransformProvider::FProviderIndirection(
							TotalBatchIndices,
							Offsets[TotalBatchIndices].CurrentTransformOffset  * sizeof(FCompressedBoneTransform),
							Offsets[TotalBatchIndices].PreviousTransformOffset * sizeof(FCompressedBoneTransform),
							Offsets[TotalBatchIndices].BoneMapOffset != INDEX_NONE ? Offsets[TotalBatchIndices].BoneMapOffset * sizeof(uint32) : INDEX_NONE,
							Offsets[TotalBatchIndices].DirtyBoneTransforms
						);
						++Range.Count;
						break;
					}
				}

				++TotalBatchIndices;
			}

			if (!ensure(TotalBatches == TotalBatchIndices))
			{
				return;
			}

			FSkinningTransformProvider::FProviderContext Context(
				TConstArrayView<FPrimitiveSceneInfo*>{},
				TConstArrayView<FSkinningSceneExtensionProxy*>{},
				BatchIndices,
				Batches,
				CurrentTime,
				GraphBuilder,
				ScatterUploadBuilder,
				Parameters.BoneTransforms->GetParent(),
				Parameters.BoneMap
			);

			TransformProvider->Broadcast(SkeletonRanges, Context);
		}

		// Primitive
		if (AllocatedHeaderDataIndices.Num() > 0)
		{
			TArray<uint8, FConcurrentLinearArrayAllocator> PrimitivesToRangeIndex;
			PrimitivesToRangeIndex.AddUninitialized(HeaderDatas.Num());

			TArray<FSkinningTransformProvider::FProviderRange, TInlineAllocator<8>> PrimitiveRanges;
			PrimitiveRanges.Reserve(PrimitiveProviderIds.Num());
			ResetRanges(PrimitiveProviderIds, PrimitiveRanges);

			TArrayView<FPrimitiveSceneInfo*> Primitives = GraphBuilder.AllocPODArrayView<FPrimitiveSceneInfo*>(AllocatedHeaderDataIndices.Num());
			TArrayView<FSkinningSceneExtensionProxy*> Proxies = GraphBuilder.AllocPODArrayView<FSkinningSceneExtensionProxy*>(AllocatedHeaderDataIndices.Num());
			TArrayView<FOffsets> Offsets = GraphBuilder.AllocPODArrayView<FOffsets>(AllocatedHeaderDataIndices.Num());

			uint32 TotalOffset = 0;

			uint32 PrimitiveCount = 0;
			for (const int32 HeaderDataIndex : AllocatedHeaderDataIndices)
			{
				FHeaderData& Header = HeaderDatas[HeaderDataIndex];
				Header.Validate();

				int32 RangeIndex = 0;

				if (!EnumHasAnyFlags(Header.DirtyBoneTransforms, EDirtyBoneTransforms::All))
				{
					continue;
				}

				for (; RangeIndex < PrimitiveRanges.Num(); ++RangeIndex)
				{
					FSkinningTransformProvider::FProviderRange& Range = PrimitiveRanges[RangeIndex];

					if (Header.ProviderId == Range.Id)
					{
						++Range.Count;
						break;
					}
				}

				check(RangeIndex != PrimitiveRanges.Num());

				PrimitivesToRangeIndex[PrimitiveCount] = RangeIndex;
				Primitives[PrimitiveCount] = Header.PrimitiveSceneInfo;
				Proxies[PrimitiveCount] = Header.Proxy;
				Offsets[PrimitiveCount].CurrentTransformOffset  = Header.TransformBufferOffset + ( Header.CurrentTransformSlot * Header.MaxTransformCount);
				Offsets[PrimitiveCount].PreviousTransformOffset = Header.TransformBufferOffset + (!Header.CurrentTransformSlot * Header.MaxTransformCount);
				Offsets[PrimitiveCount].BoneMapOffset = Header.BoneMapBufferOffset;
				Offsets[PrimitiveCount].DirtyBoneTransforms = Header.DirtyBoneTransforms;

				Header.DirtyBoneTransforms = EDirtyBoneTransforms::None;

				++PrimitiveCount;
			}

			uint32 IndirectionCount = 0;

			for (FSkinningTransformProvider::FProviderRange& Range : PrimitiveRanges)
			{
				Range.Offset = IndirectionCount;
				IndirectionCount += Range.Count;
				Range.Count = 0;
			}

			TArrayView<FSkinningTransformProvider::FProviderIndirection> PrimitiveIndices = GraphBuilder.AllocPODArrayView<FSkinningTransformProvider::FProviderIndirection>(IndirectionCount);
			for (uint32 PrimitiveIndex = 0; PrimitiveIndex < PrimitiveCount; ++PrimitiveIndex)
			{
				FSkinningTransformProvider::FProviderRange& Range = PrimitiveRanges[PrimitivesToRangeIndex[PrimitiveIndex]];
				PrimitiveIndices[Range.Offset + Range.Count] = FSkinningTransformProvider::FProviderIndirection(
					PrimitiveIndex,
					Offsets[PrimitiveIndex].CurrentTransformOffset * sizeof(FCompressedBoneTransform),
					Offsets[PrimitiveIndex].PreviousTransformOffset * sizeof(FCompressedBoneTransform),
					Offsets[PrimitiveIndex].BoneMapOffset != INDEX_NONE ? Offsets[PrimitiveIndex].BoneMapOffset * sizeof(uint32) : INDEX_NONE,
					Offsets[PrimitiveIndex].DirtyBoneTransforms
				);
				++Range.Count;
			}

			FSkinningTransformProvider::FProviderContext Context(
				Primitives,
				Proxies,
				PrimitiveIndices,
				TConstArrayView<FSkeletonBatch>(),
				CurrentTime,
				GraphBuilder,
				ScatterUploadBuilder,
				Parameters.BoneTransforms->GetParent(),
				Parameters.BoneMap
			);

			TransformProvider->Broadcast(PrimitiveRanges, Context);
		}
	}
}

bool FSkinningSceneExtension::ProcessBufferDefragmentation()
{
	// Consolidate spans
	ObjectSpaceAllocator.Consolidate();
	BoneMapAllocator.Consolidate();
	BoneHierarchyAllocator.Consolidate();
	TransformAllocator.Consolidate();

	// Decide to defragment the buffer when the used size dips below a certain multiple of the max used size.
	// Since the buffer allocates in powers of two, we pick the mid point between 1/4 and 1/2 in hopes to prevent
	// thrashing when usage is close to a power of 2.
	//
	// NOTES:
	//	* We only currently use the state of the transform buffer's fragmentation to decide to defrag all buffers
	//	* Rather than trying to minimize number of moves/uploads, we just realloc and re-upload everything. This
	//	  could be implemented in a more efficient manner if the current method proves expensive.

	const bool bAllowDefrag = GSkinningBuffersDefrag;
	static const int32 MinTransformBufferCount = GSkinningBuffersTransformDataMinSizeBytes / sizeof(FCompressedBoneTransform);
	const float LowWaterMarkRatio = GSkinningBuffersDefragLowWatermark;
	const int32 EffectiveMaxSize = FMath::RoundUpToPowerOfTwo(TransformAllocator.GetMaxSize());
	const int32 LowWaterMark = uint32(EffectiveMaxSize * LowWaterMarkRatio);
	const int32 UsedSize = TransformAllocator.GetSparselyAllocatedSize();
	
	if (!bAllowDefrag)
	{
		return false;
	}

	// Check to force a defrag
	const bool bForceDefrag = GSkinningBuffersForceDefrag != 0;
	if (GSkinningBuffersForceDefrag == 1)
	{
		GSkinningBuffersForceDefrag = 0;
	}
	
	if (!bForceDefrag && (EffectiveMaxSize <= MinTransformBufferCount || UsedSize > LowWaterMark))
	{
		// No need to defragment
		return false;
	}

	ObjectSpaceAllocator.Reset();
	BoneMapAllocator.Reset();
	BoneHierarchyAllocator.Reset();
	TransformAllocator.Reset();
	BatchHeaderDatas.Reset();
	AllocatedHeaderDataIndices.Reset();

	for (auto& Data : HeaderDatas)
	{
		if (Data.TransformBufferOffset != INDEX_NONE)
		{
			Data.PreviousTransformBufferOffset = Data.TransformBufferOffset;
			Data.PreviousTransformBufferCount  = Data.TransformBufferCount;

			Data.TransformBufferOffset = INDEX_NONE;
			Data.TransformBufferCount = 0;
		}

		if (Data.BoneMapBufferOffset != INDEX_NONE)
		{
			Data.BoneMapBufferOffset = INDEX_NONE;
			Data.BoneMapBufferCount = 0;
		}

		if (Data.HierarchyBufferOffset != INDEX_NONE)
		{
			Data.HierarchyBufferOffset = INDEX_NONE;
			Data.HierarchyBufferCount = 0;
		}

		if (Data.ObjectSpaceBufferOffset != INDEX_NONE)
		{
			Data.ObjectSpaceBufferOffset = INDEX_NONE;
			Data.ObjectSpaceBufferCount = 0;
		}
	}

	return true;
}

bool FSkinningSceneExtension::Tick(float InDeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinningSceneExtension::Tick);

	FVector NewCameraLocation = FVector::ZeroVector;
	if (UWorld* World = GetWorld())
	{
		if (auto PlayerController = World->GetFirstPlayerController<APlayerController>())
		{
			FRotator CameraRotation;
			PlayerController->GetPlayerViewPoint(NewCameraLocation, CameraRotation);
		}
		else
		{
			FVector LocationSum = FVector::Zero();
			if (World->ViewLocationsRenderedLastFrame.Num() > 0)
			{
				for (const auto& Location : World->ViewLocationsRenderedLastFrame)
				{
					LocationSum += Location;
				}

				NewCameraLocation = LocationSum / World->ViewLocationsRenderedLastFrame.Num();
			}
		}
	}

	// Takes a reference to keep the timer around since the update happens on the GT timeline.
	ENQUEUE_RENDER_COMMAND(FTickSkinningSceneExtension)
	([TickState = TickState, NewCameraLocation](FRHICommandListImmediate& RHICmdList)
	{
		TickState->CameraLocation = NewCameraLocation;
	});
	return true;
}

UWorld* FSkinningSceneExtension::GetWorld() const
{
	return WorldRef.Get();
}

void FSkinningSceneExtension::WaitForHeaderDataUpdateTasks() const
{
	UE::Tasks::Wait(MakeArrayView( { TaskHandles[FreeBufferSpaceTask], TaskHandles[InitHeaderDataTask] } ));
}

FSkinningSceneExtension::FBuffers::FBuffers()
: HeaderDataBuffer(GSkinningBuffersHeaderDataMinSizeBytes >> 2u, TEXT("Skinning.HeaderData"))
, BoneMapBuffer(GSkinningBuffersTransformDataMinSizeBytes >> 2u, TEXT("Skinning.BoneMap"))
, BoneHierarchyBuffer(GSkinningBuffersTransformDataMinSizeBytes >> 2u, TEXT("Skinning.BoneHierarchy"))
, BoneObjectSpaceBuffer(GSkinningBuffersTransformDataMinSizeBytes >> 2u, TEXT("Skinning.BoneObjectSpace"))
, TransformDataBuffer(GSkinningBuffersTransformDataMinSizeBytes >> 2u, TEXT("Skinning.BoneTransforms"))
{
}

FSkinningSceneExtension::FUpdater::FUpdater(FSkinningSceneExtension& InSceneData)
: SceneData(&InSceneData)
, bEnableAsync(GSkinningBuffersAsyncUpdate)
{
	SceneData->UpdateCounter++;
}

void FSkinningSceneExtension::FUpdater::End()
{
	// Ensure these tasks finish before we fall out of scope.
	// NOTE: This should be unnecessary if the updater shares the graph builder's lifetime but we don't enforce that
	SceneData->SyncAllTasks();
}

void FSkinningSceneExtension::FUpdater::PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet)
{
	if (!SceneData->IsEnabled())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinningSceneExtension::FUpdater::PreSceneUpdate);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(STAT_SkinningSceneExtension);

	SceneData->TaskHandles[FreeBufferSpaceTask] = GraphBuilder.AddSetupTask(
		[this, RemovedList = ChangeSet.RemovedPrimitiveIds]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Skinning::FreeBufferSpace);

			// Remove and free transform data for removed primitives
			// NOTE: Using the ID list instead of the primitive list since we're in an async task
			for (FPersistentPrimitiveIndex PersistentIndex : RemovedList)
			{
				if (SceneData->HeaderDatas.IsValidIndex(PersistentIndex.Index))
				{
					FSkinningSceneExtension::FHeaderData& Data = SceneData->HeaderDatas[PersistentIndex.Index];

					auto FreeAllocatorSlots = [this](const FHeaderData& FreeData)
					{
						if (FreeData.ObjectSpaceBufferOffset != INDEX_NONE)
						{
							SceneData->ObjectSpaceAllocator.Free(FreeData.ObjectSpaceBufferOffset, FreeData.ObjectSpaceBufferCount);
						}

						if (FreeData.BoneMapBufferOffset != INDEX_NONE)
						{
							SceneData->BoneMapAllocator.Free(FreeData.BoneMapBufferOffset, FreeData.BoneMapBufferCount);
						}

						if (FreeData.HierarchyBufferOffset != INDEX_NONE)
						{
							SceneData->BoneHierarchyAllocator.Free(FreeData.HierarchyBufferOffset, FreeData.HierarchyBufferCount);
						}

						if (FreeData.TransformBufferOffset != INDEX_NONE)
						{
							SceneData->TransformAllocator.Free(FreeData.TransformBufferOffset, FreeData.TransformBufferCount);
						}
					};

					if (Data.bIsBatched)
					{
						const FSkeletonBatchKey BatchKey = Data.GetSkeletonBatchKey();

						if (FBatchHeaderEntry* BatchEntry = SceneData->BatchHeaderDatas.Find(BatchKey))
						{
							check(BatchEntry->RefCount > 0);
							BatchEntry->RefCount--;

							if (BatchEntry->RefCount == 0)
							{
								FreeAllocatorSlots(BatchEntry->HeaderData);
								SceneData->BatchHeaderDatas.Remove(BatchKey);
							}
						}
					}
					else
					{
						FreeAllocatorSlots(Data);

						SceneData->AllocatedHeaderDataIndices.Remove(PersistentIndex.Index);

						if (Data.bIsInstanced)
						{
							SceneData->InstancedHeaderDataIndices.Remove(PersistentIndex.Index);
						}
					}

					if (Data.bVertexFactoryDirty)
					{
						SceneData->VertexFactoryUpdateList.Remove(PersistentIndex.Index);
					}

					SceneData->HeaderDatas.RemoveAt(PersistentIndex.Index);
				}
			}

			// Check to force a full upload by CVar
			// NOTE: Doesn't currently discern which scene to affect
			bForceFullUpload = GSkinningBuffersForceFullUpload != 0;
			if (GSkinningBuffersForceFullUpload == 1)
			{
				GSkinningBuffersForceFullUpload = 0;
			}

			bDefragging = SceneData->ProcessBufferDefragmentation();
			bForceFullUpload |= bDefragging;
		},
		UE::Tasks::ETaskPriority::Normal,
		bEnableAsync
	);
}

void FSkinningSceneExtension::FUpdater::PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet)
{
	if (!SceneData->IsEnabled())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinningSceneExtension::FUpdater::PostSceneUpdate);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(STAT_SkinningSceneExtension);

	// Cache the updated PrimitiveSceneInfos (this is safe as long as we only access it in updater funcs and RDG setup tasks)
	AddedList = ChangeSet.AddedPrimitiveSceneInfos;

	// Kick off a task to initialize added transform ranges
	if (AddedList.Num() > 0)
	{
		SceneData->TaskHandles[InitHeaderDataTask] = GraphBuilder.AddSetupTask(
			[this]
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(Skinning::InitHeaderData);

				for (FPrimitiveSceneInfo* PrimitiveSceneInfo : AddedList)
				{
					if (!PrimitiveSceneInfo->Proxy->IsSkinnedMesh())
					{
						continue;
					}

					FSkinningSceneExtensionProxy* Proxy = PrimitiveSceneInfo->Proxy->GetSkinningSceneExtensionProxy();
					if (!Proxy)
					{
						continue;
					}

					const FSkeletalMeshObject* MeshObject = Proxy->GetMeshObject();

					const int32 PersistentIndex = PrimitiveSceneInfo->GetPersistentIndex().Index;

					FHeaderData NewHeader;
					NewHeader.InstanceSceneDataOffset     = PrimitiveSceneInfo->GetInstanceSceneDataOffset();
					NewHeader.NumInstanceSceneDataEntries = PrimitiveSceneInfo->GetNumInstanceSceneDataEntries();
					NewHeader.ProviderId                  = Proxy->GetTransformProviderId();
					NewHeader.PrimitiveSceneInfo          = PrimitiveSceneInfo;
					NewHeader.Proxy                       = Proxy;
					NewHeader.MeshObject                  = MeshObject;
					NewHeader.MaxTransformCount           = Proxy->GetMaxBoneTransformCount();
					NewHeader.MaxBoneMapCount             = Proxy->GetMaxBoneMapCount();
					NewHeader.MaxBoneHierarchyCount       = Proxy->GetMaxBoneHierarchyCount();
					NewHeader.MaxObjectSpaceCount         = Proxy->GetMaxBoneObjectSpaceCount();
					NewHeader.MaxInfluenceCount           = Proxy->GetMaxBoneInfluenceCount();
					NewHeader.UniqueAnimationCount        = Proxy->GetUniqueAnimationCount();
					NewHeader.bIsBatched                  = Proxy->UseSkeletonBatching();
					NewHeader.bIsInstanced                = Proxy->UseInstancing();
					NewHeader.bIsRefPose                  = NewHeader.ProviderId == RefPoseProviderId;
					NewHeader.bHasScale                   = Proxy->HasScale();
					NewHeader.bIsNanite                   = MeshObject->IsNaniteMesh();

					MeshObject->SkinningSceneExtensionPrivateData.CachedPrimitiveIndex = PersistentIndex;

					SceneData->HeaderDatas.EmplaceAt(PersistentIndex, NewHeader);

					if (NewHeader.bIsInstanced && !NewHeader.bIsBatched)
					{
						SceneData->InstancedHeaderDataIndices.Emplace(PersistentIndex);
					}
				}
			},
			SceneData->TaskHandles[FreeBufferSpaceTask],
			UE::Tasks::ETaskPriority::Normal,
			bEnableAsync
		);
	}
}

bool FSkinningSceneExtension::GetHeaderFromMeshObject(const FSkeletalMeshObject* MeshObject, FHeaderData*& OutHeaderData, int32& OutPrimitiveIndex)
{
	// This skeletal mesh is not tracked by the scene extension.
	if (MeshObject->SkinningSceneExtensionPrivateData.CachedPrimitiveIndex == INDEX_NONE)
	{
		return false;
	}

	const int32 PrimitiveIndex = MeshObject->SkinningSceneExtensionPrivateData.CachedPrimitiveIndex;
	if (!HeaderDatas.IsValidIndex(PrimitiveIndex))
	{
		return false;
	}

	// We are unable to reset the mesh object primitive index on removal (the lifetimes of mesh objects don't match primitives,
	// so it's unsafe to dereference it inside of the PreSceneUpdate on removal), so instead verify that the pointers match here.
	// This avoids cases where the primitive gets removed from the scene but the mesh object is still valid and updating with a
	// stale index. An explicit map would be cleaner but it's significantly slower and these updates are frequent.
	FHeaderData& HeaderData = HeaderDatas[PrimitiveIndex];
	if (HeaderData.MeshObject != MeshObject)
	{
		return false;
	}

	OutPrimitiveIndex = PrimitiveIndex;
	OutHeaderData = &HeaderData;
	return true;
}

void FSkinningSceneExtension::FUpdater::PostMeshUpdate(FRDGBuilder& GraphBuilder, const TConstArrayView<FPrimitiveSceneInfo*>& UpdatedSceneInfoList, const FSkeletalMeshUpdater::FSubmitData* SkeletalMeshSubmitData)
{
	if (!SceneData->IsEnabled())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinningSceneExtension::FUpdater::PostMeshUpdate);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(STAT_SkinningSceneExtension);

	check(SkeletalMeshSubmitData);

	SceneData->TaskHandles[InitUpdateListTask] = GraphBuilder.AddSetupTask([this, UpdatedSceneInfoList, SkeletalMeshSubmitData]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Skinning::InitUpdateList);

			int32 NumUpdates = 0;
			int32 NumAllocations = 0;

			const auto UpdateLambda = [this, &NumAllocations, &NumUpdates] (FHeaderData& HeaderData, int32 PrimitiveIndex, int32 LODIndex, bool bFlipTransformSlots)
			{
				HeaderData.UpdateCounter        = SceneData->UpdateCounter;
				HeaderData.DirtyBoneTransforms |= EDirtyBoneTransforms::Current;

				if (bFlipTransformSlots && !HeaderData.bIsBatched)
				{
					HeaderData.CurrentTransformSlot = !HeaderData.CurrentTransformSlot;
				}

				const bool bUploadRequested             = LODIndex != INDEX_NONE ? HeaderData.Proxy->SetLOD(LODIndex) : false;
				const bool bUniqueAnimationCountChanged = HeaderData.UniqueAnimationCount != HeaderData.Proxy->GetUniqueAnimationCount();

				if (bUploadRequested || bUniqueAnimationCountChanged)
				{
					NumAllocations += bUploadRequested ? 1 : 0;
					HeaderData.UniqueAnimationCount  = HeaderData.Proxy->GetUniqueAnimationCount();
					HeaderData.MaxTransformCount     = HeaderData.Proxy->GetMaxBoneTransformCount();
					HeaderData.MaxBoneMapCount       = HeaderData.Proxy->GetMaxBoneMapCount();
					HeaderData.MaxBoneHierarchyCount = HeaderData.Proxy->GetMaxBoneHierarchyCount();
					HeaderData.MaxObjectSpaceCount   = HeaderData.Proxy->GetMaxBoneObjectSpaceCount();

					AllocationUpdateList.Emplace(PrimitiveIndex);
				}

				HeaderDataUpdateList.Emplace(PrimitiveIndex);
				NumUpdates++;
			};

			TConstArrayView<FSkeletalMeshUpdateEvent> UpdateEvents = SkeletalMeshSubmitData->GetUpdateEvents();
			AllocationUpdateList.Reserve(UpdatedSceneInfoList.Num() + UpdateEvents.Num());
			HeaderDataUpdateList.Reserve(UpdatedSceneInfoList.Num() + UpdateEvents.Num());

			// Process update events from the skeletal mesh updater first, including LOD change events.
			for (FSkeletalMeshUpdateEvent UpdateEvent : UpdateEvents)
			{
				FHeaderData* HeaderData = nullptr;
				int32 PrimitiveIndex;
				if (SceneData->GetHeaderFromMeshObject(UpdateEvent.MeshObject, HeaderData, PrimitiveIndex))
				{
					const bool bFlipTransformSlots = true;
					UpdateLambda(*HeaderData, PrimitiveIndex, UpdateEvent.LODIndex, bFlipTransformSlots);
					HeaderData->UpdateCounter = SceneData->UpdateCounter;
				}
			}

			const auto GetFirstUpdateLOD = [] (const FHeaderData& HeaderData)
			{
				// We only need to pull the LOD on first update if the skeletal mesh updater failed to send a mesh object update event
				// alongside the primitive AddToScene request. Technically it should always have an update queued but this relies on the high
				// level code managing mesh object updates and scene attachment to be ordered perfectly and enforcing this is impossible.
				// Instead, we allow for attachment without a mesh update event and instead pull the current LOD on the mesh object if no
				// update event was provided. That way, if the mesh object was updated and then the AttachToScene call occurs later, it pulls
				// the correct LOD and doesn't crash.

				return HeaderData.IsFirstUpdate() ? HeaderData.MeshObject->GetLOD() : INDEX_NONE;
			};

			// Update all unbatched instanced meshes that didn't otherwise receive an LOD update. These don't receive update events
			// from the skeletal mesh updater in the same fashion, so they are manually ticked. Only do this once per frame as we might
			// received multiple back-to-back scene updates from the high level.
			if (!SceneData->HasUpdatedThisFrame())
			{
				for (int32 PrimitiveIndex : SceneData->InstancedHeaderDataIndices)
				{
					FHeaderData& HeaderData = SceneData->HeaderDatas[PrimitiveIndex];

					if (HeaderData.UpdateCounter != SceneData->UpdateCounter)
					{
						const bool bFlipTransformSlots = true;
						UpdateLambda(HeaderData, PrimitiveIndex, GetFirstUpdateLOD(HeaderData), bFlipTransformSlots);
					}
				}
			}

			// Finally, update meshes that are new to the scene or were invalidated for other reasons.
			for (FPrimitiveSceneInfo* PrimitiveSceneInfo : UpdatedSceneInfoList)
			{
				const int32 PrimitiveIndex = PrimitiveSceneInfo->GetPersistentIndex().Index;
				if (SceneData->HeaderDatas.IsValidIndex(PrimitiveIndex))
				{
					FHeaderData& HeaderData = SceneData->HeaderDatas[PrimitiveIndex];

					if (HeaderData.UpdateCounter != SceneData->UpdateCounter)
					{
						const bool bFlipTransformSlots = false;
						UpdateLambda(HeaderData, PrimitiveIndex, GetFirstUpdateLOD(HeaderData), bFlipTransformSlots);
					}
				}
			}

			INC_DWORD_STAT_BY(STAT_SkinningSceneExtension_NumUpdates, NumUpdates);
			INC_DWORD_STAT_BY(STAT_SkinningSceneExtension_NumAllocations, NumAllocations);
		},
		MakeArrayView(
			{
				SceneData->TaskHandles[FreeBufferSpaceTask],
				SceneData->TaskHandles[InitHeaderDataTask],
				SkeletalMeshSubmitData->GetFilterTask()
			}
		),
		UE::Tasks::ETaskPriority::Normal,
		bEnableAsync
	);

	// Gets the information needed from the primitive for skinning and allocates the appropriate space in the buffer
	// for the primitive's bone transforms
	auto AllocSpaceForPrimitive = [this](const int32 HeaderDataIndex)
	{
		FHeaderData& Data = SceneData->HeaderDatas[HeaderDataIndex];
		Data.DirtyBoneTransforms = Data.bIsInstanced ? EDirtyBoneTransforms::Current : EDirtyBoneTransforms::All;

		if (Data.Proxy->UseSkeletonBatching())
		{
			const FSkeletonBatchKey BatchKey = Data.GetSkeletonBatchKey();

			if (FBatchHeaderEntry* BatchEntry = SceneData->BatchHeaderDatas.Find(BatchKey))
			{
				check(Data.bIsBatched == Data.Proxy->UseSkeletonBatching());

				const FHeaderData& SrcHeaderData = BatchEntry->HeaderData;
				BatchEntry->RefCount++;
				Data.ObjectSpaceBufferOffset	= SrcHeaderData.ObjectSpaceBufferOffset;
				Data.ObjectSpaceBufferCount		= SrcHeaderData.ObjectSpaceBufferCount;
				Data.BoneMapBufferOffset		= SrcHeaderData.BoneMapBufferOffset;
				Data.BoneMapBufferCount			= SrcHeaderData.BoneMapBufferCount;
				Data.HierarchyBufferOffset		= SrcHeaderData.HierarchyBufferOffset;
				Data.HierarchyBufferCount		= SrcHeaderData.HierarchyBufferCount;
				Data.TransformBufferOffset		= SrcHeaderData.TransformBufferOffset;
				Data.TransformBufferCount		= SrcHeaderData.TransformBufferCount;

			#if DO_CHECK
				{
					const FString SkinnedAssetName = Data.Proxy->GetSkinnedAsset()->GetName();
					const FString SkeletonName = Data.Proxy->GetSkinnedAsset()->GetSkeleton()->GetName();

					const uint32 ObjectSpaceFloatCount = Data.Proxy->GetObjectSpaceFloatCount();

					checkf(Data.ObjectSpaceBufferCount == (Data.MaxObjectSpaceCount * ObjectSpaceFloatCount),
						TEXT("Mismatch between ObjectSpaceBufferCount=%d and (MaxObjectSpaceCount * ObjectSpaceFloatCount)=%d for mesh %s with skeleton %s."),
						Data.ObjectSpaceBufferCount, (Data.MaxObjectSpaceCount * ObjectSpaceFloatCount),
						*SkinnedAssetName, *SkeletonName);

					checkf(Data.BoneMapBufferCount == Data.MaxBoneMapCount,
						TEXT("Mismatch between BoneMapBufferCount=%d and MaxBoneMapCount=%d for mesh %s with skeleton %s."),
						Data.BoneMapBufferCount, Data.MaxBoneMapCount,
						*SkinnedAssetName, *SkeletonName);

					checkf(Data.HierarchyBufferCount == Data.MaxBoneHierarchyCount,
						TEXT("Mismatch between HierarchyBufferCount=%d and MaxBoneHierarchyCount=%d for mesh %s with skeleton %s."),
						Data.HierarchyBufferCount, Data.MaxBoneHierarchyCount,
						*SkinnedAssetName, *SkeletonName);

					checkf(Data.TransformBufferCount == (Data.UniqueAnimationCount * Data.MaxTransformCount * 2u),
						TEXT("Mismatch between TransformBufferCount=%d and (Data.UniqueAnimationCount * Data.MaxTransformCount * 2u)=%d for mesh %s with skeleton %s."),
						Data.TransformBufferCount, (Data.UniqueAnimationCount * Data.MaxTransformCount * 2u),
						*SkinnedAssetName, *SkeletonName);
				}
			#endif

				return;
			}
		}

		bool bRequireUpload = false;

		const uint32 ObjectSpaceNeededSize = Data.MaxObjectSpaceCount * Data.Proxy->GetObjectSpaceFloatCount();
		if (ObjectSpaceNeededSize != Data.ObjectSpaceBufferCount)
		{
			if (Data.ObjectSpaceBufferCount > 0)
			{
				SceneData->ObjectSpaceAllocator.Free(Data.ObjectSpaceBufferOffset, Data.ObjectSpaceBufferCount);
			}

			Data.ObjectSpaceBufferOffset = ObjectSpaceNeededSize > 0 ? SceneData->ObjectSpaceAllocator.Allocate(ObjectSpaceNeededSize) : INDEX_NONE;
			Data.ObjectSpaceBufferCount = ObjectSpaceNeededSize;

			if (!bForceFullUpload)
			{
				bRequireUpload = true;
			}
		}

		const uint32 BoneMapNeededSize = Data.MaxBoneMapCount;
		if (BoneMapNeededSize != Data.BoneMapBufferCount)
		{
			if (Data.BoneMapBufferCount > 0)
			{
				SceneData->BoneMapAllocator.Free(Data.BoneMapBufferOffset, Data.BoneMapBufferCount);
			}

			Data.BoneMapBufferOffset = BoneMapNeededSize > 0 ? SceneData->BoneMapAllocator.Allocate(BoneMapNeededSize) : INDEX_NONE;
			Data.BoneMapBufferCount = BoneMapNeededSize;

			if (!bForceFullUpload)
			{
				bRequireUpload = true;
			}
		}

		const uint32 HierarchyNeededSize = Data.MaxBoneHierarchyCount;
		if (HierarchyNeededSize != Data.HierarchyBufferCount)
		{
			if (Data.HierarchyBufferCount > 0)
			{
				SceneData->BoneHierarchyAllocator.Free(Data.HierarchyBufferOffset, Data.HierarchyBufferCount);
			}

			Data.HierarchyBufferOffset = HierarchyNeededSize > 0 ? SceneData->BoneHierarchyAllocator.Allocate(HierarchyNeededSize) : INDEX_NONE;
			Data.HierarchyBufferCount = HierarchyNeededSize;

			if (!bForceFullUpload)
			{
				bRequireUpload = true;
			}
		}

		const uint32 TransformNeededSize = Data.UniqueAnimationCount * Data.MaxTransformCount * 2u; // Current and Previous
		if (bRequireUpload || (TransformNeededSize != Data.TransformBufferCount))
		{
			// Don't overwrite if already set (e.g. by ProcessBufferDefragmentation)
			if (Data.PreviousTransformBufferOffset == INDEX_NONE)
			{
				Data.PreviousTransformBufferOffset = Data.TransformBufferOffset;
				Data.PreviousTransformBufferCount  = Data.TransformBufferCount;
			}

			if (Data.TransformBufferCount > 0)
			{
				SceneData->TransformAllocator.Free(Data.TransformBufferOffset, Data.TransformBufferCount);
			}

			Data.TransformBufferOffset = TransformNeededSize > 0 ? SceneData->TransformAllocator.Allocate(TransformNeededSize) : INDEX_NONE;
			Data.TransformBufferCount = TransformNeededSize;

			// First allocation — no previous data to copy, provider must write both slots
			if (Data.PreviousTransformBufferOffset == INDEX_NONE)
			{
				Data.DirtyBoneTransforms = EDirtyBoneTransforms::All;
			}

			if (!bForceFullUpload)
			{
				bRequireUpload = true;
			}
		}

		check(Data.UniqueAnimationCount * Data.MaxTransformCount * 2u == Data.TransformBufferCount);

		if (Data.Proxy->UseSkeletonBatching())
		{
			const FSkeletonBatchKey BatchKey = Data.GetSkeletonBatchKey();

			SceneData->BatchHeaderDatas.Add(BatchKey, FBatchHeaderEntry{ Data, 1 });
		}
		else
		{
			SceneData->AllocatedHeaderDataIndices.Add(HeaderDataIndex);
		}
	};

	// Kick off the allocate task (synced just prior to header uploads)
	SceneData->TaskHandles[AllocBufferSpaceTask] = GraphBuilder.AddSetupTask(
		[this, AllocSpaceForPrimitive]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Skinning::AllocBufferSpace);
			bool bHasUpdates = false;

			if (bDefragging)
			{
				for (auto& Data : SceneData->HeaderDatas)
				{
					const int32 HeaderDataIndex = Data.PrimitiveSceneInfo->GetPersistentIndex().Index;
					if (SceneData->HeaderDatas.IsValidIndex(HeaderDataIndex))
					{
						AllocSpaceForPrimitive(HeaderDataIndex);
					}
				}
			}
			else
			{
				// Only check to reallocate space for primitives that have requested an update
				for (int32 PersistentIndex : AllocationUpdateList)
				{
					AllocSpaceForPrimitive(PersistentIndex);
					bHasUpdates = true;
				}
			}

			// Only create a new uploader here if one of the two dependent upload tasks will use it
			if (bForceFullUpload || HeaderDataUpdateList.Num() > 0 || bHasUpdates)
			{
				SceneData->Uploader = MakeUnique<FUploader>();
			}
		},
		MakeArrayView(
			{
				SceneData->TaskHandles[InitUpdateListTask]
			}
		),
		UE::Tasks::ETaskPriority::Normal,
		bEnableAsync
	);

	auto UploadHeaderData = [this](FHeaderData& Data)
	{
		const int32 PersistentIndex = Data.PrimitiveSceneInfo->GetPersistentIndex().Index;

		// Catch when/if no transform buffer data is allocated for a primitive we're tracking.
		// This should be indicative of a bug.
		ensure(Data.TransformBufferCount != 0);

		check(SceneData->Uploader.IsValid()); // Sanity check
		SceneData->Uploader->HeaderDataUploader.Add(Data.Pack(), PersistentIndex);

		if (!Data.bVertexFactoryDirty)
		{
			Data.bVertexFactoryDirty = 1;
			SceneData->VertexFactoryUpdateList.Emplace(PersistentIndex);
		}
	};

	// Kick off the header data upload task (synced when accessing the buffer)
	SceneData->TaskHandles[UploadHeaderDataTask] = GraphBuilder.AddSetupTask(
		[this, UploadHeaderData]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Skinning::UploadHeaderData);

			if (bForceFullUpload)
			{
				for (FHeaderData& Data : SceneData->HeaderDatas)
				{
					UploadHeaderData(Data);
				}

				SceneData->VertexFactoryUpdateRequests = SceneData->HeaderDatas.Num();
			}
			else
			{
				for (int32 PersistentIndex : HeaderDataUpdateList)
				{
					check(SceneData->HeaderDatas.IsValidIndex(PersistentIndex));
					UploadHeaderData(SceneData->HeaderDatas[PersistentIndex]);
				}

				SceneData->VertexFactoryUpdateRequests = HeaderDataUpdateList.Num();
			}
		},
		MakeArrayView(
			{
				SceneData->TaskHandles[AllocBufferSpaceTask]
			}
		),
		UE::Tasks::ETaskPriority::Normal,
		bEnableAsync
	);

	auto UploadHierarchyData = [this](const FHeaderData& Data)
	{
		// Bone Map
		if (Data.MaxBoneMapCount > 0)
		{
			TConstArrayView<uint16> BoneMap = Data.Proxy->GetBoneMap();
			check(BoneMap.Num() == Data.MaxBoneMapCount);
			check(Data.BoneMapBufferCount == Data.MaxBoneMapCount);
			check(SceneData->Uploader.IsValid());

			auto UploadData = SceneData->Uploader->BoneMapUploader.AddMultiple_GetRef(
				Data.BoneMapBufferOffset,
				Data.BoneMapBufferCount
			);

			uint32* DstBoneMapPtr = UploadData.GetData();
			for (int32 BoneIndex = 0; BoneIndex < Data.MaxBoneMapCount; ++BoneIndex)
			{
				DstBoneMapPtr[BoneIndex] = BoneMap[BoneIndex];
			}
		}

		// Bone Hierarchy
		if (Data.MaxBoneHierarchyCount > 0)
		{
			TConstArrayView<uint32> BoneHierarchy = Data.Proxy->GetBoneHierarchy();
			check(BoneHierarchy.Num() == Data.MaxBoneHierarchyCount);
			check(Data.HierarchyBufferCount == Data.MaxBoneHierarchyCount);
			check(SceneData->Uploader.IsValid());

			auto UploadData = SceneData->Uploader->BoneHierarchyUploader.AddMultiple_GetRef(
				Data.HierarchyBufferOffset,
				Data.HierarchyBufferCount
			);

			uint32* DstBoneHierarchyPtr = UploadData.GetData();
			for (int32 BoneIndex = 0; BoneIndex < Data.MaxBoneHierarchyCount; ++BoneIndex)
			{
				DstBoneHierarchyPtr[BoneIndex] = BoneHierarchy[BoneIndex];
			}
		}

		// Bone Object Space
		if (Data.MaxObjectSpaceCount > 0)
		{
			TConstArrayView<float> BoneObjectSpace = Data.Proxy->GetBoneObjectSpace();
			const uint32 FloatCount = Data.Proxy->GetObjectSpaceFloatCount();
			check(BoneObjectSpace.Num() == Data.MaxObjectSpaceCount * FloatCount);
			check(Data.ObjectSpaceBufferCount == Data.MaxObjectSpaceCount * FloatCount);

			auto UploadData = SceneData->Uploader->BoneObjectSpaceUploader.AddMultiple_GetRef(
				Data.ObjectSpaceBufferOffset,
				Data.ObjectSpaceBufferCount
			);

			float* DstBoneObjectSpacePtr = UploadData.GetData();
			for (uint32 BoneFloatIndex = 0; BoneFloatIndex < (Data.MaxObjectSpaceCount * FloatCount); ++BoneFloatIndex)
			{
				DstBoneObjectSpacePtr[BoneFloatIndex] = BoneObjectSpace[BoneFloatIndex];
			}
		}
	};

	auto UploadTransformData = [this](const FHeaderData& Data, bool bProvidersEnabled)
	{
		if (bProvidersEnabled && Data.Proxy->GetTransformProviderId().IsValid())
		{
			return;
		}

		// NOTE: This path is purely for debugging now - should also set "r.Skinning.Buffers.ForceFullUpload 2" to avoid caching artifacts

		check(SceneData->Uploader.IsValid());
		auto UploadData = SceneData->Uploader->TransformDataUploader.AddMultiple_GetRef(
			Data.TransformBufferOffset,
			Data.TransformBufferCount
		);

		check(Data.UniqueAnimationCount * Data.MaxTransformCount * 2u == Data.TransformBufferCount);

		FCompressedBoneTransform* DstCurrentBoneTransformsPtr = UploadData.GetData();
		FCompressedBoneTransform* DstPreviousBoneTransformsPtr = DstCurrentBoneTransformsPtr + Data.MaxTransformCount;
		const uint32 StridedPtrStep = Data.MaxTransformCount * 2u;

		for (uint32 UniqueAnimation = 0; UniqueAnimation < Data.UniqueAnimationCount; ++UniqueAnimation)
		{
			for (int32 TransformIndex = 0; TransformIndex < Data.MaxTransformCount; ++TransformIndex)
			{
				SetCompressedBoneTransformIdentity(DstCurrentBoneTransformsPtr[TransformIndex]);
				SetCompressedBoneTransformIdentity(DstPreviousBoneTransformsPtr[TransformIndex]);
			}

			DstCurrentBoneTransformsPtr += StridedPtrStep;
			DstPreviousBoneTransformsPtr += StridedPtrStep;
		}
	};

	// Kick off the hierarchy data upload task (synced when accessing the buffer)
	SceneData->TaskHandles[UploadHierarchyDataTask] = GraphBuilder.AddSetupTask(
		[this, UploadHierarchyData]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Skinning::UploadHierarchyData);

			if (bForceFullUpload)
			{
				for (auto& Data : SceneData->HeaderDatas)
				{
					UploadHierarchyData(Data);
				}
			}
			else
			{
				for (int32 PersistentIndex : AllocationUpdateList)
				{
					UploadHierarchyData(SceneData->HeaderDatas[PersistentIndex]);
				}
			}
		},
		MakeArrayView({ SceneData->TaskHandles[AllocBufferSpaceTask] }),
		UE::Tasks::ETaskPriority::Normal,
		bEnableAsync
	);

	// Kick off the transform data upload task (synced when accessing the buffer)
	SceneData->TaskHandles[UploadTransformDataTask] = GraphBuilder.AddSetupTask(
		[this, UploadTransformData]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Skinning::UploadTransformData);

			const bool bProvidersEnabled = GSkinningTransformProviders;

			if (bForceFullUpload)
			{
				for (auto& Data : SceneData->HeaderDatas)
				{
					UploadTransformData(Data, bProvidersEnabled);
				}
			}
			else
			{
				for (int32 PersistentIndex : AllocationUpdateList)
				{
					UploadTransformData(SceneData->HeaderDatas[PersistentIndex], bProvidersEnabled);
				}
			}
		},
		MakeArrayView({ SceneData->TaskHandles[AllocBufferSpaceTask] }),
		UE::Tasks::ETaskPriority::Normal,
		bEnableAsync
	);

	GraphBuilder.AddPostExecuteCallback([SceneData = SceneData]
	{
		SceneData->LastUpdateFrameNumber = GFrameNumberRenderThread;
	});
}

class FNaniteSkinningUpdateViewDataCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNaniteSkinningUpdateViewDataCS);
	SHADER_USE_PARAMETER_STRUCT(FNaniteSkinningUpdateViewDataCS, FGlobalShader)

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUScene)
		SHADER_PARAMETER_STRUCT_INCLUDE(RendererViewData::FWriterParameters, ViewDataParametersWriter)
		SHADER_PARAMETER_STRUCT_INCLUDE( FInstanceHierarchyParameters, InstanceHierarchyParameters )
		SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< FUintVector2 >, InstanceWorkGroups )
		SHADER_PARAMETER(float, DefaultAnimationMinScreenSize)		
		RDG_BUFFER_ACCESS( IndirectArgs,	ERHIAccess::IndirectArgs )
	END_SHADER_PARAMETER_STRUCT()

	static constexpr int ThreadGroupSize = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportNanite(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("VIEW_DATA_ACCESS_MODE"), VIEW_DATA_ACCESS_RW);
		// Don't access the global Scene uniform buffer but map to indivdual UBs for each used module.
		OutEnvironment.SetDefine(TEXT("USE_EXPLICIT_SCENE_UB_MODULES"), 1);

		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
	}
};
IMPLEMENT_GLOBAL_SHADER(FNaniteSkinningUpdateViewDataCS, "/Engine/Private/Nanite/NaniteSkinningUpdateViewData.usf", "NaniteSkinningUpdateViewDataCS", SF_Compute);

class FNaniteSkinningUpdateChunkCullCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FNaniteSkinningUpdateChunkCullCS);
	SHADER_USE_PARAMETER_STRUCT(FNaniteSkinningUpdateChunkCullCS, FGlobalShader)

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(RendererViewData::FWriterParameters, ViewDataParametersWriter)
		SHADER_PARAMETER_STRUCT_INCLUDE( FInstanceHierarchyParameters, InstanceHierarchyParameters )
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer< FUintVector2 >, OutInstanceWorkGroups )
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWBuffer< uint >, OutInstanceWorkArgs )

		SHADER_PARAMETER(float, DefaultAnimationMinScreenSize)		
	END_SHADER_PARAMETER_STRUCT()

	static constexpr int ThreadGroupSize = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportNanite(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("VIEW_DATA_ACCESS_MODE"), VIEW_DATA_ACCESS_RW);
		// Don't access the global Scene uniform buffer but map to indivdual UBs for each used module.
		OutEnvironment.SetDefine(TEXT("USE_EXPLICIT_SCENE_UB_MODULES"), 1);

		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
	}
};
IMPLEMENT_GLOBAL_SHADER(FNaniteSkinningUpdateChunkCullCS, "/Engine/Private/Nanite/NaniteSkinningUpdateViewData.usf", "NaniteSkinningUpdateChunkCullCS", SF_Compute);


void FSkinningSceneExtension::FRenderer::UpdateViewData(FRDGBuilder& GraphBuilder, const FRendererViewDataManager& ViewDataManager)
{
	SCOPED_NAMED_EVENT(FSkinningSceneExtension_FRenderer_UpdateViewData, FColor::Silver);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(STAT_SkinningSceneExtension);

	FSceneCullingRenderer* SceneCullingRenderer = GetSceneRenderer().GetSceneExtensionsRenderers().GetRendererPtr<FSceneCullingRenderer>();
	if (!SceneCullingRenderer || !SceneCullingRenderer->IsEnabled())
	{
		return;
	}

	FInstanceHierarchyParameters InstanceHierarchyParameters = SceneCullingRenderer->GetShaderParameters(GraphBuilder);
	int32 NumAllocatedChunks = InstanceHierarchyParameters.NumAllocatedChunks;
	// Create a buffer with enough space for all chunks
	FRDGBufferRef InstanceWorkGroupsRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FUintVector2), NumAllocatedChunks), TEXT("Skinning.UpdateViewData.WorkGroups"));
	ERHIFeatureLevel::Type FeatureLevel = SceneData->Scene.GetFeatureLevel();
	FRDGBufferRef InstanceWorkArgsRDG = CreateAndClearIndirectDispatchArgs1D(GraphBuilder, FeatureLevel, TEXT("Skinning.UpdateViewData.IndirectArgs"));
	{
		FNaniteSkinningUpdateChunkCullCS::FParameters* PassParameters = GraphBuilder.AllocParameters< FNaniteSkinningUpdateChunkCullCS::FParameters >();
		PassParameters->InstanceHierarchyParameters = InstanceHierarchyParameters;
		PassParameters->DefaultAnimationMinScreenSize = GSkinningDefaultAnimationMinScreenSize;

		PassParameters->OutInstanceWorkGroups = GraphBuilder.CreateUAV(InstanceWorkGroupsRDG);
		PassParameters->OutInstanceWorkArgs = GraphBuilder.CreateUAV(InstanceWorkArgsRDG);
		PassParameters->ViewDataParametersWriter = ViewDataManager.GetWriterShaderParameters(GraphBuilder);

		auto ComputeShader = GetGlobalShaderMap(FeatureLevel)->GetShader<FNaniteSkinningUpdateChunkCullCS>();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME( "NaniteSkinningUpdateViewDataChunks" ),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(NumAllocatedChunks, 64)
		);
	}

	{
		FNaniteSkinningUpdateViewDataCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FNaniteSkinningUpdateViewDataCS::FParameters>();
		PassParameters->GPUScene = SceneData->Scene.GPUScene.GetShaderParameters(GraphBuilder);
		PassParameters->ViewDataParametersWriter = ViewDataManager.GetWriterShaderParameters(GraphBuilder);
		PassParameters->InstanceHierarchyParameters = InstanceHierarchyParameters;
		PassParameters->DefaultAnimationMinScreenSize = GSkinningDefaultAnimationMinScreenSize;
		PassParameters->IndirectArgs = InstanceWorkArgsRDG;
		PassParameters->InstanceWorkGroups = GraphBuilder.CreateSRV(InstanceWorkGroupsRDG);

		auto ComputeShader = GetGlobalShaderMap(SceneData->Scene.GetShaderPlatform())->GetShader<FNaniteSkinningUpdateViewDataCS>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME( "NaniteSkinningUpdateViewData" ),
			ComputeShader,
			PassParameters,
			PassParameters->IndirectArgs,
			0
		);
	}
}

void FSkinningSceneExtension::FUpdater::PreGPUSceneUpdate(FRDGBuilder& GraphBuilder, const FGameTime* CurrentTime)
{
	if (!SceneData->IsEnabled())
	{
		return;
	}

	SCOPED_NAMED_EVENT(FSkinningSceneExtension_FUpdater_PreGPUSceneUpdate, FColor::Silver);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(STAT_SkinningSceneExtension);

	// Wait on dependent tasks before iterating InstancedHeaderDataIndices.
	SceneData->TaskHandles[AllocBufferSpaceTask].Wait();

	int32 NumCopies = 0;
	for (int32 PersistentIndex : SceneData->InstancedHeaderDataIndices)
	{
		FHeaderData& Data = SceneData->HeaderDatas[PersistentIndex];

		if (Data.PreviousTransformBufferOffset != INDEX_NONE && Data.PreviousTransformBufferOffset != Data.TransformBufferOffset)
		{
			NumCopies++;
		}
	}

	// Build copy commands for instanced meshes whose transform allocations moved. Collect before FinishSkinningBufferUpload since defrag may shrink the buffer.
	FRDGBufferRef DefragStagingBuffer = nullptr;
	TArrayView<FCopyTransformsHeader> CopyHeaders = GraphBuilder.AllocPODArrayView<FCopyTransformsHeader>(NumCopies);
	uint32 MaxElementsPerCopy = 0;
	int32 CopyIndex = 0;

	for (int32 PersistentIndex : SceneData->InstancedHeaderDataIndices)
	{
		FHeaderData& Data = SceneData->HeaderDatas[PersistentIndex];

		if (Data.PreviousTransformBufferOffset != INDEX_NONE && Data.PreviousTransformBufferOffset != Data.TransformBufferOffset)
		{
			const uint32 CopyCount = FMath::Min(Data.PreviousTransformBufferCount, Data.TransformBufferCount);
			CopyHeaders[CopyIndex++] =
			{
				.SrcByteOffset = Data.PreviousTransformBufferOffset * (uint32)sizeof(FCompressedBoneTransform),
				.DstByteOffset = Data.TransformBufferOffset * (uint32)sizeof(FCompressedBoneTransform),
				.Count         = CopyCount
			};
			MaxElementsPerCopy = FMath::Max(MaxElementsPerCopy, CopyCount);
			Data.DirtyBoneTransforms = EDirtyBoneTransforms::Current;
		}

		Data.PreviousTransformBufferOffset = INDEX_NONE;
		Data.PreviousTransformBufferCount = 0;
	}

	// For defrag, snapshot the transform buffer before FinishSkinningBufferUpload shrinks it.
	if (bDefragging && NumCopies > 0)
	{
		FRDGBufferRef OldTransformBuffer = SceneData->Buffers->TransformDataBuffer.Register(GraphBuilder);
		DefragStagingBuffer = GraphBuilder.CreateBuffer(OldTransformBuffer->Desc, TEXT("Skinning.DefragStagingBuffer"));

		FMemcpyResourceParams CopyParams;
		CopyParams.Count = OldTransformBuffer->Desc.GetSize() / OldTransformBuffer->Desc.BytesPerElement;
		CopyParams.SrcOffset = 0;
		CopyParams.DstOffset = 0;
		MemcpyResource(GraphBuilder, DefragStagingBuffer, OldTransformBuffer, CopyParams);
	}

	FSkinningSceneParameters Parameters;
	const bool bUpdateStats = true;
	SceneData->FinishSkinningBufferUpload(GraphBuilder, &Parameters, bUpdateStats);

	// Dispatch the copy shader
	if (NumCopies > 0)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "CopyTransforms");

		FRDGBufferRef HeaderBuffer = CreateStructuredBuffer(
			GraphBuilder,
			TEXT("Skinning.CopyTransformsHeaders"),
			sizeof(FCopyTransformsHeader),
			FMath::RoundUpToPowerOfTwo(NumCopies),
			CopyHeaders.GetData(),
			sizeof(FCopyTransformsHeader) * NumCopies,
			ERDGInitialDataFlags::NoCopy
		);

		FRDGBufferRef TransformBuffer = Parameters.BoneTransforms->GetParent();

		// When there's no defrag happening, the destination and source buffers are the same. This
		// is safe because the copy algorithm guarantees that no writes and reads overlap.
		// 
		// However, OpenGL ES 3.1 doesn't allow the buffer to be used for both reading and writing
		// like this, so on mobile (the bSelfAliasedOnMobile case) we create an additional UAV to 
		// get around this problem. There is no additional barrier, so the performance and memory 
		// overhead is minimal, but still worth the effort to avoid on other platforms.
		const bool bMobilePlatform = IsMobilePlatform(SceneData->Scene.GetShaderPlatform());
		const bool bUseStagingBuffer = (DefragStagingBuffer != nullptr) || bMobilePlatform;
		const bool bSelfAliasedOnMobile = bMobilePlatform && (DefragStagingBuffer == nullptr);

		FCopyPreviousTransformsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyPreviousTransformsCS::FParameters>();
		if (bSelfAliasedOnMobile)
		{
			PassParameters->DstTransformBuffer = GraphBuilder.CreateUAV(TransformBuffer, COMPRESSED_BONE_TRANSFORM_PIXEL_FORMAT, ERDGUnorderedAccessViewFlags::SkipBarrier);
			PassParameters->SrcTransformBuffer = GetCompressedBoneTransformSRV(GraphBuilder, TransformBuffer);
		}
		else
		{
			PassParameters->DstTransformBuffer = GetCompressedBoneTransformUAV(GraphBuilder, TransformBuffer);
			PassParameters->SrcTransformBuffer = GetCompressedBoneTransformSRV(GraphBuilder, bUseStagingBuffer ? DefragStagingBuffer : GetDefaultCompressedBoneTransformBuffer(GraphBuilder));
		}
		PassParameters->CopyHeaders = GraphBuilder.CreateSRV(HeaderBuffer);

		FCopyPreviousTransformsCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FCopyPreviousTransformsCS::FUseStagingBuffer>(bUseStagingBuffer);
		TShaderMapRef<FCopyPreviousTransformsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

		const uint32 GroupSize = FCopyPreviousTransformsCS::ThreadGroupSize;
		const uint32 NumElementGroups = FMath::DivideAndRoundUp(MaxElementsPerCopy, GroupSize);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("CopyTransforms (%d entries, %d max elements, %s)", NumCopies, MaxElementsPerCopy, bSelfAliasedOnMobile ? TEXT("self-aliased") : (bUseStagingBuffer ? TEXT("staged") : TEXT("direct"))),
			ComputeShader,
			PassParameters,
			FIntVector(NumElementGroups, NumCopies, 1)
		);
	}

	if (CurrentTime)
	{
		SceneData->PerformSkinning(Parameters, GraphBuilder, *CurrentTime);
	}
}

void FSkinningSceneExtension::FUpdater::ResolveAttachments(FRDGBuilder& GraphBuilder, const FGPUSceneWriteDelegateParams& Params)
{
	if (!SceneData->IsEnabled())
	{
		return;
	}

	SceneData->ResolveAttachments(GraphBuilder, Params);
}

void FSkinningSceneExtension::FRenderer::UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer)
{
	SCOPED_NAMED_EVENT(FSkinningSceneExtension_FRenderer_UpdateSceneUniformBuffer, FColor::Silver);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(STAT_SkinningSceneExtension);
	check(SceneData->IsEnabled());
	FSkinningSceneParameters Parameters;
	const bool bUpdateStats = false;
	SceneData->FinishSkinningBufferUpload(GraphBuilder, &Parameters, bUpdateStats);
	SceneUniformBuffer.Set(SceneUB::Skinning, Parameters);
}

void FSkinningSceneExtension::GetSkinnedPrimitives(TArray<FPrimitiveSceneInfo*>& OutPrimitives) const
{
	OutPrimitives.Reset();

	if (!IsEnabled())
	{
		return;
	}

	WaitForHeaderDataUpdateTasks();

	OutPrimitives.Reserve(HeaderDatas.Num());

	for (typename TSparseArray<FHeaderData>::TConstIterator It(HeaderDatas); It; ++It)
	{
		const FHeaderData& Header = *It;
		OutPrimitives.Add(Header.PrimitiveSceneInfo);
	}
}

const FSkinningTransformProvider::FProviderId& FSkinningSceneExtension::GetRefPoseProviderId()
{
	return RefPoseProviderId;
}

const FSkinningTransformProvider::FProviderId& FSkinningSceneExtension::GetMeshObjectProviderId()
{
	return MeshObjectProviderId;
}

void FSkinningSceneExtension::ProvideRefPoseTransforms(FSkinningTransformProvider::FProviderContext& Context)
{
	const uint32 TransformsPerGroup = FRefPoseTransformProviderCS::TransformsPerGroup;

	// TODO: Optimize further

	uint32 BlockCount = 0;
	for (const FSkinningTransformProvider::FProviderIndirection Indirection : Context.Indirections)
	{
		const FSkinningSceneExtensionProxy* Proxy = Context.Proxies[Indirection.Index];
		const uint32 TransformCount = Proxy->GetMaxBoneTransformCount();
		const uint32 AnimationCount = Proxy->GetUniqueAnimationCount();
		BlockCount += FMath::DivideAndRoundUp(TransformCount * AnimationCount, TransformsPerGroup);
	}

	if (BlockCount == 0)
	{
		return;
	}

	FRDGBuilder& GraphBuilder = Context.GraphBuilder;
	FTransformBlockHeader* BlockHeaders = GraphBuilder.AllocPODArray<FTransformBlockHeader>(BlockCount);

	uint32 BlockWrite = 0;
	for (const FSkinningTransformProvider::FProviderIndirection Indirection : Context.Indirections)
	{
		const FPrimitiveSceneInfo* Primitive = Context.Primitives[Indirection.Index];
		const FSkinningSceneExtensionProxy* Proxy = Context.Proxies[Indirection.Index];
		const uint32 TransformCount = Proxy->GetMaxBoneTransformCount();
		const uint32 AnimationCount = Proxy->GetUniqueAnimationCount();
		const uint32 TotalTransformCount = TransformCount * AnimationCount;
	
		uint32 TransformWrite =  Indirection.CurrentTransformOffset < Indirection.PreviousTransformOffset ? Indirection.CurrentTransformOffset : Indirection.PreviousTransformOffset;

		const uint32 FullBlockCount = TotalTransformCount / TransformsPerGroup;
		for (uint32 BlockIndex = 0; BlockIndex < FullBlockCount; ++BlockIndex)
		{
			BlockHeaders[BlockWrite].BlockLocalIndex = BlockIndex;
			BlockHeaders[BlockWrite].BlockTransformCount = TransformsPerGroup;
			BlockHeaders[BlockWrite].BlockTransformOffset = TransformWrite;
			++BlockWrite;

			TransformWrite += (TransformsPerGroup * 2 * sizeof(FCompressedBoneTransform));
		}

		const uint32 PartialTransformCount = TotalTransformCount - (FullBlockCount * TransformsPerGroup);
		if (PartialTransformCount > 0)
		{
			BlockHeaders[BlockWrite].BlockLocalIndex = FullBlockCount;
			BlockHeaders[BlockWrite].BlockTransformCount = PartialTransformCount;
			BlockHeaders[BlockWrite].BlockTransformOffset = TransformWrite;
			++BlockWrite;
		}
	}

	FRDGBufferRef BlockHeaderBuffer = CreateStructuredBuffer(
		GraphBuilder,
		TEXT("Skinning.RefPoseHeaders"),
		sizeof(FTransformBlockHeader),
		FMath::RoundUpToPowerOfTwo(FMath::Max(BlockCount, 1u)),
		BlockHeaders,
		sizeof(FTransformBlockHeader) * BlockCount,
		// The buffer data is allocated above on the RDG timeline
		ERDGInitialDataFlags::NoCopy
	);

	FRefPoseTransformProviderCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRefPoseTransformProviderCS::FParameters>();
	PassParameters->TransformBuffer = GetCompressedBoneTransformUAV(GraphBuilder, Context.TransformBuffer);
	PassParameters->HeaderBuffer = GraphBuilder.CreateSRV(BlockHeaderBuffer);

	auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FRefPoseTransformProviderCS>();
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("RefPoseProvider"),
		ComputeShader,
		PassParameters,
		FIntVector(BlockCount, 1, 1)
	);
}

void FSkinningSceneExtension::ProvideMeshObjectTransforms(FSkinningTransformProvider::FProviderContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinningSceneExtension::ProvideMeshObjectTransforms);
	RDG_EVENT_SCOPE(Context.GraphBuilder, "ProvideMeshObjectTransforms");

	uint32 GlobalTransformCount = 0;

	for (const FSkinningTransformProvider::FProviderIndirection Indirection : Context.Indirections)
	{
		const FSkinningSceneExtensionProxy* Proxy = Context.Proxies[Indirection.Index];
		const uint32 TransformCount = Proxy->GetMaxBoneTransformCount();
		GlobalTransformCount += TransformCount * 2; // Current and Previous
	}

	if (GlobalTransformCount == 0)
	{
		return;
	}

	FRDGBuilder& GraphBuilder = Context.GraphBuilder;
	FRDGAsyncScatterUploadBuffer& TransformUploadBuffer = *GraphBuilder.AllocObject<FRDGAsyncScatterUploadBuffer>();

	Context.ScatterUploadBuilder.AddPass(GraphBuilder, TransformUploadBuffer, Context.TransformBuffer, GlobalTransformCount, sizeof(FCompressedBoneTransform), TEXT("Skinning.AnimTransforms"),
		[Indirections = Context.Indirections, Proxies = Context.Proxies, GlobalTransformCount] (FRDGScatterUploader& ScatterUploader)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSkinningSceneExtension::ProvideMeshObjectTransformsTask);

		int64 NumBonesTotal = 0;

		for (const FSkinningTransformProvider::FProviderIndirection Indirection : Indirections)
		{
			const FSkinningSceneExtensionProxy* Proxy = Proxies[Indirection.Index];

			const uint32 MaxTransformCount = Proxy->GetMaxBoneTransformCount();
			const uint32 MaxTotalTransformCount = MaxTransformCount * 2u; // Current and Previous

			const FSkeletalMeshObject* MeshObject = Proxy->GetMeshObject();

			const FSkeletalMeshObject::FBoneTransforms Transforms = MeshObject->GetBoneTransforms();

			const int32 DstCurrentTransformIndex  = Indirection.CurrentTransformOffset  / sizeof(FCompressedBoneTransform);
			const int32 DstPreviousTransformIndex = Indirection.PreviousTransformOffset / sizeof(FCompressedBoneTransform);

			if (!Transforms.CurrentTransforms.IsEmpty())
			{
				const bool bUpdatePreviousTransforms = !Transforms.PreviousTransforms.IsEmpty() && (Transforms.UpdateMode != EPreviousBoneTransformUpdateMode::None || EnumHasAnyFlags(Indirection.DirtyBoneTransforms, EDirtyBoneTransforms::Previous));

				{
					TArrayView<FCompressedBoneTransform> DstCurrentTransforms = ScatterUploader.Add_GetRef<FCompressedBoneTransform>(DstCurrentTransformIndex, MaxTransformCount);
					uint32 DstTransformIndex = 0;

					if (Proxy->GetBoneTransformStorageMode() == EBoneTransformStorageMode::BoneMap)
					{
						for (uint32 SrcTransformIndex : Proxy->GetBoneMap())
						{
							StoreCompressedBoneTransform(&DstCurrentTransforms[DstTransformIndex], Transforms.CurrentTransforms[SrcTransformIndex]);
							DstTransformIndex++;
						}
					}
					else
					{
						for (; DstTransformIndex < MaxTransformCount; ++DstTransformIndex)
						{
							StoreCompressedBoneTransform(&DstCurrentTransforms[DstTransformIndex], Transforms.CurrentTransforms[DstTransformIndex]);
						}
					}

					NumBonesTotal += DstTransformIndex;
				}

				if (bUpdatePreviousTransforms)
				{
					TArrayView<FCompressedBoneTransform> DstPreviousTransforms = ScatterUploader.Add_GetRef<FCompressedBoneTransform>(DstPreviousTransformIndex, MaxTransformCount);
					uint32 DstTransformIndex = 0;

					if (Proxy->GetBoneTransformStorageMode() == EBoneTransformStorageMode::BoneMap)
					{
						for (uint32 SrcTransformIndex : Proxy->GetBoneMap())
						{
							StoreCompressedBoneTransform(&DstPreviousTransforms[DstTransformIndex], Transforms.PreviousTransforms[SrcTransformIndex]);
							DstTransformIndex++;
						}
					}
					else
					{
						for (; DstTransformIndex < MaxTransformCount; ++DstTransformIndex)
						{
							StoreCompressedBoneTransform(&DstPreviousTransforms[DstTransformIndex], Transforms.PreviousTransforms[DstTransformIndex]);
						}
					}

					NumBonesTotal += MaxTransformCount;
				}
			}
			else
			{
				FCompressedBoneTransform* DstCurrentTransforms = (FCompressedBoneTransform*)ScatterUploader.Add_GetRef(DstCurrentTransformIndex, MaxTransformCount);

				// Data is invalid, replace with reference pose
				for (uint32 TransformIndex = 0; TransformIndex < MaxTransformCount; ++TransformIndex)
				{
					SetCompressedBoneTransformIdentity(DstCurrentTransforms[TransformIndex]);
				}

				FCompressedBoneTransform* DstPreviousTransforms = (FCompressedBoneTransform*)ScatterUploader.Add_GetRef(DstPreviousTransformIndex, MaxTransformCount);

				// Data is invalid, replace with reference pose
				for (uint32 TransformIndex = 0; TransformIndex < MaxTransformCount; ++TransformIndex)
				{
					SetCompressedBoneTransformIdentity(DstPreviousTransforms[TransformIndex]);
				}

				NumBonesTotal += MaxTotalTransformCount;
			}
		}

		INC_MEMORY_STAT_BY(STAT_SkinningSceneExtension_BoneTransformUploadSize, NumBonesTotal * sizeof(FCompressedBoneTransform));
	});
}

// TODO: these are prototype macros for how we might expose SceneUB for direct binding. 
//       If this becomes the way we want to expose this, then we should move this to shared headers.
//       There's still some machinery we _could_ add to make it work nicely as an API, e.g., interface to get the associated sub-UB & register a provider (or something).

#define IMPLEMENT_STATIC_UNIFORM_BUFFER_SCENE_UB(StructType, MangledName) \
	IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(MangledName)	\
	IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(StructType, #MangledName, MangledName);

/**
 * Implement a Scene UB sub-struct _with_ a global UB definition for binding stand-alone.
 */
#define IMPLEMENT_SCENE_UB_STRUCT_EX(StructType, FieldName, DefaultValueFactoryType) \
	TSceneUniformBufferMemberRegistration<StructType> SceneUB::FieldName { TEXT(#FieldName), DefaultValueFactoryType }; \
	IMPLEMENT_STATIC_UNIFORM_BUFFER_SCENE_UB(StructType, SceneUbEx##FieldName)

IMPLEMENT_SCENE_UB_STRUCT_EX(FSkinningSceneParameters, Skinning, GetDefaultSkinningParameters);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Bone Attachment Resolve

class FResolveAttachmentsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FResolveAttachmentsCS);
	SHADER_USE_PARAMETER_STRUCT(FResolveAttachmentsCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<UE::HLSL::FResolveAttachmentBlockHeader>, AttachmentBlockHeaderBuffer)
		SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_SRV(SkinningTransformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneWriterParameters, GPUSceneWriterParameters)
		SHADER_PARAMETER(uint32, NumAttachments)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FResolveAttachmentsCS, "/Engine/Private/Skinning/ResolveAttachments.usf", "ResolveAttachmentsCS", SF_Compute);

void FSkinningSceneExtension::ResolveAttachments(FRDGBuilder& GraphBuilder, const FGPUSceneWriteDelegateParams& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSkinningSceneExtension::ResolveAttachments);

	if (!IsEnabled())
	{
		return;
	}

	// Collect all attachments across all child instanced primitives.
	uint32 NumTotalAttachments = 0;

	for (const FHeaderData& Header : HeaderDatas)
	{
		if (Header.bIsInstanced && Header.Proxy)
		{
			const FInstancedSkinningSceneExtensionProxy* ChildProxy = static_cast<const FInstancedSkinningSceneExtensionProxy*>(Header.Proxy);
			if (ChildProxy->HasBoneAttachments())
			{
				for (FBoneAttachmentBinding Binding : ChildProxy->GetBoneAttachmentBindings())
				{
					if (Binding.IsAttached())
					{
						++NumTotalAttachments;
					}
				}
			}
		}
	}

	if (NumTotalAttachments == 0)
	{
		return;
	}

	// Allocate the block header buffer and lock for write.
	FRDGBuffer* AttachmentHeaderBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(UE::HLSL::FResolveAttachmentBlockHeader), NumTotalAttachments),
		TEXT("SkinningSceneExtension.AttachmentHeaders"));

	GraphBuilder.ConvertToExternalBuffer(AttachmentHeaderBuffer);

	// Fill the headers asynchronously inside a command list setup task.
	// Lock/Unlock happens on the command list timeline inside the task.
	uint32& NumActualAttachments = *GraphBuilder.AllocPOD<uint32>();
	NumActualAttachments = 0;

	UE::Tasks::FTask FillTask = GraphBuilder.AddCommandListSetupTask(
		[this, AttachmentHeaderBuffer, NumTotalAttachments, &NumActualAttachments](FRHICommandList& RHICmdList)
	{
		auto* RawHeaderData = reinterpret_cast<UE::HLSL::FResolveAttachmentBlockHeader*>(
			RHICmdList.LockBuffer(AttachmentHeaderBuffer->GetRHIUnchecked(), 0,
				NumTotalAttachments * sizeof(UE::HLSL::FResolveAttachmentBlockHeader), RLM_WriteOnly));

		TRACE_CPUPROFILER_EVENT_SCOPE(FSkinningSceneExtension::FillAttachmentHeaders);

		uint32 AttachmentIndex = 0;
		TArray<uint16, SceneRenderingAllocator> ReverseBoneMap;

		// Iterate child proxies that have bone attachment parent sockets.
		for (const FHeaderData& ChildHeader : HeaderDatas)
		{
			if (!ChildHeader.bIsInstanced || !ChildHeader.Proxy)
			{
				continue;
			}

			const FInstancedSkinningSceneExtensionProxy* ChildProxy = static_cast<const FInstancedSkinningSceneExtensionProxy*>(ChildHeader.Proxy);
			if (!ChildProxy->HasBoneAttachments())
			{
				continue;
			}

			const FPrimitiveSceneInfo* ChildSceneInfo = ChildHeader.PrimitiveSceneInfo;
			if (!ChildSceneInfo)
			{
				continue;
			}

			// Pre-resolve each parent socket: find parent header, build reverse bone map, resolve transform bone index.
			struct FResolvedParentSocket
			{
				const FHeaderData* ParentHeader = nullptr;
				uint16 TransformBoneIndex = INVALID_BONE_INDEX;
			};
			TArray<FResolvedParentSocket, SceneRenderingAllocator> ResolvedSockets;
			ResolvedSockets.SetNum(ChildProxy->GetBoneAttachmentSockets().Num());

			for (int32 SocketIndex = 0; SocketIndex < ChildProxy->GetBoneAttachmentSockets().Num(); ++SocketIndex)
			{
				const FBoneAttachmentSocket& Socket = ChildProxy->GetBoneAttachmentSockets()[SocketIndex];
				FResolvedParentSocket& Resolved = ResolvedSockets[SocketIndex];

				if (!Socket.ParentComponentId.IsValid())
				{
					continue;
				}

				const FPrimitiveSceneInfo* ParentSceneInfo = Scene.GetPrimitiveSceneInfo(Socket.ParentComponentId);
				if (!ParentSceneInfo)
				{
					continue;
				}

				const int32 ParentPersistentIndex = ParentSceneInfo->GetPersistentIndex().Index;
				if (!HeaderDatas.IsValidIndex(ParentPersistentIndex))
				{
					continue;
				}

				const FHeaderData& ParentHeader = HeaderDatas[ParentPersistentIndex];
				Resolved.ParentHeader = &ParentHeader;

				// Build reverse bone map for this parent and resolve the bone index.
				TConstArrayView<uint16> ParentBoneMap = ParentHeader.Proxy ? ParentHeader.Proxy->GetBoneMap() : TConstArrayView<uint16>();
				Resolved.TransformBoneIndex = Socket.BoneIndex;

				if (!ParentBoneMap.IsEmpty() && ParentHeader.Proxy)
				{
					ReverseBoneMap.SetNumUninitialized(ParentHeader.Proxy->GetSkinnedAsset()->GetRefSkeleton().GetNum(), EAllowShrinking::No);
					FMemory::Memset(ReverseBoneMap.GetData(), 0xFF, ReverseBoneMap.Num() * sizeof(uint16));
					for (int32 BoneMapIndex = 0; BoneMapIndex < ParentBoneMap.Num(); ++BoneMapIndex)
					{
						if (ParentBoneMap[BoneMapIndex] < (uint16)ReverseBoneMap.Num())
						{
							ReverseBoneMap[ParentBoneMap[BoneMapIndex]] = (uint16)BoneMapIndex;
						}
					}

					Resolved.TransformBoneIndex = (Socket.BoneIndex < (uint16)ReverseBoneMap.Num())
						? ReverseBoneMap[Socket.BoneIndex]
						: INVALID_BONE_INDEX;
				}
			}

			// Iterate child instances and emit headers for attached ones.
			const int32 NumChildInstances = ChildProxy->GetNumBoneAttachmentBindings();
			for (int32 ChildInstanceIndex = 0; ChildInstanceIndex < NumChildInstances; ++ChildInstanceIndex)
			{
				const FBoneAttachmentBinding Binding = ChildProxy->GetBoneAttachmentBindings()[ChildInstanceIndex];
				if (!Binding.IsAttached())
				{
					continue;
				}

				const uint8 ParentSocketIndex = Binding.GetSocketIndex();
				const uint32 ParentInstanceIndex = Binding.GetParentInstanceIndex();

				if (!ResolvedSockets.IsValidIndex(ParentSocketIndex) || !ResolvedSockets[ParentSocketIndex].ParentHeader)
				{
					continue;
				}

				const FResolvedParentSocket& Resolved = ResolvedSockets[ParentSocketIndex];
				const FHeaderData& ParentHeader = *Resolved.ParentHeader;
				const FBoneAttachmentSocket& Socket = ChildProxy->GetBoneAttachmentSockets()[ParentSocketIndex];

				// Skip attachments where the bone is not in the parent's skinning buffer (e.g., LOD-culled).
				if (Resolved.TransformBoneIndex == INVALID_BONE_INDEX)
				{
					continue;
				}

				// Skip if parent instance index is out of range (parent may have had instances removed).
				if (ParentInstanceIndex >= ParentHeader.NumInstanceSceneDataEntries)
				{
					continue;
				}

				const uint32 ParentInstanceId = ParentHeader.InstanceSceneDataOffset + ParentInstanceIndex;
				const uint32 ChildGPUSceneInstanceId = ChildHeader.InstanceSceneDataOffset + ChildInstanceIndex;

				UE::HLSL::FResolveAttachmentBlockHeader& BlockHeader = RawHeaderData[AttachmentIndex++];
				BlockHeader.ParentTransformBufferOffset = ParentHeader.TransformBufferOffset;
				SetMaxTransformCount(BlockHeader, ParentHeader.MaxTransformCount);
				SetBoneIndex(BlockHeader, Resolved.TransformBoneIndex);
				SetCurrentTransformSlot(BlockHeader, ParentHeader.CurrentTransformSlot);
				SetParentInstanceId(BlockHeader, ParentInstanceId);
				BlockHeader.ChildInstanceId = ChildGPUSceneInstanceId;
				BlockHeader.ChildPrimitiveId = ChildSceneInfo->GetPersistentIndex().Index;

				BlockHeader.LocalOffsetTranslation[0] = Socket.Translation.X;
				BlockHeader.LocalOffsetTranslation[1] = Socket.Translation.Y;
				BlockHeader.LocalOffsetTranslation[2] = Socket.Translation.Z;

				BlockHeader.LocalOffsetRotation[0] = Socket.Rotation.X;
				BlockHeader.LocalOffsetRotation[1] = Socket.Rotation.Y;
				BlockHeader.LocalOffsetRotation[2] = Socket.Rotation.Z;
				BlockHeader.LocalOffsetRotation[3] = Socket.Rotation.W;

				FMemory::Memcpy(BlockHeader.BoneRefPoseRow0, Socket.RefPoseMatrix.M[0], sizeof(float) * 4);
				FMemory::Memcpy(BlockHeader.BoneRefPoseRow1, Socket.RefPoseMatrix.M[1], sizeof(float) * 4);
				FMemory::Memcpy(BlockHeader.BoneRefPoseRow2, Socket.RefPoseMatrix.M[2], sizeof(float) * 4);
			}
		}

		NumActualAttachments = AttachmentIndex;

		RHICmdList.UnlockBuffer(AttachmentHeaderBuffer->GetRHIUnchecked());
	});

	FResolveAttachmentsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FResolveAttachmentsCS::FParameters>();
	PassParameters->AttachmentBlockHeaderBuffer = GraphBuilder.CreateSRV(AttachmentHeaderBuffer);

	// Re-register the persistent transform buffer with this GraphBuilder to get an RDG SRV.
	FRDGBufferRef TransformBuffer = GraphBuilder.RegisterExternalBuffer(Buffers->TransformDataBuffer.GetPooledBuffer());
	PassParameters->SkinningTransformBuffer = GetCompressedBoneTransformSRV(GraphBuilder, TransformBuffer);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PassParameters->GPUSceneWriterParameters = Params.GPUWriteParams;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TShaderMapRef<FResolveAttachmentsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("ResolveAttachments(%d)", NumTotalAttachments),
		ComputeShader,
		PassParameters,
		[PassParameters, &NumActualAttachments]() -> FIntVector
		{
			PassParameters->NumAttachments = NumActualAttachments;
			return FIntVector(FMath::DivideAndRoundUp(NumActualAttachments, 64u), 1, 1);
		}
	);
}
