// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SpanAllocator.h"
#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "SceneExtensions.h"
#include "Skinning/SkinningTransformProvider.h"
#include "SkinningDefinitions.h"
#include "RendererPrivateUtils.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "Matrix3x4.h"
#include "Delegates/DelegateCombinations.h"
#include "Delegates/Delegate.h"
#include "SkeletalMeshUpdater.h"
#include "SceneRendering.h"

class FSkinningSceneParameters;
struct FGPUSceneWriteDelegateParams;

class FSkinnedSceneProxy;
class FSkinningSceneExtension : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FSkinningSceneExtension);
	struct FHeaderData;

public:
	class FUpdater : public ISceneExtensionUpdater
	{
		DECLARE_SCENE_EXTENSION_UPDATER(FUpdater, FSkinningSceneExtension);

	public:
		FUpdater(FSkinningSceneExtension& InSceneData);

		virtual void End();
		virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override;
		virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override;

		void PreGPUSceneUpdate(FRDGBuilder& GraphBuilder, const FGameTime* CurrentTime);

		void ResolveAttachments(FRDGBuilder& GraphBuilder, const FGPUSceneWriteDelegateParams& Params);

		void PostMeshUpdate(FRDGBuilder& GraphBuilder, const TConstArrayView<FPrimitiveSceneInfo*>& SceneInfosWithStaticDrawListUpdate, const FSkeletalMeshUpdater::FSubmitData* SkeletalMeshSubmitData);

	private:
		void AddHeaderUpdate(FHeaderData& HeaderData, int32 PrimitiveIndex);

		FSkinningSceneExtension* SceneData = nullptr;
		TConstArrayView<FPrimitiveSceneInfo*> AddedList;
		TArray<int32, FSceneRenderingArrayAllocator> AllocationUpdateList;
		TArray<int32, FSceneRenderingArrayAllocator> HeaderDataUpdateList;
		const bool bEnableAsync = true;
		bool bForceFullUpload = false;
		bool bDefragging = false;

		friend FSkinningSceneExtension;
	};

	class FRenderer : public ISceneExtensionRenderer
	{
		DECLARE_SCENE_EXTENSION_RENDERER(FRenderer, FSkinningSceneExtension);
	
	public:
		FRenderer(FSceneRendererBase& InSceneRenderer, FSkinningSceneExtension& InSceneData)
			: ISceneExtensionRenderer(InSceneRenderer)
			, SceneData(&InSceneData)
			, ViewFamily(*InSceneRenderer.GetViewFamily())
		{}

		virtual void UpdateViewData(FRDGBuilder& GraphBuilder, const FRendererViewDataManager& ViewDataManager) override;

		virtual void UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& Buffer) override;

	private:
		FSkinningSceneExtension* SceneData = nullptr;
		const FSceneViewFamily& ViewFamily;
	};

	friend class FUpdater;

	static bool ShouldCreateExtension(FScene& InScene);

	explicit FSkinningSceneExtension(FScene& InScene);
	virtual ~FSkinningSceneExtension();

	virtual void InitExtension(FScene& InScene) override;

	virtual ISceneExtensionUpdater* CreateUpdater() override;
	virtual ISceneExtensionRenderer* CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags) override;

	RENDERER_API void GetSkinnedPrimitives(TArray<FPrimitiveSceneInfo*>& OutPrimitives) const;

	RENDERER_API static const FSkinningTransformProvider::FProviderId& GetRefPoseProviderId();
	RENDERER_API static const FSkinningTransformProvider::FProviderId& GetMeshObjectProviderId();

private:
	enum ETask : uint32
	{
		FreeBufferSpaceTask,
		InitHeaderDataTask,
		InitUpdateListTask,
		AllocBufferSpaceTask,
		UploadHeaderDataTask,
		UploadHierarchyDataTask,
		UploadTransformDataTask,

		NumTasks
	};

	struct FHeaderData
	{
		FPrimitiveSceneInfo* PrimitiveSceneInfo	= nullptr;
		FSkinningSceneExtensionProxy* Proxy     = nullptr;
		const FSkeletalMeshObject* MeshObject   = nullptr;
		FGuid ProviderId;
		uint32 UpdateCounter                     = 0;
		uint32 InstanceSceneDataOffset           = 0;
		uint32 NumInstanceSceneDataEntries       = 0;
		uint32 BoneMapBufferOffset               = INDEX_NONE;
		uint32 BoneMapBufferCount                = 0;
		uint32 ObjectSpaceBufferOffset           = INDEX_NONE;
		uint32 ObjectSpaceBufferCount            = 0;
		uint32 HierarchyBufferOffset             = INDEX_NONE;
		uint32 HierarchyBufferCount              = 0;
		uint32 TransformBufferOffset             = INDEX_NONE;
		uint32 TransformBufferCount              = 0;
		uint32 PreviousTransformBufferOffset     = INDEX_NONE;
		uint32 PreviousTransformBufferCount      = 0;
		uint32 UniqueAnimationCount              = 1;
		uint16 MaxTransformCount                 = 0;
		uint16 MaxBoneMapCount                   = 0;
		uint16 MaxBoneHierarchyCount             = 0;
		uint16 MaxObjectSpaceCount               = 0;
		EDirtyBoneTransforms DirtyBoneTransforms = EDirtyBoneTransforms::All;
		uint8  MaxInfluenceCount                 = 0;
		uint8  bHasScale : 1                     = false;
		uint8  bIsBatched : 1                    = false;
		uint8  bIsInstanced : 1                  = false;
		uint8  bIsRefPose : 1                    = false;
		uint8  bVertexFactoryDirty : 1           = false;
		uint8  bIsNanite : 1                     = false;
		uint8  CurrentTransformSlot : 1          = 0;

		bool IsFirstUpdate() const
		{
			return UpdateCounter == 0;
		}

		FSkeletonBatchKey GetSkeletonBatchKey() const;

		FSkinningHeader Pack() const
		{
			Validate();

			FSkinningHeader Output;
			Output.HierarchyBufferOffset	= HierarchyBufferOffset;
			Output.TransformBufferOffset	= TransformBufferOffset;
			Output.ObjectSpaceBufferOffset	= ObjectSpaceBufferOffset != INDEX_NONE ? ObjectSpaceBufferOffset : 0;
			Output.MaxTransformCount		= MaxTransformCount;
			Output.MaxInfluenceCount		= MaxInfluenceCount;
			Output.bHasScale				= bHasScale;
			Output.bIsRefPose				= bIsRefPose;
			Output.CurrentTransformSlot		= CurrentTransformSlot;
			return Output;
		}

		void Validate() const
		{
			check(TransformBufferOffset	<= SKINNING_BUFFER_TRANSFORM_OFFSET_MAX);
			check(HierarchyBufferOffset == INDEX_NONE || HierarchyBufferOffset <= SKINNING_BUFFER_HIERARCHY_OFFSET_MAX);
			check(ObjectSpaceBufferOffset == INDEX_NONE || ObjectSpaceBufferOffset <= SKINNING_BUFFER_OBJECT_SPACE_OFFSET_MAX);
			check(MaxInfluenceCount		<= SKINNING_BUFFER_INFLUENCE_MAX);
		}
	};

	class FBuffers
	{
	public:
		FBuffers();

		TPersistentByteAddressBuffer<FSkinningHeader> HeaderDataBuffer;
		TPersistentByteAddressBuffer<uint32> BoneMapBuffer;
		TPersistentByteAddressBuffer<uint32> BoneHierarchyBuffer;
		TPersistentByteAddressBuffer<float> BoneObjectSpaceBuffer;
		FPersistentCompressedBoneTransformBuffer TransformDataBuffer;
		FRHIBuffer* TransformDataBufferRHI = nullptr;
		TRefCountPtr<FRHIShaderResourceView> TransformDataBufferSRV;
	};
	
	class FUploader
	{
	public:
		TByteAddressBufferScatterUploader<FSkinningHeader> HeaderDataUploader;
		TByteAddressBufferScatterUploader<uint32> BoneMapUploader;
		TByteAddressBufferScatterUploader<uint32> BoneHierarchyUploader;
		TByteAddressBufferScatterUploader<float> BoneObjectSpaceUploader;
		FCompressedBoneTransformBufferScatterUploader TransformDataUploader;
	};
	
	bool IsEnabled() const { return Buffers.IsValid(); }
	void SetEnabled(bool bEnabled);
	void SyncAllTasks() const { UE::Tasks::Wait(TaskHandles); }

	void FinishSkinningBufferUpload(
		FRDGBuilder& GraphBuilder,
		FSkinningSceneParameters* OutParams = nullptr,
		bool bUpdateStats = false
	);

	void PerformSkinning(
		FSkinningSceneParameters& Parameters,
		FRDGBuilder& GraphBuilder,
		const FGameTime& CurrentTime
	);

	void ResolveAttachments(FRDGBuilder& GraphBuilder, const FGPUSceneWriteDelegateParams& Params);

	bool ProcessBufferDefragmentation();

	UWorld* GetWorld() const;

	// Wait for tasks that modify HeaderData - after this the size and main fields do not change.
	void WaitForHeaderDataUpdateTasks() const;

	bool HasUpdatedThisFrame() const
	{
		return LastUpdateFrameNumber == GFrameNumberRenderThread;
	}

private:
	bool GetHeaderFromMeshObject(const FSkeletalMeshObject* MeshObject, FHeaderData*& OutHeaderData, int32& OutPrimitiveIndex);

	FSpanAllocator BoneMapAllocator;
	FSpanAllocator BoneHierarchyAllocator;
	FSpanAllocator ObjectSpaceAllocator;
	FSpanAllocator TransformAllocator;
	TSparseArray<FHeaderData> HeaderDatas;
	TSet<int32> AllocatedHeaderDataIndices;
	TSet<int32> InstancedHeaderDataIndices;
	struct FBatchHeaderEntry
	{
		FHeaderData HeaderData;
		uint32 RefCount = 0;
	};
	TMap<FSkeletonBatchKey, FBatchHeaderEntry> BatchHeaderDatas;
	TUniquePtr<FBuffers> Buffers;
	TUniquePtr<FUploader> Uploader;
	TStaticArray<UE::Tasks::FTask, NumTasks> TaskHandles;
	uint32 UpdateCounter = 0;
	uint32 LastUpdateFrameNumber = 0;
	int32 VertexFactoryUpdateRequests = 0;
	TArray<int32> VertexFactoryUpdateList;
	bool bVertexFactoryFullUpload = false;

	bool Tick(float DeltaTime);

	struct FTickState : public FRefCountedObject
	{
		FVector CameraLocation = FVector::ZeroVector;
	};

	TRefCountPtr<FTickState> TickState{ new FTickState };
	FTSTicker::FDelegateHandle UpdateTimerHandle;

	TWeakObjectPtr<UWorld> WorldRef;

public:
	RENDERER_API static void ProvideRefPoseTransforms(FSkinningTransformProvider::FProviderContext& Context);
	RENDERER_API static void ProvideMeshObjectTransforms(FSkinningTransformProvider::FProviderContext& Context);
};