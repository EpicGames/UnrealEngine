// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinningTransformProvider.h"
#include "ScenePrivate.h"
#include "RenderUtils.h"
#include "SkeletalRenderPublic.h"
#include "SkinningSceneExtension.h"

IMPLEMENT_SCENE_EXTENSION(FSkinningTransformProvider);

bool FSkinningTransformProvider::ShouldCreateExtension(FScene& InScene)
{
	return IsGPUSkinSceneExtensionEnabled() || (NaniteSkinnedMeshesSupported() && DoesRuntimeSupportNanite(GetFeatureLevelShaderPlatform(InScene.GetFeatureLevel()), true, true));
}

void FSkinningTransformProvider::RegisterProvider(const FSkinningTransformProvider::FProviderId& Id, const FOnProvideTransforms& Delegate, bool bUsesSkeletonBatches)
{
#if DO_CHECK
	for (const FTransformProvider& ProviderCheck : Providers)
	{
		check(ProviderCheck.Id != Id);
	}
#endif

	check(Delegate.IsBound());
	FTransformProvider& Provider = Providers.Emplace_GetRef();
	Provider.Id = Id;
	Provider.Delegate = Delegate;
	Provider.bUsesSkeletonBatches = bUsesSkeletonBatches;
}

void FSkinningTransformProvider::UnregisterProvider(const FSkinningTransformProvider::FProviderId& Id)
{
	for (int32 ProviderIndex = 0; ProviderIndex < Providers.Num(); ++ProviderIndex)
	{
		const FTransformProvider& Provider = Providers[ProviderIndex];
		if (Provider.Id == Id)
		{
			Providers.RemoveAtSwap(ProviderIndex);
			return;
		}
	}

	checkNoEntry(); // No provider found with this id - error!
}

void FSkinningTransformProvider::Broadcast(const TConstArrayView<FProviderRange> Ranges, FProviderContext& Context)
{
	const TConstArrayView<FSkinningTransformProvider::FProviderIndirection> IndirectionView = Context.Indirections;

	for (const FTransformProvider& Provider : Providers)
	{
		for (const FProviderRange& Range : Ranges)
		{
			if (Provider.Id == Range.Id)
			{
				if (Range.Count > 0)
				{
					Context.Indirections = MakeArrayView(IndirectionView.GetData() + Range.Offset, Range.Count);
					Provider.Delegate.ExecuteIfBound(Context);
				}
				break;
			}
		}
	}
}

const FSkinningTransformProvider::FProviderId& GetRefPoseProviderId()
{
	// TODO: Temp until skinning scene extension is refactored into a public API outside of Nanite
	return FSkinningSceneExtension::GetRefPoseProviderId();
}

const FSkinningTransformProvider::FProviderId& GetMeshObjectProviderId()
{
	// TODO: Temp until skinning scene extension is refactored into a public API outside of Nanite
	return FSkinningSceneExtension::GetMeshObjectProviderId();
}

FRDGBufferDesc GetCompressedBoneTransformBufferDesc(int32 NumTransforms)
{
#if USE_COMPRESSED_BONE_TRANSFORM
	return FRDGBufferDesc::CreateByteAddressDesc(sizeof(FCompressedBoneTransform) * NumTransforms);
#else
	return FRDGBufferDesc::CreateBufferDesc(16, (NumTransforms * sizeof(FMatrix3x4)) / 16);
#endif
}

FRDGBufferRef GetDefaultCompressedBoneTransformBuffer(FRDGBuilder& GraphBuilder)
{
#if USE_COMPRESSED_BONE_TRANSFORM
	return GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, 8u);
#else
	return GSystemTextures.GetDefaultBuffer(GraphBuilder, 16u);
#endif
}

FRDGBufferSRVRef GetCompressedBoneTransformSRV(FRDGBuilder& GraphBuilder, FRDGBuffer* Buffer)
{
	return GraphBuilder.CreateSRV(Buffer, COMPRESSED_BONE_TRANSFORM_PIXEL_FORMAT);
}

FRDGBufferUAVRef GetCompressedBoneTransformUAV(FRDGBuilder& GraphBuilder, FRDGBuffer* Buffer)
{
	return GraphBuilder.CreateUAV(Buffer, COMPRESSED_BONE_TRANSFORM_PIXEL_FORMAT);
}