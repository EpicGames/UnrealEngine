// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SpanAllocator.h"
#include "Containers/Map.h"
#include "SceneExtensions.h"
#include "RendererPrivateUtils.h"
#include "Matrix3x4.h"
#include "Delegates/DelegateCombinations.h"
#include "Delegates/Delegate.h"
#include "GameTime.h"

#define ENABLE_SKELETON_DEBUG_NAME (UE_BUILD_DEBUG | UE_BUILD_DEVELOPMENT)

struct FSkeletonBatch
{
#if ENABLE_SKELETON_DEBUG_NAME
	FName SkeletonName = "Invalid";
#endif
	FGuid SkeletonGuid;
	uint32 MaxBoneTransforms = 0u;
	uint32 UniqueAnimationCount = 0u;
};

struct FSkeletonBatchKey
{
#if ENABLE_SKELETON_DEBUG_NAME
	FName SkeletonName = "Invalid";
#endif
	FGuid SkeletonGuid;
	FGuid TransformProviderId;
	bool bNanite = false;

	bool operator == (const FSkeletonBatchKey& InOther) const
	{
		return SkeletonGuid == InOther.SkeletonGuid && TransformProviderId == InOther.TransformProviderId && bNanite == InOther.bNanite;
	}

	bool operator != (const FSkeletonBatchKey& InOther) const
	{
		return !(*this == InOther);
	}

	friend inline SIZE_T GetTypeHash(const FSkeletonBatchKey& InKey)
	{
		return HashCombine(HashCombine(GetTypeHash(InKey.SkeletonGuid), GetTypeHash(InKey.TransformProviderId)), GetTypeHash(InKey.bNanite));
	}
};

enum class EDirtyBoneTransforms : uint8
{
	None     = 0,
	Current  = 1 << 0,
	Previous = 1 << 1,
	All      = Current | Previous
};
ENUM_CLASS_FLAGS(EDirtyBoneTransforms);

class FSkinningSceneExtensionProxy;
class FRDGScatterUploadBuilder;

class FSkinningTransformProvider : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FSkinningTransformProvider);

public:
	typedef FGuid FProviderId;

	struct FProviderRange
	{
		FProviderId Id;
		uint32 Count;
		uint32 Offset;
	};

	struct FProviderIndirection
	{
		FProviderIndirection(uint32 InIndex, uint32 InCurrentTransformOffset, uint32 InPreviousTransformOffset, uint32 InBoneMapOffset, EDirtyBoneTransforms InDirtyBoneTransforms)
			: Index(InIndex)
			, CurrentTransformOffset(InCurrentTransformOffset)
			, PreviousTransformOffset(InPreviousTransformOffset)
			, BoneMapOffset(InBoneMapOffset)
			, DirtyBoneTransforms(InDirtyBoneTransforms)
		{}

		uint32 Index = 0;
		uint32 CurrentTransformOffset = 0;
		uint32 PreviousTransformOffset = 0;
		uint32 BoneMapOffset = 0;
		EDirtyBoneTransforms DirtyBoneTransforms = EDirtyBoneTransforms::None;
	};

	struct FProviderContext
	{
		FProviderContext(
			const TConstArrayView<FPrimitiveSceneInfo*> InPrimitives,
			const TConstArrayView<FSkinningSceneExtensionProxy*> InProxies,
			const TConstArrayView<FProviderIndirection> InIndirections,
			const TConstArrayView<FSkeletonBatch> InSkeletonBatches,
			const FGameTime& InCurrentTime,
			FRDGBuilder& InGraphBuilder,
			FRDGScatterUploadBuilder& InScatterUploadBuilder,
			FRDGBufferRef InTransformBuffer,
			FRDGBufferSRVRef InBoneMapBufferSRV
		)
		: Primitives(InPrimitives)
		, Proxies(InProxies)
		, Indirections(InIndirections)
		, SkeletonBatches(InSkeletonBatches)
		, CurrentTime(InCurrentTime)
		, GraphBuilder(InGraphBuilder)
		, ScatterUploadBuilder(InScatterUploadBuilder)
		, TransformBuffer(InTransformBuffer)
		, BoneMapBufferSRV(InBoneMapBufferSRV)
		{
		}

		TConstArrayView<FPrimitiveSceneInfo*> Primitives;
		TConstArrayView<FSkinningSceneExtensionProxy*> Proxies;
		TConstArrayView<FProviderIndirection> Indirections;
		TConstArrayView<FSkeletonBatch> SkeletonBatches;

		const FGameTime& CurrentTime;
		FRDGBuilder& GraphBuilder;
		FRDGScatterUploadBuilder& ScatterUploadBuilder;
		FRDGBufferRef TransformBuffer;
		FRDGBufferSRVRef BoneMapBufferSRV;
	};

	DECLARE_DELEGATE_OneParam(FOnProvideTransforms, FProviderContext&);

public:
	using ISceneExtension::ISceneExtension;

	static bool ShouldCreateExtension(FScene& InScene);

	RENDERER_API void RegisterProvider(const FProviderId& Id, const FOnProvideTransforms& Delegate, bool bUsesSkeletonBatches);
	RENDERER_API void UnregisterProvider(const FProviderId& Id);

	void Broadcast(const TConstArrayView<FProviderRange> Ranges, FProviderContext& Context);

	inline bool HasProviders() const
	{
		return !Providers.IsEmpty();
	}

	inline TArray<FProviderId> GetProviderIds() const
	{
		TArray<FProviderId> Ids;
		Ids.Reserve(Providers.Num());
		for (const FTransformProvider& Provider : Providers)
		{
			Ids.Add(Provider.Id);
		}
		return Ids;
	}

	inline TArray<FProviderId> GetPrimitiveProviderIds() const
	{
		TArray<FProviderId> Ids;
		Ids.Reserve(Providers.Num());
		for (const FTransformProvider& Provider : Providers)
		{
			if (!Provider.bUsesSkeletonBatches)
			{
				Ids.Add(Provider.Id);
			}
		}
		return Ids;
	}

	inline TArray<FProviderId> GetSkeletonProviderIds() const
	{
		TArray<FProviderId> Ids;
		Ids.Reserve(Providers.Num());
		for (const FTransformProvider& Provider : Providers)
		{
			if (Provider.bUsesSkeletonBatches)
			{
				Ids.Add(Provider.Id);
			}
		}
		return Ids;
	}

private:
	struct FTransformProvider
	{
		FProviderId Id;
		FOnProvideTransforms Delegate;
		uint8 bUsesSkeletonBatches : 1 = false;
	};

	TArray<FTransformProvider> Providers;
};

RENDERER_API const FSkinningTransformProvider::FProviderId& GetRefPoseProviderId();
RENDERER_API const FSkinningTransformProvider::FProviderId& GetMeshObjectProviderId();

RENDERER_API FRDGBufferDesc GetCompressedBoneTransformBufferDesc(int32 NumTransforms);
RENDERER_API FRDGBufferRef GetDefaultCompressedBoneTransformBuffer(FRDGBuilder& GraphBuilder);
RENDERER_API FRDGBufferSRVRef GetCompressedBoneTransformSRV(FRDGBuilder& GraphBuilder, FRDGBuffer* Buffer);
RENDERER_API FRDGBufferUAVRef GetCompressedBoneTransformUAV(FRDGBuilder& GraphBuilder, FRDGBuffer* Buffer);