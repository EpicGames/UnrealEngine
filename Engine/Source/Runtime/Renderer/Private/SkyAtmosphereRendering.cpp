// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyAtmosphereRendering.cpp
=============================================================================*/

#include "SkyAtmosphereRendering.h"
#include "CoreMinimal.h"
#include "BlueNoise.h"
#include "CanvasTypes.h"
#include "CommonRenderResources.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DistanceFieldLightingShared.h"
#include "DynamicRHI.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "ExtensibleUniformBuffer.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Math/RandomStream.h"
#include "MeshMaterialShader.h"
#include "Misc/LargeWorldCoordinates.h"
#include "PipelineStateCache.h"
#include "PixelShaderUtils.h"
#include "PostProcess/PostProcessing.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ReadOnlyCVARCache.h"
#include "RenderCommandTag.h"
#include "RendererInterface.h"

#include "RendererUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/SkyAtmosphereCommonData.h"
#include "RenderingThread.h"
#include "RenderResource.h"
#include "RenderTargetPool.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHIBufferInitializer.h"
#include "RHICommandList.h"
#include "RHIPipeline.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "RHIUtilities.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyAtmosphereSceneProxy.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneRendering.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneTypes.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "SystemTextures.h"
#include "TextureFallbacks.h"
#include "UnrealEngine.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VirtualShadowMaps/VirtualShadowMapClipmap.h"
#include "VolumeLighting.h"
#include "VolumetricCloudRendering.h"
#include "SceneViewState.h"

DEFINE_LOG_CATEGORY(LogSkyAtmosphereRendering)


//PRAGMA_DISABLE_OPTIMIZATION


// The runtime ON/OFF toggle
static TAutoConsoleVariable<int32> CVarSkyAtmosphere(
	TEXT("r.SkyAtmosphere"), 1,
	TEXT("SkyAtmosphere components are rendered when this is not 0, otherwise ignored.\n"),
	ECVF_RenderThreadSafe);

// The project setting (disable runtime and shader code)
static TAutoConsoleVariable<int32> CVarSupportSkyAtmosphere(
	TEXT("r.SupportSkyAtmosphere"),
	1,
	TEXT("Enables SkyAtmosphere rendering and shader code."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

// The project setting for the sky atmosphere component to affect the height fog (disable runtime and shader code)
static TAutoConsoleVariable<int32> CVarSupportSkyAtmosphereAffectsHeightFog(
	TEXT("r.SupportSkyAtmosphereAffectsHeightFog"),
	1,
	TEXT("Enables SkyAtmosphere affecting height fog. It requires r.SupportSkyAtmosphere to be true."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarSkyAtmosphereStateVersioning(
	TEXT("r.SkyAtmosphere.StateVersioning"), true,
	TEXT("SkyAtmosphere will only update the transmittance and multiple scattering LUT when its state changes in order to save some GPU time.\n"),
	ECVF_RenderThreadSafe);

////////////////////////////////////////////////////////////////////////// Regular sky 

static TAutoConsoleVariable<float> CVarSkyAtmosphereSampleCountMin(
	TEXT("r.SkyAtmosphere.SampleCountMin"), 2.0f,
	TEXT("The minimum sample count used to compute sky/atmosphere scattering and transmittance.\n")
	TEXT("The minimal value will be clamped to 1.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereSampleCountMax(
	TEXT("r.SkyAtmosphere.SampleCountMax"), 32.0f,
	TEXT("The maximum sample count used to compute sky/atmosphere scattering and transmittance The effective sample count is usually lower and depends on distance and SampleCountScale on the component, as well as .ini files.\n")
	TEXT("The minimal value will be clamped to r.SkyAtmosphere.SampleCountMin + 1.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereDistanceToSampleCountMax(
	TEXT("r.SkyAtmosphere.DistanceToSampleCountMax"), 150.0f,
	TEXT("The distance in kilometer after which SampleCountMax samples will be used to ray march the atmosphere."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarSkyAtmosphereSampleLightShadowmap(
	TEXT("r.SkyAtmosphere.SampleLightShadowmap"), 1,
	TEXT("Enable the sampling of atmospheric lights shadow map in order to produce volumetric shadows."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

////////////////////////////////////////////////////////////////////////// Fast sky

static TAutoConsoleVariable<int32> CVarSkyAtmosphereFastSkyLUT(
	TEXT("r.SkyAtmosphere.FastSkyLUT"), 1,
	TEXT("When enabled, a look up texture is used to render the sky.\n")
	TEXT("It is faster but can result in visual artefacts if there are some high frequency details\n")
	TEXT("in the sky such as earth shadow or scattering lob."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereFastSkyLUTSampleCountMin(
	TEXT("r.SkyAtmosphere.FastSkyLUT.SampleCountMin"), 4.0f,
	TEXT("Fast sky minimum sample count used to compute sky/atmosphere scattering and transmittance.\n")
	TEXT("The minimal value will be clamped to 1.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereFastSkyLUTSampleCountMax(
	TEXT("r.SkyAtmosphere.FastSkyLUT.SampleCountMax"), 32.0f,
	TEXT("Fast sky maximum sample count used to compute sky/atmosphere scattering and transmittance.\n")
	TEXT("The maximum sample count used to compute FastSkyLUT scattering. The effective sample count is usually lower and depends on distance and SampleCountScale on the component, as well as .ini files.\n")
	TEXT("The minimal value will be clamped to r.SkyAtmosphere.FastSkyLUT.SampleCountMin + 1.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereFastSkyLUTDistanceToSampleCountMax(
	TEXT("r.SkyAtmosphere.FastSkyLUT.DistanceToSampleCountMax"), 150.0f,
	TEXT("Fast sky distance in kilometer after which at which SampleCountMax samples will be used to ray march the atmosphere."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereFastSkyLUTWidth(
	TEXT("r.SkyAtmosphere.FastSkyLUT.Width"), 192,
	TEXT(""),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereFastSkyLUTHeight(
	TEXT("r.SkyAtmosphere.FastSkyLUT.Height"), 104,
	TEXT(""),
	ECVF_RenderThreadSafe | ECVF_Scalability);

////////////////////////////////////////////////////////////////////////// Aerial perspective

static TAutoConsoleVariable<int32> CVarSkyAtmosphereAerialPerspectiveDepthTest(
	TEXT("r.SkyAtmosphere.AerialPerspective.DepthTest"), 1,
	TEXT("When enabled, a depth test will be used to not write pixel closer to the camera than StartDepth, effectively improving performance."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

////////////////////////////////////////////////////////////////////////// Aerial perspective LUT

static TAutoConsoleVariable<float> CVarSkyAtmosphereAerialPerspectiveLUTDepthResolution(
	TEXT("r.SkyAtmosphere.AerialPerspectiveLUT.DepthResolution"), 16.0f,
	TEXT("The number of depth slice to use for the aerial perspective volume texture."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereAerialPerspectiveLUTDepth(
	TEXT("r.SkyAtmosphere.AerialPerspectiveLUT.Depth"), 96.0f,
	TEXT("The length of the LUT in kilometers (default = 96km to get nice cloud/atmosphere interactions in the distance for default sky). Further than this distance, the last slice is used."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereAerialPerspectiveLUTSampleCountMaxPerSlice(
	TEXT("r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice"), 2.0f,
	TEXT("The sample count used per slice to evaluate aerial perspective. The effective sample count is usually lower and depends on SampleCountScale on the component as well as .ini files.\n")
	TEXT("scattering and transmittance in camera frustum space froxel.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereAerialPerspectiveLUTWidth(
	TEXT("r.SkyAtmosphere.AerialPerspectiveLUT.Width"), 32,
	TEXT(""),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarSkyAtmosphereAerialPerspectiveApplyOnOpaque(
	TEXT("r.SkyAtmosphere.AerialPerspectiveLUT.FastApplyOnOpaque"), 1,
	TEXT("When enabled, the low resolution camera frustum/froxel volume containing atmospheric fog\n")
	TEXT(", usually used for fog on translucent surface, is used to render fog on opaque.\n")
	TEXT("It is faster but can result in visual artefacts if there are some high frequency details\n")
	TEXT("such as earth shadow or scattering lob."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

////////////////////////////////////////////////////////////////////////// Transmittance LUT

static TAutoConsoleVariable<int32> CVarSkyAtmosphereTransmittanceLUT(
	TEXT("r.SkyAtmosphere.TransmittanceLUT"), 1,
	TEXT("Enable the generation of the sky transmittance.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereTransmittanceLUTSampleCount(
	TEXT("r.SkyAtmosphere.TransmittanceLUT.SampleCount"), 10.0f,
	TEXT("The sample count used to evaluate transmittance."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarSkyAtmosphereTransmittanceLUTUseSmallFormat(
	TEXT("r.SkyAtmosphere.TransmittanceLUT.UseSmallFormat"), 0,
	TEXT("If true, the transmittance LUT will use a small R8BG8B8A8 format to store data at lower quality."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereTransmittanceLUTWidth(
	TEXT("r.SkyAtmosphere.TransmittanceLUT.Width"), 256,
	TEXT(""),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereTransmittanceLUTHeight(
	TEXT("r.SkyAtmosphere.TransmittanceLUT.Height"), 64,
	TEXT(""),
	ECVF_RenderThreadSafe | ECVF_Scalability);

////////////////////////////////////////////////////////////////////////// Multi-scattering LUT

static TAutoConsoleVariable<float> CVarSkyAtmosphereMultiScatteringLUTSampleCount(
	TEXT("r.SkyAtmosphere.MultiScatteringLUT.SampleCount"), 15.0f,
	TEXT("The sample count used to evaluate multi-scattering.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereMultiScatteringLUTHighQuality(
	TEXT("r.SkyAtmosphere.MultiScatteringLUT.HighQuality"), 0.0f,
	TEXT("The when enabled, 64 samples are used instead of 2, resulting in a more accurate multi scattering approximation (but also more expenssive).\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereMultiScatteringLUTWidth(
	TEXT("r.SkyAtmosphere.MultiScatteringLUT.Width"), 32,
	TEXT(""),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereMultiScatteringLUTHeight(
	TEXT("r.SkyAtmosphere.MultiScatteringLUT.Height"), 32,
	TEXT(""),
	ECVF_RenderThreadSafe | ECVF_Scalability);

////////////////////////////////////////////////////////////////////////// Distant Sky Light LUT

static TAutoConsoleVariable<int32> CVarSkyAtmosphereDistantSkyLightLUT(
	TEXT("r.SkyAtmosphere.DistantSkyLightLUT"), 1,
	TEXT("Enable the generation the sky ambient lighting value.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarSkyAtmosphereDistantSkyLightLUTAltitude(
	TEXT("r.SkyAtmosphere.DistantSkyLightLUT.Altitude"), 6.0f,
	TEXT("The altitude at which the sky samples are taken to integrate the sky lighting. Default to 6km, typicaly cirrus clouds altitude.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

////////////////////////////////////////////////////////////////////////// Debug / Visualization

static TAutoConsoleVariable<int32> CVarSkyAtmosphereLUT32(
	TEXT("r.SkyAtmosphere.LUT32"), 0,
	TEXT("Use full 32bit per-channel precision for all sky LUTs.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarSkyAtmosphereEditorNotifications(
	TEXT("r.SkyAtmosphere.EditorNotifications"), 1,
	TEXT("Enable the rendering of in editor notification to warn the user about missing sky dome pixels on screen. It is better to keep it enabled and will be removed when shipping.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarSkyAtmosphereAsyncCompute(
	TEXT("r.SkyAtmosphereAsyncCompute"), 0,
	TEXT("SkyAtmosphere on async compute (default: false). When running on the async pipe, SkyAtmosphere LUT generation will overlap with the occlusion pass.\n When enabled, opaque shadow map sampling on atmosphere will be disabled to avoid race conditions because these textures can be created in an overlaping parallel thread."),
	ECVF_RenderThreadSafe);

DECLARE_GPU_STAT(SkyAtmosphereLUTs);
DECLARE_GPU_STAT(SkyAtmosphere);
DECLARE_GPU_STAT(SkyAtmosphereEditor);
DECLARE_GPU_STAT(SkyAtmosphereDebugVisualize);

// Extra internal constants shared between all passes. It is used to render the sky itself (not shared with material)
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSkyAtmosphereInternalCommonParameters, )
	SHADER_PARAMETER(float, SampleCountMin)
	SHADER_PARAMETER(float, SampleCountMax)
	SHADER_PARAMETER(float, DistanceToSampleCountMaxInv)

	SHADER_PARAMETER(float, FastSkySampleCountMin)
	SHADER_PARAMETER(float, FastSkySampleCountMax)
	SHADER_PARAMETER(float, FastSkyDistanceToSampleCountMaxInv)

	SHADER_PARAMETER(FVector4f, CameraAerialPerspectiveVolumeSizeAndInvSize)
	SHADER_PARAMETER(float, CameraAerialPerspectiveVolumeDepthResolution)		// Also on View UB
	SHADER_PARAMETER(float, CameraAerialPerspectiveVolumeDepthResolutionInv)	// Also on View UB
	SHADER_PARAMETER(float, CameraAerialPerspectiveVolumeDepthSliceLengthKm)	// Also on View UB
	SHADER_PARAMETER(float, CameraAerialPerspectiveVolumeDepthSliceLengthKmInv)	// Also on View UB
	SHADER_PARAMETER(float, CameraAerialPerspectiveSampleCountPerSlice)

	SHADER_PARAMETER(FVector4f, TransmittanceLutSizeAndInvSize)
	SHADER_PARAMETER(FVector4f, MultiScatteredLuminanceLutSizeAndInvSize)
	SHADER_PARAMETER(FVector4f, SkyViewLutSizeAndInvSize)						// Also on View UB

	SHADER_PARAMETER(float, TransmittanceSampleCount)
	SHADER_PARAMETER(float, MultiScatteringSampleCount)
	SHADER_PARAMETER(float, AerialPespectiveViewDistanceScale)
	SHADER_PARAMETER(float, FogShowFlagFactor)

	SHADER_PARAMETER_STRUCT(FBlueNoise, BlueNoise)

	SHADER_PARAMETER(FVector3f, SkyLuminanceFactor)
	SHADER_PARAMETER(FVector3f, SkyAndAerialPerspectiveLuminanceFactor)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FAtmosphereUniformShaderParameters, "Atmosphere");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSkyAtmosphereInternalCommonParameters, "SkyAtmosphere");


BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSkyAtmosphereRealTimeReflectionLUTParameters, )
	SHADER_PARAMETER(FMatrix44f, SkyViewLutReferential)
	SHADER_PARAMETER(FVector4f, SkyPlanetTranslatedWorldCenterAndViewHeight)
	SHADER_PARAMETER(FVector3f, SkyCameraTranslatedWorldOrigin)
	SHADER_PARAMETER(float, RealTimeReflectionCapture)
END_GLOBAL_SHADER_PARAMETER_STRUCT()
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSkyAtmosphereRealTimeReflectionLUTParameters, "SkyAtmosphereRealTimeReflectionLUTParameters");

auto GetDefaultSkyAtmosphereRealTimeReflectionLUTUniformBuffer()
{
	FSkyAtmosphereRealTimeReflectionLUTParameters Parameters;
	Parameters.SkyViewLutReferential = FMatrix44f::Identity;
	Parameters.SkyPlanetTranslatedWorldCenterAndViewHeight = FVector4f::Zero();
	Parameters.SkyCameraTranslatedWorldOrigin = FVector3f::Zero();
	Parameters.RealTimeReflectionCapture = 0.0f;
	return TUniformBufferRef<FSkyAtmosphereRealTimeReflectionLUTParameters>::CreateUniformBufferImmediate(Parameters, UniformBuffer_SingleDraw);
}


#define GET_VALID_DATA_FROM_CVAR \
	auto ValidateLUTResolution = [](int32 Value) \
	{ \
		return Value < 4 ? 4 : Value; \
	}; \
	int32 TransmittanceLutWidth = ValidateLUTResolution(CVarSkyAtmosphereTransmittanceLUTWidth.GetValueOnRenderThread()); \
	int32 TransmittanceLutHeight = ValidateLUTResolution(CVarSkyAtmosphereTransmittanceLUTHeight.GetValueOnRenderThread()); \
	int32 MultiScatteredLuminanceLutWidth = ValidateLUTResolution(CVarSkyAtmosphereMultiScatteringLUTWidth.GetValueOnRenderThread()); \
	int32 MultiScatteredLuminanceLutHeight = ValidateLUTResolution(CVarSkyAtmosphereMultiScatteringLUTHeight.GetValueOnRenderThread()); \
	int32 SkyViewLutWidth = ValidateLUTResolution(CVarSkyAtmosphereFastSkyLUTWidth.GetValueOnRenderThread()); \
	int32 SkyViewLutHeight = ValidateLUTResolution(CVarSkyAtmosphereFastSkyLUTHeight.GetValueOnRenderThread()); \
	int32 CameraAerialPerspectiveVolumeScreenResolution = ValidateLUTResolution(CVarSkyAtmosphereAerialPerspectiveLUTWidth.GetValueOnRenderThread()); \
	int32 CameraAerialPerspectiveVolumeDepthResolution = ValidateLUTResolution(CVarSkyAtmosphereAerialPerspectiveLUTDepthResolution.GetValueOnRenderThread()); \
	float CameraAerialPerspectiveVolumeDepthKm = CVarSkyAtmosphereAerialPerspectiveLUTDepth.GetValueOnRenderThread(); \
	CameraAerialPerspectiveVolumeDepthKm = CameraAerialPerspectiveVolumeDepthKm < 1.0f ? 1.0f : CameraAerialPerspectiveVolumeDepthKm;	/* 1 kilometer minimum */ \
	float CameraAerialPerspectiveVolumeDepthSliceLengthKm = CameraAerialPerspectiveVolumeDepthKm / CameraAerialPerspectiveVolumeDepthResolution;

#define KM_TO_CM  100000.0f
#define CM_TO_KM  (1.0f / KM_TO_CM)

static bool GetSkyAtmosphereAsyncComputeEnabled()
{
	return CVarSkyAtmosphereAsyncCompute.GetValueOnRenderThread() != 0;
}

ESkyAtmospherePassLocation GetSkyAtmospherePassLocation()
{
	if (GetSkyAtmosphereAsyncComputeEnabled())
	{
		return ESkyAtmospherePassLocation::BeforePrePass;
	}
	// When SkyAtmLUT is running on graphics pipe, kick before BasePass to have a better overlap when DistanceFieldShadows is running async
	return ESkyAtmospherePassLocation::BeforeBasePass;
}

float GetValidAerialPerspectiveStartDepthInCm(const FViewInfo& View, const FSkyAtmosphereSceneProxy& SkyAtmosphereProxy)
{
	float AerialPerspectiveStartDepthKm = SkyAtmosphereProxy.GetAerialPerspectiveStartDepthKm();
	AerialPerspectiveStartDepthKm = AerialPerspectiveStartDepthKm < 0.0f ? 0.0f : AerialPerspectiveStartDepthKm;
	// For sky reflection capture, the start depth can be super large. So we max it to make sure the triangle is never in front the NearClippingDistance.
	const float StartDepthInCm = FMath::Max(AerialPerspectiveStartDepthKm * KM_TO_CM, View.NearClippingDistance);
	return StartDepthInCm;
}

static bool VirtualShadowMapSamplingSupported(EShaderPlatform ShaderPlatform)
{
	return GetMaxSupportedFeatureLevel(ShaderPlatform) >= ERHIFeatureLevel::SM5;
}


bool ShouldSkySampleAtmosphereLightsOpaqueShadow(const FScene& Scene, const FVisibleLightInfoArray& VisibleLightInfos, SkyAtmosphereLightShadowData& LightShadowData, FDynamicShadowsTaskData* FilterDynamicShadowTaskData)
{
	LightShadowData.LightVolumetricShadowSceneinfo0 = Scene.AtmosphereLights[0];
	LightShadowData.LightVolumetricShadowSceneinfo1 = Scene.AtmosphereLights[1];

	bool bCastShadowOnAtmosphereLight0 = LightShadowData.LightVolumetricShadowSceneinfo0 && LightShadowData.LightVolumetricShadowSceneinfo0->Proxy && LightShadowData.LightVolumetricShadowSceneinfo0->Proxy->GetCastShadowsOnAtmosphere();
	bool bCastShadowOnAtmosphereLight1 = LightShadowData.LightVolumetricShadowSceneinfo1 && LightShadowData.LightVolumetricShadowSceneinfo1->Proxy && LightShadowData.LightVolumetricShadowSceneinfo1->Proxy->GetCastShadowsOnAtmosphere();
	
	if (FilterDynamicShadowTaskData != nullptr && (bCastShadowOnAtmosphereLight0 || bCastShadowOnAtmosphereLight1))
	{
		// FilterDynamicShadows runs on a parallel task and writes = VisibleLightInfo.ShadowsToProject concurrently
		// Wait before accessing = VisibleLightInfo.ShadowsToProject in GetFirstWholeSceneShadowMap.
		extern void WaitForFilterDynamicShadowsTask(FDynamicShadowsTaskData* TaskData);
		WaitForFilterDynamicShadowsTask(FilterDynamicShadowTaskData);
	}
	
	if (bCastShadowOnAtmosphereLight0)
	{
		const FVisibleLightInfo& Light0 = VisibleLightInfos[LightShadowData.LightVolumetricShadowSceneinfo0->Id];
		LightShadowData.ProjectedShadowInfo0 = GetFirstWholeSceneShadowMap(Light0);	
		// NOTE: This will be an arbitrary clipmap if multiple exist, but this is similar to the CSM select above,
		// which also does not take multiple views into account.
		if (Light0.VirtualShadowMapClipmaps.Num() > 0)
		{
			LightShadowData.VirtualShadowMapId0 = Light0.VirtualShadowMapClipmaps[0]->GetVirtualShadowMapId();
		}
		else
		{
			LightShadowData.VirtualShadowMapId0 = INDEX_NONE;
		}
	}
	if (bCastShadowOnAtmosphereLight1)
	{
		const FVisibleLightInfo& Light1 = VisibleLightInfos[LightShadowData.LightVolumetricShadowSceneinfo1->Id];
		LightShadowData.ProjectedShadowInfo1 = GetFirstWholeSceneShadowMap(Light1);
		if (Light1.VirtualShadowMapClipmaps.Num() > 0)
		{
			LightShadowData.VirtualShadowMapId1 = Light1.VirtualShadowMapClipmaps[0]->GetVirtualShadowMapId();
		}
		else
		{
			LightShadowData.VirtualShadowMapId1 = INDEX_NONE;
		}
	}

	return CVarSkyAtmosphereSampleLightShadowmap.GetValueOnRenderThread() > 0 &&
		(LightShadowData.ProjectedShadowInfo0 || LightShadowData.ProjectedShadowInfo1 ||
		LightShadowData.VirtualShadowMapId0 != INDEX_NONE || LightShadowData.VirtualShadowMapId1 != INDEX_NONE);
}
void GetSkyAtmosphereLightsUniformBuffers(
	FRDGBuilder& GraphBuilder,
	TRDGUniformBufferRef<FVolumeShadowingShaderParametersGlobal0>& OutLightShadowShaderParams0UniformBuffer,
	TRDGUniformBufferRef<FVolumeShadowingShaderParametersGlobal1>& OutLightShadowShaderParams1UniformBuffer,
	const SkyAtmosphereLightShadowData& LightShadowData,
	const FViewInfo& ViewInfo,
	const bool bShouldSampleOpaqueShadow,
	const EUniformBufferUsage UniformBufferUsage
)
{
	FVolumeShadowingShaderParametersGlobal0& LightShadowShaderParams0 = *GraphBuilder.AllocParameters<FVolumeShadowingShaderParametersGlobal0>();
	FVolumeShadowingShaderParametersGlobal1& LightShadowShaderParams1 = *GraphBuilder.AllocParameters<FVolumeShadowingShaderParametersGlobal1>();

	bool bSampleOpaqueShadow = bShouldSampleOpaqueShadow;
	if (GetSkyAtmosphereAsyncComputeEnabled())
	{
		// When enabling async compute, we can run RenderSkyAtmosphereLookupTables in parallel with FilterDynamicShadows, which allocates the depth buffers.
		// So when running sky atmosphere LUT generation on the async compute pipe, we disable opaque shadow sampling.
		bSampleOpaqueShadow = false;
	}

	if (bSampleOpaqueShadow && LightShadowData.LightVolumetricShadowSceneinfo0)
	{
		SetVolumeShadowingShaderParameters(GraphBuilder, LightShadowShaderParams0, ViewInfo,
			LightShadowData.LightVolumetricShadowSceneinfo0, LightShadowData.ProjectedShadowInfo0);
	}
	else
	{
		SetVolumeShadowingDefaultShaderParameters(GraphBuilder, LightShadowShaderParams0);
	}
	if (bSampleOpaqueShadow && LightShadowData.LightVolumetricShadowSceneinfo1)
	{
		SetVolumeShadowingShaderParameters(GraphBuilder, LightShadowShaderParams1, ViewInfo,
			LightShadowData.LightVolumetricShadowSceneinfo1, LightShadowData.ProjectedShadowInfo1);
	}
	else
	{
		SetVolumeShadowingDefaultShaderParameters(GraphBuilder, LightShadowShaderParams1);
	}
	OutLightShadowShaderParams0UniformBuffer = GraphBuilder.CreateUniformBuffer(&LightShadowShaderParams0);
	OutLightShadowShaderParams1UniformBuffer = GraphBuilder.CreateUniformBuffer(&LightShadowShaderParams1);
}

bool ShouldRenderSkyAtmosphere(const FScene* Scene, const FEngineShowFlags& EngineShowFlags)
{
	if (Scene && Scene->HasSkyAtmosphere() && EngineShowFlags.Atmosphere)
	{
		EShaderPlatform ShaderPlatform = Scene->GetShaderPlatform();
		const FSkyAtmosphereRenderSceneInfo* SkyAtmosphere = Scene->GetSkyAtmosphereSceneInfo();
		check(SkyAtmosphere);

		return FReadOnlyCVARCache::SupportSkyAtmosphere() && CVarSkyAtmosphere.GetValueOnRenderThread() > 0;
	}
	return false;
}

void SkyAtmosphereOnRenderBegin(FScene& Scene, const FEngineShowFlags& EngineShowFlags)
{
	if (ShouldRenderSkyAtmosphere(&Scene, EngineShowFlags))
	{
		for (int32 LightIndex = 0; LightIndex < NUM_ATMOSPHERE_LIGHTS; ++LightIndex)
		{
			if (Scene.AtmosphereLights[LightIndex])
			{
				PrepareSunLightProxy(*Scene.GetSkyAtmosphereSceneInfo(), LightIndex, *Scene.AtmosphereLights[LightIndex]);
			}
		}
	}
	else
	{
		Scene.ResetAtmosphereLightsProperties();
	}
}

static auto GetSizeAndInvSize = [](int32 Width, int32 Height)
{
	float FWidth = float(Width);
	float FHeight = float(Height);
	return FVector4f(FWidth, FHeight, 1.0f / FWidth, 1.0f / FHeight);
};

static bool ShouldApplyAerialPerspective(EShaderPlatform ShaderPlatform)
{
	extern bool PlatformUsesBasicFogFeatures(EShaderPlatform ShaderPlatform);
	return PlatformUsesBasicFogFeatures(ShaderPlatform) == false;
}

void SetupSkyAtmosphereViewSharedUniformShaderParameters(const FViewInfo& View, const FSkyAtmosphereSceneProxy& SkyAtmosphereProxy, FSkyAtmosphereViewSharedUniformShaderParameters& OutParameters)
{
	GET_VALID_DATA_FROM_CVAR;

	FRHITexture* SkyAtmosphereCameraAerialPerspectiveVolume = nullptr;
	if (View.SkyAtmosphereCameraAerialPerspectiveVolume)
	{
		SkyAtmosphereCameraAerialPerspectiveVolume = View.SkyAtmosphereCameraAerialPerspectiveVolume->GetRHI();
	}

	OutParameters.CameraAerialPerspectiveVolumeSizeAndInvSize = GetSizeAndInvSize(CameraAerialPerspectiveVolumeScreenResolution, CameraAerialPerspectiveVolumeScreenResolution);
	OutParameters.ApplyCameraAerialPerspectiveVolume = (View.SkyAtmosphereCameraAerialPerspectiveVolume == nullptr) || !ShouldApplyAerialPerspective(View.GetShaderPlatform()) ? 0.0f : 1.0f;
	OutParameters.CameraAerialPerspectiveVolumeDepthResolution = float(CameraAerialPerspectiveVolumeDepthResolution);
	OutParameters.CameraAerialPerspectiveVolumeDepthResolutionInv = 1.0f / OutParameters.CameraAerialPerspectiveVolumeDepthResolution;
	OutParameters.CameraAerialPerspectiveVolumeDepthSliceLengthKm = CameraAerialPerspectiveVolumeDepthSliceLengthKm;
	OutParameters.CameraAerialPerspectiveVolumeDepthSliceLengthKmInv = 1.0f / OutParameters.CameraAerialPerspectiveVolumeDepthSliceLengthKm;

	OutParameters.AerialPerspectiveStartDepthKm = GetValidAerialPerspectiveStartDepthInCm(View, SkyAtmosphereProxy) * CM_TO_KM;

	SetBlackAlpha13DIfNull(SkyAtmosphereCameraAerialPerspectiveVolume); // Needs to be after we set ApplyCameraAerialPerspectiveVolume
}

static void CopyAtmosphereSetupToUniformShaderParameters(FAtmosphereUniformShaderParameters& out, const FAtmosphereSetup& Atmosphere)
{
#define COPYMACRO(MemberName) out.MemberName = Atmosphere.MemberName 
	COPYMACRO(MultiScatteringFactor);
	COPYMACRO(BottomRadiusKm);
	COPYMACRO(TopRadiusKm);
	COPYMACRO(RayleighDensityExpScale);
	COPYMACRO(RayleighScattering);
	COPYMACRO(MieScattering);
	COPYMACRO(MieDensityExpScale);
	COPYMACRO(MieExtinction);
	COPYMACRO(MiePhaseG);
	COPYMACRO(MieAbsorption);
	COPYMACRO(AbsorptionDensity0LayerWidth);
	COPYMACRO(AbsorptionDensity0ConstantTerm);
	COPYMACRO(AbsorptionDensity0LinearTerm);
	COPYMACRO(AbsorptionDensity1ConstantTerm);
	COPYMACRO(AbsorptionDensity1LinearTerm);
	COPYMACRO(AbsorptionExtinction);
	COPYMACRO(GroundAlbedo);
#undef COPYMACRO
}

static FLinearColor GetLightDiskLuminance(FLightSceneInfo& Light)
{
	
	const float SunSolidAngle = 2.0f * PI * (1.0f - FMath::Cos(Light.Proxy->GetSunLightHalfApexAngleRadian()));			// Solid angle from aperture https://en.wikipedia.org/wiki/Solid_angle 
	return  Light.Proxy->GetAtmosphereSunDiskColorScale() * Light.Proxy->GetOuterSpaceIlluminance() / SunSolidAngle;	// approximation
}

void PrepareSunLightProxy(const FSkyAtmosphereRenderSceneInfo& SkyAtmosphere, uint32 AtmosphereLightIndex, FLightSceneInfo& AtmosphereLight)
{
	// See explanation in "Physically Based Sky, Atmosphere	and Cloud Rendering in Frostbite" page 26
	const FSkyAtmosphereSceneProxy& SkyAtmosphereProxy = SkyAtmosphere.GetSkyAtmosphereSceneProxy();
	const FVector AtmosphereLightDirection = SkyAtmosphereProxy.GetAtmosphereLightDirection(AtmosphereLightIndex, -AtmosphereLight.Proxy->GetDirection());
	const FLinearColor TransmittanceTowardSun = SkyAtmosphereProxy.GetAtmosphereSetup().GetTransmittanceAtGroundLevel(AtmosphereLightDirection);

	const FLinearColor SunDiskOuterSpaceLuminance = GetLightDiskLuminance(AtmosphereLight);

	AtmosphereLight.Proxy->SetAtmosphereRelatedProperties(TransmittanceTowardSun, SunDiskOuterSpaceLuminance);
}

bool IsLightAtmospherePerPixelTransmittanceEnabled(const FScene* Scene, const FViewInfo& View, const FLightSceneInfo* const LightSceneInfo)
{
	if (Scene 
		&& LightSceneInfo 
		&& LightSceneInfo->Proxy->GetLightType() == LightType_Directional 
		&& ShouldRenderSkyAtmosphere(LightSceneInfo->Scene, View.Family->EngineShowFlags))
	{
		FLightSceneProxy* AtmosphereLight0Proxy = Scene->AtmosphereLights[0] ? Scene->AtmosphereLights[0]->Proxy : nullptr;
		FLightSceneProxy* AtmosphereLight1Proxy = Scene->AtmosphereLights[1] ? Scene->AtmosphereLights[1]->Proxy : nullptr;

		// The light must be one of the atmospheric directional lights and has per pixel atmosphere transmittance enabled.
		return (AtmosphereLight0Proxy == LightSceneInfo->Proxy && AtmosphereLight0Proxy && AtmosphereLight0Proxy->GetUsePerPixelAtmosphereTransmittance())
			|| (AtmosphereLight1Proxy == LightSceneInfo->Proxy && AtmosphereLight1Proxy && AtmosphereLight1Proxy->GetUsePerPixelAtmosphereTransmittance());
	}
	return false;
}



/*=============================================================================
	FSkyAtmosphereRenderSceneInfo implementation.
=============================================================================*/


FSkyAtmosphereRenderSceneInfo::FSkyAtmosphereRenderSceneInfo(FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxyIn)
	:SkyAtmosphereSceneProxy(SkyAtmosphereSceneProxyIn)
{
	CopyAtmosphereSetupToUniformShaderParameters(AtmosphereUniformShaderParameters, SkyAtmosphereSceneProxy.GetAtmosphereSetup());

	// FSkyAtmosphereRenderSceneInfo must be created on the render thread to be able to create a multiframe uniform buffer. (see FScene::AddSkyAtmosphere)
	check(IsInRenderingThread());
	AtmosphereUniformBuffer = TUniformBufferRef<FAtmosphereUniformShaderParameters>::CreateUniformBufferImmediate(AtmosphereUniformShaderParameters, UniformBuffer_MultiFrame);
}

FSkyAtmosphereRenderSceneInfo::~FSkyAtmosphereRenderSceneInfo()
{
}

void FSkyAtmosphereRenderSceneInfo::CreateDistantSkyLightLutBufferAndSRV(FRDGBuilder& GraphBuilder)
{
	DistantSkyLightLutBuffer = AllocatePooledBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FVector4f), 1), TEXT("SkyAtmosphere.DistantSkyLightLutBuffer"));
	DistantSkyLightLutBufferSRV = DistantSkyLightLutBuffer->GetOrCreateSRV(GraphBuilder.RHICmdList, FRHIBufferSRVCreateInfo(PF_A32B32G32R32F));

	MobileDistantSkyLightLutBuffer = AllocatePooledBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(FVector4f), 1), TEXT("SkyAtmosphere.MobileDistantSkyLightLutBuffer"));
	MobileDistantSkyLightLutBufferSRV = MobileDistantSkyLightLutBuffer->GetOrCreateSRV(GraphBuilder.RHICmdList, FRHIBufferSRVCreateInfo(PF_A32B32G32R32F));
}

TRefCountPtr<FRDGPooledBuffer>& FSkyAtmosphereRenderSceneInfo::GetDistantSkyLightLutBuffer()
{
	check(CVarSkyAtmosphereDistantSkyLightLUT.GetValueOnRenderThread() > 0);
	return DistantSkyLightLutBuffer;
}

TRefCountPtr<FRDGPooledBuffer>& FSkyAtmosphereRenderSceneInfo::GetMobileDistantSkyLightLutBuffer()
{
	check(CVarSkyAtmosphereDistantSkyLightLUT.GetValueOnRenderThread() > 0);
	return MobileDistantSkyLightLutBuffer;
}

FRHIShaderResourceView* FSkyAtmosphereRenderSceneInfo::GetDistantSkyLightLutBufferSRV()
{
	if (CVarSkyAtmosphereDistantSkyLightLUT.GetValueOnRenderThread() > 0)
	{
		return DistantSkyLightLutBufferSRV;
	}
	return GBlackFloat4StructuredBufferWithSRV->ShaderResourceViewRHI;
}

FRHIShaderResourceView* FSkyAtmosphereRenderSceneInfo::GetMobileDistantSkyLightLutBufferSRV()
{
	if (CVarSkyAtmosphereDistantSkyLightLUT.GetValueOnRenderThread() > 0)
	{
		return MobileDistantSkyLightLutBufferSRV;
	}
	return GBlackFloat4VertexBufferWithSRV->ShaderResourceViewRHI;
}



/*=============================================================================
	FScene functions
=============================================================================*/



void FScene::AddSkyAtmosphere(FSkyAtmosphereSceneProxy* SkyAtmosphereSceneProxy, bool bStaticLightingBuilt)
{
	check(SkyAtmosphereSceneProxy);
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FAddSkyAtmosphereCommand)(
		[Scene, SkyAtmosphereSceneProxy, bStaticLightingBuilt](FRHICommandListImmediate& RHICmdList)
		{
			check(!Scene->SkyAtmosphereStack.Contains(SkyAtmosphereSceneProxy));
			Scene->SkyAtmosphereStack.Push(SkyAtmosphereSceneProxy);

			SkyAtmosphereSceneProxy->RenderSceneInfo = new FSkyAtmosphereRenderSceneInfo(*SkyAtmosphereSceneProxy);

			// Use the most recently enabled SkyAtmosphere
			Scene->SkyAtmosphere = SkyAtmosphereSceneProxy->RenderSceneInfo;
			SkyAtmosphereSceneProxy->bStaticLightingBuilt = bStaticLightingBuilt;
			if (!SkyAtmosphereSceneProxy->bStaticLightingBuilt)
			{
				FPlatformAtomics::InterlockedIncrement(&Scene->NumUncachedStaticLightingInteractions);
			}
			Scene->InvalidatePathTracedOutput();
		} );
}

void FScene::RemoveSkyAtmosphere(FSkyAtmosphereSceneProxy* SkyAtmosphereSceneProxy)
{
	check(SkyAtmosphereSceneProxy);
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FRemoveSkyAtmosphereCommand)(
		[Scene, SkyAtmosphereSceneProxy](FRHICommandListImmediate& RHICmdList)
		{
			if (!SkyAtmosphereSceneProxy->bStaticLightingBuilt)
			{
				FPlatformAtomics::InterlockedDecrement(&Scene->NumUncachedStaticLightingInteractions);
			}

			delete SkyAtmosphereSceneProxy->RenderSceneInfo;
			SkyAtmosphereSceneProxy->RenderSceneInfo = nullptr;
			Scene->SkyAtmosphereStack.RemoveSingle(SkyAtmosphereSceneProxy);

			if (Scene->SkyAtmosphereStack.Num() > 0)
			{
				// Use the most recently enabled SkyAtmosphere
				Scene->SkyAtmosphere = Scene->SkyAtmosphereStack.Last()->RenderSceneInfo;
			}
			else
			{
				Scene->SkyAtmosphere = nullptr;
			}
			Scene->InvalidatePathTracedOutput();
		} );
}

void FScene::ResetAtmosphereLightsProperties()
{
	// Also rest the current atmospheric light to default atmosphere
	for (int32 LightIndex = 0; LightIndex < NUM_ATMOSPHERE_LIGHTS; ++LightIndex)
	{
		FLightSceneInfo* Light = AtmosphereLights[LightIndex];
		if (Light)
		{
			const FLinearColor TransmittanceTowardSun = FLinearColor::White;
			Light->Proxy->SetAtmosphereRelatedProperties(TransmittanceTowardSun, GetLightDiskLuminance(*Light));
		}
	}
}



/*=============================================================================
	Sky/Atmosphere rendering functions
=============================================================================*/



namespace
{

class FHighQualityMultiScatteringApprox : SHADER_PERMUTATION_BOOL("HIGHQUALITY_MULTISCATTERING_APPROX_ENABLED");
class FFastSky : SHADER_PERMUTATION_BOOL("FASTSKY_ENABLED");
class FFastAerialPespective : SHADER_PERMUTATION_BOOL("FASTAERIALPERSPECTIVE_ENABLED");
class FSecondAtmosphereLight : SHADER_PERMUTATION_BOOL("SECOND_ATMOSPHERE_LIGHT_ENABLED");
class FRenderSky : SHADER_PERMUTATION_BOOL("RENDERSKY_ENABLED");
class FSampleOpaqueShadow : SHADER_PERMUTATION_BOOL("SAMPLE_OPAQUE_SHADOW");
class FSampleCloudShadow : SHADER_PERMUTATION_BOOL("SAMPLE_CLOUD_SHADOW");
class FSampleCloudSkyAO : SHADER_PERMUTATION_BOOL("SAMPLE_CLOUD_SKYAO");
class FAtmosphereOnClouds : SHADER_PERMUTATION_BOOL("SAMPLE_ATMOSPHERE_ON_CLOUDS");
class FMSAASampleCount : SHADER_PERMUTATION_SPARSE_INT("MSAA_SAMPLE_COUNT", 1, 2, 4, 8);
class FSeparateMieAndRayleighScattering : SHADER_PERMUTATION_BOOL("SEPARATE_MIE_RAYLEIGH_SCATTERING");
class FRealTimeReflectionLUT : SHADER_PERMUTATION_BOOL("PERMUTATION_REALTIME_REFLECTION_LUT");

}

//////////////////////////////////////////////////////////////////////////

class FRenderSkyAtmosphereVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderSkyAtmosphereVS);
	SHADER_USE_PARAMETER_STRUCT(FRenderSkyAtmosphereVS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, StartDepthZ)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Mobile must use a skydome mesh with a sky material to achieve good GPU performance
		return !IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRenderSkyAtmosphereVS, "/Engine/Private/SkyAtmosphere.usf", "SkyAtmosphereVS", SF_Vertex);

class FRenderSkyAtmospherePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderSkyAtmospherePS);
	SHADER_USE_PARAMETER_STRUCT(FRenderSkyAtmospherePS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSampleCloudSkyAO, FFastSky, FFastAerialPespective, FSecondAtmosphereLight, FRenderSky, FSampleOpaqueShadow, FSampleCloudShadow, FAtmosphereOnClouds, FMSAASampleCount>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FAtmosphereUniformShaderParameters, Atmosphere)
		SHADER_PARAMETER_STRUCT_REF(FSkyAtmosphereInternalCommonParameters, SkyAtmosphere)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		RENDER_TARGET_BINDING_SLOTS()
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, TransmittanceLutTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, MultiScatteredLuminanceLutTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, SkyViewLutTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, CameraAerialPerspectiveVolumeTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, VolumetricCloudShadowMapTexture0)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, VolumetricCloudShadowMapTexture1)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, VolumetricCloudSkyAOTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float> , VolumetricCloudDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, InputCloudLuminanceTransmittanceTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS<float>, MSAADepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TransmittanceLutTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, MultiScatteredLuminanceLutTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, SkyViewLutTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, CameraAerialPerspectiveVolumeTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudShadowMapTexture0Sampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudShadowMapTexture1Sampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudSkyAOTextureSampler)
		SHADER_PARAMETER(float, AerialPerspectiveStartDepthKm)
		SHADER_PARAMETER(uint32, bPropagateAlphaNonReflection)
		SHADER_PARAMETER(uint32, SourceDiskEnabled)
		SHADER_PARAMETER(uint32, DepthReadDisabled)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVolumeShadowingShaderParametersGlobal0, Light0Shadow)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVolumeShadowingShaderParametersGlobal1, Light1Shadow)
		SHADER_PARAMETER(float, VolumetricCloudShadowStrength0)
		SHADER_PARAMETER(float, VolumetricCloudShadowStrength1)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMap)
		SHADER_PARAMETER(int32, VirtualShadowMapId0)
		SHADER_PARAMETER(int32, VirtualShadowMapId1)
		SHADER_PARAMETER_STRUCT_REF(FVolumetricCloudCommonGlobalShaderParameters, VolumetricCloudCommonGlobalParams)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		// If not rendering the sky, ignore the fastsky and sundisk permutations
		if (PermutationVector.Get<FRenderSky>() == false)
		{
			PermutationVector.Set<FFastSky>(false);
		}

		if (PermutationVector.Get<FAtmosphereOnClouds>() == true)
		{
			PermutationVector.Set<FFastSky>(false);
			PermutationVector.Set<FFastAerialPespective>(false);
			PermutationVector.Set<FRenderSky>(false);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (IsMobilePlatform(Parameters.Platform))
		{
			// Mobile must use a skydome mesh with a sky material to achieve good GPU performance
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// If not rendering the sky, ignore the FFastSky permutation
		if (PermutationVector.Get<FRenderSky>() == false && PermutationVector.Get<FFastSky>())
		{
			return false;
		}

		if (PermutationVector.Get<FAtmosphereOnClouds>() == true)
		{
			// FSampleCloudSkyAO, FFastSky, FFastAerialPespective, FSecondAtmosphereLight, FRenderSky, FSampleOpaqueShadow, FSampleCloudShadow
			// When tracing atmosphere on clouds, this is because we want crisp light shaft on them.
			if (PermutationVector.Get<FFastSky>() || PermutationVector.Get<FFastAerialPespective>() || PermutationVector.Get<FRenderSky>())
			{
				return false;
			}
		}

		if ((!IsForwardShadingEnabled(Parameters.Platform) || !RHISupportsMSAA(Parameters.Platform)) && PermutationVector.Get<FMSAASampleCount>() > 1)
		{
			// We only compile the MSAA support when Forward shading is enabled because MSAA can only be used in this case.
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("PER_PIXEL_NOISE"), 1);
		OutEnvironment.SetDefine(TEXT("MULTISCATTERING_APPROX_SAMPLING_ENABLED"), 1);
		OutEnvironment.SetDefine(TEXT("SOURCE_DISK_ENABLED"), 1);

		
		if (PermutationVector.Get<FSampleOpaqueShadow>() && VirtualShadowMapSamplingSupported(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}
	}
};
IMPLEMENT_GLOBAL_SHADER(FRenderSkyAtmospherePS, "/Engine/Private/SkyAtmosphere.usf", "RenderSkyAtmosphereRayMarchingPS", SF_Pixel);

//////////////////////////////////////////////////////////////////////////

class FRenderTransmittanceLutCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderTransmittanceLutCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderTransmittanceLutCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

public:
	const static uint32 GroupSize = 8;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FAtmosphereUniformShaderParameters, Atmosphere)
		SHADER_PARAMETER_STRUCT_REF(FSkyAtmosphereInternalCommonParameters, SkyAtmosphere)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, TransmittanceLutUAV)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GroupSize);
		OutEnvironment.SetDefine(TEXT("WHITE_TRANSMITTANCE"),1); // Workaround for some compiler not culling enough unused code (e.g. when computing TransmittanceLUT, Transmittance texture is still requested but we are computing it) 
		OutEnvironment.SetDefine(TEXT("TRANSMITTANCE_PASS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRenderTransmittanceLutCS, "/Engine/Private/SkyAtmosphere.usf", "RenderTransmittanceLutCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////

class FRenderMultiScatteredLuminanceLutCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderMultiScatteredLuminanceLutCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderMultiScatteredLuminanceLutCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FHighQualityMultiScatteringApprox>;

public:
	const static uint32 GroupSize = 8;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FAtmosphereUniformShaderParameters, Atmosphere)
		SHADER_PARAMETER_STRUCT_REF(FSkyAtmosphereInternalCommonParameters, SkyAtmosphere)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, MultiScatteredLuminanceLutUAV)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, TransmittanceLutTexture)
		SHADER_PARAMETER_SRV(Buffer<float4>, UniformSphereSamplesBuffer)
		SHADER_PARAMETER_SAMPLER(SamplerState, TransmittanceLutTextureSampler)
		SHADER_PARAMETER(uint32, UniformSphereSamplesBufferSampleCount)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GroupSize);
		OutEnvironment.SetDefine(TEXT("MULTISCATT_PASS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRenderMultiScatteredLuminanceLutCS, "/Engine/Private/SkyAtmosphere.usf", "RenderMultiScatteredLuminanceLutCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////

class FRenderDistantSkyLightLutCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderDistantSkyLightLutCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderDistantSkyLightLutCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSecondAtmosphereLight>;

public:
	const static uint32 GroupSize = 8;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FAtmosphereUniformShaderParameters, Atmosphere)
		SHADER_PARAMETER_STRUCT_REF(FSkyAtmosphereInternalCommonParameters, SkyAtmosphere)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, DistantSkyLightLutBufferUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float4>, MobileDistantSkyLightLutBufferUAV)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, TransmittanceLutTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, MultiScatteredLuminanceLutTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TransmittanceLutTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, MultiScatteredLuminanceLutTextureSampler)
		SHADER_PARAMETER_SRV(Buffer<float4>, UniformSphereSamplesBuffer)
		SHADER_PARAMETER(FVector4f, AtmosphereLightDirection0)
		SHADER_PARAMETER(FVector4f, AtmosphereLightDirection1)
		SHADER_PARAMETER(FLinearColor, AtmosphereLightIlluminanceOuterSpace0)
		SHADER_PARAMETER(FLinearColor, AtmosphereLightIlluminanceOuterSpace1)
		SHADER_PARAMETER(float, DistantSkyLightSampleAltitude)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GroupSize);
		OutEnvironment.SetDefine(TEXT("SKYLIGHT_PASS"), 1);
		OutEnvironment.SetDefine(TEXT("MULTISCATTERING_APPROX_SAMPLING_ENABLED"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRenderDistantSkyLightLutCS, "/Engine/Private/SkyAtmosphere.usf", "RenderDistantSkyLightLutCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////

class FRenderSkyViewLutCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderSkyViewLutCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderSkyViewLutCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSampleCloudSkyAO, FSecondAtmosphereLight, FSampleOpaqueShadow, FSampleCloudShadow, FRealTimeReflectionLUT>;

public:
	const static uint32 GroupSize = 8;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FAtmosphereUniformShaderParameters, Atmosphere)
		SHADER_PARAMETER_STRUCT_REF(FSkyAtmosphereInternalCommonParameters, SkyAtmosphere)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_REF(FSkyAtmosphereRealTimeReflectionLUTParameters, SkyAtmosphereRealTimeReflectionLUTUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, SkyViewLutUAV)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, TransmittanceLutTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, MultiScatteredLuminanceLutTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, VolumetricCloudShadowMapTexture0)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, VolumetricCloudShadowMapTexture1)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, VolumetricCloudSkyAOTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TransmittanceLutTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, MultiScatteredLuminanceLutTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudShadowMapTexture0Sampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudShadowMapTexture1Sampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudSkyAOTextureSampler)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVolumeShadowingShaderParametersGlobal0, Light0Shadow)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVolumeShadowingShaderParametersGlobal1, Light1Shadow)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMap)
		SHADER_PARAMETER(int32, VirtualShadowMapId0)
		SHADER_PARAMETER(int32, VirtualShadowMapId1)
		SHADER_PARAMETER(float, VolumetricCloudShadowStrength0)
		SHADER_PARAMETER(float, VolumetricCloudShadowStrength1)
		SHADER_PARAMETER_STRUCT_REF(FVolumetricCloudCommonGlobalShaderParameters, VolumetricCloudCommonGlobalParams)
		SHADER_PARAMETER(uint32, SourceDiskEnabled)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GroupSize);
		OutEnvironment.SetDefine(TEXT("SKYVIEWLUT_PASS"), 1);
		OutEnvironment.SetDefine(TEXT("MULTISCATTERING_APPROX_SAMPLING_ENABLED"), 1);
		OutEnvironment.SetDefine(TEXT("SOURCE_DISK_ENABLED"), 1);

		if (PermutationVector.Get<FSampleOpaqueShadow>() && VirtualShadowMapSamplingSupported(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}
	}
};
IMPLEMENT_GLOBAL_SHADER(FRenderSkyViewLutCS, "/Engine/Private/SkyAtmosphere.usf", "RenderSkyViewLutCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////

class FRenderCameraAerialPerspectiveVolumeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCameraAerialPerspectiveVolumeCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCameraAerialPerspectiveVolumeCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSampleCloudSkyAO, FSecondAtmosphereLight, FSampleOpaqueShadow, FSampleCloudShadow, FSeparateMieAndRayleighScattering, FRealTimeReflectionLUT>;

public:
	const static uint32 GroupSize = 4;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FAtmosphereUniformShaderParameters, Atmosphere)
		SHADER_PARAMETER_STRUCT_REF(FSkyAtmosphereInternalCommonParameters, SkyAtmosphere)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_REF(FSkyAtmosphereRealTimeReflectionLUTParameters, SkyAtmosphereRealTimeReflectionLUTUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, CameraAerialPerspectiveVolumeUAV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, CameraAerialPerspectiveVolumeMieOnlyUAV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, CameraAerialPerspectiveVolumeRayOnlyUAV)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, TransmittanceLutTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, MultiScatteredLuminanceLutTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, VolumetricCloudShadowMapTexture0)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, VolumetricCloudShadowMapTexture1)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, VolumetricCloudSkyAOTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TransmittanceLutTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, MultiScatteredLuminanceLutTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudShadowMapTexture0Sampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudShadowMapTexture1Sampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudSkyAOTextureSampler)
		SHADER_PARAMETER(float, AerialPerspectiveStartDepthKm)
		SHADER_PARAMETER(float, RealTimeReflection360Mode)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVolumeShadowingShaderParametersGlobal0, Light0Shadow)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVolumeShadowingShaderParametersGlobal1, Light1Shadow)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMap)
		SHADER_PARAMETER(int32, VirtualShadowMapId0)
		SHADER_PARAMETER(int32, VirtualShadowMapId1)
		SHADER_PARAMETER(float, VolumetricCloudShadowStrength0)
		SHADER_PARAMETER(float, VolumetricCloudShadowStrength1)
		SHADER_PARAMETER_STRUCT_REF(FVolumetricCloudCommonGlobalShaderParameters, VolumetricCloudCommonGlobalParams)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GroupSize);
		OutEnvironment.SetDefine(TEXT("MULTISCATTERING_APPROX_SAMPLING_ENABLED"), 1);

		if (PermutationVector.Get<FSampleOpaqueShadow>() && VirtualShadowMapSamplingSupported(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}
	}
};
IMPLEMENT_GLOBAL_SHADER(FRenderCameraAerialPerspectiveVolumeCS, "/Engine/Private/SkyAtmosphere.usf", "RenderCameraAerialPerspectiveVolumeCS", SF_Compute);

//////////////////////////////////////////////////////////////////////////

class FRenderDebugSkyAtmospherePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderDebugSkyAtmospherePS);
	SHADER_USE_PARAMETER_STRUCT(FRenderDebugSkyAtmospherePS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FAtmosphereUniformShaderParameters, Atmosphere)
		SHADER_PARAMETER_STRUCT_REF(FSkyAtmosphereInternalCommonParameters, SkyAtmosphere)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, TransmittanceLutTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, MultiScatteredLuminanceLutTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TransmittanceLutTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, MultiScatteredLuminanceLutTextureSampler)
		SHADER_PARAMETER(float, ViewPortWidth)
		SHADER_PARAMETER(float, ViewPortHeight)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// TODO: Exclude when shipping.
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MULTISCATTERING_APPROX_SAMPLING_ENABLED"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRenderDebugSkyAtmospherePS, "/Engine/Private/SkyAtmosphere.usf", "RenderSkyAtmosphereDebugPS", SF_Pixel);

//////////////////////////////////////////////////////////////////////////

class RenderSkyAtmosphereEditorHudPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(RenderSkyAtmosphereEditorHudPS);
	SHADER_USE_PARAMETER_STRUCT(RenderSkyAtmosphereEditorHudPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// TODO: Exclude when shipping.
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_EDITOR_HUD"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(RenderSkyAtmosphereEditorHudPS, "/Engine/Private/SkyAtmosphere.usf", "RenderSkyAtmosphereEditorHudPS", SF_Pixel);


/*=============================================================================
	FUniformSphereSamplesBuffer
=============================================================================*/

class FUniformSphereSamplesBuffer : public FRenderResource
{
public:
	FReadBuffer UniformSphereSamplesBuffer;

	uint32 GetSampletCount()
	{
		return FRenderDistantSkyLightLutCS::GroupSize;
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const uint32 GroupSize = GetSampletCount();
		const float GroupSizeInv = 1.0f / float(GroupSize);

		UniformSphereSamplesBuffer.InitializeWithData(RHICmdList, TEXT("UniformSphereSamplesBuffer"), sizeof(FVector4f), GroupSize * GroupSize, PF_A32B32G32R32F, EBufferUsageFlags::Static,
			[GroupSize, GroupSizeInv](FRHIBufferInitializer& Initializer)
			{
				TArrayView<FVector4f> Dest = Initializer.GetWriteView<FVector4f>();

				FRandomStream RandomStream(0xDE4DC0DE);
				for (uint32 i = 0; i < GroupSize; ++i)
				{
					for (uint32 j = 0; j < GroupSize; ++j)
					{
						const float u0 = (float(i) + RandomStream.GetFraction()) * GroupSizeInv;
						const float u1 = (float(j) + RandomStream.GetFraction()) * GroupSizeInv;

						const float a = 1.0f - 2.0f * u0;
						const float b = FMath::Sqrt(1.0f - a * a);
						const float phi = 2 * PI * u1;

						uint32 idx = j * GroupSize + i;
						Dest[idx].X = b * FMath::Cos(phi);
						Dest[idx].Y = b * FMath::Sin(phi);
						Dest[idx].Z = a;
						Dest[idx].W = 0.0f;
					}
				}
			});
	}

	virtual void ReleaseRHI()
	{
		UniformSphereSamplesBuffer.Release();
	}
};
TGlobalResource<FUniformSphereSamplesBuffer> GUniformSphereSamplesBuffer;



/*=============================================================================
	FSceneRenderer functions
=============================================================================*/

void FSceneRenderer::InitSkyAtmosphereForViews(FRHICommandListImmediate& RHICmdList, FRDGBuilder& GraphBuilder)
{
	InitSkyAtmosphereForScene(RHICmdList, GraphBuilder, Scene);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];
		InitSkyAtmosphereForView(RHICmdList, Scene, View);
	}
}

static EPixelFormat GetSkyLutTextureFormat(ERHIFeatureLevel::Type FeatureLevel)
{
	EPixelFormat TextureLUTFormat = PF_FloatRGB;
	const bool bSupportsAlpha = IsPostProcessingWithAlphaChannelSupported();
	if (FeatureLevel <= ERHIFeatureLevel::ES3_1 || bSupportsAlpha)
	{
		// OpenGL ES3.1 does not support storing into 3-component images
		// TODO: check if need this for Metal, Vulkan
		TextureLUTFormat = PF_FloatRGBA;
	}

	return TextureLUTFormat;
}
static EPixelFormat GetSkyLutSmallTextureFormat()
{
	return PF_R8G8B8A8;
}

void InitSkyAtmosphereForScene(FRHICommandListImmediate& RHICmdList, FRDGBuilder& GraphBuilder, FScene* Scene)
{
	if (Scene)
	{
		GET_VALID_DATA_FROM_CVAR;

		FPooledRenderTargetDesc Desc;
		check(Scene->GetSkyAtmosphereSceneInfo());
		FSkyAtmosphereRenderSceneInfo& SkyInfo = *Scene->GetSkyAtmosphereSceneInfo();

		EPixelFormat TextureLUTFormat = GetSkyLutTextureFormat(Scene->GetFeatureLevel());
		EPixelFormat TextureLUTSmallFormat = GetSkyLutSmallTextureFormat();

		bool bCachedLUTsResourcesAreTheSame = true;

		//
		// Initialise per scene/atmosphere resources
		//
		if (CVarSkyAtmosphereTransmittanceLUT.GetValueOnAnyThread() > 0)
		{
			const bool TranstmittanceLUTUseSmallFormat = CVarSkyAtmosphereTransmittanceLUTUseSmallFormat.GetValueOnRenderThread() > 0;

			TRefCountPtr<IPooledRenderTarget>& TransmittanceLutTexture = SkyInfo.GetTransmittanceLutTexture();
			Desc = FPooledRenderTargetDesc::Create2DDesc(
				FIntPoint(TransmittanceLutWidth, TransmittanceLutHeight),
				TranstmittanceLUTUseSmallFormat ? TextureLUTSmallFormat : TextureLUTFormat, FClearValueBinding::None, TexCreate_None, TexCreate_ShaderResource | TexCreate_UAV, false);
			bCachedLUTsResourcesAreTheSame = GRenderTargetPool.FindFreeElement(RHICmdList, Desc, TransmittanceLutTexture, TEXT("SkyAtmosphere.TransmittanceLut")) && bCachedLUTsResourcesAreTheSame;
		}
		else
		{
			SkyInfo.GetTransmittanceLutTexture() = GSystemTextures.WhiteDummy;
		}

		TRefCountPtr<IPooledRenderTarget>& MultiScatteredLuminanceLutTexture = SkyInfo.GetMultiScatteredLuminanceLutTexture();
		Desc = FPooledRenderTargetDesc::Create2DDesc(
			FIntPoint(MultiScatteredLuminanceLutWidth, MultiScatteredLuminanceLutHeight),
			TextureLUTFormat, FClearValueBinding::None, TexCreate_None, TexCreate_ShaderResource | TexCreate_UAV, false);
		bCachedLUTsResourcesAreTheSame = GRenderTargetPool.FindFreeElement(RHICmdList, Desc, MultiScatteredLuminanceLutTexture, TEXT("SkyAtmosphere.MultiScatteredLuminanceLut")) && bCachedLUTsResourcesAreTheSame;

		// If the resources have changed, we must recache them.
		if (!bCachedLUTsResourcesAreTheSame)
		{
			Scene->SetCachedTransmittanceAndMultiScatteringLUTsVersion(0);
		}

		if (CVarSkyAtmosphereDistantSkyLightLUT.GetValueOnRenderThread() > 0)
		{
			SkyInfo.CreateDistantSkyLightLutBufferAndSRV(GraphBuilder);
		}
	}
}

void InitSkyAtmosphereForView(FRHICommandListImmediate& RHICmdList, const FScene* Scene, FViewInfo& View)
{
	if (Scene)
	{
		GET_VALID_DATA_FROM_CVAR;

		check(ShouldRenderSkyAtmosphere(Scene, View.Family->EngineShowFlags)); // This should not be called if we should not render SkyAtmosphere
		FPooledRenderTargetDesc Desc;
		check(Scene->GetSkyAtmosphereSceneInfo());
		const FSkyAtmosphereRenderSceneInfo& SkyInfo = *Scene->GetSkyAtmosphereSceneInfo();

		EPixelFormat TextureLUTFormat = GetSkyLutTextureFormat(Scene->GetFeatureLevel());
		EPixelFormat TextureLUTSmallFormat = GetSkyLutSmallTextureFormat();
		EPixelFormat TextureAerialLUTFormat = (CVarSkyAtmosphereLUT32.GetValueOnAnyThread() != 0) ? PF_A32B32G32R32F : PF_FloatRGBA;

		//
		// Initialise view resources.
		//
		
		FPooledRenderTargetDesc SkyAtmosphereViewLutTextureDesc = FPooledRenderTargetDesc::Create2DDesc(
			FIntPoint(SkyViewLutWidth, SkyViewLutHeight),
			TextureLUTFormat, FClearValueBinding::None, TexCreate_None, TexCreate_ShaderResource | TexCreate_UAV, false);

		FPooledRenderTargetDesc SkyAtmosphereCameraAerialPerspectiveVolumeDesc = FPooledRenderTargetDesc::CreateVolumeDesc(
			CameraAerialPerspectiveVolumeScreenResolution, CameraAerialPerspectiveVolumeScreenResolution, CameraAerialPerspectiveVolumeDepthResolution,
			TextureAerialLUTFormat, FClearValueBinding::None, TexCreate_None, TexCreate_ShaderResource | TexCreate_UAV, false);

		// Set textures and data that will be needed later on the view.
		View.SkyAtmosphereUniformShaderParameters = SkyInfo.GetAtmosphereShaderParameters();
		GRenderTargetPool.FindFreeElement(RHICmdList, SkyAtmosphereViewLutTextureDesc, View.SkyAtmosphereViewLutTexture, TEXT("SkyAtmosphere.SkyViewLut"));

		const bool bSeparatedAtmosphereMieRayLeigh = VolumetricCloudWantsSeparatedAtmosphereMieRayLeigh(Scene);
		if (View.ViewState)
		{
			// Per view non transient double buffered resources when needed.
			View.ViewState->PersistentSkyAtmosphereData.InitialiseOrNextFrame(View.FeatureLevel, SkyAtmosphereCameraAerialPerspectiveVolumeDesc, RHICmdList, bSeparatedAtmosphereMieRayLeigh);
			View.SkyAtmosphereCameraAerialPerspectiveVolume = View.ViewState->PersistentSkyAtmosphereData.GetCurrentCameraAerialPerspectiveVolume();
			if (bSeparatedAtmosphereMieRayLeigh)
			{
				View.SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly = View.ViewState->PersistentSkyAtmosphereData.GetCurrentCameraAerialPerspectiveVolumeMieOnly();
				View.SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly = View.ViewState->PersistentSkyAtmosphereData.GetCurrentCameraAerialPerspectiveVolumeRayOnly();
			}
		}
		else
		{
			// Per frame transient resource for reflection views
			GRenderTargetPool.FindFreeElement(RHICmdList, SkyAtmosphereCameraAerialPerspectiveVolumeDesc, View.SkyAtmosphereCameraAerialPerspectiveVolume, TEXT("SkyAtmosphere.CameraAPVolume"));
			if (bSeparatedAtmosphereMieRayLeigh)
			{
				GRenderTargetPool.FindFreeElement(RHICmdList, SkyAtmosphereCameraAerialPerspectiveVolumeDesc, View.SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly, TEXT("SkyAtmosphere.CameraAPVolumeMieOnly"));
				GRenderTargetPool.FindFreeElement(RHICmdList, SkyAtmosphereCameraAerialPerspectiveVolumeDesc, View.SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly, TEXT("SkyAtmosphere.CameraAPVolumeRayOnly"));
			}
		}
	}
}

static void SetupSkyAtmosphereInternalCommonParameters(
	FSkyAtmosphereInternalCommonParameters& InternalCommonParameters, 
	const FScene& Scene,
	const FSceneViewFamily& ViewFamily,
	const FSkyAtmosphereRenderSceneInfo& SkyInfo)
{
	GET_VALID_DATA_FROM_CVAR;

	InternalCommonParameters.TransmittanceLutSizeAndInvSize = GetSizeAndInvSize(TransmittanceLutWidth, TransmittanceLutHeight);
	InternalCommonParameters.MultiScatteredLuminanceLutSizeAndInvSize = GetSizeAndInvSize(MultiScatteredLuminanceLutWidth, MultiScatteredLuminanceLutHeight);
	InternalCommonParameters.SkyViewLutSizeAndInvSize = GetSizeAndInvSize(SkyViewLutWidth, SkyViewLutHeight);

	const float SkyAtmosphereBaseSampleCount = 32.0f;
	const float AerialPerspectiveBaseSampleCountPerSlice = 1.0f;

	InternalCommonParameters.SampleCountMin = CVarSkyAtmosphereSampleCountMin.GetValueOnRenderThread();
	InternalCommonParameters.SampleCountMax = FMath::Min(SkyAtmosphereBaseSampleCount * SkyInfo.GetSkyAtmosphereSceneProxy().GetTraceSampleCountScale(), float(CVarSkyAtmosphereSampleCountMax.GetValueOnRenderThread()));
	float DistanceToSampleCountMaxInv = CVarSkyAtmosphereDistanceToSampleCountMax.GetValueOnRenderThread();

	InternalCommonParameters.FastSkySampleCountMin = CVarSkyAtmosphereFastSkyLUTSampleCountMin.GetValueOnRenderThread();
	InternalCommonParameters.FastSkySampleCountMax = FMath::Min(SkyAtmosphereBaseSampleCount * SkyInfo.GetSkyAtmosphereSceneProxy().GetTraceSampleCountScale(), float(CVarSkyAtmosphereFastSkyLUTSampleCountMax.GetValueOnRenderThread()));
	float FastSkyDistanceToSampleCountMaxInv = CVarSkyAtmosphereFastSkyLUTDistanceToSampleCountMax.GetValueOnRenderThread();

	InternalCommonParameters.CameraAerialPerspectiveVolumeDepthResolution = float(CameraAerialPerspectiveVolumeDepthResolution);
	InternalCommonParameters.CameraAerialPerspectiveVolumeDepthResolutionInv = 1.0f / InternalCommonParameters.CameraAerialPerspectiveVolumeDepthResolution;
	InternalCommonParameters.CameraAerialPerspectiveVolumeDepthSliceLengthKm = CameraAerialPerspectiveVolumeDepthSliceLengthKm;
	InternalCommonParameters.CameraAerialPerspectiveVolumeDepthSliceLengthKmInv = 1.0f / CameraAerialPerspectiveVolumeDepthSliceLengthKm;
	InternalCommonParameters.CameraAerialPerspectiveSampleCountPerSlice = FMath::Max(AerialPerspectiveBaseSampleCountPerSlice, FMath::Min(2.0f * SkyInfo.GetSkyAtmosphereSceneProxy().GetTraceSampleCountScale(), float(CVarSkyAtmosphereAerialPerspectiveLUTSampleCountMaxPerSlice.GetValueOnRenderThread())));

	InternalCommonParameters.TransmittanceSampleCount = CVarSkyAtmosphereTransmittanceLUTSampleCount.GetValueOnRenderThread();
	InternalCommonParameters.MultiScatteringSampleCount = CVarSkyAtmosphereMultiScatteringLUTSampleCount.GetValueOnRenderThread();

	const FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy = SkyInfo.GetSkyAtmosphereSceneProxy();
	InternalCommonParameters.SkyLuminanceFactor = FVector3f(SkyAtmosphereSceneProxy.GetSkyLuminanceFactor());
	InternalCommonParameters.SkyAndAerialPerspectiveLuminanceFactor = FVector3f(SkyAtmosphereSceneProxy.GetSkyAndAerialPerspectiveLuminanceFactor());
	InternalCommonParameters.AerialPespectiveViewDistanceScale = SkyAtmosphereSceneProxy.GetAerialPespectiveViewDistanceScale();
	InternalCommonParameters.FogShowFlagFactor = ViewFamily.EngineShowFlags.Fog ? 1.0f : 0.0f;

	InternalCommonParameters.BlueNoise = GetBlueNoiseGlobalParameters();

	auto ValidateDistanceValue = [](float& Value)
	{
		Value = Value < KINDA_SMALL_NUMBER ? KINDA_SMALL_NUMBER : Value;
	};
	auto ValidateSampleCountValue = [](float& Value)
	{
		Value = Value < 1.0f ? 1.0f : Value;
	};
	auto ValidateMaxSampleCountValue = [](float& Value, float& MinValue)
	{
		Value = Value < MinValue ? MinValue : Value;
	};
	ValidateSampleCountValue(InternalCommonParameters.SampleCountMin);
	ValidateMaxSampleCountValue(InternalCommonParameters.SampleCountMax, InternalCommonParameters.SampleCountMin);
	ValidateSampleCountValue(InternalCommonParameters.FastSkySampleCountMin);
	ValidateMaxSampleCountValue(InternalCommonParameters.FastSkySampleCountMax, InternalCommonParameters.FastSkySampleCountMin);
	ValidateSampleCountValue(InternalCommonParameters.CameraAerialPerspectiveSampleCountPerSlice);
	ValidateSampleCountValue(InternalCommonParameters.TransmittanceSampleCount);
	ValidateSampleCountValue(InternalCommonParameters.MultiScatteringSampleCount);
	ValidateDistanceValue(DistanceToSampleCountMaxInv);
	ValidateDistanceValue(FastSkyDistanceToSampleCountMaxInv);

	// Derived values post validation
	InternalCommonParameters.DistanceToSampleCountMaxInv = 1.0f / DistanceToSampleCountMaxInv;
	InternalCommonParameters.FastSkyDistanceToSampleCountMaxInv = 1.0f / FastSkyDistanceToSampleCountMaxInv;
	InternalCommonParameters.CameraAerialPerspectiveVolumeSizeAndInvSize = GetSizeAndInvSize(CameraAerialPerspectiveVolumeScreenResolution, CameraAerialPerspectiveVolumeScreenResolution);
}

void FSceneRenderer::RenderSkyAtmosphereLookUpTables(FRDGBuilder& GraphBuilder, class FSkyAtmospherePendingRDGResources& PendingRDGResources, FDynamicShadowsTaskData* FilterDynamicShadowTaskData)
{
	check(ShouldRenderSkyAtmosphere(Scene, ViewFamily.EngineShowFlags)); // This should not be called if we should not render SkyAtmosphere

	RDG_EVENT_SCOPE_STAT(GraphBuilder, SkyAtmosphereLUTs, "SkyAtmosphereLUTs");
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, SkyAtmosphere);
	SCOPED_NAMED_EVENT(RenderSkyAtmosphereLookUpTables, FColor::Emerald);

	FSkyAtmosphereRenderSceneInfo& SkyInfo = *Scene->GetSkyAtmosphereSceneInfo();
	const FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy = SkyInfo.GetSkyAtmosphereSceneProxy();

	PendingRDGResources.SceneRenderer = this;
	PendingRDGResources.ViewResources.SetNum(Views.Num());

	const bool bHighQualityMultiScattering = CVarSkyAtmosphereMultiScatteringLUTHighQuality.GetValueOnRenderThread() > 0;
	const bool bSecondAtmosphereLightEnabled = Scene->IsSecondAtmosphereLightEnabled();
	const bool bSeparatedAtmosphereMieRayLeigh = VolumetricCloudWantsSeparatedAtmosphereMieRayLeigh(Scene);

	FRHISamplerState* SamplerLinearClamp = TStaticSamplerState<SF_Trilinear>::GetRHI();

	// Initialise common internal parameters on the sky info for this frame
	FSkyAtmosphereInternalCommonParameters InternalCommonParameters;
	SetupSkyAtmosphereInternalCommonParameters(InternalCommonParameters, *Scene, ViewFamily, SkyInfo);
	SkyInfo.GetInternalCommonParametersUniformBuffer() = TUniformBufferRef<FSkyAtmosphereInternalCommonParameters>::CreateUniformBufferImmediate(InternalCommonParameters, UniformBuffer_SingleFrame);

	FRDGTextureRef TransmittanceLut = GraphBuilder.RegisterExternalTexture(SkyInfo.GetTransmittanceLutTexture());
	FRDGTextureRef MultiScatteredLuminanceLut = GraphBuilder.RegisterExternalTexture(SkyInfo.GetMultiScatteredLuminanceLutTexture());
	FRDGTextureUAVRef MultiScatteredLuminanceLutUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(MultiScatteredLuminanceLut, 0));

	ERDGPassFlags PassFlag = CVarSkyAtmosphereAsyncCompute.GetValueOnAnyThread() ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute;

	bool bEvaluateTransmittanceAndMultiScatteringLUTs = false;
	const FAtmosphereSetup& AtmosphereSetup = SkyAtmosphereSceneProxy.GetAtmosphereSetup();
	const uint32 IncomingTransmittanceAndMultiScatteringLUTsVersion = AtmosphereSetup.GetTransmittanceAndMultiScatteringLUTsVersion();
	uint32 CachedTransmittanceAndMultiScatteringLUTsVersion = Scene->GetCachedTransmittanceAndMultiScatteringLUTsVersion();
	if (CachedTransmittanceAndMultiScatteringLUTsVersion==0 
		|| IncomingTransmittanceAndMultiScatteringLUTsVersion != CachedTransmittanceAndMultiScatteringLUTsVersion
		|| CVarSkyAtmosphereStateVersioning.GetValueOnAnyThread() == false)
	{
		bEvaluateTransmittanceAndMultiScatteringLUTs = true;
		Scene->SetCachedTransmittanceAndMultiScatteringLUTsVersion(IncomingTransmittanceAndMultiScatteringLUTsVersion);
	}

	// Transmittance LUT
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);
	if (bEvaluateTransmittanceAndMultiScatteringLUTs && CVarSkyAtmosphereTransmittanceLUT.GetValueOnRenderThread() > 0)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "SkyAtmosphere::TransmittanceLut");
		TShaderMapRef<FRenderTransmittanceLutCS> ComputeShader(GlobalShaderMap);

		FRenderTransmittanceLutCS::FParameters * PassParameters = GraphBuilder.AllocParameters<FRenderTransmittanceLutCS::FParameters>();
		PassParameters->Atmosphere = Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereUniformBuffer();
		PassParameters->SkyAtmosphere = SkyInfo.GetInternalCommonParametersUniformBuffer();
		PassParameters->TransmittanceLutUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(TransmittanceLut, 0));

		FIntVector TextureSize = TransmittanceLut->Desc.GetSize();
		TextureSize.Z = 1;
		const FIntVector NumGroups = FIntVector::DivideAndRoundUp(TextureSize, FRenderTransmittanceLutCS::GroupSize);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("TransmittanceLut"), PassFlag, ComputeShader, PassParameters, NumGroups);
	}

	// Multi-Scattering LUT
	if(bEvaluateTransmittanceAndMultiScatteringLUTs)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "SkyAtmosphere::MultiScatteringLut");
		FRenderMultiScatteredLuminanceLutCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FHighQualityMultiScatteringApprox>(bHighQualityMultiScattering);
		TShaderMapRef<FRenderMultiScatteredLuminanceLutCS> ComputeShader(GlobalShaderMap, PermutationVector);

		FRenderMultiScatteredLuminanceLutCS::FParameters * PassParameters = GraphBuilder.AllocParameters<FRenderMultiScatteredLuminanceLutCS::FParameters>();
		PassParameters->Atmosphere = Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereUniformBuffer();
		PassParameters->SkyAtmosphere = SkyInfo.GetInternalCommonParametersUniformBuffer();
		PassParameters->TransmittanceLutTextureSampler = SamplerLinearClamp;
		PassParameters->TransmittanceLutTexture = TransmittanceLut;
		PassParameters->UniformSphereSamplesBuffer = GUniformSphereSamplesBuffer.UniformSphereSamplesBuffer.SRV;
		PassParameters->UniformSphereSamplesBufferSampleCount = GUniformSphereSamplesBuffer.GetSampletCount();
		PassParameters->MultiScatteredLuminanceLutUAV = MultiScatteredLuminanceLutUAV;

		FIntVector TextureSize = MultiScatteredLuminanceLut->Desc.GetSize();
		TextureSize.Z = 1;
		const FIntVector NumGroups = FIntVector::DivideAndRoundUp(TextureSize, FRenderMultiScatteredLuminanceLutCS::GroupSize);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("MultiScatteringLut"), PassFlag, ComputeShader, PassParameters, NumGroups);
	}

	// Distant Sky Light LUT
	if(CVarSkyAtmosphereDistantSkyLightLUT.GetValueOnRenderThread() > 0)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "SkyAtmosphere::DistantSkyLightLut");
		FRDGBufferRef DistantSkyLightLutBuffer = GraphBuilder.RegisterExternalBuffer(SkyInfo.GetDistantSkyLightLutBuffer());
		FRDGBufferUAVRef DistantSkyLightLutBufferUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(DistantSkyLightLutBuffer, PF_A32B32G32R32F));
		FRDGBufferRef MobileDistantSkyLightLutBuffer = GraphBuilder.RegisterExternalBuffer(SkyInfo.GetMobileDistantSkyLightLutBuffer());
		FRDGBufferUAVRef MobileDistantSkyLightLutBufferUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(MobileDistantSkyLightLutBuffer, PF_A32B32G32R32F));

		FRenderDistantSkyLightLutCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FSecondAtmosphereLight>(bSecondAtmosphereLightEnabled);
		TShaderMapRef<FRenderDistantSkyLightLutCS> ComputeShader(GlobalShaderMap, PermutationVector);

		FRenderDistantSkyLightLutCS::FParameters * PassParameters = GraphBuilder.AllocParameters<FRenderDistantSkyLightLutCS::FParameters>();
		PassParameters->Atmosphere = Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereUniformBuffer();
		PassParameters->SkyAtmosphere = SkyInfo.GetInternalCommonParametersUniformBuffer();
		PassParameters->TransmittanceLutTextureSampler = SamplerLinearClamp;
		PassParameters->MultiScatteredLuminanceLutTextureSampler = SamplerLinearClamp;
		PassParameters->TransmittanceLutTexture = TransmittanceLut;
		PassParameters->MultiScatteredLuminanceLutTexture = MultiScatteredLuminanceLut;
		PassParameters->UniformSphereSamplesBuffer = GUniformSphereSamplesBuffer.UniformSphereSamplesBuffer.SRV;
		PassParameters->DistantSkyLightLutBufferUAV = DistantSkyLightLutBufferUAV;
		PassParameters->MobileDistantSkyLightLutBufferUAV = MobileDistantSkyLightLutBufferUAV;

		FLightSceneInfo* Light0 = Scene->AtmosphereLights[0];
		FLightSceneInfo* Light1 = Scene->AtmosphereLights[1];
		if (Light0)
		{
			PassParameters->AtmosphereLightDirection0 = FVector3f(SkyAtmosphereSceneProxy.GetAtmosphereLightDirection(0, -Light0->Proxy->GetDirection()));
			PassParameters->AtmosphereLightIlluminanceOuterSpace0 = Light0->Proxy->GetOuterSpaceIlluminance();
		}
		else
		{
			PassParameters->AtmosphereLightDirection0 = FVector4f(0.0f, 0.0f, 1.0f, 1.0f);
			PassParameters->AtmosphereLightIlluminanceOuterSpace0 = FLinearColor::Black;
		}
		if (Light1)
		{
			PassParameters->AtmosphereLightDirection1 = FVector3f(SkyAtmosphereSceneProxy.GetAtmosphereLightDirection(1, -Light1->Proxy->GetDirection()));
			PassParameters->AtmosphereLightIlluminanceOuterSpace1 = Light1->Proxy->GetOuterSpaceIlluminance();
		}
		else
		{
			PassParameters->AtmosphereLightDirection1 = FVector4f(0.0f, 0.0f, 1.0f, 1.0f);
			PassParameters->AtmosphereLightIlluminanceOuterSpace1 = FLinearColor::Black;
		}
		PassParameters->DistantSkyLightSampleAltitude = CVarSkyAtmosphereDistantSkyLightLUTAltitude.GetValueOnAnyThread();

		FIntVector TextureSize = FIntVector(1, 1, 1);
		const FIntVector NumGroups = FIntVector::DivideAndRoundUp(TextureSize, FRenderDistantSkyLightLutCS::GroupSize);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("DistantSkyLightLut"), PassFlag, ComputeShader, PassParameters, NumGroups);

		PendingRDGResources.DistantSkyLightLutBuffer = DistantSkyLightLutBuffer;
		PendingRDGResources.MobileDistantSkyLightLutBuffer = MobileDistantSkyLightLutBuffer;
	}

	SkyAtmosphereLightShadowData LightShadowData;
	const bool bShouldSampleOpaqueShadow = ShouldSkySampleAtmosphereLightsOpaqueShadow(*Scene, VisibleLightInfos, LightShadowData, FilterDynamicShadowTaskData);

	FLightSceneProxy* AtmosphereLight0Proxy = Scene->AtmosphereLights[0] ? Scene->AtmosphereLights[0]->Proxy : nullptr;
	FLightSceneProxy* AtmosphereLight1Proxy = Scene->AtmosphereLights[1] ? Scene->AtmosphereLights[1]->Proxy : nullptr;
	const float CloudShadowOnAtmosphereStrength0 = AtmosphereLight0Proxy ? AtmosphereLight0Proxy->GetCloudShadowOnAtmosphereStrength() : 0.0f;
	const float CloudShadowOnAtmosphereStrength1 = AtmosphereLight1Proxy ? AtmosphereLight1Proxy->GetCloudShadowOnAtmosphereStrength() : 0.0f;

	// SkyViewLUT texture is required if there are any sky dome material that could potentially sample it.
	// This texture is sampled on skydome mesh with a sky material when rendered into a cubemap real time capture.
	const bool bRealTimeReflectionCaptureSkyAtmosphereViewLutTexture = Views.Num() > 0 && Scene->SkyLight && Scene->SkyLight->bRealTimeCaptureEnabled;
	// CameraAP volume is required if there is a skydome or a volumetric cloud component rendered in a cubemap real time capture.
	const bool bRealTimeReflectionCapture360APLutTexture = Views.Num() > 0 && Scene->SkyLight && Scene->SkyLight->bRealTimeCaptureEnabled && (Views[0].bSceneHasSkyMaterial || Scene->HasVolumetricCloud());
	if (bRealTimeReflectionCaptureSkyAtmosphereViewLutTexture || bRealTimeReflectionCapture360APLutTexture)
	{
		FViewInfo& ViewForReflection = Views[0];
		const float AerialPerspectiveStartDepthInCm = GetValidAerialPerspectiveStartDepthInCm(ViewForReflection, SkyAtmosphereSceneProxy);

		// Setup a constant referential for each of the faces of the dynamic reflection capture.
		const FVector3f SkyViewLutReferentialForward = FVector3f(1.0f, 0.0f, 0.0f);
		const FVector3f SkyViewLutReferentialRight = FVector3f(0.0f, 1.0f, 0.0f);
		FVector3f SkyCameraTranslatedWorldOrigin;
		FMatrix44f SkyViewLutReferential;
		FVector4f TempSkyPlanetData;
		AtmosphereSetup.ComputeViewData(
			Scene->SkyLight->CapturePosition, ViewForReflection.ViewMatrices.GetPreViewTranslation(), SkyViewLutReferentialForward, SkyViewLutReferentialRight,
			SkyCameraTranslatedWorldOrigin, TempSkyPlanetData, SkyViewLutReferential);

		// Here, it would be convenient to create a new ViewUB using Scene->SkyLight->CapturePosition,
		// while setting up correct view parameters such as SkyPlanetTranslatedWorldCenterAndViewHeight, SkyViewLutReferential, etc.
		// But this is expenssive and can increase the cost of that function by 3 on low end platform.
		// As an optimisation to avoid creating a new View UB, shaders will fetch from constants what they need.
		FSkyAtmosphereRealTimeReflectionLUTParameters Parameters;
		Parameters.SkyViewLutReferential = SkyViewLutReferential;
		Parameters.SkyPlanetTranslatedWorldCenterAndViewHeight = TempSkyPlanetData;
		Parameters.SkyCameraTranslatedWorldOrigin = SkyCameraTranslatedWorldOrigin;
		Parameters.RealTimeReflectionCapture = 0.0f;
		TUniformBufferRef<FSkyAtmosphereRealTimeReflectionLUTParameters> SkyAtmosphereRealTimeReflectionLUTUB = TUniformBufferRef<FSkyAtmosphereRealTimeReflectionLUTParameters>::CreateUniformBufferImmediate(Parameters, UniformBuffer_SingleFrame);

		FVolumetricCloudRenderSceneInfo* CloudInfo = Scene->GetVolumetricCloudSceneInfo();
		FCloudShadowAOData CloudShadowAOData;
		GetCloudShadowAOData(CloudInfo, ViewForReflection, GraphBuilder, CloudShadowAOData);
		const bool bShouldSampleCloudShadow = CloudShadowAOData.bShouldSampleCloudShadow && (CloudShadowOnAtmosphereStrength0 > 0.0f || CloudShadowOnAtmosphereStrength1 > 0.0f);

		if (bRealTimeReflectionCaptureSkyAtmosphereViewLutTexture)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "SkyAtmosphere::RealTimeCaptureSkyViewLut");
			FIntVector SkyViewLutSize = ViewForReflection.SkyAtmosphereViewLutTexture->GetDesc().GetSize();
			FRDGTextureRef RealTimeReflectionCaptureSkyAtmosphereViewLutTexture = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(FIntPoint(SkyViewLutSize.X, SkyViewLutSize.Y), GetSkyLutTextureFormat(Scene->GetFeatureLevel()), FClearValueBinding::None,
					TexCreate_ShaderResource | TexCreate_UAV), TEXT("SkyAtmosphere.RealTimeSkyCapViewLut"));
			FRDGTextureUAVRef RealTimeReflectionCaptureSkyAtmosphereViewLutTextureUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(RealTimeReflectionCaptureSkyAtmosphereViewLutTexture, 0));

			FRenderSkyViewLutCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSampleCloudSkyAO>(CloudShadowAOData.bShouldSampleCloudSkyAO);
			PermutationVector.Set<FSecondAtmosphereLight>(bSecondAtmosphereLightEnabled);
			PermutationVector.Set<FSampleOpaqueShadow>(false); // bShouldSampleOpaqueShadow); Off for now to not have to generate Light0Shadow and Light1Shadow
			PermutationVector.Set<FSampleCloudShadow>(bShouldSampleCloudShadow);
			PermutationVector.Set<FRealTimeReflectionLUT>(true);
			TShaderMapRef<FRenderSkyViewLutCS> ComputeShader(ViewForReflection.ShaderMap, PermutationVector);

			FRenderSkyViewLutCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderSkyViewLutCS::FParameters>();
			PassParameters->Atmosphere = Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereUniformBuffer();
			PassParameters->SkyAtmosphere = SkyInfo.GetInternalCommonParametersUniformBuffer();
			PassParameters->ViewUniformBuffer = ViewForReflection.ViewUniformBuffer;
			PassParameters->SkyAtmosphereRealTimeReflectionLUTUniformBuffer = SkyAtmosphereRealTimeReflectionLUTUB;
			PassParameters->TransmittanceLutTextureSampler = SamplerLinearClamp;
			PassParameters->MultiScatteredLuminanceLutTextureSampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudShadowMapTexture0Sampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudShadowMapTexture1Sampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudSkyAOTextureSampler = SamplerLinearClamp;
			PassParameters->TransmittanceLutTexture = TransmittanceLut;
			PassParameters->MultiScatteredLuminanceLutTexture = MultiScatteredLuminanceLut;
			PassParameters->VolumetricCloudShadowMapTexture0 = CloudShadowAOData.VolumetricCloudShadowMap[0];
			PassParameters->VolumetricCloudShadowMapTexture1 = CloudShadowAOData.VolumetricCloudShadowMap[1];
			PassParameters->VolumetricCloudSkyAOTexture = CloudShadowAOData.VolumetricCloudSkyAO;
			PassParameters->VolumetricCloudShadowStrength0 = CloudShadowOnAtmosphereStrength0;
			PassParameters->VolumetricCloudShadowStrength1 = CloudShadowOnAtmosphereStrength1;
			PassParameters->SkyViewLutUAV = RealTimeReflectionCaptureSkyAtmosphereViewLutTextureUAV;
			if (bShouldSampleCloudShadow || CloudShadowAOData.bShouldSampleCloudSkyAO)
			{
				PassParameters->VolumetricCloudCommonGlobalParams = CloudInfo->GetVolumetricCloudCommonShaderParametersUB();
			}
			PassParameters->SourceDiskEnabled = 0;

			FIntVector TextureSize = RealTimeReflectionCaptureSkyAtmosphereViewLutTexture->Desc.GetSize();
			TextureSize.Z = 1;
			const FIntVector NumGroups = FIntVector::DivideAndRoundUp(TextureSize, FRenderSkyViewLutCS::GroupSize);
			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RealTimeCaptureSkyViewLut"), PassFlag, ComputeShader, PassParameters, NumGroups);

			PendingRDGResources.RealTimeReflectionCaptureSkyAtmosphereViewLutTexture = RealTimeReflectionCaptureSkyAtmosphereViewLutTexture;
		}

		if (bRealTimeReflectionCapture360APLutTexture)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "SkyAtmosphere::RealTimeCaptureCamera360VolumeLut");
			FIntVector CameraAPLutSize = ViewForReflection.SkyAtmosphereCameraAerialPerspectiveVolume->GetDesc().GetSize();
			EPixelFormat TextureAerialLUTFormat = (CVarSkyAtmosphereLUT32.GetValueOnAnyThread() != 0) ? PF_A32B32G32R32F : PF_FloatRGBA;
			FRDGTextureRef RealTimeReflectionCaptureCamera360APLutTexture = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create3D(CameraAPLutSize, TextureAerialLUTFormat, FClearValueBinding::None, 
					TexCreate_ShaderResource | TexCreate_UAV), TEXT("SkyAtmosphere.RealTimeSkyCapCamera360APLut"));
			FRDGTextureUAVRef RealTimeReflectionCaptureSkyAtmosphereViewLutTextureUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(RealTimeReflectionCaptureCamera360APLutTexture, 0));

			FRenderCameraAerialPerspectiveVolumeCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSampleCloudSkyAO>(CloudShadowAOData.bShouldSampleCloudSkyAO);
			PermutationVector.Set<FSecondAtmosphereLight>(bSecondAtmosphereLightEnabled);
			PermutationVector.Set<FSampleOpaqueShadow>(false); // bShouldSampleOpaqueShadow); Off for now to not have to generate Light0Shadow and Light1Shadow
			PermutationVector.Set<FSampleCloudShadow>(bShouldSampleCloudShadow);
			PermutationVector.Set<FSeparateMieAndRayleighScattering>(false);	// Not for reflections for now
			PermutationVector.Set<FRealTimeReflectionLUT>(true);
			TShaderMapRef<FRenderCameraAerialPerspectiveVolumeCS> ComputeShader(ViewForReflection.ShaderMap, PermutationVector);

			FRenderCameraAerialPerspectiveVolumeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderCameraAerialPerspectiveVolumeCS::FParameters>();
			PassParameters->Atmosphere = Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereUniformBuffer();
			PassParameters->SkyAtmosphere = SkyInfo.GetInternalCommonParametersUniformBuffer();
			PassParameters->ViewUniformBuffer = ViewForReflection.ViewUniformBuffer;
			PassParameters->SkyAtmosphereRealTimeReflectionLUTUniformBuffer = SkyAtmosphereRealTimeReflectionLUTUB;
			PassParameters->TransmittanceLutTextureSampler = SamplerLinearClamp;
			PassParameters->MultiScatteredLuminanceLutTextureSampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudShadowMapTexture0Sampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudShadowMapTexture1Sampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudSkyAOTextureSampler = SamplerLinearClamp;
			PassParameters->TransmittanceLutTexture = TransmittanceLut;
			PassParameters->MultiScatteredLuminanceLutTexture = MultiScatteredLuminanceLut;
			PassParameters->VolumetricCloudShadowMapTexture0 = CloudShadowAOData.VolumetricCloudShadowMap[0];
			PassParameters->VolumetricCloudShadowMapTexture1 = CloudShadowAOData.VolumetricCloudShadowMap[1];
			PassParameters->VolumetricCloudSkyAOTexture = CloudShadowAOData.VolumetricCloudSkyAO;
			PassParameters->VolumetricCloudShadowStrength0 = CloudShadowOnAtmosphereStrength0;
			PassParameters->VolumetricCloudShadowStrength1 = CloudShadowOnAtmosphereStrength1;
			PassParameters->CameraAerialPerspectiveVolumeUAV = RealTimeReflectionCaptureSkyAtmosphereViewLutTextureUAV;
			PassParameters->AerialPerspectiveStartDepthKm = AerialPerspectiveStartDepthInCm * CM_TO_KM;
			PassParameters->RealTimeReflection360Mode = 1.0f;
			//PassParameters->Light0Shadow = nullptr;
			//PassParameters->Light1Shadow = nullptr;
			if (bShouldSampleCloudShadow || CloudShadowAOData.bShouldSampleCloudSkyAO)
			{
				PassParameters->VolumetricCloudCommonGlobalParams = CloudInfo->GetVolumetricCloudCommonShaderParametersUB();
			}

			FIntVector TextureSize = RealTimeReflectionCaptureCamera360APLutTexture->Desc.GetSize();
			const FIntVector NumGroups = FIntVector::DivideAndRoundUp(TextureSize, FRenderCameraAerialPerspectiveVolumeCS::GroupSize);
			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RealTimeCaptureCamera360VolumeLut"), PassFlag, ComputeShader, PassParameters, NumGroups);

			PendingRDGResources.RealTimeReflectionCaptureCamera360APLutTexture = RealTimeReflectionCaptureCamera360APLutTexture;
		}
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];
		const float AerialPerspectiveStartDepthInCm = GetValidAerialPerspectiveStartDepthInCm(View, SkyAtmosphereSceneProxy);
		const bool bLightDiskEnabled = !View.bIsReflectionCapture;

		FRDGTextureRef SkyAtmosphereViewLutTexture = GraphBuilder.RegisterExternalTexture(View.SkyAtmosphereViewLutTexture);
		FRDGTextureUAVRef SkyAtmosphereViewLutTextureUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SkyAtmosphereViewLutTexture, 0));
		FRDGTextureRef SkyAtmosphereCameraAerialPerspectiveVolume = GraphBuilder.RegisterExternalTexture(View.SkyAtmosphereCameraAerialPerspectiveVolume);
		FRDGTextureUAVRef SkyAtmosphereCameraAerialPerspectiveVolumeUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SkyAtmosphereCameraAerialPerspectiveVolume, 0));

		FRDGTextureRef SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly = nullptr;
		FRDGTextureUAVRef SkyAtmosphereCameraAerialPerspectiveVolumeMieOnlyUAV = nullptr;
		FRDGTextureRef SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly = nullptr;
		FRDGTextureUAVRef SkyAtmosphereCameraAerialPerspectiveVolumeRayOnlyUAV = nullptr;
		if (bSeparatedAtmosphereMieRayLeigh)
		{
			SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly = GraphBuilder.RegisterExternalTexture(View.SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly);
			SkyAtmosphereCameraAerialPerspectiveVolumeMieOnlyUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly, 0));

			SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly = GraphBuilder.RegisterExternalTexture(View.SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly);
			SkyAtmosphereCameraAerialPerspectiveVolumeRayOnlyUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly, 0));
		}

		TRDGUniformBufferRef<FVolumeShadowingShaderParametersGlobal0> LightShadowShaderParams0UniformBuffer{};
		TRDGUniformBufferRef<FVolumeShadowingShaderParametersGlobal1> LightShadowShaderParams1UniformBuffer{};
		GetSkyAtmosphereLightsUniformBuffers(GraphBuilder, LightShadowShaderParams0UniformBuffer, LightShadowShaderParams1UniformBuffer,
			LightShadowData, View, bShouldSampleOpaqueShadow, UniformBuffer_SingleFrame);

		FVolumetricCloudRenderSceneInfo* CloudInfo = Scene->GetVolumetricCloudSceneInfo();
		FCloudShadowAOData CloudShadowAOData;
		GetCloudShadowAOData(CloudInfo, View, GraphBuilder, CloudShadowAOData);
		const bool bShouldSampleCloudShadow = CloudShadowAOData.bShouldSampleCloudShadow && (CloudShadowOnAtmosphereStrength0 > 0.0f || CloudShadowOnAtmosphereStrength1 > 0.0f);

		// Sky View LUT
		{
			RDG_EVENT_SCOPE(GraphBuilder, "SkyAtmosphere::SkyViewLut");
			FRenderSkyViewLutCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSampleCloudSkyAO>(CloudShadowAOData.bShouldSampleCloudSkyAO);
			PermutationVector.Set<FSecondAtmosphereLight>(bSecondAtmosphereLightEnabled);
			PermutationVector.Set<FSampleOpaqueShadow>(bShouldSampleOpaqueShadow);
			PermutationVector.Set<FSampleCloudShadow>(bShouldSampleCloudShadow);
			PermutationVector.Set<FRealTimeReflectionLUT>(false);
			TShaderMapRef<FRenderSkyViewLutCS> ComputeShader(View.ShaderMap, PermutationVector);

			FRenderSkyViewLutCS::FParameters * PassParameters = GraphBuilder.AllocParameters<FRenderSkyViewLutCS::FParameters>();
			PassParameters->Atmosphere = Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereUniformBuffer();
			PassParameters->SkyAtmosphere = SkyInfo.GetInternalCommonParametersUniformBuffer();
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->SkyAtmosphereRealTimeReflectionLUTUniformBuffer = GetDefaultSkyAtmosphereRealTimeReflectionLUTUniformBuffer();
			PassParameters->TransmittanceLutTextureSampler = SamplerLinearClamp;
			PassParameters->MultiScatteredLuminanceLutTextureSampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudShadowMapTexture0Sampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudShadowMapTexture1Sampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudSkyAOTextureSampler = SamplerLinearClamp;
			PassParameters->TransmittanceLutTexture = TransmittanceLut;
			PassParameters->MultiScatteredLuminanceLutTexture = MultiScatteredLuminanceLut;
			PassParameters->VolumetricCloudShadowMapTexture0 = CloudShadowAOData.VolumetricCloudShadowMap[0];
			PassParameters->VolumetricCloudShadowMapTexture1 = CloudShadowAOData.VolumetricCloudShadowMap[1];
			PassParameters->VolumetricCloudSkyAOTexture = CloudShadowAOData.VolumetricCloudSkyAO;
			PassParameters->VolumetricCloudShadowStrength0 = CloudShadowOnAtmosphereStrength0;
			PassParameters->VolumetricCloudShadowStrength1 = CloudShadowOnAtmosphereStrength1;
			PassParameters->SkyViewLutUAV = SkyAtmosphereViewLutTextureUAV;
			PassParameters->Light0Shadow = LightShadowShaderParams0UniformBuffer;
			PassParameters->Light1Shadow = LightShadowShaderParams1UniformBuffer;
			PassParameters->VirtualShadowMap = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
			PassParameters->VirtualShadowMapId0 = LightShadowData.VirtualShadowMapId0;
			PassParameters->VirtualShadowMapId1 = LightShadowData.VirtualShadowMapId1;
			if (bShouldSampleCloudShadow || CloudShadowAOData.bShouldSampleCloudSkyAO)
			{
				PassParameters->VolumetricCloudCommonGlobalParams = CloudInfo->GetVolumetricCloudCommonShaderParametersUB();
			}
			PassParameters->SourceDiskEnabled = bLightDiskEnabled ? 1 : 0;

			FIntVector TextureSize = SkyAtmosphereViewLutTexture->Desc.GetSize();
			TextureSize.Z = 1;
			const FIntVector NumGroups = FIntVector::DivideAndRoundUp(TextureSize, FRenderSkyViewLutCS::GroupSize);
			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("SkyViewLut"), PassFlag, ComputeShader, PassParameters, NumGroups);
		}

		// Camera Atmosphere Volume
		{
			RDG_EVENT_SCOPE(GraphBuilder, "SkyAtmosphere::CameraVolumeLut");
			FRenderCameraAerialPerspectiveVolumeCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FSampleCloudSkyAO>(CloudShadowAOData.bShouldSampleCloudSkyAO);
			PermutationVector.Set<FSecondAtmosphereLight>(bSecondAtmosphereLightEnabled);
			PermutationVector.Set<FSampleOpaqueShadow>(bShouldSampleOpaqueShadow);
			PermutationVector.Set<FSampleCloudShadow>(bShouldSampleCloudShadow);
			PermutationVector.Set<FSeparateMieAndRayleighScattering>(bSeparatedAtmosphereMieRayLeigh);
			PermutationVector.Set<FRealTimeReflectionLUT>(false);
			TShaderMapRef<FRenderCameraAerialPerspectiveVolumeCS> ComputeShader(View.ShaderMap, PermutationVector);

			FRenderCameraAerialPerspectiveVolumeCS::FParameters * PassParameters = GraphBuilder.AllocParameters<FRenderCameraAerialPerspectiveVolumeCS::FParameters>();
			PassParameters->Atmosphere = Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereUniformBuffer();
			PassParameters->SkyAtmosphere = SkyInfo.GetInternalCommonParametersUniformBuffer();
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->SkyAtmosphereRealTimeReflectionLUTUniformBuffer = GetDefaultSkyAtmosphereRealTimeReflectionLUTUniformBuffer();
			PassParameters->TransmittanceLutTextureSampler = SamplerLinearClamp;
			PassParameters->MultiScatteredLuminanceLutTextureSampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudShadowMapTexture0Sampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudShadowMapTexture1Sampler = SamplerLinearClamp;
			PassParameters->VolumetricCloudSkyAOTextureSampler = SamplerLinearClamp;
			PassParameters->TransmittanceLutTexture = TransmittanceLut;
			PassParameters->MultiScatteredLuminanceLutTexture = MultiScatteredLuminanceLut;
			PassParameters->VolumetricCloudShadowMapTexture0 = CloudShadowAOData.VolumetricCloudShadowMap[0];
			PassParameters->VolumetricCloudShadowMapTexture1 = CloudShadowAOData.VolumetricCloudShadowMap[1];
			PassParameters->VolumetricCloudSkyAOTexture = CloudShadowAOData.VolumetricCloudSkyAO;
			PassParameters->VolumetricCloudShadowStrength0 = CloudShadowOnAtmosphereStrength0;
			PassParameters->VolumetricCloudShadowStrength1 = CloudShadowOnAtmosphereStrength1;
			PassParameters->CameraAerialPerspectiveVolumeUAV = SkyAtmosphereCameraAerialPerspectiveVolumeUAV;
			PassParameters->CameraAerialPerspectiveVolumeMieOnlyUAV = SkyAtmosphereCameraAerialPerspectiveVolumeMieOnlyUAV;
			PassParameters->CameraAerialPerspectiveVolumeRayOnlyUAV = SkyAtmosphereCameraAerialPerspectiveVolumeRayOnlyUAV;

			PassParameters->AerialPerspectiveStartDepthKm = AerialPerspectiveStartDepthInCm * CM_TO_KM;
			PassParameters->RealTimeReflection360Mode = 0.0f;
			PassParameters->Light0Shadow = LightShadowShaderParams0UniformBuffer;
			PassParameters->Light1Shadow = LightShadowShaderParams1UniformBuffer;
			PassParameters->VirtualShadowMap = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
			PassParameters->VirtualShadowMapId0 = LightShadowData.VirtualShadowMapId0;
			PassParameters->VirtualShadowMapId1 = LightShadowData.VirtualShadowMapId1;
			if (bShouldSampleCloudShadow || CloudShadowAOData.bShouldSampleCloudSkyAO)
			{
				PassParameters->VolumetricCloudCommonGlobalParams = CloudInfo->GetVolumetricCloudCommonShaderParametersUB();
			}

			FIntVector TextureSize = SkyAtmosphereCameraAerialPerspectiveVolume->Desc.GetSize();
			const FIntVector NumGroups = FIntVector::DivideAndRoundUp(TextureSize, FRenderCameraAerialPerspectiveVolumeCS::GroupSize);
			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("CameraVolumeLut"), PassFlag, ComputeShader, PassParameters, NumGroups);
		}

		PendingRDGResources.ViewResources[ViewIndex].SkyAtmosphereViewLutTexture = SkyAtmosphereViewLutTexture;
		PendingRDGResources.ViewResources[ViewIndex].SkyAtmosphereCameraAerialPerspectiveVolume = SkyAtmosphereCameraAerialPerspectiveVolume;
	}

	PendingRDGResources.TransmittanceLut = TransmittanceLut;
}

void FSkyAtmospherePendingRDGResources::CommitToSceneAndViewUniformBuffers(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue) const
{
	check(SceneRenderer);
	FScene* Scene = SceneRenderer->Scene;
	FSkyAtmosphereRenderSceneInfo& SkyInfo = *Scene->GetSkyAtmosphereSceneInfo();

	if (DistantSkyLightLutBuffer && CVarSkyAtmosphereDistantSkyLightLUT.GetValueOnRenderThread() > 0)
	{
		SkyInfo.GetDistantSkyLightLutBuffer() = ConvertToExternalAccessBuffer(GraphBuilder, ExternalAccessQueue, DistantSkyLightLutBuffer, ERHIAccess::SRVMask, ERHIPipeline::All);
	}
	if (MobileDistantSkyLightLutBuffer && CVarSkyAtmosphereDistantSkyLightLUT.GetValueOnRenderThread() > 0)
	{
		SkyInfo.GetMobileDistantSkyLightLutBuffer() = ConvertToExternalAccessBuffer(GraphBuilder, ExternalAccessQueue, MobileDistantSkyLightLutBuffer, ERHIAccess::SRVMask, ERHIPipeline::All);
	}

	if (RealTimeReflectionCaptureSkyAtmosphereViewLutTexture)
	{
		Scene->RealTimeReflectionCaptureSkyAtmosphereViewLutTexture = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, RealTimeReflectionCaptureSkyAtmosphereViewLutTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
	}
	else
	{
		Scene->RealTimeReflectionCaptureSkyAtmosphereViewLutTexture = nullptr;
	}

	if (RealTimeReflectionCaptureCamera360APLutTexture)
	{
		Scene->RealTimeReflectionCaptureCamera360APLutTexture = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, RealTimeReflectionCaptureCamera360APLutTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
	}
	else
	{
		Scene->RealTimeReflectionCaptureCamera360APLutTexture = nullptr;
	}

	for (int32 ViewIndex = 0; ViewIndex < SceneRenderer->Views.Num(); ViewIndex++)
	{
		FViewInfo& View = SceneRenderer->Views[ViewIndex];
		View.SkyAtmosphereViewLutTexture = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, ViewResources[ViewIndex].SkyAtmosphereViewLutTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
		View.SkyAtmosphereCameraAerialPerspectiveVolume = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, ViewResources[ViewIndex].SkyAtmosphereCameraAerialPerspectiveVolume, ERHIAccess::SRVMask, ERHIPipeline::All);
	}

	// We always need to run the conversion even if TransmittanceLut has not been written to, as long as RegisterExternalTexture has been called on it. This in order to finish any asynchronous resource transition.
	check(TransmittanceLut);
	SkyInfo.GetTransmittanceLutTexture() = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, TransmittanceLut, ERHIAccess::SRVMask, ERHIPipeline::All);
}

FSkyAtmosphereRenderContext::FSkyAtmosphereRenderContext()
{
	bAPOnCloudMode = false;
	VolumetricCloudDepthTexture = nullptr;
	InputCloudLuminanceTransmittanceTexture = nullptr;

	MSAASampleCount = 1;
	MSAADepthTexture = nullptr;
}

void FSceneRenderer::RenderSkyAtmosphereInternal(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureShaderParameters& SceneTextures,
	FSkyAtmosphereRenderContext& SkyRC)
{
	check(Scene->HasSkyAtmosphere());

	FSkyAtmosphereRenderSceneInfo& SkyInfo = *Scene->GetSkyAtmosphereSceneInfo();
	const FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy = SkyInfo.GetSkyAtmosphereSceneProxy();
	const FAtmosphereSetup& Atmosphere = SkyAtmosphereSceneProxy.GetAtmosphereSetup();

	const FViewMatrices& ViewMatrices = *SkyRC.ViewMatrices;

	FRHISamplerState* SamplerLinearClamp = TStaticSamplerState<SF_Trilinear>::GetRHI();
	const float AerialPerspectiveStartDepthInCm = SkyRC.AerialPerspectiveStartDepthInCm;

	const FVector3f ViewOrigin = (FVector3f)ViewMatrices.GetViewOrigin();
	const FVector3f PlanetCenter = (FVector3f)Atmosphere.PlanetCenterKm * KM_TO_CM;	// LWC_TODO: Precision Loss
	const float TopOfAtmosphere = Atmosphere.TopRadiusKm * KM_TO_CM;
	const float PLANET_RADIUS_RATIO_SAFE_EDGE = 1.00000155763f; // must match PLANET_RADIUS_SAFE_TRACE_EDGE
	const bool ForceRayMarching = SkyRC.bForceRayMarching || (FVector3f::Distance(ViewOrigin, PlanetCenter) >= (TopOfAtmosphere * PLANET_RADIUS_RATIO_SAFE_EDGE));
	const bool bDisableBlending = SkyRC.bDisableBlending;

	// We only support MSAA up to 8 sample and in forward
	check(SkyRC.MSAASampleCount <= 8);
	// We only support MSAA in forward, not in deferred.
	const bool bForwardShading = IsForwardShadingEnabled(Scene->GetShaderPlatform());
	check(bForwardShading || (!bForwardShading && SkyRC.MSAASampleCount == 1));

	// Render the sky, and optionally the atmosphere aerial perspective, on the scene luminance buffer
	{
		FLightSceneProxy* AtmosphereLight0Proxy = Scene->AtmosphereLights[0] ? Scene->AtmosphereLights[0]->Proxy : nullptr;
		FLightSceneProxy* AtmosphereLight1Proxy = Scene->AtmosphereLights[1] ? Scene->AtmosphereLights[1]->Proxy : nullptr;
		const float CloudShadowOnAtmosphereStrength0 = AtmosphereLight0Proxy ? AtmosphereLight0Proxy->GetCloudShadowOnAtmosphereStrength() : 0.0f;
		const float CloudShadowOnAtmosphereStrength1 = AtmosphereLight1Proxy ? AtmosphereLight1Proxy->GetCloudShadowOnAtmosphereStrength() : 0.0f;
		const bool bShouldSampleCloudShadow = SkyRC.bShouldSampleCloudShadow && (CloudShadowOnAtmosphereStrength0 > 0.0f || CloudShadowOnAtmosphereStrength1 > 0.0f);

		const bool bFastAerialPerspectiveDepthTest = SkyRC.bFastAerialPerspectiveDepthTest;
		const bool SkyAtmosphereOutputsAlpha = IsPostProcessingWithAlphaChannelSupported();
		const bool SkyAtmosphereAlphaHoldOut = SkyAtmosphereOutputsAlpha && SkyAtmosphereSceneProxy.IsHoldout();
		const bool bRenderSkyPixel = SkyRC.bRenderSkyPixel || (SkyAtmosphereOutputsAlpha && !SkyRC.bSceneHasSkyMaterial);	// In this case we need to write alpha holdout values in the sky pixels. If there is no IsSky dmoe meshes.

		FRenderSkyAtmospherePS::FPermutationDomain PsPermutationVector;
		PsPermutationVector.Set<FSampleCloudSkyAO>(SkyRC.bShouldSampleCloudSkyAO);
		PsPermutationVector.Set<FFastSky>(SkyRC.bFastSky && !ForceRayMarching);
		PsPermutationVector.Set<FFastAerialPespective>(SkyRC.bFastAerialPerspective && !ForceRayMarching);
		PsPermutationVector.Set<FSecondAtmosphereLight>(SkyRC.bSecondAtmosphereLightEnabled);
		PsPermutationVector.Set<FRenderSky>(bRenderSkyPixel);
		PsPermutationVector.Set<FSampleOpaqueShadow>(SkyRC.bShouldSampleOpaqueShadow);
		PsPermutationVector.Set<FSampleCloudShadow>(bShouldSampleCloudShadow);
		PsPermutationVector.Set<FAtmosphereOnClouds>(SkyRC.bAPOnCloudMode);
		PsPermutationVector.Set<FMSAASampleCount>(SkyRC.MSAASampleCount);
		PsPermutationVector = FRenderSkyAtmospherePS::RemapPermutation(PsPermutationVector);
		TShaderMapRef<FRenderSkyAtmospherePS> PixelShader(GetGlobalShaderMap(SkyRC.FeatureLevel), PsPermutationVector);

		FRenderSkyAtmosphereVS::FPermutationDomain VsPermutationVector;
		TShaderMapRef<FRenderSkyAtmosphereVS> VertexShader(GetGlobalShaderMap(SkyRC.FeatureLevel), VsPermutationVector);

		FRenderSkyAtmospherePS::FParameters* PsPassParameters = GraphBuilder.AllocParameters<FRenderSkyAtmospherePS::FParameters>();
		PsPassParameters->Atmosphere = Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereUniformBuffer();
		PsPassParameters->SkyAtmosphere = SkyInfo.GetInternalCommonParametersUniformBuffer();
		PsPassParameters->ViewUniformBuffer = SkyRC.ViewUniformBuffer;
		PsPassParameters->Scene = SkyRC.SceneUniformBuffer;
		PsPassParameters->RenderTargets = SkyRC.RenderTargets;
		PsPassParameters->SceneTextures = SceneTextures;
		PsPassParameters->MSAADepthTexture = SkyRC.MSAADepthTexture;
		PsPassParameters->TransmittanceLutTextureSampler = SamplerLinearClamp;
		PsPassParameters->MultiScatteredLuminanceLutTextureSampler = SamplerLinearClamp;
		PsPassParameters->SkyViewLutTextureSampler = SamplerLinearClamp;
		PsPassParameters->CameraAerialPerspectiveVolumeTextureSampler = SamplerLinearClamp;
		PsPassParameters->VolumetricCloudShadowMapTexture0Sampler = SamplerLinearClamp;
		PsPassParameters->VolumetricCloudShadowMapTexture1Sampler = SamplerLinearClamp;
		PsPassParameters->VolumetricCloudSkyAOTextureSampler = SamplerLinearClamp;
		PsPassParameters->TransmittanceLutTexture = SkyRC.TransmittanceLut;
		PsPassParameters->MultiScatteredLuminanceLutTexture = SkyRC.MultiScatteredLuminanceLut;
		PsPassParameters->SkyViewLutTexture = SkyRC.SkyAtmosphereViewLutTexture;
		PsPassParameters->CameraAerialPerspectiveVolumeTexture = SkyRC.SkyAtmosphereCameraAerialPerspectiveVolume;
		PsPassParameters->VolumetricCloudShadowMapTexture0 = SkyRC.VolumetricCloudShadowMap[0];
		PsPassParameters->VolumetricCloudShadowMapTexture1 = SkyRC.VolumetricCloudShadowMap[1];
		PsPassParameters->VolumetricCloudSkyAOTexture = SkyRC.VolumetricCloudSkyAO;
		PsPassParameters->VolumetricCloudShadowStrength0 = CloudShadowOnAtmosphereStrength0;
		PsPassParameters->VolumetricCloudShadowStrength1 = CloudShadowOnAtmosphereStrength1;
		PsPassParameters->VolumetricCloudDepthTexture = SkyRC.VolumetricCloudDepthTexture;
		PsPassParameters->InputCloudLuminanceTransmittanceTexture = SkyRC.InputCloudLuminanceTransmittanceTexture;
		PsPassParameters->AerialPerspectiveStartDepthKm = AerialPerspectiveStartDepthInCm * CM_TO_KM;
		PsPassParameters->bPropagateAlphaNonReflection = (IsPostProcessingWithAlphaChannelSupported() && !SkyRC.bIsReflectionCapture) ? 1 : 0;
		PsPassParameters->SourceDiskEnabled = SkyRC.bIsReflectionCapture ? 0 : 1;
		PsPassParameters->DepthReadDisabled = SkyRC.bDepthReadDisabled ? 1 : 0;
		if (bShouldSampleCloudShadow || SkyRC.bShouldSampleCloudSkyAO)
		{
			PsPassParameters->VolumetricCloudCommonGlobalParams = Scene->GetVolumetricCloudSceneInfo()->GetVolumetricCloudCommonShaderParametersUB();
		}

		PsPassParameters->Light0Shadow = SkyRC.LightShadowShaderParams0UniformBuffer;
		PsPassParameters->Light1Shadow = SkyRC.LightShadowShaderParams1UniformBuffer;

		PsPassParameters->VirtualShadowMap = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, 0);
		PsPassParameters->VirtualShadowMapId0 = SkyRC.VirtualShadowMapId0;
		PsPassParameters->VirtualShadowMapId1 = SkyRC.VirtualShadowMapId1;

		ClearUnusedGraphResources(PixelShader, PsPassParameters);

		float StartDepthZ = 0.1f;
		if (SkyRC.bFastAerialPerspectiveDepthTest)
		{
			const FMatrix ProjectionMatrix = ViewMatrices.GetViewToClip();
			float StartDepthViewCm;
			if (ViewMatrices.IsPerspectiveProjection())
			{
				float HalfHorizontalFOV = FMath::Atan(1.0f / ProjectionMatrix.M[0][0]);
				float HalfVerticalFOV = FMath::Atan(1.0f / ProjectionMatrix.M[1][1]);
				StartDepthViewCm = FMath::Cos(FMath::Max(HalfHorizontalFOV, HalfVerticalFOV)) * AerialPerspectiveStartDepthInCm;
			}
			else
			{
				StartDepthViewCm = AerialPerspectiveStartDepthInCm;
			}
			StartDepthViewCm = FMath::Max(StartDepthViewCm, SkyRC.NearClippingDistance); // In any case, we need to limit the distance to frustum near plane to not be clipped away.
			const FVector4 Projected = ProjectionMatrix.TransformFVector4(FVector4(0.0f, 0.0f, StartDepthViewCm, 1.0f));
			StartDepthZ = float(Projected.Z / Projected.W); // LWC_TODO: precision loss
		}

		FIntRect Viewport = SkyRC.Viewport;
		GraphBuilder.AddPass(
			{},
			PsPassParameters,
			ERDGPassFlags::Raster,
			[PsPassParameters, VertexShader, PixelShader, Viewport, bFastAerialPerspectiveDepthTest, bRenderSkyPixel, bDisableBlending, StartDepthZ, SkyAtmosphereOutputsAlpha, SkyAtmosphereAlphaHoldOut](FRDGAsyncTask, FRHICommandList& RHICmdListLambda)
		{
			RHICmdListLambda.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdListLambda.ApplyCachedRenderTargets(GraphicsPSOInit);

			if (bDisableBlending)
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			}
			else
			{
				if (SkyAtmosphereOutputsAlpha)
				{
					if (SkyAtmosphereAlphaHoldOut)
					{
						// We might need to write alpha holdout data. Since this is the sky writing to 
						GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI();
					}
					else
					{
						// Maintain alpha untouched
						GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
					}
				}
				else
				{
					// Disable alpha writes 
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha>::GetRHI();
				}
			}
			if (bFastAerialPerspectiveDepthTest)
			{
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
			}
			else
			{
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			}
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			if (!bRenderSkyPixel && GSupportsDepthBoundsTest)
			{
				// We do not want to process sky pixels so we take advantage of depth bound test when available to skip launching pointless GPU wavefront/work.
				GraphicsPSOInit.bDepthBounds = true;

				FDepthBounds::FDepthBoundsValues Values = FDepthBounds::CalculateNearFarDepthExcludingSky();
				RHICmdListLambda.SetDepthBounds(Values.MinDepth, Values.MaxDepth);
			}

			SetGraphicsPipelineState(RHICmdListLambda, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdListLambda, PixelShader, PixelShader.GetPixelShader(), *PsPassParameters);

			FRenderSkyAtmosphereVS::FParameters VsPassParameters;
			VsPassParameters.StartDepthZ = StartDepthZ;
			SetShaderParameters(RHICmdListLambda, VertexShader, VertexShader.GetVertexShader(), VsPassParameters);

			RHICmdListLambda.DrawPrimitive(0, 1, 1);
		});
	}
}



void FSceneRenderer::RenderSkyAtmosphere(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures)
{
	check(!IsMobilePlatform(Scene->GetShaderPlatform()));

	check(ShouldRenderSkyAtmosphere(Scene, ViewFamily.EngineShowFlags)); // This should not be called if we should not render SkyAtmosphere
	check(ViewFamily.EngineShowFlags.DeferredAtmospherePass);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, SkyAtmosphere, "SkyAtmosphere");
	SCOPED_NAMED_EVENT(SkyAtmosphere, FColor::Emerald);

	FSkyAtmosphereRenderSceneInfo& SkyInfo = *Scene->GetSkyAtmosphereSceneInfo();
	const FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy = SkyInfo.GetSkyAtmosphereSceneProxy();
	FVolumetricCloudRenderSceneInfo* CloudInfo = Scene->GetVolumetricCloudSceneInfo();

	FSkyAtmosphereRenderContext SkyRC;
	SkyRC.ViewMatrices = nullptr;

	SkyRC.SceneUniformBuffer = GetSceneUniforms().GetBuffer(GraphBuilder);

	const FAtmosphereSetup& Atmosphere = SkyAtmosphereSceneProxy.GetAtmosphereSetup();
	SkyRC.bFastSky = CVarSkyAtmosphereFastSkyLUT.GetValueOnRenderThread() > 0;
	SkyRC.bFastAerialPerspective = CVarSkyAtmosphereAerialPerspectiveApplyOnOpaque.GetValueOnRenderThread() > 0;
	SkyRC.bFastAerialPerspectiveDepthTest = CVarSkyAtmosphereAerialPerspectiveDepthTest.GetValueOnRenderThread() > 0;
	SkyRC.bSecondAtmosphereLightEnabled = Scene->IsSecondAtmosphereLightEnabled();

	SkyAtmosphereLightShadowData LightShadowData;
	SkyRC.bShouldSampleOpaqueShadow = ShouldSkySampleAtmosphereLightsOpaqueShadow(*Scene, VisibleLightInfos, LightShadowData);
	SkyRC.bUseDepthBoundTestIfPossible = true;
	SkyRC.bForceRayMarching = false;
	SkyRC.bDepthReadDisabled = false;
	SkyRC.bDisableBlending = false;

	SkyRC.TransmittanceLut = GraphBuilder.RegisterExternalTexture(SkyInfo.GetTransmittanceLutTexture());
	SkyRC.MultiScatteredLuminanceLut = GraphBuilder.RegisterExternalTexture(SkyInfo.GetMultiScatteredLuminanceLutTexture());

	SkyRC.RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
	SkyRC.RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);

	SkyRC.MSAASampleCount = SceneTextures.Depth.Target->Desc.NumSamples;
	SkyRC.MSAADepthTexture = SceneTextures.Depth.Target;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		FViewInfo& View = Views[ViewIndex];
		SkyRC.ViewMatrices = &View.ViewMatrices;
		SkyRC.ViewUniformBuffer = View.ViewUniformBuffer;
		SkyRC.bSceneHasSkyMaterial = View.bSceneHasSkyMaterial;

		SkyRC.Viewport = View.ViewRect;
		SkyRC.bIsReflectionCapture = View.bIsReflectionCapture;
		SkyRC.AerialPerspectiveStartDepthInCm = GetValidAerialPerspectiveStartDepthInCm(View, SkyAtmosphereSceneProxy);
		SkyRC.NearClippingDistance = View.NearClippingDistance;
		SkyRC.FeatureLevel = View.FeatureLevel;

		// If the scene contains Sky material then it is likely rendering the sky using a sky dome mesh.
		// In this case we can use a simpler shader during this pass to only render aerial perspective
		// and sky pixels can likely be optimised out.
		SkyRC.bRenderSkyPixel = !View.bSceneHasSkyMaterial;

		SkyRC.SkyAtmosphereViewLutTexture = GraphBuilder.RegisterExternalTexture(View.SkyAtmosphereViewLutTexture);
		SkyRC.SkyAtmosphereCameraAerialPerspectiveVolume = GraphBuilder.RegisterExternalTexture(View.SkyAtmosphereCameraAerialPerspectiveVolume);

		GetSkyAtmosphereLightsUniformBuffers(GraphBuilder, SkyRC.LightShadowShaderParams0UniformBuffer, SkyRC.LightShadowShaderParams1UniformBuffer,
			LightShadowData, View, SkyRC.bShouldSampleOpaqueShadow, UniformBuffer_SingleDraw);
		SkyRC.VirtualShadowMapId0 = LightShadowData.VirtualShadowMapId0;
		SkyRC.VirtualShadowMapId1 = LightShadowData.VirtualShadowMapId1;

		FCloudShadowAOData CloudShadowAOData;
		GetCloudShadowAOData(CloudInfo, View, GraphBuilder, CloudShadowAOData);
		SkyRC.bShouldSampleCloudShadow = CloudShadowAOData.bShouldSampleCloudShadow;
		SkyRC.VolumetricCloudShadowMap[0] = CloudShadowAOData.VolumetricCloudShadowMap[0];
		SkyRC.VolumetricCloudShadowMap[1] = CloudShadowAOData.VolumetricCloudShadowMap[1];
		SkyRC.bShouldSampleCloudSkyAO = CloudShadowAOData.bShouldSampleCloudSkyAO;
		SkyRC.VolumetricCloudSkyAO = CloudShadowAOData.VolumetricCloudSkyAO;

		RenderSkyAtmosphereInternal(GraphBuilder, GetSceneTextureShaderParameters(SceneTextures.UniformBuffer), SkyRC);
	}

#if WITH_EDITOR
	if (CVarSkyAtmosphereFastSkyLUT.GetValueOnAnyThread() == 0 && CVarSkyAtmosphereAerialPerspectiveApplyOnOpaque.GetValueOnAnyThread() > 0)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];

			AddDrawCanvasPass(GraphBuilder, {}, View, FScreenPassRenderTarget(SceneTextures.Color.Target, View.ViewRect, ERenderTargetLoadAction::ELoad),
				[&View](FCanvas& Canvas)
			{
				const float ViewPortWidth = float(View.ViewRect.Width());
				const float ViewPortHeight = float(View.ViewRect.Height());
				FLinearColor TextColor(1.0f, 0.5f, 0.0f);
				FString Text = "You are using a FastAerialPespective without FastSky, visuals might look wrong.";
				Canvas.DrawShadowedString(ViewPortWidth * 0.5f - Text.Len() * 7.0f, ViewPortHeight * 0.4f, *Text, GetStatsFont(), TextColor);
			});
		}
	}
#endif
}

bool FSceneRenderer::ShouldRenderSkyAtmosphereEditorNotifications(TArrayView<FViewInfo> InViews)
{
#if WITH_EDITOR
	if (CVarSkyAtmosphereEditorNotifications.GetValueOnAnyThread() > 0)
	{
		bool bAnyViewHasSkyMaterial = false;
		for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ViewIndex++)
		{
			const FViewInfo& View = InViews[ViewIndex];
			bAnyViewHasSkyMaterial |= (View.bSceneHasSkyMaterial && View.Family->EngineShowFlags.Atmosphere);
		}
		return bAnyViewHasSkyMaterial;
	}
#endif
	return false;
}

void FSceneRenderer::RenderSkyAtmosphereEditorNotifications(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, FRDGTextureRef SceneColorTexture) const
{
#if WITH_EDITOR
	RDG_EVENT_SCOPE_STAT(GraphBuilder, SkyAtmosphereEditor, "SkyAtmosphereEditor");

	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ViewIndex++)
	{
		const FViewInfo& View = InViews[ViewIndex];
		if (View.bSceneHasSkyMaterial)
		{
			RenderSkyAtmosphereEditorHudPS::FPermutationDomain PermutationVector;
			TShaderMapRef<RenderSkyAtmosphereEditorHudPS> PixelShader(View.ShaderMap, PermutationVector);

			RenderSkyAtmosphereEditorHudPS::FParameters* PassParameters = GraphBuilder.AllocParameters<RenderSkyAtmosphereEditorHudPS::FParameters>();
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->MiniFontTexture = GSystemTextures.AsciiTexture->GetRHI();

			FPixelShaderUtils::AddFullscreenPass<RenderSkyAtmosphereEditorHudPS>(GraphBuilder, View.ShaderMap, RDG_EVENT_NAME("SkyAtmosphereEditor"), PixelShader, PassParameters, View.ViewRect);
		}
	}
#endif
}



bool ShouldRenderSkyAtmosphereDebugPasses(const FScene* Scene, const FEngineShowFlags& EngineShowFlags)
{
	return EngineShowFlags.VisualizeSkyAtmosphere && ShouldRenderSkyAtmosphere(Scene, EngineShowFlags);
}

FScreenPassTexture AddSkyAtmosphereDebugPasses(FRDGBuilder& GraphBuilder, FScene* Scene, const FSceneViewFamily& ViewFamily, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor)
{
#if WITH_EDITOR
	check(ShouldRenderSkyAtmosphere(Scene, ViewFamily.EngineShowFlags)); // This should not be called if we should not render SkyAtmosphere

	RDG_EVENT_SCOPE_STAT(GraphBuilder, SkyAtmosphereDebugVisualize, "SkyAtmosphereDebugVisualize");

	const bool bSkyAtmosphereVisualizeShowFlag = ViewFamily.EngineShowFlags.VisualizeSkyAtmosphere;
	FSkyAtmosphereRenderSceneInfo& SkyInfo = *Scene->GetSkyAtmosphereSceneInfo();
	const FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy = SkyInfo.GetSkyAtmosphereSceneProxy();

	const FAtmosphereSetup& Atmosphere = SkyAtmosphereSceneProxy.GetAtmosphereSetup();

	if (bSkyAtmosphereVisualizeShowFlag)
	{
		FRDGTextureRef TransmittanceLut = GraphBuilder.RegisterExternalTexture(SkyInfo.GetTransmittanceLutTexture());
		FRDGTextureRef MultiScatteredLuminanceLut = GraphBuilder.RegisterExternalTexture(SkyInfo.GetMultiScatteredLuminanceLutTexture());

		FRHIBlendState* PreMultipliedColorTransmittanceBlend = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
		FRHIDepthStencilState* DepthStencilStateWrite = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		FRHISamplerState* SamplerLinearClamp = TStaticSamplerState<SF_Trilinear>::GetRHI();

		// Render the sky and atmosphere on the scene luminance buffer
		{
			FRenderDebugSkyAtmospherePS::FPermutationDomain PermutationVector;
			TShaderMapRef<FRenderDebugSkyAtmospherePS> PixelShader(View.ShaderMap, PermutationVector);

			FRenderDebugSkyAtmospherePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderDebugSkyAtmospherePS::FParameters>();
			PassParameters->Atmosphere = SkyInfo.GetAtmosphereUniformBuffer();
			PassParameters->SkyAtmosphere = SkyInfo.GetInternalCommonParametersUniformBuffer();
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenPassSceneColor.Texture, ERenderTargetLoadAction::ELoad);
			PassParameters->TransmittanceLutTextureSampler = SamplerLinearClamp;
			PassParameters->MultiScatteredLuminanceLutTextureSampler = SamplerLinearClamp;
			PassParameters->TransmittanceLutTexture = TransmittanceLut;
			PassParameters->MultiScatteredLuminanceLutTexture = MultiScatteredLuminanceLut;
			PassParameters->ViewPortWidth = float(View.ViewRect.Width());
			PassParameters->ViewPortHeight = float(View.ViewRect.Height());

			FPixelShaderUtils::AddFullscreenPass<FRenderDebugSkyAtmospherePS>(GraphBuilder, View.ShaderMap, RDG_EVENT_NAME("SkyAtmosphere"), PixelShader, PassParameters,
				View.ViewRect, PreMultipliedColorTransmittanceBlend, nullptr, DepthStencilStateWrite);
		}
	}

	// Now debug print
	AddDrawCanvasPass(GraphBuilder, {}, View, FScreenPassRenderTarget(ScreenPassSceneColor, ERenderTargetLoadAction::ELoad),
		[bSkyAtmosphereVisualizeShowFlag, &View, &Atmosphere](FCanvas& Canvas)
	{
		FLinearColor TextColor(FLinearColor::White);
		FLinearColor GrayTextColor(FLinearColor::Gray);
		FLinearColor WarningColor(1.0f, 0.5f, 0.0f);
		FString Text;

		const float ViewPortWidth = float(View.ViewRect.Width());
		const float ViewPortHeight = float(View.ViewRect.Height());

		if (bSkyAtmosphereVisualizeShowFlag)
		{
			const float ViewPlanetAltitude = (View.ViewLocation * FAtmosphereSetup::CmToSkyUnit - (FVector)Atmosphere.PlanetCenterKm).Size() - Atmosphere.BottomRadiusKm;
			const bool bViewUnderGroundLevel = ViewPlanetAltitude < 0.0f;
			if (bViewUnderGroundLevel)
			{
				Text = FString::Printf(TEXT("SkyAtmosphere: View is %.3f km under the planet ground level!"), -ViewPlanetAltitude);
				Canvas.DrawShadowedString(ViewPortWidth * 0.5 - 250.0f, ViewPortHeight * 0.5f, *Text, GetStatsFont(), WarningColor);
			}

			// This needs to stay in sync with RenderSkyAtmosphereDebugPS.
			const float DensityViewTop = ViewPortHeight * 0.1f;
			const float DensityViewBottom = ViewPortHeight * 0.8f;
			const float DensityViewLeft = ViewPortWidth * 0.8f;
			const float Margin = 2.0f;
			const float TimeOfDayViewHeight = 64.0f;
			const float TimeOfDayViewTop = ViewPortHeight - (TimeOfDayViewHeight + Margin * 2.0);
			const float HemiViewHeight = ViewPortWidth * 0.25f;
			const float HemiViewTop = ViewPortHeight - HemiViewHeight - TimeOfDayViewHeight - Margin * 2.0;

			Text = FString::Printf(TEXT("Atmosphere top = %.1f km"), Atmosphere.TopRadiusKm - Atmosphere.BottomRadiusKm);
			Canvas.DrawShadowedString(DensityViewLeft, DensityViewTop, *Text, GetStatsFont(), TextColor);
			Text = FString::Printf(TEXT("Rayleigh extinction"));
			Canvas.DrawShadowedString(DensityViewLeft + 60.0f, DensityViewTop + 30.0f, *Text, GetStatsFont(), FLinearColor(FLinearColor::Red));
			Text = FString::Printf(TEXT("Mie extinction"));
			Canvas.DrawShadowedString(DensityViewLeft + 60.0f, DensityViewTop + 45.0f, *Text, GetStatsFont(), FLinearColor(FLinearColor::Green));
			Text = FString::Printf(TEXT("Absorption"));
			Canvas.DrawShadowedString(DensityViewLeft + 60.0f, DensityViewTop + 60.0f, *Text, GetStatsFont(), FLinearColor(FLinearColor::Blue));
			Text = FString::Printf(TEXT("<=== Low visual contribution"));
			Canvas.DrawShadowedString(DensityViewLeft + 2.0, DensityViewTop + 150.0f, *Text, GetStatsFont(), GrayTextColor);
			Text = FString::Printf(TEXT("High visual contribution ===>"));
			Canvas.DrawShadowedString(ViewPortWidth - 170.0f, DensityViewTop + 166.0f, *Text, GetStatsFont(), GrayTextColor);
			Text = FString::Printf(TEXT("Ground level"));
			Canvas.DrawShadowedString(DensityViewLeft, DensityViewBottom, *Text, GetStatsFont(), TextColor);

			Text = FString::Printf(TEXT("Time-of-day preview"));
			Canvas.DrawShadowedString(ViewPortWidth * 0.5f - 80.0f, TimeOfDayViewTop, *Text, GetStatsFont(), TextColor);

			Text = FString::Printf(TEXT("Hemisphere view"));
			Canvas.DrawShadowedString(Margin, HemiViewTop, *Text, GetStatsFont(), TextColor);
		}
	});

#endif
	return MoveTemp(ScreenPassSceneColor);
}

