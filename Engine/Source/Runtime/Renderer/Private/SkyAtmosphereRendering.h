// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyAtmosphereRendering.h
=============================================================================*/

#pragma once

#include "EngineDefines.h"
#include "RenderGraph.h"
#include "RenderResource.h"
#include "Rendering/SkyAtmosphereCommonData.h"
#include "RenderGraphResources.h"
#include "SceneView.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSkyAtmosphereRendering, Log, All)

class FScene;
class FViewInfo;
class FViewFamilyInfo;
class FLightSceneInfo;
class FVisibleLightInfo;
class USkyAtmosphereComponent;
class FSkyAtmosphereSceneProxy;
class FProjectedShadowInfo;

class FSkyAtmosphereInternalCommonParameters;
class FVolumeShadowingShaderParametersGlobal0;
class FVolumeShadowingShaderParametersGlobal1;

struct FScreenPassTexture;
struct FEngineShowFlags;

DECLARE_UNIFORM_BUFFER_STRUCT(FSceneUniformParameters, RENDERER_API)


// Use as a global shader parameter struct and also the CPU structure representing the atmosphere it self.
// This is static for a version of a component. When a component is changed/tweaked, it is recreated.
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FAtmosphereUniformShaderParameters, )
	SHADER_PARAMETER(float, MultiScatteringFactor)
	SHADER_PARAMETER(float, BottomRadiusKm)
	SHADER_PARAMETER(float, TopRadiusKm)
	SHADER_PARAMETER(float, RayleighDensityExpScale)
	SHADER_PARAMETER(FLinearColor, RayleighScattering)
	SHADER_PARAMETER(FLinearColor, MieScattering)
	SHADER_PARAMETER(float, MieDensityExpScale)
	SHADER_PARAMETER(FLinearColor, MieExtinction)
	SHADER_PARAMETER(float, MiePhaseG)
	SHADER_PARAMETER(FLinearColor, MieAbsorption)
	SHADER_PARAMETER(float, AbsorptionDensity0LayerWidth)
	SHADER_PARAMETER(float, AbsorptionDensity0ConstantTerm)
	SHADER_PARAMETER(float, AbsorptionDensity0LinearTerm)
	SHADER_PARAMETER(float, AbsorptionDensity1ConstantTerm)
	SHADER_PARAMETER(float, AbsorptionDensity1LinearTerm)
	SHADER_PARAMETER(FLinearColor, AbsorptionExtinction)
	SHADER_PARAMETER(FLinearColor, GroundAlbedo)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

// These parameters are shared on the view global uniform buffer and are dynamically changed with cvars.
struct FSkyAtmosphereViewSharedUniformShaderParameters
{
	FVector4f CameraAerialPerspectiveVolumeSizeAndInvSize;
	float AerialPerspectiveStartDepthKm;
	float CameraAerialPerspectiveVolumeDepthResolution;
	float CameraAerialPerspectiveVolumeDepthResolutionInv;
	float CameraAerialPerspectiveVolumeDepthSliceLengthKm;
	float CameraAerialPerspectiveVolumeDepthSliceLengthKmInv;
	float ApplyCameraAerialPerspectiveVolume;
};

// Structure with data necessary to specify a sky render.
struct FSkyAtmosphereRenderContext
{
	///////////////////////////////////
	// Per scene parameters

	bool bUseDepthBoundTestIfPossible;
	bool bForceRayMarching;
	bool bDepthReadDisabled;// Do not apply scene depth texture. As such, only far Z will be considered
	bool bDisableBlending;	// Do not do any blending. The sky will clear the target when rendering a sky reflection capture for instance.
	bool bFastSky;
	bool bFastAerialPerspective;
	bool bFastAerialPerspectiveDepthTest;
	bool bSecondAtmosphereLightEnabled;
	bool bShouldSampleOpaqueShadow;

	FRDGTextureRef TransmittanceLut;
	FRDGTextureRef MultiScatteredLuminanceLut;
	FRDGTextureRef SkyAtmosphereViewLutTexture;
	FRDGTextureRef SkyAtmosphereCameraAerialPerspectiveVolume;
	FRDGTextureRef SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly;
	FRDGTextureRef SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly;

	///////////////////////////////////
	// Per view parameters

	FViewMatrices* ViewMatrices;			// The actual view matrices we use to render the sky
	TUniformBufferRef<FViewUniformShaderParameters> ViewUniformBuffer;
	TRDGUniformBufferRef<FSceneUniformParameters> SceneUniformBuffer;
	bool bSceneHasSkyMaterial;

	FRenderTargetBindingSlots RenderTargets;

	FIntRect Viewport;

	bool bIsReflectionCapture;
	bool bRenderSkyPixel;
	float AerialPerspectiveStartDepthInCm;
	float NearClippingDistance;
	ERHIFeatureLevel::Type FeatureLevel;

	TRDGUniformBufferRef<FVolumeShadowingShaderParametersGlobal0> LightShadowShaderParams0UniformBuffer{};
	TRDGUniformBufferRef<FVolumeShadowingShaderParametersGlobal1> LightShadowShaderParams1UniformBuffer{};

	int VirtualShadowMapId0 = INDEX_NONE;
	int VirtualShadowMapId1 = INDEX_NONE;

	bool bShouldSampleCloudShadow;
	FRDGTextureRef VolumetricCloudShadowMap[2];

	bool bShouldSampleCloudSkyAO;
	FRDGTextureRef VolumetricCloudSkyAO;

	bool bAPOnCloudMode;
	FRDGTextureRef VolumetricCloudDepthTexture;
	FRDGTextureRef InputCloudLuminanceTransmittanceTexture;

	uint8 MSAASampleCount;
	FRDGTextureRef MSAADepthTexture;

	FSkyAtmosphereRenderContext();

private:
};



class FSkyAtmosphereRenderSceneInfo
{
public:

	/** Initialization constructor. */
	explicit FSkyAtmosphereRenderSceneInfo(FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy);
	~FSkyAtmosphereRenderSceneInfo();

	const TUniformBufferRef<FAtmosphereUniformShaderParameters>& GetAtmosphereUniformBuffer() { return AtmosphereUniformBuffer; }
	TRefCountPtr<IPooledRenderTarget>& GetTransmittanceLutTexture() { return TransmittanceLutTexture; }
	TRefCountPtr<IPooledRenderTarget>& GetMultiScatteredLuminanceLutTexture() { return MultiScatteredLuminanceLutTexture; }

	void CreateDistantSkyLightLutBufferAndSRV(FRDGBuilder& GraphBuilder);
	TRefCountPtr<FRDGPooledBuffer>& GetDistantSkyLightLutBuffer();
	TRefCountPtr<FRDGPooledBuffer>& GetMobileDistantSkyLightLutBuffer();
	FRHIShaderResourceView* GetDistantSkyLightLutBufferSRV();
	FRHIShaderResourceView* GetMobileDistantSkyLightLutBufferSRV();

	FRDGTextureRef GetTransmittanceLutTexture(FRDGBuilder& GraphBuilder) const { return GraphBuilder.RegisterExternalTexture(TransmittanceLutTexture); }

	const FAtmosphereUniformShaderParameters* GetAtmosphereShaderParameters() const { return &AtmosphereUniformShaderParameters; }
	const FSkyAtmosphereSceneProxy& GetSkyAtmosphereSceneProxy() const { return SkyAtmosphereSceneProxy; }

	TUniformBufferRef<FSkyAtmosphereInternalCommonParameters>& GetInternalCommonParametersUniformBuffer() { return InternalCommonParametersUniformBuffer; }

private:

	FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy;

	FAtmosphereUniformShaderParameters AtmosphereUniformShaderParameters;

	TUniformBufferRef<FAtmosphereUniformShaderParameters> AtmosphereUniformBuffer;

	TUniformBufferRef<FSkyAtmosphereInternalCommonParameters> InternalCommonParametersUniformBuffer;

	TRefCountPtr<IPooledRenderTarget> TransmittanceLutTexture;
	TRefCountPtr<IPooledRenderTarget> MultiScatteredLuminanceLutTexture;
	TRefCountPtr<FRDGPooledBuffer> DistantSkyLightLutBuffer;
	FRHIShaderResourceView* DistantSkyLightLutBufferSRV = nullptr;
	TRefCountPtr<FRDGPooledBuffer> MobileDistantSkyLightLutBuffer;
	FRHIShaderResourceView* MobileDistantSkyLightLutBufferSRV = nullptr;
};

/** Pending RDG resource to commit after the pre-pass / nanite rasterization so that RenderSkyAtmosphereLookUpTables() can overlap them on async compute. */
class FSkyAtmospherePendingRDGResources
{
public:
	// Sky env map capture uses the view UB, which contains the LUTs computed above. We need to transition them to readable now.
	void CommitToSceneAndViewUniformBuffers(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue) const;

private:
	struct FViewRDGResources
	{
		FRDGTextureRef SkyAtmosphereViewLutTexture = nullptr;
		FRDGTextureRef SkyAtmosphereCameraAerialPerspectiveVolume = nullptr;
	};

	FSceneRenderer* SceneRenderer = nullptr;
	TArray<FViewRDGResources, TInlineAllocator<4>> ViewResources;

	FRDGBufferRef DistantSkyLightLutBuffer = nullptr;
	FRDGBufferRef MobileDistantSkyLightLutBuffer = nullptr;
	FRDGTextureRef RealTimeReflectionCaptureSkyAtmosphereViewLutTexture = nullptr;
	FRDGTextureRef RealTimeReflectionCaptureCamera360APLutTexture = nullptr;

	FRDGTextureRef TransmittanceLut = nullptr;

	friend class FSceneRenderer;
};

enum class ESkyAtmospherePassLocation : uint32
{
	// Renders just before the BasePass.
	BeforeBasePass,

	// Renders just before the PrePass but can overlap on async compute with it too.
	BeforePrePass,
};

// Returns the location in the frame where SkyAtmosphere is rendered.
extern ESkyAtmospherePassLocation GetSkyAtmospherePassLocation();


bool ShouldRenderSkyAtmosphere(const FScene* Scene, const FEngineShowFlags& EngineShowFlags);

void SkyAtmosphereOnRenderBegin(FScene& Scene, const FEngineShowFlags& EngineShowFlags);

void InitSkyAtmosphereForScene(FRHICommandListImmediate& RHICmdList, FRDGBuilder& GraphBuilder, FScene* Scene);
void InitSkyAtmosphereForView(FRHICommandListImmediate& RHICmdList, const FScene* Scene, FViewInfo& View);

extern void SetupSkyAtmosphereViewSharedUniformShaderParameters(const class FViewInfo& View, const FSkyAtmosphereSceneProxy& SkyAtmosphereProxy, FSkyAtmosphereViewSharedUniformShaderParameters& OutParameters);

// Prepare the sun light data as a function of the atmosphere state. 
void PrepareSunLightProxy(const FSkyAtmosphereRenderSceneInfo& SkyAtmosphere, uint32 AtmosphereLightIndex, FLightSceneInfo& AtmosphereLight);

bool IsLightAtmospherePerPixelTransmittanceEnabled(const FScene* Scene, const FViewInfo& View, const FLightSceneInfo* const LightSceneInfo);

float GetValidAerialPerspectiveStartDepthInCm(const FViewInfo& View, const FSkyAtmosphereSceneProxy& SkyAtmosphereProxy);

struct SkyAtmosphereLightShadowData
{
	const FLightSceneInfo* LightVolumetricShadowSceneinfo0 = nullptr;
	const FLightSceneInfo* LightVolumetricShadowSceneinfo1 = nullptr;
	const FProjectedShadowInfo* ProjectedShadowInfo0 = nullptr;
	const FProjectedShadowInfo* ProjectedShadowInfo1 = nullptr;
	int VirtualShadowMapId0 = INDEX_NONE;
	int VirtualShadowMapId1 = INDEX_NONE;
};
bool ShouldSkySampleAtmosphereLightsOpaqueShadow(const FScene& Scene, const FVisibleLightInfoArray& VisibleLightInfos, SkyAtmosphereLightShadowData& LightShadowData, struct FDynamicShadowsTaskData* FilterDynamicShadowTaskData = nullptr);
void GetSkyAtmosphereLightsUniformBuffers(
	FRDGBuilder& GraphBuilder,
	TRDGUniformBufferRef<FVolumeShadowingShaderParametersGlobal0>& OutLightShadowShaderParams0UniformBuffer,
	TRDGUniformBufferRef<FVolumeShadowingShaderParametersGlobal1>& OutLightShadowShaderParams1UniformBuffer,
	const SkyAtmosphereLightShadowData& LightShadowData,
	const FViewInfo& ViewInfo,
	const bool bShouldSampleOpaqueShadow,
	const EUniformBufferUsage UniformBufferUsage);


bool ShouldRenderSkyAtmosphereDebugPasses(const FScene* Scene, const FEngineShowFlags& EngineShowFlags);
FScreenPassTexture AddSkyAtmosphereDebugPasses(FRDGBuilder& GraphBuilder, FScene* Scene, const FSceneViewFamily& ViewFamily, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor);

