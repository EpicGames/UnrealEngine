// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyPassRendering.cpp: Sky pass rendering implementation.
=============================================================================*/

#include "SkyPassRendering.h"
#include "BasePassRendering.h"
#include "MobileBasePassRendering.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "MeshPassProcessor.inl"



FSkyPassMeshProcessor::FSkyPassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::SkyPass, Scene, InFeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InPassDrawRenderState)
{
}

void FSkyPassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material && Material->IsSky())
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

bool FSkyPassMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
	return Process(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, MeshFillMode, MeshCullMode);
}

bool FSkyPassMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	typedef FUniformLightMapPolicy LightMapPolicyType;
	FUniformLightMapPolicy NoLightmapPolicy(LMP_NO_LIGHTMAP);
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Deferred)
	{
		TMeshProcessorShaders<
			TBasePassVertexShaderPolicyParamType<LightMapPolicyType>,
			TBasePassPixelShaderPolicyParamType<LightMapPolicyType>> SkyPassShaders;

		if (!GetBasePassShaders<LightMapPolicyType>(
			MaterialResource,
			VertexFactory->GetType(),
			NoLightmapPolicy,
			FeatureLevel,
			false, // 128bit
			false, // bIsDebug
			GBL_Default,
			nullptr, // Mesh Shader
			&SkyPassShaders.VertexShader,
			&SkyPassShaders.PixelShader
			))
		{
			return false;
		}

		TBasePassShaderElementData<LightMapPolicyType> ShaderElementData(nullptr);
		ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

		const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(SkyPassShaders.VertexShader, SkyPassShaders.PixelShader);

		BuildMeshDrawCommands(
			MeshBatch,
			BatchElementMask,
			PrimitiveSceneProxy,
			MaterialRenderProxy,
			MaterialResource,
			PassDrawRenderState,
			SkyPassShaders,
			MeshFillMode,
			MeshCullMode,
			SortKey,
			EMeshPassFeatures::Default,
			ShaderElementData);
	}
	else
	{
		TMeshProcessorShaders<
			TMobileBasePassVSPolicyParamType<LightMapPolicyType>,
			TMobileBasePassPSPolicyParamType<LightMapPolicyType>> SkyPassShaders;

		if (!MobileBasePass::GetShaders(
			NoLightmapPolicy,
			EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED,
			MaterialResource,
			VertexFactory->GetType(),
			SkyPassShaders.VertexShader,
			SkyPassShaders.PixelShader
		))
		{
			return false;
		}

		SetStateForMobile();
		
		TMobileBasePassShaderElementData<LightMapPolicyType> ShaderElementData(nullptr, false);
		ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

		const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(SkyPassShaders.VertexShader, SkyPassShaders.PixelShader);

		BuildMeshDrawCommands(
			MeshBatch,
			BatchElementMask,
			PrimitiveSceneProxy,
			MaterialRenderProxy,
			MaterialResource,
			PassDrawRenderState,
			SkyPassShaders,
			MeshFillMode,
			MeshCullMode,
			SortKey,
			EMeshPassFeatures::Default,
			ShaderElementData);
	}

	return true;
}

void FSkyPassMeshProcessor::SetStateForMobile()
{
	if (SkyPassType == SPT_Default)
	{
		// Mask sky pixels so we can skip them when rendering per-pixel fog (USE_VERTEX_FOG is applying for on sky pixels).
		PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<
			false, CF_DepthNearOrEqual,
			true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			0x00, STENCIL_MOBILE_SKY_MASK>::GetRHI());

		PassDrawRenderState.SetStencilRef(STENCIL_MOBILE_SKY_MASK);
	}
	else if (SkyPassType == SPT_RealTimeCapture_DepthWrite)
	{
		// Capturing real time sky light, writing depth only.
		PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<
			true, CF_DepthNearOrEqual>::GetRHI());
	}
	else if (SkyPassType == SPT_RealTimeCapture_DepthNop)
	{
		// Capturing real time sky light, no depth writes.
		PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<
			false, CF_Always>::GetRHI());
	}
	else
	{
		check(false);
	}
}

void FSkyPassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	// Early out if not sky
	if (!Material.IsSky())
	{
		return;
	}

	typedef FUniformLightMapPolicy LightMapPolicyType;
	FUniformLightMapPolicy NoLightmapPolicy(LMP_NO_LIGHTMAP);

	if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Deferred)
	{
		TMeshProcessorShaders<
			TBasePassVertexShaderPolicyParamType<LightMapPolicyType>,
			TBasePassPixelShaderPolicyParamType<LightMapPolicyType>> SkyPassShaders;
				
		if (GetBasePassShaders<LightMapPolicyType>(
			Material,
			VertexFactoryData.VertexFactoryType,
			NoLightmapPolicy,
			FeatureLevel,
			false, // 128bit
			false, // bIsDebug
			GBL_Default,
			nullptr, // Mesh Shader
			&SkyPassShaders.VertexShader,
			&SkyPassShaders.PixelShader
		))
		{
			CollectPSOInitializersInternal(SkyPassShaders, SceneTexturesConfig, Material, VertexFactoryData, PreCacheParams, PSOInitializers);
		}
	}
	else
	{
		TMeshProcessorShaders<
			TMobileBasePassVSPolicyParamType<LightMapPolicyType>,
			TMobileBasePassPSPolicyParamType<LightMapPolicyType>> SkyPassShaders;

		if (MobileBasePass::GetShaders(
			NoLightmapPolicy,
			EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED,
			Material,
			VertexFactoryData.VertexFactoryType,
			SkyPassShaders.VertexShader,
			SkyPassShaders.PixelShader
		))
		{
			SetStateForMobile();
			CollectPSOInitializersInternal(SkyPassShaders, SceneTexturesConfig, Material, VertexFactoryData, PreCacheParams, PSOInitializers);
		}
	}
}

template <typename T>
void FSkyPassMeshProcessor::CollectPSOInitializersInternal(T& SkyPassShaders, const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

	static auto CVarRealTimeReflectionCaptureDepthBufferStatic = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SkyLight.RealTimeReflectionCapture.DepthBuffer"));
	const bool bUseDepthBuffer = CVarRealTimeReflectionCaptureDepthBufferStatic ? CVarRealTimeReflectionCaptureDepthBufferStatic->GetInt() > 0 : false;

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	SetupGBufferRenderTargetInfo(SceneTexturesConfig, RenderTargetsInfo, bUseDepthBuffer);

	// Mobile path renders sky in a BasePass so we should have exactly same RT setup
	if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Deferred)
	{
		// Remove all but the SceneColor		
		RenderTargetsInfo.RenderTargetsEnabled = 1;
		for (uint32 i = 1; i < MaxSimultaneousRenderTargets; ++i)
		{
			RenderTargetsInfo.RenderTargetFormats[i] = 0;
			RenderTargetsInfo.RenderTargetFlags[i] = ETextureCreateFlags::None;
		}

		FBasePassMeshProcessor::AddBasePassGraphicsPipelineStateInitializer(
			FeatureLevel,
			VertexFactoryData,
			Material,
			PassDrawRenderState,
			RenderTargetsInfo,
			SkyPassShaders,
			MeshFillMode,
			MeshCullMode,
			(EPrimitiveType)PreCacheParams.PrimitiveType,
			true /*bPrecacheAlphaColorChannel*/,
			PSOCollectorIndex,
			PSOInitializers);
	}
	else
	{
		// subpass info set during the submission of the draws in mobile renderer
		bool bDeferredShading = IsMobileDeferredShadingEnabled(GMaxRHIShaderPlatform);
		ESubpassHint SubpassHint = GetSubpassHint(GMaxRHIShaderPlatform, bDeferredShading, RenderTargetsInfo.MultiViewCount > 1, RenderTargetsInfo.NumSamples);

		AddGraphicsPipelineStateInitializer(
			VertexFactoryData,
			Material,
			PassDrawRenderState,
			RenderTargetsInfo,
			SkyPassShaders,
			MeshFillMode,
			MeshCullMode,
			(EPrimitiveType)PreCacheParams.PrimitiveType,
			EMeshPassFeatures::Default,
			SubpassHint,
			0,
			true /*bRequired*/,
			PSOCollectorIndex,
			PSOInitializers);
	}

	// Also generate with depth write when dpeth buffer is enabled which is used during CaptureSkyMeshReflection
	{
		const FExclusiveDepthStencil::Type SceneBasePassDepthStencilAccess = FScene::GetDefaultBasePassDepthStencilAccess(FeatureLevel);
		FMeshPassProcessorRenderState SkyCaptureDrawRenderState;
		FExclusiveDepthStencil::Type BasePassDepthStencilAccess_Sky = bUseDepthBuffer ? FExclusiveDepthStencil::Type(SceneBasePassDepthStencilAccess | FExclusiveDepthStencil::DepthWrite)
			: FExclusiveDepthStencil::Type(SceneBasePassDepthStencilAccess & ~FExclusiveDepthStencil::DepthWrite);
		SetupBasePassState(BasePassDepthStencilAccess_Sky, false, SkyCaptureDrawRenderState);

		// Also change render target format
		FRDGTextureDesc SkyCaptureRenderTargetDesc = FSkyPassMeshProcessor::GetCaptureFrameSkyEnvMapTextureDesc(1, 1);

		FGraphicsPipelineRenderTargetsInfo SkyCaptureRenderTargetsInfo;
		SkyCaptureRenderTargetsInfo.NumSamples = 1;
		AddRenderTargetInfo(SkyCaptureRenderTargetDesc.Format, SkyCaptureRenderTargetDesc.Flags, SkyCaptureRenderTargetsInfo);
		if (bUseDepthBuffer)
		{
			SetupDepthStencilInfo(PF_DepthStencil, SceneTexturesConfig.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
				ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite, SkyCaptureRenderTargetsInfo);
		}

		AddGraphicsPipelineStateInitializer(
			VertexFactoryData,
			Material,
			SkyCaptureDrawRenderState,
			SkyCaptureRenderTargetsInfo,
			SkyPassShaders,
			MeshFillMode,
			MeshCullMode,
			(EPrimitiveType)PreCacheParams.PrimitiveType,
			EMeshPassFeatures::Default,
			true /*bRequired*/,
			PSOInitializers);
	}
}

FRDGTextureDesc FSkyPassMeshProcessor::GetCaptureFrameSkyEnvMapTextureDesc(uint32 CubeWidth, uint32 CubeMipCount)
{
	return FRDGTextureDesc::CreateCube(CubeWidth,
		PF_FloatR11G11B10, FClearValueBinding::Black, TexCreate_TargetArraySlicesIndependently |
		TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable, CubeMipCount);
}

FMeshPassProcessor* CreateSkyPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	const FExclusiveDepthStencil::Type SceneBasePassDepthStencilAccess = FScene::GetDefaultBasePassDepthStencilAccess(FeatureLevel);
	
	FMeshPassProcessorRenderState DrawRenderState;
	FExclusiveDepthStencil::Type BasePassDepthStencilAccess_NoDepthWrite = FExclusiveDepthStencil::Type(SceneBasePassDepthStencilAccess & ~FExclusiveDepthStencil::DepthWrite);
	SetupBasePassState(BasePassDepthStencilAccess_NoDepthWrite, false, DrawRenderState);

	return new FSkyPassMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, DrawRenderState, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(SkyPass, CreateSkyPassProcessor, EShadingPath::Deferred, EMeshPass::SkyPass, EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileSkyPass, CreateSkyPassProcessor, EShadingPath::Mobile, EMeshPass::SkyPass, EMeshPassFlags::MainView);
