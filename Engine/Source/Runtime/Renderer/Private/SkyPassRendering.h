// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyPassRendering.h: Sky pass rendering implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "MeshPassProcessor.h"



class FSkyPassMeshProcessor : public FSceneRenderingAllocatorObject<FSkyPassMeshProcessor>, public FMeshPassProcessor
{
public:

	FSkyPassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

	static FRDGTextureDesc GetCaptureFrameSkyEnvMapTextureDesc(uint32 CubeWidth, uint32 CubeMipCount);

	FMeshPassProcessorRenderState PassDrawRenderState;
	EShadingPath ShadingPath;

	enum ESkyPassType
	{
		SPT_Default,
		SPT_RealTimeCapture_DepthWrite,
		SPT_RealTimeCapture_DepthNop,
	};
	ESkyPassType SkyPassType = SPT_Default;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	void SetStateForMobile();

	template <typename T>
	void CollectPSOInitializersInternal(T& SkyPassShaders, const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers);
};


