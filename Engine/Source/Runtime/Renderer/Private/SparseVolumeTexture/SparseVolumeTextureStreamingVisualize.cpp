// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTextureStreamingVisualize.h"
#include "SparseVolumeTexture/ISparseVolumeTextureStreamingManager.h"
#include "RenderGraph.h"
#include "RenderGraphBuilder.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"

static int32 GSVTStreamingShowDebugInfo = 0;
static FAutoConsoleVariableRef CVarSVTStreamingShowDebugInfo(
	TEXT("r.SparseVolumeTexture.Streaming.ShowDebugInfo"),
	GSVTStreamingShowDebugInfo,
	TEXT("Prints debug info about the streaming SVT instances to the screen."),
	ECVF_RenderThreadSafe
);

void UE::SVT::AddStreamingDebugPass(FRDGBuilder& GraphBuilder, const FSceneView& View, FScreenPassTexture Output)
{
	if (GSVTStreamingShowDebugInfo == 0)
	{
		return;
	}

	const FStreamingDebugInfo* DebugInfo = UE::SVT::GetStreamingManager().GetStreamingDebugInfo(GraphBuilder);
	FScreenPassRenderTarget OutputTarget = FScreenPassRenderTarget(Output, ERenderTargetLoadAction::ELoad);
	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("SVT::StreamingDebug"), View, OutputTarget, [DebugInfo](FCanvas& Canvas)
	{
		const double LineSpacing = 14.0;
		double YOffset = 32.0;
		TStringBuilder<128> StringBuilder;

		StringBuilder.Appendf(TEXT("Sparse Volume Texture Streaming Stats\nRequested Peak Bandwidth: %#08.2f MiB/s, Bandwidth Limit: %#08.2f MiB/s, Bandwidth Throttle Scale: %6.2f%%\n"), 
			DebugInfo->RequestedBandwidth, DebugInfo->BandwidthLimit, float(DebugInfo->BandwidthScale * 100.0));
		FCanvasTextItem HeaderTextItem(FVector2D(8.0, YOffset), FText::FromStringView(StringBuilder.ToView()), GEngine->GetSmallFont(), FLinearColor::White);
		HeaderTextItem.EnableShadow(FLinearColor::Black);
		HeaderTextItem.Draw(&Canvas);
		YOffset += LineSpacing * 2.0; // We just printed two lines
		StringBuilder.Reset();

		for (int32 SVTIndex = 0; SVTIndex < DebugInfo->NumSVTs; ++SVTIndex)
		{
			const FStreamingDebugInfo::FSVT& SVT = DebugInfo->SVTs[SVTIndex];

			StringBuilder.Appendf(TEXT("    Asset: '%s', Num Frames: %i\n"), *SVT.AssetName.ToString(), SVT.NumFrames);
			FCanvasTextItem SVTTextItem(FVector2D(8.0, YOffset), FText::FromStringView(StringBuilder.ToView()), GEngine->GetSmallFont(), FLinearColor::White);
			SVTTextItem.EnableShadow(FLinearColor::Black);
			SVTTextItem.Draw(&Canvas);
			YOffset += LineSpacing;
			StringBuilder.Reset();

			const double TileYOffset = YOffset;
			const double TileHeight = 24.0;
			const double TileWidth = 8.0;
			for (int32 FrameIndex = 0; FrameIndex < SVT.NumFrames; ++FrameIndex)
			{
				const double XOffset = 8.0 + FrameIndex * (TileWidth + 1.0);
				const float Residency = SVT.FrameResidencyPercentages[FrameIndex];
				const float Streaming = SVT.FrameStreamingPercentages[FrameIndex];
				const FLinearColor ResidencyColor = FMath::Lerp(FLinearColor::Red, FLinearColor::Green, Residency);

				FCanvasTileItem StreamingFrameTileItem(FVector2D(XOffset, YOffset + FMath::Min(TileHeight - 1.0, TileHeight * (1.0 - Streaming))), FVector2D(TileWidth, FMath::Max(1.0, TileHeight * Streaming)), FLinearColor(0.0f, 0.0f, 0.5f));
				StreamingFrameTileItem.Draw(&Canvas);
				FCanvasTileItem ResidencyFrameTileItem(FVector2D(XOffset, YOffset + FMath::Min(TileHeight - 1.0, TileHeight * (1.0 - Residency))), FVector2D(TileWidth, FMath::Max(1.0, TileHeight * Residency)), ResidencyColor);
				ResidencyFrameTileItem.Draw(&Canvas);
			}
			YOffset += TileHeight + 2.0;

			for (int32 InstanceIndex = 0; InstanceIndex < SVT.NumInstances; ++InstanceIndex)
			{
				const FStreamingDebugInfo::FSVT::FInstance& Instance = SVT.Instances[InstanceIndex];

				StringBuilder.Appendf(TEXT("        Instance Key: %u, Frame: %#06.2f, Estimated Frame Rate: %#06.2f, Requested Peak Bandwidth: %#07.2f MiB/s, Allocated Bandwidth: %#07.2f MiB/s, Requested Mip: %#5.2f, Bandwidth Budget Mip: %#5.2f\n"),
					Instance.Key, Instance.Frame, Instance.FrameRate, Instance.RequestedBandwidth, Instance.AllocatedBandwidth, Instance.RequestedMip, Instance.InBudgetMip);
				FCanvasTextItem InstanceTextItem(FVector2D(8.0, YOffset), FText::FromStringView(StringBuilder.ToView()), GEngine->GetSmallFont(), FLinearColor::White);
				InstanceTextItem.EnableShadow(FLinearColor::Black);
				InstanceTextItem.Draw(&Canvas);
				YOffset += LineSpacing;
				StringBuilder.Reset();

				const double FrameLineX = 8.0 + FMath::FloorToDouble(Instance.Frame) * (TileWidth + 1.0) + FMath::Frac(Instance.Frame) * TileWidth;
				FCanvasLineItem InstanceLineItem(FVector2D(FrameLineX, TileYOffset - 1.0), FVector2D(FrameLineX, TileYOffset + TileHeight + 1.0));
				InstanceLineItem.SetColor(FLinearColor::Blue);
				InstanceLineItem.Draw(&Canvas);
			}
		}
	});
}
