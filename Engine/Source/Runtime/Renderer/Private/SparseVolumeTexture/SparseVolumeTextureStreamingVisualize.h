// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "ScreenPass.h"

class FRDGBuilder;

namespace UE
{
namespace SVT
{
	void AddStreamingDebugPass(FRDGBuilder& GraphBuilder, const FSceneView& View, FScreenPassTexture Output);
}
}