// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTexture/SparseVolumeTextureViewerComponent.h"

#include "SparseVolumeTexture/SparseVolumeTextureViewerSceneProxy.h"
#include "Components/ArrowComponent.h"
#include "Components/BillboardComponent.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Internationalization/Text.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"
#include "UObject/UObjectIterator.h"
#include "UObject/ConstructorHelpers.h"

#if WITH_EDITOR
#include "ObjectEditorUtils.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SparseVolumeTextureViewerComponent)

#define LOCTEXT_NAMESPACE "SparseVolumeTextureViewerComponent"

constexpr int32 SVTViewerDefaultVolumeResolution = 128;

/*=============================================================================
	USparseVolumeTextureViewerComponent implementation.
=============================================================================*/

USparseVolumeTextureViewerComponent::USparseVolumeTextureViewerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, SparseVolumeTexturePreview(nullptr)
	, Frame(0.0f)
	, FrameRate(24.0f)
	, bPlaying(false)
	, bLooping(false)
	, bReversePlayback(false)
	, bBlockingStreamingRequests(false)
	, bApplyPerFrameTransforms(true)
	, bPivotAtCentroid(false)
	, VoxelSize(1.0f)
	, PreviewAttribute(ESVTPA_AttributesA_R)
	, MipLevel(0)
	, Extinction(0.025f)
	, SparseVolumeTextureViewerSceneProxy(nullptr)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.TickGroup = TG_DuringPhysics;
	bTickInEditor = true;
}

USparseVolumeTextureViewerComponent::~USparseVolumeTextureViewerComponent()
{
}

#if WITH_EDITOR

void USparseVolumeTextureViewerComponent::CheckForErrors()
{
}

void USparseVolumeTextureViewerComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName;
	if (PropertyChangedEvent.Property)
	{
		PropertyName = PropertyChangedEvent.Property->GetFName();
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(USparseVolumeTextureViewerComponent, Frame))
	{
		if (SparseVolumeTexturePreview)
		{
			const int32 FrameCount = SparseVolumeTexturePreview->GetNumFrames();
			Frame = FMath::Clamp(Frame, 0, FrameCount - 1);
		}
	}

	MarkRenderStateDirty();
}

#endif // WITH_EDITOR

void USparseVolumeTextureViewerComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
}

FBoxSphereBounds USparseVolumeTextureViewerComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FVector VolumeResolution = FVector(SVTViewerDefaultVolumeResolution);
	if (SparseVolumeTexturePreview)
	{
		VolumeResolution = FVector(SparseVolumeTexturePreview->GetVolumeResolution());
	}
	
	FBoxSphereBounds NewBounds;
	FVector HalfVolumeResolution = FVector(VolumeResolution) * 0.5;
	if (bPivotAtCentroid)
	{
		NewBounds.Origin = FVector::ZeroVector;
	}
	else
	{
		NewBounds.Origin = HalfVolumeResolution;
	}
	NewBounds.BoxExtent = HalfVolumeResolution;
	NewBounds.SphereRadius = NewBounds.BoxExtent.Length();

	if (SparseVolumeTextureFrame)
	{
		return NewBounds.TransformBy(SparseVolumeTextureFrame->GetFrameTransform() * LocalToWorld);
	}
	else
	{
		return NewBounds.TransformBy(LocalToWorld);
	}
}

void USparseVolumeTextureViewerComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);
	// If one day we need to look up lightmass built data, lookup it up here using the guid from the correct MapBuildData.

	bool bHidden = false;
#if WITH_EDITORONLY_DATA
	bHidden = GetOwner() ? GetOwner()->bHiddenEdLevel : false;
#endif // WITH_EDITORONLY_DATA
	if (!ShouldComponentAddToScene())
	{
		bHidden = true;
	}

	if (GetVisibleFlag() && !bHidden &&
		ShouldComponentAddToScene() && ShouldRender() && IsRegistered() && (GetOuter() == NULL || !GetOuter()->HasAnyFlags(RF_ClassDefaultObject)))
	{
		// Create the scene proxy.
		SparseVolumeTextureViewerSceneProxy = new FSparseVolumeTextureViewerSceneProxy(this);
		GetWorld()->Scene->AddSparseVolumeTextureViewer(SparseVolumeTextureViewerSceneProxy);
		SendRenderTransformCommand();
	}
}

void USparseVolumeTextureViewerComponent::DestroyRenderState_Concurrent()
{
	Super::DestroyRenderState_Concurrent();

	if (SparseVolumeTextureViewerSceneProxy)
	{
		GetWorld()->Scene->RemoveSparseVolumeTextureViewer(SparseVolumeTextureViewerSceneProxy);

		FSparseVolumeTextureViewerSceneProxy* SVTViewerSceneProxy = SparseVolumeTextureViewerSceneProxy;
		ENQUEUE_RENDER_COMMAND(FDestroySparseVolumeTextureViewerSceneProxyCommand)(
			[SVTViewerSceneProxy](FRHICommandList& RHICmdList)
			{
				delete SVTViewerSceneProxy;
			});

		SparseVolumeTextureViewerSceneProxy = nullptr;
	}
}

void USparseVolumeTextureViewerComponent::SendRenderTransform_Concurrent()
{
	Super::SendRenderTransform_Concurrent();

	SendRenderTransformCommand();
}

void USparseVolumeTextureViewerComponent::SendRenderTransformCommand()
{
	if (SparseVolumeTextureViewerSceneProxy)
	{
		FVector VolumeResolution = FVector(SVTViewerDefaultVolumeResolution);
		FTransform FrameTransform = FTransform::Identity;
		if (SparseVolumeTextureFrame)
		{
			VolumeResolution = FVector(SparseVolumeTextureFrame->GetVolumeResolution());
			if (bApplyPerFrameTransforms)
			{
				FrameTransform = SparseVolumeTextureFrame->GetFrameTransform();
			}
		}

		const FTransform GlobalTransform = GetComponentTransform();
		const FVector3f VolumeRes3f = FVector3f(VolumeResolution.X, VolumeResolution.Y, VolumeResolution.Z);

		FSparseVolumeTextureViewerSceneProxy* SVTSceneProxy = SparseVolumeTextureViewerSceneProxy;
		ENQUEUE_RENDER_COMMAND(FUpdateSparseVolumeTextureViewerProxyTransformCommand)(
		[SVTSceneProxy, GlobalTransform, FrameTransform, VolumeRes3f, CompIdx = (uint32)PreviewAttribute, Ext = Extinction, Mip = MipLevel, VoxelSizeFactor = VoxelSize, bCentroidPivot = (bool)bPivotAtCentroid](FRHICommandList& RHICmdList)
		{
			SVTSceneProxy->GlobalTransform = GlobalTransform;
			SVTSceneProxy->FrameTransform = FrameTransform;
			SVTSceneProxy->VolumeResolution = VolumeRes3f;
			SVTSceneProxy->MipLevel = Mip;
			SVTSceneProxy->ComponentToVisualize = CompIdx;
			SVTSceneProxy->Extinction = Ext;
			SVTSceneProxy->VoxelSizeFactor = VoxelSizeFactor;
			SVTSceneProxy->bPivotAtCentroid = bCentroidPivot;
		});
	}
}

void USparseVolumeTextureViewerComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	if (SparseVolumeTexturePreview)
	{
		const int32 NumFrames = SparseVolumeTexturePreview->GetNumFrames();

		if (bPlaying)
		{
			Frame += DeltaTime * FrameRate * (bReversePlayback ? -1.0f : 1.0f);
		}

		if (bLooping)
		{
			// Simple way of dealing with looping when playing back in reverse: add NumFrames to the X input of Fmod.
			Frame = FMath::Fmod(Frame + NumFrames, (float)NumFrames);
		}
		else
		{
			Frame = FMath::Clamp(Frame, 0.0f, (float)(NumFrames - 1));
		}

		const bool bHasValidFrameRate = bPlaying != 0;
		SparseVolumeTextureFrame = USparseVolumeTextureFrame::GetFrameAndIssueStreamingRequest(SparseVolumeTexturePreview, GetTypeHash(this), FrameRate, Frame, MipLevel, bBlockingStreamingRequests, bHasValidFrameRate);
	}
	else
	{
		SparseVolumeTextureFrame = nullptr;
	}

	MarkRenderStateDirty();
}

/*=============================================================================
	ASparseVolumeTextureViewer implementation.
=============================================================================*/

#if WITH_EDITOR
#include "ObjectEditorUtils.h"
#endif

ASparseVolumeTextureViewer::ASparseVolumeTextureViewer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SparseVolumeTextureViewerComponent = CreateDefaultSubobject<USparseVolumeTextureViewerComponent>(TEXT("SparseVolumeTextureViewerComponent"));
	RootComponent = SparseVolumeTextureViewerComponent;

#if WITH_EDITORONLY_DATA

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> SparseVolumeTextureViewerTextureObject;
			FName ID_SparseVolumeTextureViewer;
			FText NAME_SparseVolumeTextureViewer;
			FConstructorStatics()
				: SparseVolumeTextureViewerTextureObject(TEXT("/Engine/EditorResources/S_VolumetricCloud"))	// SVT_TODO set a specific icon
				, ID_SparseVolumeTextureViewer(TEXT("Fog"))
				, NAME_SparseVolumeTextureViewer(NSLOCTEXT("SpriteCategory", "Fog", "Fog"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (GetSpriteComponent())
		{
			GetSpriteComponent()->Sprite = ConstructorStatics.SparseVolumeTextureViewerTextureObject.Get();
			GetSpriteComponent()->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			GetSpriteComponent()->SpriteInfo.Category = ConstructorStatics.ID_SparseVolumeTextureViewer;
			GetSpriteComponent()->SpriteInfo.DisplayName = ConstructorStatics.NAME_SparseVolumeTextureViewer;
			GetSpriteComponent()->SetupAttachment(SparseVolumeTextureViewerComponent);
		}
	}
#endif // WITH_EDITORONLY_DATA

	PrimaryActorTick.bCanEverTick = true;
	SetHidden(false);
}

#undef LOCTEXT_NAMESPACE
