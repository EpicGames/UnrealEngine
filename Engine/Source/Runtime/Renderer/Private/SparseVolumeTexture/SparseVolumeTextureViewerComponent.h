// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/SceneComponent.h"
#include "Components/PrimitiveComponent.h"
#include "EngineDefines.h"
#include "GameFramework/Info.h"

#include "SparseVolumeTexture/SparseVolumeTexture.h"

#include "SparseVolumeTextureViewerComponent.generated.h"


class FSparseVolumeTextureViewerSceneProxy;

UENUM(BlueprintType)
enum ESparseVolumeTexturePreviewAttribute : uint8
{
	ESVTPA_AttributesA_R UMETA(DisplayName = "AttributesA Red"),
	ESVTPA_AttributesA_G UMETA(DisplayName = "AttributesA Green"),
	ESVTPA_AttributesA_B UMETA(DisplayName = "AttributesA Blue"),
	ESVTPA_AttributesA_A UMETA(DisplayName = "AttributesA Alpha"),
	ESVTPA_AttributesB_R UMETA(DisplayName = "AttributesB Red"),
	ESVTPA_AttributesB_G UMETA(DisplayName = "AttributesB Green"),
	ESVTPA_AttributesB_B UMETA(DisplayName = "AttributesB Blue"),
	ESVTPA_AttributesB_A UMETA(DisplayName = "AttributesB Alpha"),
};

/**
 * A component used to inspect sparse volume textures.
 */
UCLASS(ClassGroup = Rendering, collapsecategories, hidecategories = (Object, Mobility, Activation, "Components|Activation"), editinlinenew, meta = (BlueprintSpawnableComponent), MinimalAPI)
class USparseVolumeTextureViewerComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

	~USparseVolumeTextureViewerComponent();
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= "Asset Preview")
	TObjectPtr<class USparseVolumeTexture> SparseVolumeTexturePreview;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation, meta = (EditCondition = "bPlaying == false"))
	float Frame;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	float FrameRate = 24.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	uint32 bPlaying : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Animation)
	uint32 bLooping : 1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Animation)
	uint32 bReversePlayback : 1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Asset Preview")
	uint32 bBlockingStreamingRequests : 1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Asset Preview")
	uint32 bApplyPerFrameTransforms : 1;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Asset Preview")
	uint32 bPivotAtCentroid : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Asset Preview")
	float VoxelSize = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Asset Preview")
	TEnumAsByte<ESparseVolumeTexturePreviewAttribute> PreviewAttribute;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Asset Preview", meta = (UIMin = 0, UIMax = 11, ClampMin = 0, ClampMax = 11))
	int32 MipLevel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Asset Preview", meta = (UIMin = 0.0, UIMax = 1.0, ClampMin = 0))
	float Extinction = 0.025f;

	TObjectPtr<class USparseVolumeTextureFrame> SparseVolumeTextureFrame;

	//~ Begin UPrimitiveComponent Interface.
	virtual bool SupportsStaticLighting() const override { return false; }
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface.
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const  override;
	//~ End USceneComponent Interface.


	//~ Begin UActorComponent Interface.
	virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	virtual void DestroyRenderState_Concurrent() override;
	virtual void SendRenderTransform_Concurrent() override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent Interface.

protected:

public:

	//~ Begin UObject Interface. 
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin UActorComponent Interface.
#if WITH_EDITOR
	virtual void CheckForErrors() override;
#endif // WITH_EDITOR
	//~ End UActorComponent Interface.

private:
	void SendRenderTransformCommand();

	FSparseVolumeTextureViewerSceneProxy* SparseVolumeTextureViewerSceneProxy;
};


/**
 * A placeable actor that represents a participating media material around a planet, e.g. clouds.
 */
UCLASS(showcategories = (Movement, Rendering, Transformation, DataLayers, "Input|MouseInput", "Input|TouchInput"), ClassGroup = Fog, hidecategories = (Info, Object, Input), MinimalAPI)
class ASparseVolumeTextureViewer : public AInfo
{
	GENERATED_UCLASS_BODY()

private:

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Atmosphere, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class USparseVolumeTextureViewerComponent> SparseVolumeTextureViewerComponent;

#if WITH_EDITOR
	virtual bool ActorTypeSupportsDataLayer() const override { return true; }
#endif

};
