// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTexture/SparseVolumeTextureViewerRendering.h"
#include "SparseVolumeTexture/SparseVolumeTextureViewerSceneProxy.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "SceneView.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "RendererInterface.h"
#include "UniformBuffer.h"
#include "SceneTextureParameters.h"
#include "ShaderCompiler.h"
#include "RendererUtils.h"
#include "EngineAnalytics.h"


////////////////////////////////////////////////////////////////////////////////////////////////////

class FVisualizeSparseVolumeTextureVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeSparseVolumeTextureVS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeSparseVolumeTextureVS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, DepthAsDeviceZ)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return !IsMobilePlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("VERTEX_SHADER"), 1);
	}
};

class FVisualizeSparseVolumeTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeSparseVolumeTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeSparseVolumeTexturePS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
		SHADER_PARAMETER_SAMPLER(SamplerState, TileDataTextureSampler)
		SHADER_PARAMETER_TEXTURE(Texture3D<uint>, SparseVolumeTexturePageTable)
		SHADER_PARAMETER_TEXTURE(Texture3D, SparseVolumeTextureA)
		SHADER_PARAMETER_TEXTURE(Texture3D, SparseVolumeTextureB)
		SHADER_PARAMETER(FUintVector4, PackedSVTUniforms0)
		SHADER_PARAMETER(FUintVector4, PackedSVTUniforms1)
		SHADER_PARAMETER(FVector3f, SparseVolumeTextureResolution)
		SHADER_PARAMETER(int32, MipLevel)
		SHADER_PARAMETER(FVector4f, WorldToLocal0)
		SHADER_PARAMETER(FVector4f, WorldToLocal1)
		SHADER_PARAMETER(FVector4f, WorldToLocal2)
		SHADER_PARAMETER(FVector3f, WorldToLocalRotation0)
		SHADER_PARAMETER(FVector3f, WorldToLocalRotation1)
		SHADER_PARAMETER(FVector3f, WorldToLocalRotation2)
		SHADER_PARAMETER(uint32, ComponentToVisualize)
		SHADER_PARAMETER(float, Extinction)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return !IsMobilePlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("PIXEL_SHADER"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeSparseVolumeTextureVS, "/Engine/Private/SparseVolumeTexture/VisualizeSparseVolumeTexture.usf", "VisualizeSparseVolumeTextureVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FVisualizeSparseVolumeTexturePS, "/Engine/Private/SparseVolumeTexture/VisualizeSparseVolumeTexture.usf", "VisualizeSparseVolumeTexturePS", SF_Pixel);


////////////////////////////////////////////////////////////////////////////////////////////////////

DECLARE_GPU_STAT(SparseVolumeTextureViewer);

void AddSparseVolumeTextureViewerRenderPass(FRDGBuilder& GraphBuilder, FSceneRenderer& SceneRenderer, FSceneTextures& SceneTextures)
{
#if WITH_EDITORONLY_DATA // The SparseVolumeTexture viewer is only supported in editor builds

	FScene* Scene = SceneRenderer.Scene;

	if (Scene->SparseVolumeTextureViewers.Num() == 0)
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, SparseVolumeTextureViewer, "SparseVolumeTextureViewer");
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, SparseVolumeTextureViewer);

	for (FViewInfo& View : SceneRenderer.Views)
	{
		const FIntRect& ViewportRect = View.ViewRect;

		for (auto& SVTSceneProxy : Scene->SparseVolumeTextureViewers)
		{
			const FVector& PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();

			FTransform GlobalTransform = SVTSceneProxy->GlobalTransform;
			GlobalTransform.AddToTranslation(PreViewTranslation); // Move into translated world space

			FTransform FrameTransform = SVTSceneProxy->FrameTransform;
			FrameTransform.MultiplyScale3D(FVector(SVTSceneProxy->VoxelSizeFactor));
			FrameTransform.ScaleTranslation(SVTSceneProxy->VoxelSizeFactor);

			const FTransform Transform = FrameTransform * GlobalTransform;
			const FMatrix InvTransformMat = Transform.ToMatrixWithScale().Inverse();
			const FQuat InvRotation = Transform.GetRotation().Inverse();

			const FVector RcpVolumeRes = FVector(SVTSceneProxy->VolumeResolution).Reciprocal();
			FMatrix UnitSpaceMat = FMatrix::Identity;
			UnitSpaceMat.SetColumn(0, FVector(RcpVolumeRes.X * 2.0, 0.0, 0.0));
			UnitSpaceMat.SetColumn(1, FVector(0.0, RcpVolumeRes.Y * 2.0, 0.0));
			UnitSpaceMat.SetColumn(2, FVector(0.0, 0.0, RcpVolumeRes.Z * 2.0));
			UnitSpaceMat.SetOrigin(FVector(SVTSceneProxy->bPivotAtCentroid ? 0.0  : -1.0));

			const FMatrix44f WorldToLocal = FMatrix44f(InvTransformMat * UnitSpaceMat);
			const FMatrix44f WorldToLocalRot = FMatrix44f(InvRotation.ToMatrix());
			
			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			FVisualizeSparseVolumeTextureVS::FPermutationDomain VsPermutationVector;
			TShaderMapRef<FVisualizeSparseVolumeTextureVS> VertexShader(GlobalShaderMap, VsPermutationVector);
			FVisualizeSparseVolumeTexturePS::FPermutationDomain PsPermutationVector;
			TShaderMapRef<FVisualizeSparseVolumeTexturePS> PixelShader(GlobalShaderMap, PsPermutationVector);

			FVisualizeSparseVolumeTexturePS::FParameters* PsPassParameters = GraphBuilder.AllocParameters<FVisualizeSparseVolumeTexturePS::FParameters>();
			PsPassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PsPassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
			PsPassParameters->TileDataTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PsPassParameters->SparseVolumeTexturePageTable = GBlackUintVolumeTexture->TextureRHI;
			PsPassParameters->SparseVolumeTextureA = GBlackVolumeTexture->TextureRHI;
			PsPassParameters->SparseVolumeTextureB = GBlackVolumeTexture->TextureRHI;
			PsPassParameters->PackedSVTUniforms0 = FUintVector4(0);
			PsPassParameters->PackedSVTUniforms1 = FUintVector4(0);
			PsPassParameters->SparseVolumeTextureResolution = SVTSceneProxy->VolumeResolution;
			PsPassParameters->MipLevel = SVTSceneProxy->MipLevel;
			PsPassParameters->WorldToLocal0 = FVector4f(WorldToLocal.M[0][0], WorldToLocal.M[1][0], WorldToLocal.M[2][0], WorldToLocal.M[3][0]);
			PsPassParameters->WorldToLocal1 = FVector4f(WorldToLocal.M[0][1], WorldToLocal.M[1][1], WorldToLocal.M[2][1], WorldToLocal.M[3][1]);
			PsPassParameters->WorldToLocal2 = FVector4f(WorldToLocal.M[0][2], WorldToLocal.M[1][2], WorldToLocal.M[2][2], WorldToLocal.M[3][2]);
			PsPassParameters->WorldToLocalRotation0 = FVector3f(WorldToLocalRot.M[0][0], WorldToLocalRot.M[1][0], WorldToLocalRot.M[2][0]);
			PsPassParameters->WorldToLocalRotation1 = FVector3f(WorldToLocalRot.M[0][1], WorldToLocalRot.M[1][1], WorldToLocalRot.M[2][1]);
			PsPassParameters->WorldToLocalRotation2 = FVector3f(WorldToLocalRot.M[0][2], WorldToLocalRot.M[1][2], WorldToLocalRot.M[2][2]);
			PsPassParameters->ComponentToVisualize = SVTSceneProxy->ComponentToVisualize;
			PsPassParameters->Extinction = SVTSceneProxy->Extinction;
			
			const UE::SVT::FTextureRenderResources* RenderResources = SVTSceneProxy->TextureRenderResources;
			if (RenderResources)
			{
				FRHITexture* PageTableTexture = RenderResources->GetPageTableTexture();
				FRHITexture* TextureA = RenderResources->GetPhysicalTileDataATexture();
				FRHITexture* TextureB = RenderResources->GetPhysicalTileDataBTexture();

				PsPassParameters->SparseVolumeTexturePageTable = PageTableTexture ? PageTableTexture : PsPassParameters->SparseVolumeTexturePageTable;
				PsPassParameters->SparseVolumeTextureA = TextureA ? TextureA : PsPassParameters->SparseVolumeTextureA;
				PsPassParameters->SparseVolumeTextureB = TextureB ? TextureB : PsPassParameters->SparseVolumeTextureB;

				SVTSceneProxy->TextureRenderResources->GetPackedUniforms(PsPassParameters->PackedSVTUniforms0, PsPassParameters->PackedSVTUniforms1);
			}

			ClearUnusedGraphResources(PixelShader, PsPassParameters);

			float VsDepthAsDeviceZ = 0.1f;

			GraphBuilder.AddPass(
				{},
				PsPassParameters,
				ERDGPassFlags::Raster,
				[PsPassParameters, VertexShader, PixelShader, ViewportRect, VsDepthAsDeviceZ](FRDGAsyncTask, FRHICommandList& RHICmdListLambda)
				{
					RHICmdListLambda.SetViewport(ViewportRect.Min.X, ViewportRect.Min.Y, 0.0f, ViewportRect.Max.X, ViewportRect.Max.Y, 1.0f);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdListLambda.ApplyCachedRenderTargets(GraphicsPSOInit);

					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;
					SetGraphicsPipelineState(RHICmdListLambda, GraphicsPSOInit, 0);

					SetShaderParameters(RHICmdListLambda, PixelShader, PixelShader.GetPixelShader(), *PsPassParameters);

					FVisualizeSparseVolumeTextureVS::FParameters VsPassParameters;
					VsPassParameters.DepthAsDeviceZ = VsDepthAsDeviceZ;
					SetShaderParameters(RHICmdListLambda, VertexShader, VertexShader.GetVertexShader(), VsPassParameters);

					RHICmdListLambda.DrawPrimitive(0, 1, 1);
				});
		}
	}

#endif // WITH_EDITORONLY_DATA
}
