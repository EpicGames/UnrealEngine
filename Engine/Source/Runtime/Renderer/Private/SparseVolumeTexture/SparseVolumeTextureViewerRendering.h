// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FScene;
class FSceneRenderer;
class FRDGBuilder;
struct FMinimalSceneTextures;
struct FSceneTextures;
struct FScreenPassTexture;


void AddSparseVolumeTextureViewerRenderPass(FRDGBuilder& GraphBuilder, FSceneRenderer& SceneRenderer, FSceneTextures& SceneTextures);

