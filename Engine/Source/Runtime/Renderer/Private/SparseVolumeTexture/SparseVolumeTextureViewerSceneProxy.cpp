// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseVolumeTexture/SparseVolumeTextureViewerSceneProxy.h"
#include "SparseVolumeTexture/SparseVolumeTextureViewerComponent.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"

#include "RenderGraphUtils.h"
#include "ScenePrivate.h"
#include "SceneView.h"

#define LOCTEXT_NAMESPACE "FSparseVolumeTextureViewerSceneProxy"

void FScene::AddSparseVolumeTextureViewer(FSparseVolumeTextureViewerSceneProxy* SVTV)
{
	check(SVTV);
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FAddSparseVolumeTextureViewerCommand)(
		[Scene, SVTV](FRHICommandListImmediate& RHICmdList)
		{
			Scene->SparseVolumeTextureViewers.Emplace(SVTV);
		});
}

void FScene::RemoveSparseVolumeTextureViewer(FSparseVolumeTextureViewerSceneProxy* SVTV)
{
	check(SVTV);
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FRemoveSparseVolumeTextureCommand)(
		[Scene, SVTV](FRHICommandListImmediate& RHICmdList)
		{
			Scene->SparseVolumeTextureViewers.Remove(SVTV);
		});
}

////////////////////////////////////////////////////////////////////////////////////////////////

FSparseVolumeTextureViewerSceneProxy::FSparseVolumeTextureViewerSceneProxy(const USparseVolumeTextureViewerComponent* InComponent, FName ResourceName)
	: FPrimitiveSceneProxy((UPrimitiveComponent*)InComponent, ResourceName)
	, TextureRenderResources(nullptr)
{
	if (InComponent->SparseVolumeTextureFrame)
	{
		TextureRenderResources = InComponent->SparseVolumeTextureFrame->GetTextureRenderResources();
	}
}

SIZE_T FSparseVolumeTextureViewerSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer = 0;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FSparseVolumeTextureViewerSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FSceneView* View = Views[ViewIndex];

		if (IsShown(View) && (VisibilityMap & (1 << ViewIndex)))
		{
			// Only render Bounds
			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
			FBoxSphereBounds LocalBound = GetBounds();
			RenderBounds(PDI, ViewFamily.EngineShowFlags, LocalBound, IsSelected());
		}
	}
}

FPrimitiveViewRelevance FSparseVolumeTextureViewerSceneProxy::GetViewRelevance(const FSceneView * View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View);
	Result.bDynamicRelevance = true;
	Result.bStaticRelevance = false;
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bOpaque = false;
	Result.bNormalTranslucency = true;
	return Result;
}

#undef LOCTEXT_NAMESPACE
