// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PrimitiveSceneProxy.h"
#include "RendererInterface.h"
#include "SceneViewExtension.h"

class USparseVolumeTextureViewerComponent;
namespace UE
{
	namespace SVT
	{
		class FTextureRenderResources;
	}
}


// Proxy representing the rendering of the SparseVolumeTextureViewer component on the render thread.
class FSparseVolumeTextureViewerSceneProxy : public FPrimitiveSceneProxy
{
public:
	FSparseVolumeTextureViewerSceneProxy(const USparseVolumeTextureViewerComponent* InComponent, FName ResourceName = NAME_None);
	virtual ~FSparseVolumeTextureViewerSceneProxy() = default;

	const UE::SVT::FTextureRenderResources* TextureRenderResources;
	FTransform GlobalTransform;
	FTransform FrameTransform;
	FVector3f VolumeResolution;
	int32 MipLevel;
	uint32 ComponentToVisualize;
	float Extinction;
	float VoxelSizeFactor;
	bool bPivotAtCentroid;

protected:

	//~ Begin FPrimitiveSceneProxy Interface

	virtual SIZE_T GetTypeHash() const override;
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override {};
	virtual void DestroyRenderThreadResources() override {};
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	virtual uint32 GetMemoryFootprint() const override { return(sizeof(*this) + GetAllocatedSize()); }

	//~ End FPrimitiveSceneProxy Interface

private:
};
