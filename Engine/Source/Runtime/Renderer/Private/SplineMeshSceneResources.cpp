// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineMeshSceneResources.h"
#include "SplineMeshShaderParams.h"
#include "SplineMeshSceneProxy.h"
#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "SystemTextures.h"
#include "LogRenderer.h"
#include "ScenePrivate.h"
#include "SceneUniformBuffer.h"
#include "RHIGlobals.h"
#include "RHIStaticStates.h"
#include "RenderCaptureInterface.h"

static TAutoConsoleVariable<int32> CVarSplineMeshSceneTextures(
	TEXT("r.SplineMesh.SceneTextures"),
	1,
	TEXT("Whether to cache all spline mesh splines in the scene to textures (performance optimization)."),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarSplineMeshSceneTexturesForceUpdate(
	TEXT("r.SplineMesh.SceneTextures.ForceUpdate"),
	0,
	TEXT("When true, will force an update of the whole spline mesh scene texture each frame (for debugging)."),
	ECVF_RenderThreadSafe
);

int32 GSplineMeshSceneTexturesCaptureNextUpdate = 0;
static FAutoConsoleVariableRef CVarSplineMeshSceneTexturesCaptureNextUpdate(
	TEXT("r.SplineMesh.SceneTextures.CaptureNextUpdate"),
	GSplineMeshSceneTexturesCaptureNextUpdate,
	TEXT("Set to 1 to perform a capture of the next spline mesh texture update. ")
	TEXT("Set to > 1 to capture the next N updates."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarSplineMeshSceneTexturesInstanceIDUploadCopy(
	TEXT("r.SplineMesh.SceneTextures.InstanceIDUploadCopy"),
	true,
	TEXT("When true, will make a copy of the registered instance IDs on buffer upload."),
	ECVF_RenderThreadSafe
);

BEGIN_SHADER_PARAMETER_STRUCT(FSplineMeshSceneResourceParameters, RENDERER_API)
	SHADER_PARAMETER(FVector2f, SplineTextureInvExtent)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, SplinePosTexture)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, SplineRotTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SplineSampler)
END_SHADER_PARAMETER_STRUCT()

DECLARE_SCENE_UB_STRUCT(FSplineMeshSceneResourceParameters, SplineMesh, RENDERER_API)

namespace SplineMesh
{
	inline uint32 GetTileIndex(uint32 SplineIndex)
	{
		return SplineIndex >> SPLINE_MESH_TEXEL_WIDTH_BITS;
	}

	inline uint32 GetIndexInTile(uint32 SplineIndex)
	{
		return SplineIndex & SPLINE_MESH_TEXEL_WIDTH_MASK;
	}

	inline FUintVector2 CalcTilePosition(uint32 TileIndex)
	{
		return FUintVector2(FMath::ReverseMortonCode2(TileIndex),
							FMath::ReverseMortonCode2(TileIndex >> 1));
	}

	inline FUintVector2 CalcTextureCoord(uint32 SplineIndex)
	{
		FUintVector2 Coord = CalcTilePosition(GetTileIndex(SplineIndex));
		Coord *= SPLINE_MESH_TEXEL_WIDTH;
		Coord.Y += GetIndexInTile(SplineIndex);

		return Coord;
	}

	inline uint32 CalcTextureSize(uint32 MaxSplines)
	{
		const FUintVector2 TilePosition = CalcTilePosition(GetTileIndex(MaxSplines - 1));
		const uint32 MaxDimension = FMath::RoundUpToPowerOfTwo(FMath::Max(TilePosition.X, TilePosition.Y) + 1);

		return MaxDimension * SPLINE_MESH_TEXEL_WIDTH;
	}

	static void GetDefaultResourceParameters(FSplineMeshSceneResourceParameters& ShaderParams, FRDGBuilder& GraphBuilder)
	{
		// Initialize global system textures (pass-through if already initialized).
		GSystemTextures.InitializeTextures(GraphBuilder.RHICmdList, GMaxRHIFeatureLevel);

		ShaderParams.SplinePosTexture = GraphBuilder.CreateSRV(GSystemTextures.GetBlackDummy(GraphBuilder));
		ShaderParams.SplineRotTexture = GraphBuilder.CreateSRV(GSystemTextures.GetBlackDummy(GraphBuilder));
		ShaderParams.SplineSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		ShaderParams.SplineTextureInvExtent = FVector2f::One();
	}
}

IMPLEMENT_SCENE_UB_STRUCT(FSplineMeshSceneResourceParameters, SplineMesh, SplineMesh::GetDefaultResourceParameters);

class FSplineMeshTextureFillCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSplineMeshTextureFillCS);
	SHADER_USE_PARAMETER_STRUCT(FSplineMeshTextureFillCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUSceneParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SplinePosTextureOut)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SplineRotTextureOut)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, InstanceIdLookup)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, UpdateRequests)
		SHADER_PARAMETER(uint32, NumUpdateRequests)
		SHADER_PARAMETER(float, TextureHeight)
		SHADER_PARAMETER(float, TextureHeightInv)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FGlobalShader::ShouldCompilePermutation(Parameters) &&
			UseSplineMeshSceneResources(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FSplineMeshTextureFillCS, "/Engine/Private/SplineMeshSceneTexture.usf", "FillTexture", SF_Compute)

IMPLEMENT_SCENE_EXTENSION(FSplineMeshSceneExtension);

bool FSplineMeshSceneExtension::ShouldCreateExtension(FScene& Scene)
{
	return UseSplineMeshSceneResources(GetFeatureLevelShaderPlatform(Scene.GetFeatureLevel()));
}

ISceneExtensionUpdater* FSplineMeshSceneExtension::CreateUpdater()
{
	return new FSplineMeshSceneUpdater(*this);
}

ISceneExtensionRenderer* FSplineMeshSceneExtension::CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags)
{
	return new FSplineMeshSceneRenderer(InSceneRenderer, *this);	
}

FSplineMeshSceneExtension::FPrimitiveSlot& FSplineMeshSceneExtension::Register(const FPrimitiveSceneInfo& PrimitiveSceneInfo)
{	
	FPrimitiveSlot& Slot = RegisteredPrimitives.FindOrAdd(&PrimitiveSceneInfo);
	if (ensureMsgf(Slot.NumSplines == 0, TEXT("This primitive was already registered!")))
	{
		// Alloc space for the new splines and ensure they are included in the next update
		AllocTextureSpace(PrimitiveSceneInfo, GetNumSplines(PrimitiveSceneInfo), Slot);
	}

	return Slot;
}

void FSplineMeshSceneExtension::Unregister(const FPrimitiveSceneInfo& PrimitiveSceneInfo)
{
	FPrimitiveSlot* Slot = RegisteredPrimitives.Find(&PrimitiveSceneInfo);
	if (!ensure(Slot != nullptr))
	{
		return;
	}

	SlotAllocator.Free(Slot->FirstSplineIndex, Slot->NumSplines);

	// Clear the instance IDs of the newly freed range
	for (uint32 i = 0; i < Slot->NumSplines; ++i)
	{
		RegisteredInstanceIds[Slot->FirstSplineIndex + i] = INDEX_NONE;
	}
	
	// shorten this look-up if the allocator shrinks
	RegisteredInstanceIds.SetNumUninitialized(FMath::Min(SlotAllocator.GetMaxSize(), RegisteredInstanceIds.Num()));

	RegisteredPrimitives.Remove(&PrimitiveSceneInfo);

	// Ensure we repopulate the instance ID lookup next update
	bInstanceLookupDirty = true;
}

void FSplineMeshSceneExtension::AllocTextureSpace(const FPrimitiveSceneInfo& PrimitiveSceneInfo, uint32 NumSplines, FPrimitiveSlot& OutSlot)
{
	check(NumSplines > 0);

	OutSlot.NumSplines = NumSplines;
	OutSlot.FirstSplineIndex = SlotAllocator.Allocate(NumSplines);

	// If we don't have the space for these instances in the registered ID list, allocate them
	if (RegisteredInstanceIds.Num() < SlotAllocator.GetMaxSize())
	{
		RegisteredInstanceIds.Reserve(SlotAllocator.GetMaxSize());
		for (uint32 i = RegisteredInstanceIds.Num(); i < OutSlot.FirstSplineIndex + NumSplines; ++i)
		{
			RegisteredInstanceIds.Add(INDEX_NONE);
		}
	}

	// Store the instance ID for each spline slot allocated (will be used to fill texels with spline data)
	// NOTE: This is assuming a spline per instance. If that changes, this look-up will need to as well
	const int32 InstanceSceneDataOffset = PrimitiveSceneInfo.GetInstanceSceneDataOffset();
	for (uint32 i = 0; i < NumSplines; ++i)
	{
		const uint32 SplineIndex = OutSlot.FirstSplineIndex + i;
		check(RegisteredInstanceIds[SplineIndex] == INDEX_NONE); // sanity check it has been cleared
		RegisteredInstanceIds[SplineIndex] = InstanceSceneDataOffset + i;
	}

	bInstanceLookupDirty = true;

	// Give the scene proxy the coordinates allocated so it can place it in its scene data.
	AssignCoordinates(PrimitiveSceneInfo, OutSlot);
}

uint32 FSplineMeshSceneExtension::GetNumSplines(const FPrimitiveSceneInfo& SceneInfo)
{
	// We only support spline mesh scene proxies currently, and we assume the number of splines is equal to the
	// number of instance scene data entries. Support could be added later for other scene proxy types that need
	// baked down splines.
	check(SceneInfo.Proxy->IsSplineMesh());

	return SceneInfo.GetNumInstanceSceneDataEntries();
}

void FSplineMeshSceneExtension::AssignCoordinates(const FPrimitiveSceneInfo& SceneInfo, const FPrimitiveSlot& Slot)
{
	check(SceneInfo.Proxy->IsSplineMesh()); // sanity check

	if (SceneInfo.Proxy->IsNaniteMesh())
	{
		AssignCoordinates(static_cast<FNaniteSplineMeshSceneProxy*>(SceneInfo.Proxy), Slot);
	}
	else
	{
		AssignCoordinates(static_cast<FSplineMeshSceneProxy*>(SceneInfo.Proxy), Slot);
	}
}

template<typename TSplineMeshSceneProxy>
void FSplineMeshSceneExtension::AssignCoordinates(TSplineMeshSceneProxy* SceneProxy, const FPrimitiveSlot& Slot)
{
	for (uint32 i = 0; i < Slot.NumSplines; ++i)
	{
		SceneProxy->SetSplineTextureCoord_RenderThread(i, SplineMesh::CalcTextureCoord(Slot.FirstSplineIndex + i));
	}
}

void FSplineMeshSceneExtension::DefragTexture()
{
	// NOTE: Currently not attempting to reduce motions to a minimal set, we're just ditching our cache
	// and re-assigning space in the new texture that will be created next update
	SavedPosTexture = nullptr;
	SavedRotTexture = nullptr;
	SlotAllocator.Reset();
	RegisteredInstanceIds.Reset();
	for (auto& Pair : RegisteredPrimitives)
	{
		AllocTextureSpace(*Pair.Key, Pair.Value.NumSplines, Pair.Value);
	}

	// Sanity check defragmentation results
	check(SlotAllocator.GetMaxSize() == SlotAllocator.GetSparselyAllocatedSize());
}

// Check to replace or update the cached Instance ID lookup buffer
FRDGBufferSRVRef FSplineMeshSceneExtension::GetInstanceIdLookupSRV(FRDGBuilder& GraphBuilder, bool bForceUpdate)
{
	const uint32 InstanceIdLookupSize = RegisteredInstanceIds.Num();
	const uint32 CurInstanceIdLookupSize = SavedIdLookup.IsValid() ? SavedIdLookup->Desc.NumElements : 0u;
	const bool bNeedsResize = bForceUpdate || InstanceIdLookupSize != CurInstanceIdLookupSize;
	const bool bNeedsUpload = bForceUpdate || !SavedIdLookup.IsValid() || bInstanceLookupDirty;

	FRDGBufferRef InstanceIdLookup = nullptr;
	if (bNeedsResize)
	{
		InstanceIdLookup = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), InstanceIdLookupSize),
			TEXT("SplineMesh.InstanceIdLookup")
		);

		// Only persist the buffer if we're not force-updating every frame (keeps it in transient memory, otherwise)
		SavedIdLookup = bForceUpdate ? nullptr : GraphBuilder.ConvertToExternalBuffer(InstanceIdLookup);
	}
	else
	{
		InstanceIdLookup = GraphBuilder.RegisterExternalBuffer(SavedIdLookup, TEXT("SplineMesh.InstanceIdLookup"));
	}

	if (bNeedsUpload)
	{
		// Upload the contents
		const ERDGInitialDataFlags Flags = CVarSplineMeshSceneTexturesInstanceIDUploadCopy.GetValueOnRenderThread() ?
			ERDGInitialDataFlags::None : ERDGInitialDataFlags::NoCopy;
		GraphBuilder.QueueBufferUpload<uint32>(InstanceIdLookup, RegisteredInstanceIds, Flags);
	}

	bInstanceLookupDirty = false;
	return GraphBuilder.CreateSRV(InstanceIdLookup);
}

void FSplineMeshSceneExtension::ClearAllCache()
{
	SavedPosTexture = nullptr;
	SavedRotTexture = nullptr;
	SavedIdLookup = nullptr;
}


void FSplineMeshSceneUpdater::PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet)
{
	for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.RemovedPrimitiveSceneInfos)
	{
		if (PrimitiveSceneInfo->Proxy->IsSplineMesh())
		{
			SceneData->Unregister(*PrimitiveSceneInfo);
		}
	}

	// Consolidate free spans on the allocator after a batch of frees.
	// NOTE: This is important to ensure the allocator trims down its max size naturally when freeing space off the end.
	// This allows us to downsize the texture without defragging, which is a much heavier operation.
	SceneData->SlotAllocator.Consolidate();
}

void FSplineMeshSceneUpdater::PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet)
{
	auto RequestUpdate = [this](const FSplineMeshSceneExtension::FPrimitiveSlot& Slot)
	{
		for (uint32 i = 0; i < Slot.NumSplines; ++i)
		{
			UpdateRequests.AddUnique(Slot.FirstSplineIndex + i);
		}
	};

	ChangeSet.PrimitiveUpdates.ForEachUpdateCommand(ESceneUpdateCommandFilter::AddedUpdated, FUpdateInstanceCommand::IdBit | FUpdateTransformCommand::IdBit, [&](const FPrimitiveUpdateCommand& Cmd)
	{
		if (Cmd.GetSceneInfo()->Proxy->IsSplineMesh())
		{
			if (Cmd.IsAdd())
			{
				RequestUpdate(SceneData->Register(*Cmd.GetSceneInfo()));
			}
			else
			{
				RequestUpdate(SceneData->RegisteredPrimitives.FindChecked(Cmd.GetSceneInfo()));
			}
		}
	});

	if (SceneData->NumRegisteredPrimitives() > 0)
	{
		// Check to defrag the texture when we could halve the dimensions of the texture just by doing so. Must do this
		// before GPU scene updates to give the primitives a chance to re-upload their new texcoord assignments.
		const uint32 CurSize = SceneData->SlotAllocator.GetMaxSize();
		const uint32 DefraggedSize = SceneData->SlotAllocator.GetSparselyAllocatedSize();
		if (SplineMesh::CalcTextureSize(DefraggedSize) < SplineMesh::CalcTextureSize(CurSize))
		{
			SceneData->DefragTexture();
		}
	}
	else
	{
		// No active splines, just clear all cache
		SceneData->ClearAllCache();
	}
}

void FSplineMeshSceneUpdater::PostGPUSceneUpdate(FRDGBuilder& GraphBuilder)
{
	if (SceneData->NumRegisteredPrimitives() == 0)
	{
		return; // nothing to do
	}

	// Check if we need to re-size the cached texture
	uint32 NeededSize = SplineMesh::CalcTextureSize(SceneData->SlotAllocator.GetMaxSize());
	const uint32 CurSize = SceneData->SavedPosTexture.IsValid() ? SceneData->SavedPosTexture->GetDesc().Extent.X : 0;

	// Clamp to the max dimension and check to report an error about over-sizing the spline mesh texture
	if (NeededSize > SPLINE_MESH_TEXTURE_MAX_DIMENSION)
	{
		NeededSize = SPLINE_MESH_TEXTURE_MAX_DIMENSION;
		if (!SceneData->bOverflowError)
		{
			UE_LOGF(LogRenderer, Error,
				"Too many spline meshes have been registered with the scene. The spline mesh texture has grown " "to its max size (%dx%d - see r.SplineMesh.BakeToTexture.MaxDimension) and has ran out of space. " "Expect some spline meshes to render incorrectly.",
				NeededSize, NeededSize
			);
			SceneData->bOverflowError = true;
		}
	}

	// Check if we are forcing a full update because we have no cache or are debugging
	const bool bForceUpdate = CVarSplineMeshSceneTexturesForceUpdate.GetValueOnRenderThread() != 0;
	bool bFullUpdate = !SceneData->SavedPosTexture.IsValid() || bForceUpdate;

	// Register or create the spline texture
	FRDGTextureRef PosTexture = nullptr;
	FRDGTextureRef RotTexture = nullptr;
	if (NeededSize != CurSize)
	{
		PosTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(
				FIntPoint(NeededSize, NeededSize),
				PF_A32B32G32R32F,
				EClearBinding::ENoneBound,
				TexCreate_UAV | TexCreate_ShaderResource
			),
			TEXT("SplineMesh.SplinePosTexture")
		);
		RotTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(
				FIntPoint(NeededSize, NeededSize),
				PF_R16G16B16A16_SNORM, // Optimal format for normalized quaternions
				EClearBinding::ENoneBound,
				TexCreate_UAV | TexCreate_ShaderResource
			),
			TEXT("SplineMesh.SplineRotTexture")
		);

		if (!bFullUpdate)
		{
			// We are resizing, so copy the previous contents to this frame
			const uint32 CopyExtent = FMath::Min(NeededSize, CurSize);

			FRDGTextureRef CopySrc = GraphBuilder.RegisterExternalTexture(
				SceneData->SavedPosTexture,
				TEXT("SplineMesh.PrevSplinePosTexture")
			);
			AddCopyTexturePass(
				GraphBuilder,
				CopySrc,
				PosTexture,
				FIntPoint::ZeroValue, // InputPosition
				FIntPoint::ZeroValue, // OutputPosition
				FIntPoint(CopyExtent, CopyExtent) // Size
			);
			
			CopySrc = GraphBuilder.RegisterExternalTexture(
				SceneData->SavedRotTexture,
				TEXT("SplineMesh.PrevSplineRotTexture")
			);
			AddCopyTexturePass(
				GraphBuilder,
				CopySrc,
				RotTexture,
				FIntPoint::ZeroValue, // InputPosition
				FIntPoint::ZeroValue, // OutputPosition
				FIntPoint(CopyExtent, CopyExtent) // Size
			);
		}

		// Don't store off the texture if we're updating every frame (keeps it transient, otherwise)
		SceneData->SavedPosTexture = bForceUpdate ? nullptr : GraphBuilder.ConvertToExternalTexture(PosTexture);
		SceneData->SavedRotTexture = bForceUpdate ? nullptr : GraphBuilder.ConvertToExternalTexture(RotTexture);
	}
	else
	{
		check(SceneData->SavedPosTexture.IsValid());
		PosTexture = GraphBuilder.RegisterExternalTexture(SceneData->SavedPosTexture, TEXT("SplineMesh.SplinePosTexture"));

		check(SceneData->SavedRotTexture.IsValid());
		RotTexture = GraphBuilder.RegisterExternalTexture(SceneData->SavedRotTexture, TEXT("SplineMesh.SplineRotTexture"));
	}

	// Perform the update and clear pending requests
	const FVector2f Extent = FVector2f(float(NeededSize));
	const FVector2f InvExtent = FVector2f(1.0f / Extent.X, 1.0f / Extent.Y);
	if (bFullUpdate || UpdateRequests.Num() > 0)
	{
		AddUpdatePass(
			GraphBuilder,
			PosTexture,
			RotTexture,
			Extent,
			InvExtent,
			bFullUpdate,
			bForceUpdate
		);
	}
}

void FSplineMeshSceneUpdater::AddUpdatePass(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef PosTexture,
	FRDGTextureRef RotTexture,
	FVector2f Extent,
	FVector2f InvExtent,
	bool bFullUpdate,
	bool bForceUpdate)
{
	RenderCaptureInterface::FScopedCapture Capture(
		GSplineMeshSceneTexturesCaptureNextUpdate > 0,
		GraphBuilder,
		TEXT("Spline Mesh Texture Update")
	);
	if (GSplineMeshSceneTexturesCaptureNextUpdate > 0)
	{
		--GSplineMeshSceneTexturesCaptureNextUpdate;
	}

	FRDGTextureUAVRef PosTextureUAV = GraphBuilder.CreateUAV(PosTexture);
	FRDGTextureUAVRef RotTextureUAV = GraphBuilder.CreateUAV(RotTexture);
	if (bForceUpdate)
	{
		// If we're debugging, clear the texture first so we can catch bugs
		AddClearUAVPass(GraphBuilder, PosTextureUAV, FLinearColor::Black);
		AddClearUAVPass(GraphBuilder, RotTextureUAV, FLinearColor::Black);
	}

	FRDGBufferRef UpdateRequestBuffer = nullptr;
	const uint32 NumUpdateRequests = bFullUpdate ? 0 : UpdateRequests.Num();
	if (NumUpdateRequests > 0)
	{
		// Update only select instances
		UpdateRequestBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NumUpdateRequests),
			TEXT("SplineMesh.UpdateRequests")
		);
		GraphBuilder.QueueBufferUpload<uint32>(UpdateRequestBuffer, UpdateRequests);
	}
	else
	{
		// This will be unused
		UpdateRequestBuffer = GSystemTextures.GetDefaultStructuredBuffer<uint32>(GraphBuilder);
	}

	FScene& Scene = SceneData->Scene;
	auto* PassParameters = GraphBuilder.AllocParameters<FSplineMeshTextureFillCS::FParameters>();

	PassParameters->GPUSceneParameters = Scene.GPUScene.GetShaderParameters(GraphBuilder);
	PassParameters->SplinePosTextureOut = PosTextureUAV;
	PassParameters->SplineRotTextureOut = RotTextureUAV;
	PassParameters->InstanceIdLookup = SceneData->GetInstanceIdLookupSRV(GraphBuilder, bForceUpdate);
	PassParameters->UpdateRequests = GraphBuilder.CreateSRV(UpdateRequestBuffer);
	PassParameters->NumUpdateRequests = NumUpdateRequests;
	PassParameters->TextureHeight = Extent.Y;
	PassParameters->TextureHeightInv = InvExtent.Y;

	auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FSplineMeshTextureFillCS>();

	const uint32 NumThreadGroups = NumUpdateRequests ? NumUpdateRequests : SceneData->SlotAllocator.GetMaxSize();
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("SplineMeshTextureUpdate"),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCountWrapped(NumThreadGroups)
	);
}


void FSplineMeshSceneRenderer::UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniforms)
{
	if (SceneData->NumRegisteredPrimitives() > 0)
	{
		FRDGTextureRef PosTexture = GraphBuilder.RegisterExternalTexture(SceneData->SavedPosTexture);
		FRDGTextureRef RotTexture = GraphBuilder.RegisterExternalTexture(SceneData->SavedRotTexture);

		const FIntPoint Extent = SceneData->SavedPosTexture->GetDesc().Extent;
		const FVector2f InvExtent = FVector2f(1.0f / Extent.X, 1.0f / Extent.Y);
	
		// Lastly, set up the scene uniforms for spline meshes
		FSplineMeshSceneResourceParameters ShaderParams;
		ShaderParams.SplinePosTexture = GraphBuilder.CreateSRV(PosTexture);
		ShaderParams.SplineRotTexture = GraphBuilder.CreateSRV(RotTexture);
		ShaderParams.SplineSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		ShaderParams.SplineTextureInvExtent = InvExtent;
	
		SceneUniforms.Set(SceneUB::SplineMesh, ShaderParams);
	}
}