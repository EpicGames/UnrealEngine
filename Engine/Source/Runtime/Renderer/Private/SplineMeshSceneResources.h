// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SpanAllocator.h"
#include "SceneUniformBuffer.h"
#include "RHIShaderPlatform.h"
#include "RendererInterface.h"
#include "Containers/Map.h"
#include "RenderGraphResources.h"
#include "SceneExtensions.h"

struct IPooledRenderTarget;

/**
 *	FSplineMeshSceneExtension
 *
 *	This class manages a texture that is used to bake down spline position values at a fixed number of texels
 *	to lower the cost of sampling these splines when deforming spline mesh vertices. Each spline in the scene is
 *	allocated an Nx1 region of the scene-wide 2D texture, and the system performs minimal updates to the texture
 *	on demand as spline mesh scene proxies are created or request updates due to changes to their parameters at run time.
 *
 *	Notes about texture allocation:
 *	- The scene spline mesh texture is allocated in square pow2 tiles of size SPLINE_MESH_TEXEL_WIDTH^2.
 *	- Tiles are encoded in Morton order so that the texture can be resized as the upper bound grows or shrinks
 *	  without changing the assigned texture coordinate of registered spline meshes.
 *	- Defragmentation of the texture will occur if the texture could be 1/4 the size of the current texture
 *	  size when tightly allocated.
 */
class FSplineMeshSceneExtension : public ISceneExtension
{
	friend class FSplineMeshSceneUpdater;
	friend class FSplineMeshSceneRenderer;

	DECLARE_SCENE_EXTENSION(RENDERER_API, FSplineMeshSceneExtension);

public:
	using ISceneExtension::ISceneExtension;

	static bool ShouldCreateExtension(FScene& InScene);
	
	virtual ISceneExtensionUpdater* CreateUpdater() override;
	virtual ISceneExtensionRenderer* CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags) override;

	uint32 NumRegisteredPrimitives() const { return RegisteredPrimitives.Num(); }

private:
	struct FPrimitiveSlot
	{
		uint32 FirstSplineIndex = INDEX_NONE;
		uint32 NumSplines = 0;
	};
	using FPrimitiveSlotMap = TMap<const FPrimitiveSceneInfo*, FPrimitiveSlot>;

	FPrimitiveSlot& Register(const FPrimitiveSceneInfo& PrimitiveSceneInfo);
	void Unregister(const FPrimitiveSceneInfo& PrimitiveSceneInfo);
	void AllocTextureSpace(const FPrimitiveSceneInfo& PrimitiveSceneInfo, uint32 NumSplines, FPrimitiveSlot& OutSlot);
	static uint32 GetNumSplines(const FPrimitiveSceneInfo& SceneInfo);
	void AssignCoordinates(const FPrimitiveSceneInfo& SceneInfo, const FPrimitiveSlot& Slot);
	template<typename TSplineMeshSceneProxy>
	void AssignCoordinates(TSplineMeshSceneProxy* SceneProxy, const FPrimitiveSlot& Slot);
	void DefragTexture();
	FRDGBufferSRVRef GetInstanceIdLookupSRV(FRDGBuilder& GraphBuilder, bool bForceUpdate);
	void ClearAllCache();

private:
	FPrimitiveSlotMap RegisteredPrimitives;
	TArray<uint32> RegisteredInstanceIds;
	FSpanAllocator SlotAllocator;
	TRefCountPtr<IPooledRenderTarget> SavedPosTexture;
	TRefCountPtr<IPooledRenderTarget> SavedRotTexture;
	TRefCountPtr<FRDGPooledBuffer> SavedIdLookup;
	bool bInstanceLookupDirty = true;
	bool bOverflowError = false;
};

/** This class performs updates to the persistent spline mesh resources. */
class FSplineMeshSceneUpdater : public ISceneExtensionUpdater
{
	DECLARE_SCENE_EXTENSION_UPDATER(FSplineMeshSceneUpdater, FSplineMeshSceneExtension);

public:
	FSplineMeshSceneUpdater(FSplineMeshSceneExtension& InSceneData) : SceneData(&InSceneData) {}

	virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override;
	virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override;
	virtual void PostGPUSceneUpdate(FRDGBuilder& GraphBuilder) override;

private:
	void AddUpdatePass(
		FRDGBuilder& GraphBuilder,
		FRDGTextureRef PosTexture,
		FRDGTextureRef RotTexture,
		FVector2f Extent,
		FVector2f InvExtent,
		bool bFullUpdate,
		bool bForceUpdate
	);

	FSplineMeshSceneExtension* SceneData = nullptr;
	TArray<uint32, FSceneRenderingArrayAllocator> UpdateRequests;
};

/** This class is used to insert the spline mesh scene uniforms into the scene uniform buffer for any given renderer */
class FSplineMeshSceneRenderer : public ISceneExtensionRenderer
{
	DECLARE_SCENE_EXTENSION_RENDERER(FSplineMeshSceneRenderer, FSplineMeshSceneExtension);

public:
	FSplineMeshSceneRenderer(FSceneRendererBase& InSceneRenderer, FSplineMeshSceneExtension& InSceneData) : ISceneExtensionRenderer(InSceneRenderer), SceneData(&InSceneData) {}
	virtual void UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniforms) override;

private:
	FSplineMeshSceneExtension* SceneData = nullptr;
};