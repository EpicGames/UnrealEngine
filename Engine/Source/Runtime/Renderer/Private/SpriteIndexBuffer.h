// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"
#include "RHI.h"
#include "RHICommandList.h"

template< uint32 NumSprites >
class FSpriteIndexBuffer : public FIndexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateIndex<uint16>(TEXT("FSpriteIndexBuffer"), 6 * NumSprites)
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		TRHIBufferInitializer<uint16> Indices = RHICmdList.CreateBufferInitializer(CreateDesc);

		for (uint32 SpriteIndex = 0; SpriteIndex < NumSprites; ++SpriteIndex)
		{
			Indices[SpriteIndex*6 + 0] = SpriteIndex*4 + 0;
			Indices[SpriteIndex*6 + 1] = SpriteIndex*4 + 3;
			Indices[SpriteIndex*6 + 2] = SpriteIndex*4 + 2;
			Indices[SpriteIndex*6 + 3] = SpriteIndex*4 + 0;
			Indices[SpriteIndex*6 + 4] = SpriteIndex*4 + 1;
			Indices[SpriteIndex*6 + 5] = SpriteIndex*4 + 3;
		}

		IndexBufferRHI = Indices.Finalize();
	}
};