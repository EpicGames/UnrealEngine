// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExponentialHeightFogStateStreamImpl.h"
#include "SceneInterface.h"
#include "StateStreamCreator.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

void FExponentialHeightFogStateStreamImpl::Render_OnCreate(const FExponentialHeightFogStaticState& Ss, const FExponentialHeightFogDynamicState& Ds, FExponentialHeightFogSceneProxy*& UserData, bool IsDestroyedInSameFrame)
{
	LaneUserData->AddExponentialHeightFog(uint64(&Ss), Ds);
}

void FExponentialHeightFogStateStreamImpl::Render_OnUpdate(const FExponentialHeightFogStaticState& Ss, const FExponentialHeightFogDynamicState& Ds, FExponentialHeightFogSceneProxy*& UserData)
{
}

void FExponentialHeightFogStateStreamImpl::Render_OnDestroy(const FExponentialHeightFogStaticState& Ss, const FExponentialHeightFogDynamicState& Ds, FExponentialHeightFogSceneProxy*& UserData)
{
	LaneUserData->RemoveExponentialHeightFog(uint64(&Ss));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STATESTREAM_CREATOR_INSTANCE(FExponentialHeightFogStateStreamImpl)

////////////////////////////////////////////////////////////////////////////////////////////////////
