// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericStateStream.h"
#include "StateStream/ExponentialHeightFogStateStream.h"

#define UE_API RENDERER_API

class FSceneInterface;
class FExponentialHeightFogSceneProxy;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FExponentialHeightFogStateStreamSettings : TStateStreamSettings<IExponentialHeightFogStateStream, FExponentialHeightFogSceneProxy, FSceneInterface>
{
	static inline constexpr bool SkipCreatingDeletes = true;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FExponentialHeightFogStateStreamImpl : public TStateStream<FExponentialHeightFogStateStreamSettings>
{
	UE_API virtual void Render_OnCreate(const FExponentialHeightFogStaticState& Ss, const FExponentialHeightFogDynamicState& Ds, FExponentialHeightFogSceneProxy*& UserData, bool IsDestroyedInSameFrame) override;
	UE_API virtual void Render_OnUpdate(const FExponentialHeightFogStaticState& Ss, const FExponentialHeightFogDynamicState& Ds, FExponentialHeightFogSceneProxy*& UserData) override;
	UE_API virtual void Render_OnDestroy(const FExponentialHeightFogStaticState& Ss, const FExponentialHeightFogDynamicState& Ds, FExponentialHeightFogSceneProxy*& UserData) override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
