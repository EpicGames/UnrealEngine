// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParticleSystemStateStreamImpl.h"
#include "Engine/World.h"
#include "ParticleHelper.h"
#include "ParticleSystemSceneProxy.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleLODLevel.h"
#include "Particles/ParticleSystem.h"
#include "ScenePrivate.h"
#include "StateStreamCreator.h"
#include "StateStreamManagerImpl.h"
#include "TransformStateStream.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

FParticleSystemObjectCascade::~FParticleSystemObjectCascade()
{
	FSceneInterface& Scene = PrimitiveSceneData.SceneProxy->GetScene();
	Scene.RemovePrimitive(&PrimitiveSceneDesc);

	if (TransformObject)
	{
		TransformObject->RemoveListener(this);
		TransformObject = nullptr;
	}
}

void FParticleSystemObjectCascade::OnTransformObjectDirty()
{
	FTransformObject::Info Info = TransformObject->GetInfo();
	// TODO: Implement this to handle changing visibility and movement
}

void FParticleSystemObjectCascade::InitializeSystem(FParticleSystemSceneProxyDesc& OutDesc, const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds)
{
	if (Template == NULL)
	{
		return;
	}

	// Variables from UParticleSystemComponent and base classes
	float WarmupTime = Template->WarmupTime;
	float WarmupTickRate = Template->WarmupTickRate;
	bool bIsViewRelevanceDirty = true;
	bool bWasCompleted = false;
	EDetailMode DetailMode = DM_Medium;

	const int32 GlobalDetailMode = PDM_Medium;//GetCurrentDetailMode();
	const bool bCanEverRender = true;//CanEverRender();

		

	//simplified version.
	int32 NumInstances = EmitterInstances.Num();
	int32 NumEmitters = Template->Emitters.Num();
	const bool bIsFirstCreate = NumInstances == 0;
	EmitterInstances.SetNumZeroed(NumEmitters);

	bWasCompleted = bIsFirstCreate ? false : bWasCompleted;

	bool bClearDynamicData = false;
	int32 PreferredLODLevel = LODLevel;
	bool bSetLodLevels = LODLevel > 0; //We should set the lod level even when creating all emitters if the requested LOD is not 0. 

	for (int32 Idx = 0; Idx < NumEmitters; Idx++)
	{
		UParticleEmitter* Emitter = Template->Emitters[Idx];
		if (Emitter)
		{
			FParticleEmitterInstance* Instance = NumInstances == 0 ? NULL : EmitterInstances[Idx];
			check(GlobalDetailMode < EParticleDetailMode::PDM_MAX);
			const bool bDetailModeAllowsRendering = DetailMode <= GlobalDetailMode && (Emitter->DetailModeBitmask & (1 << GlobalDetailMode));
			const bool bShouldCreateAndOrInit = bDetailModeAllowsRendering && Emitter->HasAnyEnabledLODs() && bCanEverRender;

			if (bShouldCreateAndOrInit)
			{
				if (Instance)
				{
					Instance->SetHaltSpawning(false);
					Instance->SetHaltSpawningExternal(false);
				}
				else
				{
					Instance = Emitter->CreateInstance(*this);
					EmitterInstances[Idx] = Instance;
				}

				if (Instance)
				{
					Instance->bEnabled = true;
					Instance->InitParameters(Emitter);
					Instance->Init();

					UParticleLODLevel* EmitterLODLevel = Emitter->LODLevels[0];//LODIndex];
					//FMaterialRelevance& LODViewRel = CachedViewRelevanceFlags[LODIndex];
					Instance->GatherMaterialRelevance(&OutDesc.MaterialRelevance, EmitterLODLevel, GMaxRHIShaderPlatform);

					PreferredLODLevel = FMath::Min(PreferredLODLevel, Emitter->LODLevels.Num());
					bSetLodLevels |= !bIsFirstCreate;//Only set lod levels if we init any instances and it's not the first creation time.
				}
			}
			else
			{
				if (Instance)
				{
#if STATS
					Instance->PreDestructorCall();
#endif
					delete Instance;
					EmitterInstances[Idx] = NULL;
					bClearDynamicData = true;
				}
			}
		}
	}

	if (bClearDynamicData)
	{
		//ClearDynamicData();
	}

	if (bSetLodLevels)
	{
		if (PreferredLODLevel != LODLevel)
		{
			// This should never be higher...
			check(PreferredLODLevel < LODLevel);
			LODLevel = PreferredLODLevel;
		}

		for (int32 Idx = 0; Idx < EmitterInstances.Num(); Idx++)
		{
			FParticleEmitterInstance* Instance = EmitterInstances[Idx];
			// set the LOD levels here
			if (Instance)
			{
				Instance->CurrentLODLevelIndex	= LODLevel;

				// small safety net for OR-11322; can be removed if the ensure never fires after the change in SetTemplate (reset all instances LOD indices to 0)
				if (Instance->CurrentLODLevelIndex >= Instance->SpriteTemplate->LODLevels.Num())
				{
					Instance->CurrentLODLevelIndex = Instance->SpriteTemplate->LODLevels.Num()-1;
					ensureMsgf(false, TEXT("LOD access out of bounds (OR-11322). Please let olaf.piesche or simon.tovey know."));
				}
				Instance->CurrentLODLevel = Instance->SpriteTemplate->LODLevels[Instance->CurrentLODLevelIndex];
			}
		}
	}

	LWCTile = FLargeWorldRenderScalar::GetTileFor(GetComponentTransform().GetLocation());
}

void FParticleSystemObjectCascade::Update()
{
	float DeltaTimeTick = 1.0f/30.0f;
	bool bSuppressSpawning = false;

	// Tick Subemitters.
	int32 EmitterIndex;
	//NumSignificantEmitters = 0;
	for (EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
	{
		FParticleEmitterInstance* Instance = EmitterInstances[EmitterIndex];

		if (EmitterIndex + 1 < EmitterInstances.Num())
		{
			FParticleEmitterInstance* NextInstance = EmitterInstances[EmitterIndex+1];
			FPlatformMisc::Prefetch(NextInstance);
		}

		if (Instance && Instance->SpriteTemplate)
		{
			check(Instance->SpriteTemplate->LODLevels.Num() > 0);

			UParticleLODLevel* SpriteLODLevel = Instance->SpriteTemplate->GetCurrentLODLevel(Instance);
			if (SpriteLODLevel && SpriteLODLevel->bEnabled)
			{
				Instance->Tick(DeltaTimeTick, bSuppressSpawning);

				Instance->Tick_MaterialOverrides(EmitterIndex);
				//TotalActiveParticles += Instance->ActiveParticles;
			}
		}
	}

	FParticleDynamicData* ParticleDynamicData = CreateDynamicData(PrimitiveSceneData.SceneProxy->GetScene().GetFeatureLevel());
	FParticleSystemSceneProxy* Proxy = (FParticleSystemSceneProxy*)PrimitiveSceneData.SceneProxy;
	Proxy->UpdateData( ParticleDynamicData );

	bJustRegistered = false;
}

FParticleDynamicData* FParticleSystemObjectCascade::CreateDynamicData(ERHIFeatureLevel::Type InFeatureLevel)
{
	FParticleDynamicData* ParticleDynamicData = new FParticleDynamicData();

	if (Template)
	{
		ParticleDynamicData->SystemPositionForMacroUVs = GetComponentTransform().TransformPosition(Template->MacroUVPosition);
		ParticleDynamicData->SystemRadiusForMacroUVs = Template->MacroUVRadius;
	}

	{
		// Is the particle system allowed to run?
		if (true)//( bForcedInActive == false )
		{
			//SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_CreateDynamicData_Gather);
			ParticleDynamicData->DynamicEmitterDataArray.Reset();
			ParticleDynamicData->DynamicEmitterDataArray.Reserve(EmitterInstances.Num());

			//QUICK_SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_GetDynamicData);
			for (int32 EmitterIndex = 0; EmitterIndex < EmitterInstances.Num(); EmitterIndex++)
			{
				FDynamicEmitterDataBase* NewDynamicEmitterData = NULL;
				FParticleEmitterInstance* EmitterInst = EmitterInstances[EmitterIndex];
				if (EmitterInst)
				{
					//FScopeCycleCounterEmitter AdditionalScope(EmitterInst);

					// Generate the dynamic data for this emitter
					{
						//SCOPE_CYCLE_COUNTER(STAT_ParticleSystemComponent_GetDynamicData);
						bool bIsOwnerSeleted = false;
						NewDynamicEmitterData = EmitterInst->GetDynamicData(bIsOwnerSeleted, InFeatureLevel);
					}
					if( NewDynamicEmitterData != NULL )
					{
						NewDynamicEmitterData->bValid = true;
						ParticleDynamicData->DynamicEmitterDataArray.Add( NewDynamicEmitterData );
						NewDynamicEmitterData->EmitterIndex = EmitterIndex;

						NewDynamicEmitterData->EmitterIndex = EmitterIndex;
					}
				}
			}
		}
	}

	return ParticleDynamicData;
}

const FTransform& FParticleSystemObjectCascade::GetAsyncComponentToWorld() const { return TransformObject->GetInfo().WorldTransform; }
UObject* FParticleSystemObjectCascade::GetDistributionData() const { return nullptr; }
const FTransform& FParticleSystemObjectCascade::GetComponentTransform() const { return TransformObject->GetInfo().WorldTransform; }
FRotator FParticleSystemObjectCascade::GetComponentRotation() const { return {}; }
const FTransform& FParticleSystemObjectCascade::GetComponentToWorld() const { return TransformObject->GetInfo().WorldTransform; }
const FBoxSphereBounds& FParticleSystemObjectCascade::GetBounds() const { return PrimitiveSceneDesc.Bounds; }
TWeakObjectPtr<UWorld> FParticleSystemObjectCascade::GetWeakWorld() const { return {}; }
bool FParticleSystemObjectCascade::HasWorld() const { return true; }
bool FParticleSystemObjectCascade::HasWorldSettings() const { return {}; }
bool FParticleSystemObjectCascade::IsGameWorld() const { return true; }
float FParticleSystemObjectCascade::GetWorldTimeSeconds() const { return {}; }
float FParticleSystemObjectCascade::GetWorldEffectiveTimeDilation() const { return {}; }
FIntVector FParticleSystemObjectCascade::GetWorldOriginLocation() const { return {}; }
FSceneInterface* FParticleSystemObjectCascade::GetScene() const { return &PrimitiveSceneData.SceneProxy->GetScene(); }
bool FParticleSystemObjectCascade::GetFloatParameter(const FName InName, float& OutFloat) { return {}; }
const FVector3f& FParticleSystemObjectCascade::GetLWCTile() const { return LWCTile; }
FString FParticleSystemObjectCascade::GetName() const { return {}; }
FString FParticleSystemObjectCascade::GetFullName() const { return {}; }
FString FParticleSystemObjectCascade::GetPathName() const { return {}; }
bool FParticleSystemObjectCascade::IsActive() const { return {}; }
bool FParticleSystemObjectCascade::IsValidLowLevel() const { return {}; }
TArrayView<const FParticleSysParam> FParticleSystemObjectCascade::GetAsyncInstanceParameters() { return {}; }
int32 FParticleSystemObjectCascade::GetCurrentDetailMode() const { return {}; }
int32 FParticleSystemObjectCascade::GetCurrentLODIndex() const { return {}; }
const FVector& FParticleSystemObjectCascade::GetPartSysVelocity() const { return FVector::ZeroVector; }
const FVector& FParticleSystemObjectCascade::GetOldPosition() const { return FVector::ZeroVector; }
FFXSystem* FParticleSystemObjectCascade::GetFXSystem() const { return {}; }
const UParticleSystem* FParticleSystemObjectCascade::GetTemplate() const { return Template; }
TArrayView<const FParticleSysParam> FParticleSystemObjectCascade::GetInstanceParameters() const { return {}; }
TArrayView<FParticleEmitterInstance*> FParticleSystemObjectCascade::GetEmitterInstances() const { return {}; }
TArrayView<TObjectPtr<UMaterialInterface>> FParticleSystemObjectCascade::GetEmitterMaterials() const { return {}; }
FPrimitiveSceneProxy* FParticleSystemObjectCascade::GetSceneProxy() const { return PrimitiveSceneData.SceneProxy; }
bool FParticleSystemObjectCascade::GetIsWarmingUp() const { return {}; }
bool FParticleSystemObjectCascade::GetJustRegistered() const { return bJustRegistered; }
float FParticleSystemObjectCascade::GetWarmupTime() const { return {}; }
float FParticleSystemObjectCascade::GetEmitterDelay() const { return {}; }
FRandomStream& FParticleSystemObjectCascade::GetRandomStream() { return RandomStream; }
void FParticleSystemObjectCascade::SetComponentToWorld(const FTransform& NewComponentToWorld) { }
void FParticleSystemObjectCascade::DeactivateNextTick() { }
UParticleSystemComponent* FParticleSystemObjectCascade::AsComponent() const { return {}; }
void FParticleSystemObjectCascade::ReportEventSpawn(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData) { }
void FParticleSystemObjectCascade::ReportEventDeath(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData, const float InParticleTime) { }
void FParticleSystemObjectCascade::ReportEventCollision(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InDirection, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData, const float InParticleTime, const FVector InNormal, const float InTime, const int32 InItem, const FName InBoneName, UPhysicalMaterial* PhysMat) { }
void FParticleSystemObjectCascade::ReportEventBurst(const FName InEventName, const float InEmitterTime, const int32 ParticleCount, const FVector InLocation, const TArray<class UParticleModuleEventSendToGame*>& InEventData) { }
TArrayView<FParticleEventSpawnData> FParticleSystemObjectCascade::GetSpawnEvents() const { return {}; }
TArrayView<FParticleEventDeathData> FParticleSystemObjectCascade::GetDeathEvents() const { return {}; }
TArrayView<FParticleEventCollideData> FParticleSystemObjectCascade::GetCollisionEvents() const { return {}; }
TArrayView<FParticleEventBurstData> FParticleSystemObjectCascade::GetBurstEvents() const { return {}; }
TArrayView<FParticleEventKismetData> FParticleSystemObjectCascade::GetKismetEvents() const { return {}; }

////////////////////////////////////////////////////////////////////////////////////////////////////

FParticleSystemStateStreamCascade::FParticleSystemStateStreamCascade()
{
}

void FParticleSystemStateStreamCascade::SetTransformObject(FParticleSystemObjectCascade& Object, const FParticleSystemDynamicState& Ds)
{
	if (Object.TransformObject)
	{
		Object.TransformObject->RemoveListener(&Object);
		Object.TransformObject = nullptr;
	}

	const FTransformHandle& TransformHandle = Ds.GetTransform();
	if (TransformHandle.IsValid())
	{
		FTransformObject* TransformObject = static_cast<FTransformObject*>(TransformHandle.Render_GetUserData());
		check(TransformObject);
		TransformObject->AddListener(&Object);
		Object.TransformObject = TransformObject;
	}
}

void FParticleSystemStateStreamCascade::Render_OnCreate(const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds, FParticleSystemObjectCascade*& UserData, bool IsDestroyedInSameFrame)
{
	check(!UserData);

	FParticleSystemObjectCascade& Object = *new FParticleSystemObjectCascade();
	Object.AddRef();

	SetTransformObject(Object, Ds);

	FTransformObject::Info Info = Object.TransformObject->GetInfo();
	const FTransform& Transform = Info.WorldTransform;

	//FBoxSphereBounds LocalBounds(ForceInitToZero);
	FBoxSphereBounds LocalBounds(FVector(-1000.0, -1000.0, -1000.0), FVector(1000.0, 1000.0, 1000.0), 1000.0);
	//if (Mesh)
	//{
	//	LocalBounds = Mesh->GetBounds();
	//}	
	FSceneInterface& Scene = *LaneUserData;

	Object.PrimitiveSceneDesc.RenderMatrix = Transform.ToMatrixWithScale();
	Object.PrimitiveSceneDesc.AttachmentRootPosition = Transform.GetLocation();
	Object.PrimitiveSceneDesc.PrimitiveSceneData = &Object.PrimitiveSceneData;
	Object.PrimitiveSceneDesc.LocalBounds = LocalBounds;
	Object.PrimitiveSceneDesc.Bounds = LocalBounds.TransformBy(Transform);

	UParticleSystem* Asset = static_cast<UParticleSystem*>(Ds.GetSystemAsset());
	FParticleSystemSceneProxyDesc Desc;
	Desc.SystemAsset = Asset;
	//Desc.OverrideMaterials = const_cast<TArray<TObjectPtr<UMaterialInterface>>&>(Ds.GetOverrideMaterials());
	Desc.CustomPrimitiveData = &Object.CustomPrimitiveData;
	Desc.Scene = &Scene;
	Desc.FeatureLevel = Scene.GetFeatureLevel();
	Desc.bIsVisible = Info.bVisible;
	//Desc.MaterialRelevance.Raw = Ss.GetMaterialRelevance();

	//Desc.DynamicData = CreateDynamicData(Scene->GetFeatureLevel());
	//Desc.bOnlyOwnerSee = Ds.GetOnlyOwnerSee();
	//Desc.bOwnerNoSee = Ds.GetOwnerNoSee();
	//Desc.ActorOwners = Ds.GetOwners();


	#if WITH_EDITOR
	//Desc.TextureStreamingTransformScale = Transform.GetMaximumAxisScale();
	#endif

	Object.Template = Asset;
	Object.InitializeSystem(Desc, Ss, Ds);

	Object.PrimitiveSceneData.SceneProxy = new FParticleSystemSceneProxy(Desc);
	Scene.AddPrimitive(&Object.PrimitiveSceneDesc);

	UserData = &Object;
	Objects.Add(&Object);
}

void FParticleSystemStateStreamCascade::Render_OnUpdate(const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds, FParticleSystemObjectCascade*& UserData)
{
	if (!UserData)
	{
		return;
	}

	FParticleSystemObjectCascade& Object = *UserData;

	if (Ds.TransformModified())
	{
		SetTransformObject(Object, Ds);
	}
}

void FParticleSystemStateStreamCascade::Render_OnDestroy(const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds, FParticleSystemObjectCascade*& UserData)
{
	if (UserData)
	{
		Objects.Remove(UserData);
		UserData->Release();
		UserData = nullptr;
	}
}

void FParticleSystemStateStreamCascade::Render_PostUpdate()
{
	TStateStream<FParticleSystemStateStreamSettingsCascade>::Render_PostUpdate();

	for (FParticleSystemObjectCascade* Object : Objects)
	{
		Object->Update();
	}
}

STATESTREAM_CREATOR_INSTANCE_WITH_FUNC(FParticleSystemStateStreamCascade, [](const FStateStreamRegisterContext& Context, FParticleSystemStateStreamCascade& Impl)
	{
		Context.RegisterDependency(ParticleSystemStateStreamCascadeId, TransformStateStreamId);
		static_cast<FParticleSystemStateStreamImpl*>(Context.Manager.Render_GetStream(ParticleSystemStateStreamId))->CascadeBackend = &Impl;
	})

////////////////////////////////////////////////////////////////////////////////////////////////////

FParticleSystemHandle FParticleSystemStateStreamImpl::Game_CreateInstance(const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds, uint32 LaneId)
{
	if (Cast<UParticleSystem>(Ds.GetSystemAsset()))
		return CascadeBackend->Game_CreateInstance(Ss, Ds);
	check(OtherBackend);
	if (OtherBackend)
		return OtherBackend->Game_CreateInstance(Ss, Ds);
	return {};
}

void FParticleSystemStateStreamImpl::SetOtherBackend(IParticleSystemStateStream* Other)
{
	OtherBackend = Other;
}

STATESTREAM_CREATOR_INSTANCE(FParticleSystemStateStreamImpl)

////////////////////////////////////////////////////////////////////////////////////////////////////
