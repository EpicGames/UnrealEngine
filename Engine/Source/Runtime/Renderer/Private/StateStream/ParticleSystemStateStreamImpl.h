// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ParticleEmitterInstanceOwner.h"
#include "PrimitiveSceneDesc.h"
#include "SceneTypes.h"
#include "StateStream/ParticleSystemStateStream.h"
#include "TransformStateStreamImpl.h"

#define UE_API RENDERER_API

class FParticleDynamicData;
class FSceneInterface;
struct FParticleSystemSceneProxyDesc;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FParticleSystemObjectCascade final : public TRefCountingMixin<FParticleSystemObjectCascade>, public FTransformObjectListener, public IParticleEmitterInstanceOwner
{
public:

private:
	virtual ~FParticleSystemObjectCascade();
	virtual void OnTransformObjectDirty() override final;

	TRefCountPtr<FTransformObject> TransformObject;

	FCustomPrimitiveData CustomPrimitiveData;
	FPrimitiveSceneInfoData	PrimitiveSceneData;
	FPrimitiveSceneDesc PrimitiveSceneDesc;

	friend class FParticleSystemStateStreamCascade;
	friend class TRefCountingMixin<FParticleSystemObjectCascade>;

	void InitializeSystem(FParticleSystemSceneProxyDesc& OutDesc, const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds);
	void Update();
	FParticleDynamicData* CreateDynamicData(ERHIFeatureLevel::Type InFeatureLevel);

	virtual const FTransform& GetAsyncComponentToWorld() const override;

	virtual UObject* GetDistributionData() const override;
	virtual const FTransform& GetComponentTransform() const override;
	virtual FRotator GetComponentRotation() const override;
	virtual const FTransform& GetComponentToWorld() const override;
	virtual const FBoxSphereBounds& GetBounds() const override;
	virtual TWeakObjectPtr<UWorld> GetWeakWorld() const override;
	virtual bool HasWorld() const override;
	virtual bool HasWorldSettings() const override;
	virtual bool IsGameWorld() const override;
	virtual float GetWorldTimeSeconds() const override;
	virtual float GetWorldEffectiveTimeDilation() const override;
	virtual FIntVector GetWorldOriginLocation() const override;
	virtual FSceneInterface* GetScene() const override;
	virtual bool GetFloatParameter(const FName InName, float& OutFloat) override;
	virtual const FVector3f& GetLWCTile() const override;
	virtual FString GetName() const override;
	virtual FString GetFullName() const override;
	virtual FString GetPathName() const override;
	virtual bool IsActive() const override;
	virtual bool IsValidLowLevel() const override;
	virtual TArrayView<const FParticleSysParam> GetAsyncInstanceParameters() override;
	virtual int32 GetCurrentDetailMode() const override;
	virtual int32 GetCurrentLODIndex() const override;
	virtual const FVector& GetPartSysVelocity() const override;
	virtual const FVector& GetOldPosition() const override;
	virtual FFXSystem* GetFXSystem() const override;
	virtual const UParticleSystem* GetTemplate() const override;
	virtual TArrayView<const FParticleSysParam> GetInstanceParameters() const override;
	virtual TArrayView<FParticleEmitterInstance*> GetEmitterInstances() const override;
	virtual TArrayView<TObjectPtr<UMaterialInterface>> GetEmitterMaterials() const override;
	virtual FPrimitiveSceneProxy* GetSceneProxy() const override;
	virtual bool GetIsWarmingUp() const override;
	virtual bool GetJustRegistered() const override;
	virtual float GetWarmupTime() const override;
	virtual float GetEmitterDelay() const override;
	virtual FRandomStream& GetRandomStream() override;

	virtual void SetComponentToWorld(const FTransform& NewComponentToWorld) override;
	virtual void DeactivateNextTick() override;

	virtual UParticleSystemComponent* AsComponent() const override;

	virtual void ReportEventSpawn(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData) override;
	virtual void ReportEventDeath(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData, const float InParticleTime) override;
	virtual void ReportEventCollision(const FName InEventName, const float InEmitterTime, const FVector InLocation, const FVector InDirection, const FVector InVelocity, const TArray<class UParticleModuleEventSendToGame*>& InEventData, const float InParticleTime, const FVector InNormal, const float InTime, const int32 InItem, const FName InBoneName, UPhysicalMaterial* PhysMat) override;
	virtual void ReportEventBurst(const FName InEventName, const float InEmitterTime, const int32 ParticleCount, const FVector InLocation, const TArray<class UParticleModuleEventSendToGame*>& InEventData) override;

	virtual TArrayView<FParticleEventSpawnData> GetSpawnEvents() const override;
	virtual TArrayView<FParticleEventDeathData> GetDeathEvents() const override;
	virtual TArrayView<FParticleEventCollideData> GetCollisionEvents() const override;
	virtual TArrayView<FParticleEventBurstData> GetBurstEvents() const override;
	virtual TArrayView<FParticleEventKismetData> GetKismetEvents() const override;

	const UParticleSystem* Template;
	TArray<FParticleEmitterInstance*> EmitterInstances;
	FRandomStream RandomStream;
	FVector3f LWCTile = FVector3f::ZeroVector;
	bool bJustRegistered = true;
	int32 LODLevel = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

inline constexpr uint32 ParticleSystemStateStreamCascadeId = 5;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FParticleSystemStateStreamSettingsCascade : TStateStreamSettings<IParticleSystemStateStream, FParticleSystemObjectCascade, FSceneInterface>
{
	enum { Id = ParticleSystemStateStreamCascadeId };
	static inline constexpr const TCHAR* DebugName = TEXT("ParticleSystem(Cascade)");
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FParticleSystemStateStreamCascade final : public TStateStream<FParticleSystemStateStreamSettingsCascade>
{
public:
	FParticleSystemStateStreamCascade();
private:
	void SetTransformObject(FParticleSystemObjectCascade& Object, const FParticleSystemDynamicState& Ds);
	UE_API virtual void Render_OnCreate(const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds, FParticleSystemObjectCascade*& UserData, bool IsDestroyedInSameFrame) override;
	UE_API virtual void Render_OnUpdate(const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds, FParticleSystemObjectCascade*& UserData) override;
	UE_API virtual void Render_OnDestroy(const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds, FParticleSystemObjectCascade*& UserData) override;

	virtual void Render_PostUpdate() override;

	TArray<FParticleSystemObjectCascade*> Objects;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FParticleSystemStateStreamImpl final : public IParticleSystemStateStream, public IStateStream
{
public:
	FParticleSystemStateStreamImpl() {}

	virtual FParticleSystemHandle Game_CreateInstance(const FParticleSystemStaticState& Ss, const FParticleSystemDynamicState& Ds, uint32 LaneId = DefaultLaneId) override;
	virtual void SetOtherBackend(IParticleSystemStateStream* Other) override;

	virtual void Game_BeginTick(uint32 LaneId) override {}
	virtual void Game_EndTick(StateStreamTime AbsoluteTime, uint32 LaneId) override {}
	virtual void Game_SetDefaultLane(uint32 LaneId) override {}
	virtual void Game_Exit() override {}
	virtual void* Game_GetVoidPointer() override { return this; }
	virtual void Render_Update(StateStreamTime AbsoluteTime) override {}
	virtual void Render_PostUpdate() override {}
	virtual void Render_Exit() override {}
	virtual void Render_GarbageCollect() override {}
	virtual void Render_Enable(bool Enable) override {};
	virtual uint32 GetId() override { return ParticleSystemStateStreamId; }

	FParticleSystemStateStreamCascade* CascadeBackend = nullptr;
	IParticleSystemStateStream* OtherBackend = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
