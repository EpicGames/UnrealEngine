// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinnedMeshStateStreamImpl.h"
#include "HAL/PlatformMisc.h"
#include "Rendering/RenderCommandPipes.h"
#include "ScenePrivate.h"
#include "SkeletalMeshSceneProxy.h"
#include "SkeletalMeshUpdater.h"
#include "SkeletalRenderPublic.h"
#include "SkinnedMeshSceneProxyDesc.h"
#include "StateStreamCreator.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

FSkinnedMeshObject::~FSkinnedMeshObject()
{
	FSceneInterface& Scene = PrimitiveSceneData.SceneProxy->GetScene();
	Scene.RemovePrimitive(&PrimitiveSceneDesc);

	if (TransformObject)
	{
		TransformObject->RemoveListener(this);
		TransformObject = nullptr;
	}
}

void FSkinnedMeshObject::OnTransformObjectDirty()
{
	FPrimitiveSceneProxy& Proxy = *PrimitiveSceneData.SceneProxy;
	FScene& Scene = static_cast<FScene&>(Proxy.GetScene());

	FTransformObject::Info Info = TransformObject->GetInfo();

	#if UE_DEBUG_OFFSET_SKINNED_MESH
	FTransform Transform = Info.WorldTransform;
	Transform.AddToTranslation(FVector(10.0f*DebugIndex, 10.0f*DebugIndex, 0));
	#else
	const FTransform& Transform = Info.WorldTransform;
	#endif

	FMatrix Mat = Transform.ToMatrixWithScale();
	MeshObject->SetTransform(Mat, Scene.GetFrameNumber());
	MeshObject->RefreshClothingTransforms(Mat, Scene.GetFrameNumber());

	const FBoxSphereBounds& LocalBounds = Proxy.GetLocalBounds();
	FBoxSphereBounds Bounds = LocalBounds.TransformBy(Transform);
	FVector ActorPosForRendering(ForceInitToZero);// Proxy.GetActorPositionForRenderer(); TODO This need to be revisited
	Scene.UpdatePrimitiveTransform_RenderThread(&Proxy, Bounds, LocalBounds, Mat, ActorPosForRendering, {});

	int32 LODLevel = 0;

	TArray<FExternalMorphSets> ExternalMorphSets;
	ExternalMorphSets.Add({});
	FSkinnedMeshSceneProxyDynamicData ProxyData;
	ProxyData.ExternalMorphSets = ExternalMorphSets;
	ProxyData.ComponentSpaceTransforms = Info.BoneTransforms;
	ProxyData.PreviousComponentSpaceTransforms = PrevTransforms;
	ProxyData.ComponentWorldTransform = Transform;
	ProxyData.PreviousBoneTransformRevisionNumber = BoneTransformRevisionNumber;
	ProxyData.CurrentBoneTransformFrame = BoneTransformRevisionNumber;
	ProxyData.CurrentBoneTransformRevisionNumber = ++BoneTransformRevisionNumber;
	ProxyData.NumLODs = 1;

	//TArray<uint8> VisibilityStates;
	//VisibilityStates.SetNumUninitialized(Info.BoneTransforms.Num());
	//memset(VisibilityStates.GetData(), 1, Info.BoneTransforms.Num());

	//ProxyData.BoneVisibilityStates = VisibilityStates;
	//ProxyData.PreviousBoneVisibilityStates = VisibilityStates;

	MeshObject->Update(LODLevel, ProxyData, &Proxy, SkinnedAsset, FMorphTargetWeightMap(), TArray<float>(), EPreviousBoneTransformUpdateMode::None, FExternalMorphWeightData());
	MeshObject->bHasBeenUpdatedAtLeastOnce = true;

	PrevTransforms = Info.BoneTransforms;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSkinnedMeshStateStreamImpl::FSkinnedMeshStateStreamImpl()
{
}

void FSkinnedMeshStateStreamImpl::SetTransformObject(FSkinnedMeshObject& Object, const FSkinnedMeshDynamicState& Ds)
{
	if (Object.TransformObject)
	{
		Object.TransformObject->RemoveListener(&Object);
		Object.TransformObject = nullptr;
	}

	const FTransformHandle& TransformHandle = Ds.GetTransform();
	if (TransformHandle.IsValid())
	{
		FTransformObject* TransformObject = static_cast<FTransformObject*>(TransformHandle.Render_GetUserData());
		check(TransformObject);
		TransformObject->AddListener(&Object);
		Object.TransformObject = TransformObject;
	}
}

void FSkinnedMeshStateStreamImpl::Render_OnCreate(const FSkinnedMeshStaticState& Ss, const FSkinnedMeshDynamicState& Ds, FSkinnedMeshObject*& UserData, bool IsDestroyedInSameFrame)
{
	check(!IsDestroyedInSameFrame);
	check(!UserData);

	USkinnedAsset* SkinnedAsset = Ds.GetSkinnedAsset();
	if (!SkinnedAsset)
	{
		return;
	}

	FSceneInterface& Scene = *LaneUserData;

	FSkinnedMeshObject& Object = *new FSkinnedMeshObject();
	Object.AddRef();
	Object.SkinnedAsset = SkinnedAsset;

	SetTransformObject(Object, Ds);

	check(Object.TransformObject);
	FTransformObject::Info Info = Object.TransformObject->GetInfo();

	#if UE_DEBUG_OFFSET_SKINNED_MESH
	FTransform Transform = Info.WorldTransform;
	static uint32 DebugCounter = 0;
	Object.DebugIndex = 5;
	Transform.AddToTranslation(FVector(10.0f*Object.DebugIndex, 10.0f*Object.DebugIndex, 0));
	#else
	const FTransform& Transform = Info.WorldTransform;
	#endif

	FBoxSphereBounds LocalBounds(ForceInitToZero);
	if (SkinnedAsset)
	{
		LocalBounds = SkinnedAsset->GetBounds();
	}	

	Object.PrimitiveSceneDesc.RenderMatrix = Transform.ToMatrixWithScale();
	Object.PrimitiveSceneDesc.PrimitiveSceneData = &Object.PrimitiveSceneData;
	Object.PrimitiveSceneDesc.LocalBounds = LocalBounds;
	Object.PrimitiveSceneDesc.Bounds = LocalBounds.TransformBy(Transform);

	FSkinnedMeshSceneProxyDesc Desc;
	Desc.SkinnedAsset = SkinnedAsset;
	Desc.OverrideMaterials = const_cast<TArray<TObjectPtr<UMaterialInterface>>&>(Ds.GetOverrideMaterials());
	Desc.CustomPrimitiveData = &Object.CustomPrimitiveData;
	Desc.Scene = &Scene;
	Desc.FeatureLevel = Scene.GetFeatureLevel();
	Desc.bIsVisible = Info.bVisible;
	Desc.bPerBoneMotionBlur = true;
	Desc.VisibilityId = -1;
	Desc.bReceivesDecals = false;
	Desc.bCollisionEnabled = true;
	Desc.MaterialRelevance.Raw = Ss.GetMaterialRelevance();
	Desc.CastShadow = true;
	Desc.bCastDynamicShadow = true;
	Desc.bCastStaticShadow = true;
	Desc.bCastContactShadow = true;
	Desc.bUseAsOccluder = true;
	
	FSkeletalMeshObject* MeshObject = FSkinnedMeshSceneProxyDesc::CreateMeshObject(Desc);
	Desc.MeshObject = MeshObject;
	Object.MeshObject = MeshObject;

	#if WITH_EDITOR
	//Desc.TextureStreamingTransformScale = Transform.GetMaximumAxisScale();
	#endif

	FPrimitiveSceneProxy* SceneProxy = FSkinnedMeshSceneProxyDesc::CreateSceneProxy(Desc, false);
	Object.PrimitiveSceneData.SceneProxy = SceneProxy;

	SceneProxy->SetPrimitiveColor(FLinearColor::White);

	int32 LODLevel = Desc.GetPredictedLODLevel();

	TArray<FExternalMorphSets> ExternalMorphSets;
	ExternalMorphSets.Add({});
	FSkinnedMeshSceneProxyDynamicData ProxyData;
	ProxyData.ExternalMorphSets = ExternalMorphSets;
	ProxyData.ComponentSpaceTransforms = Info.BoneTransforms;
	ProxyData.ComponentWorldTransform = Info.WorldTransform;

	MeshObject->Update(LODLevel, ProxyData, SceneProxy, SkinnedAsset, FMorphTargetWeightMap(), TArray<float>(), EPreviousBoneTransformUpdateMode::None, FExternalMorphWeightData());

	Scene.AddPrimitive(&Object.PrimitiveSceneDesc);

	UserData = &Object;
}

void FSkinnedMeshStateStreamImpl::Render_OnUpdate(const FSkinnedMeshStaticState& Ss, const FSkinnedMeshDynamicState& Ds, FSkinnedMeshObject*& Object)
{
	if (!Object)
	{
		return;
	}
	if (Ds.TransformModified())
	{
		SetTransformObject(*Object, Ds);
	}
}

void FSkinnedMeshStateStreamImpl::Render_OnDestroy(const FSkinnedMeshStaticState& Ss, const FSkinnedMeshDynamicState& Ds, FSkinnedMeshObject*& Object)
{
	if (Object)
	{
		Object->MeshObject->ReleaseResources();
		Object->Release();
		Object = nullptr;
	}
}

void FSkinnedMeshStateStreamImpl::Render_PostUpdate()
{
	TStateStream<FSkinnedMeshStateStreamSettings>::Render_PostUpdate();

	FSceneInterface& Scene = *LaneUserData;

	// TODO: Get rid of this entirely.. no need to create these queues of stuff when it can be done directly render side
	Scene.GetSkeletalMeshUpdater()->PushChangesToStream();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STATESTREAM_CREATOR_INSTANCE_WITH_DEPENDENCY(FSkinnedMeshStateStreamImpl, TransformStateStreamId)

////////////////////////////////////////////////////////////////////////////////////////////////////
