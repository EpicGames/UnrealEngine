// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PrimitiveSceneDesc.h"
#include "SceneTypes.h"
#include "StateStream/SkinnedMeshStateStream.h"
#include "TransformStateStreamImpl.h"

#define UE_API RENDERER_API

class FSceneInterface;
class FSkeletalMeshObject;
struct FPrimitiveSceneDesc;

#define UE_DEBUG_OFFSET_SKINNED_MESH 0 // For debugging purposes

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSkinnedMeshObject : public TRefCountingMixin<FSkinnedMeshObject>, public FTransformObjectListener
{
public:

private:
	virtual ~FSkinnedMeshObject();
	virtual void OnTransformObjectDirty() override final;

	TRefCountPtr<FTransformObject> TransformObject;
	USkinnedAsset* SkinnedAsset;
	FSkeletalMeshObject* MeshObject;

	FCustomPrimitiveData CustomPrimitiveData;
	FPrimitiveSceneInfoData	PrimitiveSceneData;
	FPrimitiveSceneDesc PrimitiveSceneDesc;
	TArray<FTransform> PrevTransforms;
	uint32 BoneTransformRevisionNumber = 0;

	#if UE_DEBUG_OFFSET_SKINNED_MESH
	uint32 DebugIndex = 0;
	#endif

	friend class FSkinnedMeshStateStreamImpl;
	friend class TRefCountingMixin<FSkinnedMeshObject>;
};


////////////////////////////////////////////////////////////////////////////////////////////////////

struct FSkinnedMeshStateStreamSettings : TStateStreamSettings<ISkinnedMeshStateStream, FSkinnedMeshObject, FSceneInterface>
{
	static inline constexpr bool SkipCreatingDeletes = true;
};


////////////////////////////////////////////////////////////////////////////////////////////////////

class FSkinnedMeshStateStreamImpl : public TStateStream<FSkinnedMeshStateStreamSettings>
{
public:
	FSkinnedMeshStateStreamImpl();
private:
	void SetTransformObject(FSkinnedMeshObject& Object, const FSkinnedMeshDynamicState& Ds);

	UE_API virtual void Render_PostUpdate() override;

	UE_API virtual void Render_OnCreate(const FSkinnedMeshStaticState& Ss, const FSkinnedMeshDynamicState& Ds, FSkinnedMeshObject*& UserData, bool IsDestroyedInSameFrame) override;
	UE_API virtual void Render_OnUpdate(const FSkinnedMeshStaticState& Ss, const FSkinnedMeshDynamicState& Ds, FSkinnedMeshObject*& UserData) override;
	UE_API virtual void Render_OnDestroy(const FSkinnedMeshStaticState& Ss, const FSkinnedMeshDynamicState& Ds, FSkinnedMeshObject*& UserData) override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
