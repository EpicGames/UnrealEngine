// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkyAtmosphereStateStreamImpl.h"
#include "SceneInterface.h"
#include "SceneProxies/SkyAtmosphereSceneProxy.h"
#include "StateStreamCreator.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

FSkyAtmosphereStateStreamImpl::FSkyAtmosphereStateStreamImpl()
{
}

void FSkyAtmosphereStateStreamImpl::Render_OnCreate(const FSkyAtmosphereStaticState& Ss, const FSkyAtmosphereDynamicState& Ds, FSkyAtmosphereSceneProxy*& UserData, bool IsDestroyedInSameFrame)
{
	FSceneInterface& Scene = *LaneUserData;

	UserData = new FSkyAtmosphereSceneProxy(Ds);
	Scene.AddSkyAtmosphere(UserData, Ds.GetBuilt());
}

void FSkyAtmosphereStateStreamImpl::Render_OnUpdate(const FSkyAtmosphereStaticState& Ss, const FSkyAtmosphereDynamicState& Ds, FSkyAtmosphereSceneProxy*& UserData)
{
	FSceneInterface& Scene = *LaneUserData;

	// For some reason we need to recreate the entire sceneproxy every time
#if 0
	UserData->AtmosphereSetup = FAtmosphereSetup(Ds);
	Scene.InvalidatePathTracedOutput();
#else
	Scene.RemoveSkyAtmosphere(UserData);
	delete UserData;
	UserData = new FSkyAtmosphereSceneProxy(Ds);
	Scene.AddSkyAtmosphere(UserData, Ds.GetBuilt());
#endif
}

void FSkyAtmosphereStateStreamImpl::Render_OnDestroy(const FSkyAtmosphereStaticState& Ss, const FSkyAtmosphereDynamicState& Ds, FSkyAtmosphereSceneProxy*& UserData)
{
	FSceneInterface& Scene = *LaneUserData;
	Scene.RemoveSkyAtmosphere(UserData);
	delete UserData;
	UserData = nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STATESTREAM_CREATOR_INSTANCE(FSkyAtmosphereStateStreamImpl)

////////////////////////////////////////////////////////////////////////////////////////////////////
