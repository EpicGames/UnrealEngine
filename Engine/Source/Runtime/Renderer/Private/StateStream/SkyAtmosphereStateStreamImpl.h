// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericStateStream.h"
#include "StateStream/SkyAtmosphereStateStream.h"

#define UE_API RENDERER_API

class FSceneInterface;
class FSkyAtmosphereSceneProxy;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FSkyAtmosphereStateStreamSettings : TStateStreamSettings<ISkyAtmosphereStateStream, FSkyAtmosphereSceneProxy, FSceneInterface>
{
	static inline constexpr bool SkipCreatingDeletes = true;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FSkyAtmosphereStateStreamImpl : public TStateStream<FSkyAtmosphereStateStreamSettings>
{
public:
	FSkyAtmosphereStateStreamImpl();
private:
	UE_API virtual void Render_OnCreate(const FSkyAtmosphereStaticState& Ss, const FSkyAtmosphereDynamicState& Ds, FSkyAtmosphereSceneProxy*& UserData, bool IsDestroyedInSameFrame) override;
	UE_API virtual void Render_OnUpdate(const FSkyAtmosphereStaticState& Ss, const FSkyAtmosphereDynamicState& Ds, FSkyAtmosphereSceneProxy*& UserData) override;
	UE_API virtual void Render_OnDestroy(const FSkyAtmosphereStaticState& Ss, const FSkyAtmosphereDynamicState& Ds, FSkyAtmosphereSceneProxy*& UserData) override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
