// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshStateStreamImpl.h"
#include "Engine/StaticMesh.h"
#include "ScenePrivate.h"
#include "StateStreamCreator.h"
#include "StaticMeshSceneProxy.h"
#include "StaticMeshSceneProxyDesc.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

FStaticMeshObject::~FStaticMeshObject()
{
	FSceneInterface& Scene = PrimitiveSceneData.SceneProxy->GetScene();
	Scene.RemovePrimitive(&PrimitiveSceneDesc);

	if (TransformObject)
	{
		TransformObject->RemoveListener(this);
		TransformObject = nullptr;
	}
}

void FStaticMeshObject::OnTransformObjectDirty()
{
	FTransformObject::Info Info = TransformObject->GetInfo();
	// TODO: Implement this to handle changing visibility and movement
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FStaticMeshStateStreamImpl::FStaticMeshStateStreamImpl()
{
}

void FStaticMeshStateStreamImpl::SetTransformObject(FStaticMeshObject& Object, const FStaticMeshDynamicState& Ds)
{
	if (Object.TransformObject)
	{
		Object.TransformObject->RemoveListener(&Object);
		Object.TransformObject = nullptr;
	}

	const FTransformHandle& TransformHandle = Ds.GetTransform();
	if (TransformHandle.IsValid())
	{
		FTransformObject* TransformObject = static_cast<FTransformObject*>(TransformHandle.Render_GetUserData());
		check(TransformObject);
		TransformObject->AddListener(&Object);
		Object.TransformObject = TransformObject;
	}
}

void FStaticMeshStateStreamImpl::Render_OnCreate(const FStaticMeshStaticState& Ss, const FStaticMeshDynamicState& Ds, FStaticMeshObject*& UserData, bool IsDestroyedInSameFrame)
{
	check(!UserData);

	UStaticMesh* Mesh = Ds.GetMesh();
	if (!Mesh)
	{
		return;
	}

	FSceneInterface& Scene = *LaneUserData;

	FStaticMeshObject& Object = *new FStaticMeshObject();
	Object.AddRef();

	SetTransformObject(Object, Ds);

	FTransformObject::Info Info = Object.TransformObject->GetInfo();
	const FTransform& Transform = Info.WorldTransform;

	FBoxSphereBounds LocalBounds(ForceInitToZero);
	if (Mesh)
	{
		LocalBounds = Mesh->GetBounds();
	}	

	Object.PrimitiveSceneDesc.RenderMatrix = Transform.ToMatrixWithScale();
	Object.PrimitiveSceneDesc.AttachmentRootPosition = Transform.GetLocation();
	Object.PrimitiveSceneDesc.PrimitiveSceneData = &Object.PrimitiveSceneData;
	Object.PrimitiveSceneDesc.LocalBounds = LocalBounds;
	Object.PrimitiveSceneDesc.Bounds = LocalBounds.TransformBy(Transform);

	FStaticMeshSceneProxyDesc Desc;
	Desc.StaticMesh = Mesh;
	Desc.OverrideMaterials = const_cast<TArray<TObjectPtr<UMaterialInterface>>&>(Ds.GetOverrideMaterials());
	Desc.CustomPrimitiveData = &Object.CustomPrimitiveData;
	Desc.Scene = &Scene;
	Desc.FeatureLevel = Scene.GetFeatureLevel();
	Desc.bIsVisible = Info.bVisible;
	Desc.bOnlyOwnerSee = Ds.GetOnlyOwnerSee();
	Desc.bOwnerNoSee = Ds.GetOwnerNoSee();
	Desc.ActorOwners = Ds.GetOwners();


	#if WITH_EDITOR
	Desc.TextureStreamingTransformScale = Transform.GetMaximumAxisScale();
	#endif

	Object.PrimitiveSceneData.SceneProxy = new FStaticMeshSceneProxy(Desc, false);

	Scene.AddPrimitive(&Object.PrimitiveSceneDesc);

	UserData = &Object;
}

void FStaticMeshStateStreamImpl::Render_OnUpdate(const FStaticMeshStaticState& Ss, const FStaticMeshDynamicState& Ds, FStaticMeshObject*& UserData)
{
	if (!UserData)
	{
		return;
	}

	FStaticMeshObject& Object = *UserData;

	if (Ds.TransformModified())
	{
		SetTransformObject(Object, Ds);
	}
}

void FStaticMeshStateStreamImpl::Render_OnDestroy(const FStaticMeshStaticState& Ss, const FStaticMeshDynamicState& Ds, FStaticMeshObject*& UserData)
{
	if (UserData)
	{
		UserData->Release();
		UserData = nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STATESTREAM_CREATOR_INSTANCE_WITH_DEPENDENCY(FStaticMeshStateStreamImpl, TransformStateStreamId)

////////////////////////////////////////////////////////////////////////////////////////////////////
