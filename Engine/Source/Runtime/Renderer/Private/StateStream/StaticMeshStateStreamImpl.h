// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PrimitiveSceneDesc.h"
#include "SceneTypes.h"
#include "StateStream/StaticMeshStateStream.h"
#include "TransformStateStreamImpl.h"

#define UE_API RENDERER_API

class FSceneInterface;
struct FPrimitiveSceneDesc;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStaticMeshObject : public TRefCountingMixin<FStaticMeshObject>, public FTransformObjectListener
{
public:

private:
	virtual ~FStaticMeshObject();
	virtual void OnTransformObjectDirty() override final;

	TRefCountPtr<FTransformObject> TransformObject;

	FCustomPrimitiveData CustomPrimitiveData;
	FPrimitiveSceneInfoData	PrimitiveSceneData;
	FPrimitiveSceneDesc PrimitiveSceneDesc;

	friend class FStaticMeshStateStreamImpl;
	friend class TRefCountingMixin<FStaticMeshObject>;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FStaticMeshStateStreamSettings : TStateStreamSettings<IStaticMeshStateStream, FStaticMeshObject, FSceneInterface>
{
	static inline constexpr bool SkipCreatingDeletes = true;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStaticMeshStateStreamImpl : public TStateStream<FStaticMeshStateStreamSettings>
{
public:
	FStaticMeshStateStreamImpl();
private:
	void SetTransformObject(FStaticMeshObject& Object, const FStaticMeshDynamicState& Ds);
	UE_API virtual void Render_OnCreate(const FStaticMeshStaticState& Ss, const FStaticMeshDynamicState& Ds, FStaticMeshObject*& UserData, bool IsDestroyedInSameFrame) override;
	UE_API virtual void Render_OnUpdate(const FStaticMeshStaticState& Ss, const FStaticMeshDynamicState& Ds, FStaticMeshObject*& UserData) override;
	UE_API virtual void Render_OnDestroy(const FStaticMeshStaticState& Ss, const FStaticMeshDynamicState& Ds, FStaticMeshObject*& UserData) override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef UE_API
