// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshBatch.h"

#include "PrimitiveSceneInfo.h"
#include "ScenePrivate.h"

FStaticMeshBatch::FStaticMeshBatch(FPrimitiveSceneInfo* InPrimitiveSceneInfo, const FMeshBatch& InMesh, FHitProxyId InHitProxyId)
	: FMeshBatch(InMesh)
	, PrimitiveSceneInfo(InPrimitiveSceneInfo)
	, Id(INDEX_NONE)
{
	BatchHitProxyId = InHitProxyId;
}

/** Private copy constructor. */
FStaticMeshBatch::FStaticMeshBatch(const FStaticMeshBatch& InStaticMesh)
	: FMeshBatch(InStaticMesh)
	, PrimitiveSceneInfo(InStaticMesh.PrimitiveSceneInfo)
	, Id(InStaticMesh.Id)
{}

FStaticMeshBatch::~FStaticMeshBatch()
{
	FScene* Scene = PrimitiveSceneInfo->Scene;
	// Remove this static mesh from the scene's list.
	Scene->StaticMeshes.RemoveAt(Id);
}


FStaticMeshBatchRelevance::FStaticMeshBatchRelevance(
	const FStaticMeshBatch& StaticMesh, 
	float InScreenSize, 
	bool InbSupportsCachingMeshDrawCommands, 
	bool InbUseSkyMaterial, 
	bool bInUseSingleLayerWaterMaterial, 
	bool bInUseAnisotropy, 
	bool bInSupportsNaniteRendering, 
	bool bInSupportsGPUScene, 
	bool bInUseForWaterInfoTextureDepth, 
	bool bInUseForLumenSceneCapture, 
	uint8 InDecalRenderTargetModeMask,
	ERHIFeatureLevel::Type FeatureLevel)
	: Id(StaticMesh.Id)
	, ScreenSize(InScreenSize)
	, NumElements(StaticMesh.Elements.Num())
	, CommandInfosBase(0)
	, UnsignedLODIndex(StaticMesh.LODIndex)
	, bInvalidLODIndex(StaticMesh.LODIndex < 0)
	, bDitheredLODTransition(StaticMesh.bDitheredLODTransition)
	, bSelectable(StaticMesh.bSelectable)
	, CastShadow(StaticMesh.CastShadow)
	, bUseForMaterial(StaticMesh.bUseForMaterial)
	, bUseForDepthPass(StaticMesh.bUseForDepthPass)
	, bUseAsOccluder(StaticMesh.bUseAsOccluder)
	, bUseSkyMaterial(InbUseSkyMaterial)
	, bUseSingleLayerWaterMaterial(bInUseSingleLayerWaterMaterial)
	, bUseHairStrands(StaticMesh.UseForHairStrands(FeatureLevel))
	, bUseAnisotropy(bInUseAnisotropy)
	, bOverlayMaterial(StaticMesh.bOverlayMaterial)
	, bRenderToVirtualTexture(StaticMesh.bRenderToVirtualTexture)
	, RuntimeVirtualTextureMaterialType(StaticMesh.RuntimeVirtualTextureMaterialType)
	, bSupportsCachingMeshDrawCommands(InbSupportsCachingMeshDrawCommands)
	, bSupportsNaniteRendering(bInSupportsNaniteRendering)
	, bSupportsGPUScene(bInSupportsGPUScene)
	, bUseForWaterInfoTextureDepth(bInUseForWaterInfoTextureDepth)
	, bUseForLumenSceneCapture(bInUseForLumenSceneCapture)
	, DecalRenderTargetModeMask(InDecalRenderTargetModeMask)
{
	check(GetLODIndex() == StaticMesh.LODIndex);
}

int32 FStaticMeshBatchRelevance::GetStaticMeshCommandInfoIndex(EMeshPass::Type MeshPass) const
{
	int32 CommandInfoIndex = CommandInfosBase;

	if (!CommandInfosMask.Get(MeshPass))
	{
		return -1;
	}

	for (int32 MeshPassIndex = 0; MeshPassIndex < MeshPass; ++MeshPassIndex)
	{
		if (CommandInfosMask.Get((EMeshPass::Type)MeshPassIndex))
		{
			++CommandInfoIndex;
		}
	}

	return CommandInfoIndex;
}
