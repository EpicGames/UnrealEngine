// Copyright Epic Games, Inc. All Rights Reserved.

#include "StochasticLighting.h"
#include "Lumen/LumenScreenProbeGather.h"
#include "Lumen/LumenShortRangeAO.h"
#include "Lumen/LumenReflections.h"
#include "Lumen/RayTracedTranslucency.h"
#include "MegaLights/MegaLights.h"
#include "MegaLights/MegaLightsViewState.h"
#include "BasePassRendering.h"
#include "SceneViewState.h"
#include "DeferredShadingRenderer.h"

static TAutoConsoleVariable<int32> CVarStochasticLightingFixedStateFrameIndex(
	TEXT("r.StochasticLighting.FixedStateFrameIndex"),
	-1,
	TEXT("Whether to override View.StateFrameIndex for debugging."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarStochasticLightingAsyncCompute(
	TEXT("r.StochasticLighting.AsyncCompute"),
	1,
	TEXT("Whether to run the tile classification mark pass on async compute:\n")
	TEXT("  0 - Disabled\n")
	TEXT("  1 - Enable only when downstream features (MegaLights, Lumen) are async compute\n")
	TEXT("  2 - Enabled always"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

namespace StochasticLighting
{
	constexpr int32 TileSize = 8;
	constexpr int32 DownsampleFactor = 2;

	int32 GetStateFrameIndex(const FSceneViewState* ViewState)
	{
		int32 StateFrameIndex = CVarStochasticLightingFixedStateFrameIndex.GetValueOnRenderThread();
		if (StateFrameIndex < 0)
		{
			StateFrameIndex = ViewState ? ViewState->GetFrameIndex() : 0;
		}
		return StateFrameIndex;
	}

	bool IsStateFrameIndexOverridden()
	{
		return CVarStochasticLightingFixedStateFrameIndex.GetValueOnRenderThread() >= 0;
	}

	void InitDefaultHistoryScreenParameters(FHistoryScreenParameters& OutHistoryScreenParameters)
	{
		OutHistoryScreenParameters.EncodedHistoryScreenCoordTexture = nullptr;
		OutHistoryScreenParameters.PackedPixelDataTexture = nullptr;
		OutHistoryScreenParameters.HistoryScreenPositionScaleBias = FVector4f(1.0f, 1.0f, 0.0f, 0.0f);
		OutHistoryScreenParameters.HistoryUVMinMax = FVector4f::Zero();
		OutHistoryScreenParameters.HistoryGatherUVMinMax = FVector4f::Zero();
		OutHistoryScreenParameters.HistoryBufferSizeAndInvSize = FVector4f::Zero();
		OutHistoryScreenParameters.HistorySubPixelGridSizeAndInvSize = FVector4f::Zero();
		OutHistoryScreenParameters.HistoryScreenCoordDecodeShift = FIntPoint::ZeroValue;
	}

	void GetHistoryScreenCoordCodec(FIntPoint HistoryBufferSize, FVector4f& OutSubPixelGridSizeAndInvSize, FIntPoint& OutDecodeShift)
	{
		if (HistoryBufferSize.X <= 0 || HistoryBufferSize.Y <= 0)
		{
			OutSubPixelGridSizeAndInvSize = FVector4f::Zero();
			OutDecodeShift = FIntPoint::ZeroValue;
		}
		else
		{
			FIntPoint SubPixelGridSize = FIntPoint(MAX_uint16) / HistoryBufferSize;
			OutDecodeShift = FIntPoint(FMath::FloorLog2(SubPixelGridSize.X), FMath::FloorLog2(SubPixelGridSize.Y));
			SubPixelGridSize = FIntPoint(1u << OutDecodeShift.X, 1u << OutDecodeShift.Y);
			OutSubPixelGridSizeAndInvSize = FVector4f(float(SubPixelGridSize.X), float(SubPixelGridSize.Y), 1.0f / SubPixelGridSize.X, 1.0f / SubPixelGridSize.Y);
		}
	}

	FHistoryScreenParameters GetHistoryScreenParameters(const FViewInfo& View)
	{
		FHistoryScreenParameters Parameters;
		InitDefaultHistoryScreenParameters(Parameters);

		if (View.ViewState && !View.bCameraCut && !View.bPrevTransformsReset)
		{
			const FStochasticLightingViewState& StochasticLightingViewState = View.ViewState->StochasticLighting;

			Parameters.HistoryScreenPositionScaleBias = StochasticLightingViewState.HistoryScreenPositionScaleBias;
			Parameters.HistoryUVMinMax = StochasticLightingViewState.HistoryUVMinMax;
			Parameters.HistoryGatherUVMinMax = StochasticLightingViewState.HistoryGatherUVMinMax;
			Parameters.HistoryBufferSizeAndInvSize = StochasticLightingViewState.HistoryBufferSizeAndInvSize;
			GetHistoryScreenCoordCodec(
				FIntPoint(Parameters.HistoryBufferSizeAndInvSize.X, Parameters.HistoryBufferSizeAndInvSize.Y),
				Parameters.HistorySubPixelGridSizeAndInvSize,
				Parameters.HistoryScreenCoordDecodeShift);
		}

		return Parameters;
	}
}

class FStochasticLightingTileClassificationMarkCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FStochasticLightingTileClassificationMarkCS)
	SHADER_USE_PARAMETER_STRUCT(FStochasticLightingTileClassificationMarkCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FFrontLayerTranslucencyGBufferParameters, FrontLayerTranslucencyGBufferParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthHistoryTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, NormalAndShadingInfoHistory)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, ClosureHistory)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, MegaLightsNumFramesAccumulatedHistory)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, RWLumenTileBitmask)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, RWMegaLightsTileBitmask)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, RWNormalTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWClosureTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWDownsampledSceneDepth2x1)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWDownsampledSceneDepth2x2)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<UNORM float3>, RWDownsampledWorldNormal2x1)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<UNORM float3>, RWDownsampledWorldNormal2x2)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint2>, RWEncodedHistoryScreenCoord)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, RWLumenPackedPixelData)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWMegaLightsPackedPixelData)
		SHADER_PARAMETER_STRUCT_INCLUDE(LumenScreenProbeGather::FTileClassifyParameters, ScreenProbeGatherTileClassifyParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(LumenReflections::FCompositeParameters, ReflectionsCompositeParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(MegaLights::FTileClassifyParameters, MegaLightsTileClassifyParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(StochasticLighting::FHistoryScreenParameters, HistoryScreenParameters)
		SHADER_PARAMETER(uint32, ReflectionPass)
		SHADER_PARAMETER(FIntPoint, DownsampledViewMin2x1)
		SHADER_PARAMETER(FIntPoint, DownsampledViewSize2x1)
		SHADER_PARAMETER(FIntPoint, DownsampledViewMin2x2)
		SHADER_PARAMETER(FIntPoint, DownsampledViewSize2x2)
		SHADER_PARAMETER(uint32, LumenStochasticSampleMode)
		SHADER_PARAMETER(uint32, MegaLightsStochasticSampleMode)
		SHADER_PARAMETER(uint32, StochasticLightingStateFrameIndex)
		SHADER_PARAMETER(uint32, OutputDownsampledDepthAndNormal)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)
		SHADER_PARAMETER(uint32, TileEncoding)
		SHADER_PARAMETER(uint32, bRectPrimitive)
		SHADER_PARAMETER_ARRAY(FUintVector4, TileListBufferOffsets, [SUBSTRATE_TILE_TYPE_COUNT])
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileDrawIndirectDataBufferUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileListBufferUAV)
		RDG_BUFFER_ACCESS(TileIndirectBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileIndirectDispatchData)
	END_SHADER_PARAMETER_STRUCT()

	class FCopyDepthAndNormal : SHADER_PERMUTATION_BOOL("COPY_DEPTH_AND_NORMAL");
	class FStochasticSampleOffset : SHADER_PERMUTATION_ENUM_CLASS("STOCHASTIC_SAMPLE_OFFSET", StochasticLighting::EStochasticSampleOffset);
	class FTileClassifyLumen : SHADER_PERMUTATION_BOOL("TILE_CLASSIFY_LUMEN");
	class FTileClassifyMegaLights : SHADER_PERMUTATION_BOOL("TILE_CLASSIFY_MEGALIGHTS");
	class FTileClassifySubstrate : SHADER_PERMUTATION_BOOL("TILE_CLASSIFY_SUBSTRATE");
	class FReprojectLumen : SHADER_PERMUTATION_BOOL("REPROJECT_LUMEN");
	class FReprojectMegaLights : SHADER_PERMUTATION_BOOL("REPROJECT_MEGALIGHTS");
	class FHistoryRejectBasedOnNormal : SHADER_PERMUTATION_BOOL("HISTORY_REJECT_BASED_ON_NORMAL");
	class FMaterialSource : SHADER_PERMUTATION_ENUM_CLASS("MATERIAL_SOURCE", StochasticLighting::EMaterialSource);
	class FOverflowTile : SHADER_PERMUTATION_BOOL("PERMUTATION_OVERFLOW_TILE");
	class FTileIndirectDispatch : SHADER_PERMUTATION_BOOL("TILE_INDIRECT_DISPATCH");
	using FPermutationDomain = TShaderPermutationDomain<
		FCopyDepthAndNormal,
		FStochasticSampleOffset,
		FTileClassifyLumen,
		FTileClassifyMegaLights,
		FTileClassifySubstrate,
		FReprojectLumen,
		FReprojectMegaLights,
		FHistoryRejectBasedOnNormal,
		FMaterialSource,
		FOverflowTile,
		FTileIndirectDispatch>;

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector, EShaderPlatform InPlatform)
	{
		if (!Substrate::IsSubstrateEnabled())
		{
			PermutationVector.Set<FOverflowTile>(false);
			PermutationVector.Set<FTileClassifySubstrate>(false);
		}
		else if (!Substrate::IsSubstrateBlendableGBufferEnabled(InPlatform))
		{
			// Only available with Format=0 (blendable GBuffer)
			PermutationVector.Set<FTileClassifySubstrate>(false);
		}

		if (PermutationVector.Get<FStochasticSampleOffset>() == StochasticLighting::EStochasticSampleOffset::Both)
		{
			PermutationVector.Set<FMaterialSource>(StochasticLighting::EMaterialSource::GBuffer);
		}

		if (PermutationVector.Get<FMaterialSource>() != StochasticLighting::EMaterialSource::GBuffer)
		{
			PermutationVector.Set<FOverflowTile>(false);

			if (PermutationVector.Get<FMaterialSource>() == StochasticLighting::EMaterialSource::HairStrands)
			{
				PermutationVector.Set<FTileClassifyLumen>(false);
				PermutationVector.Set<FTileClassifyMegaLights>(true);
				PermutationVector.Set<FReprojectMegaLights>(true);
			}
			else if (PermutationVector.Get<FMaterialSource>() == StochasticLighting::EMaterialSource::FrontLayerGBuffer)
			{
				PermutationVector.Set<FCopyDepthAndNormal>(false);
				PermutationVector.Set<FReprojectLumen>(false);
			}
			else
			{
				unimplemented();
			}
		}

		if (PermutationVector.Get<FOverflowTile>())
		{
			PermutationVector.Set<FCopyDepthAndNormal>(false);
			PermutationVector.Set<FStochasticSampleOffset>(StochasticLighting::EStochasticSampleOffset::None);
			PermutationVector.Set<FTileClassifyMegaLights>(false);
		}

		if (PermutationVector.Get<FMaterialSource>() != StochasticLighting::EMaterialSource::FrontLayerGBuffer)
		{
			PermutationVector.Set<FTileIndirectDispatch>(false);
		}

		if (!PermutationVector.Get<FTileClassifyLumen>())
		{
			PermutationVector.Set<FReprojectLumen>(false);
		}

		if (!PermutationVector.Get<FTileClassifyMegaLights>())
		{
			PermutationVector.Set<FReprojectMegaLights>(false);
		}

		if (!PermutationVector.Get<FReprojectLumen>())
		{
			PermutationVector.Set<FHistoryRejectBasedOnNormal>(false);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (RemapPermutation(PermutationVector, Parameters.Platform) != PermutationVector)
		{
			return false;
		}

		if (PermutationVector.Get<FTileClassifyLumen>() && !DoesPlatformSupportLumenGI(Parameters.Platform))
		{
			return false;
		}

		if (PermutationVector.Get<FTileClassifyMegaLights>() && !MegaLights::ShouldCompileShaders(Parameters.Platform))
		{
			return false;
		}

		return DoesPlatformSupportLumenGI(Parameters.Platform) || MegaLights::ShouldCompileShaders(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return 8;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FMaterialSource>() == StochasticLighting::EMaterialSource::FrontLayerGBuffer)
		{
			OutEnvironment.SetDefine(TEXT("FRONT_LAYER_TRANSLUCENCY"), 1);
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FStochasticLightingTileClassificationMarkCS, "/Engine/Private/StochasticLighting/StochasticLightingTileClassification.usf", "StochasticLightingTileClassificationMarkCS", SF_Compute);

FMegaLightsViewState::FResources& GetMegaLightsViewState(const FViewInfo& View, StochasticLighting::EMaterialSource MaterialSource)
{
	static FMegaLightsViewState::FResources DummyState;

	if (View.ViewState)
	{
		switch (MaterialSource)
		{
		case StochasticLighting::EMaterialSource::GBuffer: return View.ViewState->MegaLights.GBuffer;
		case StochasticLighting::EMaterialSource::HairStrands: return View.ViewState->MegaLights.HairStrands;
		case StochasticLighting::EMaterialSource::FrontLayerGBuffer: return View.ViewState->MegaLights.FrontLayerTranslucency;
		default: checkf(false, TEXT("GetMegaLightsViewState not implemented for MaterialSource")); return DummyState;
		};
	}

	return DummyState;
}

StochasticLighting::FContext::FContext(
	FRDGBuilder& InGraphBuilder,
	const FMinimalSceneTextures& InSceneTextures,
	const FFrontLayerTranslucencyGBufferParameters& InFrontLayerTranslucencyGBuffer,
	StochasticLighting::EMaterialSource InMaterialSource)
	: GraphBuilder(InGraphBuilder)
	, SceneTextures(InSceneTextures)
	, FrontLayerTranslucencyGBuffer(InFrontLayerTranslucencyGBuffer)
	, MaterialSource(InMaterialSource)
{}

void StochasticLighting::FContext::Validate(const StochasticLighting::FRunConfig& RunConfig) const
{
	if (RunConfig.bSubstrateOverflow)
	{
		check(MaterialSource == EMaterialSource::GBuffer);
	}

	if (RunConfig.bCopyDepthAndNormal)
	{
		check(DepthHistoryUAV && NormalHistoryUAV);
		if (MaterialSource == EMaterialSource::GBuffer && Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(RunConfig.View.GetShaderPlatform()))
		{
			check(ClosureHistoryUAV);
		}
	}

	if (RunConfig.bDownsampleDepthAndNormal2x1)
	{
		check(DownsampledSceneDepth2x1UAV && DownsampledWorldNormal2x1UAV);
	}

	if (RunConfig.bDownsampleDepthAndNormal2x2)
	{
		check(DownsampledSceneDepth2x2UAV && DownsampledWorldNormal2x2UAV);
	}

	if (RunConfig.bTileClassifyLumen)
	{
		check(LumenTileBitmaskUAV);
	}

	if (RunConfig.bTileClassifyMegaLights)
	{
		check(MegaLightsTileBitmaskUAV);
	}

	if (RunConfig.bReprojectLumen)
	{
		check(RunConfig.bTileClassifyLumen && EncodedHistoryScreenCoordUAV && LumenPackedPixelDataUAV);
	}

	if (RunConfig.bReprojectMegaLights)
	{
		check(RunConfig.bTileClassifyMegaLights && EncodedHistoryScreenCoordUAV && MegaLightsPackedPixelDataUAV);
	}
}

void StochasticLighting::FContext::Run(const FRunConfig& RunConfig)
{
	const FViewInfo& View = RunConfig.View;

	Validate(RunConfig);

	checkf((View.ViewRect.Min.X % TileSize) == 0 && (View.ViewRect.Min.Y % TileSize) == 0, TEXT("Viewport rect must be %d-pixel aligned."), TileSize);

	FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
	TUniformBufferRef<FBlueNoise> BlueNoiseUniformBuffer = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

	FHistoryScreenParameters HistoryScreenParameters = GetHistoryScreenParameters(View);

	FRDGTextureRef DepthHistoryTexture = nullptr;
	FRDGTextureRef NormalAndShadingInfoHistory = nullptr;
	FRDGTextureRef MegaLightsNumFramesAccumulatedHistory = nullptr;
	FRDGTextureRef ClosureHistory = nullptr;
	
	if (View.ViewState)
	{
		const FMegaLightsViewState::FResources& MegaLightsViewState = GetMegaLightsViewState(View, MaterialSource);
		const FStochasticLightingViewState& StochasticLightingViewState = View.ViewState->StochasticLighting;

		if (!View.bCameraCut && !View.bPrevTransformsReset)
		{
			if (MaterialSource == EMaterialSource::GBuffer)
			{
				DepthHistoryTexture = TryRegisterExternalTexture(GraphBuilder, StochasticLightingViewState.SceneDepthHistory);
				NormalAndShadingInfoHistory = TryRegisterExternalTexture(GraphBuilder, StochasticLightingViewState.SceneNormalHistory);
				ClosureHistory = TryRegisterExternalTexture(GraphBuilder, StochasticLightingViewState.SceneClosureHistory);
			}
			else if (MaterialSource == EMaterialSource::HairStrands)
			{
				DepthHistoryTexture = TryRegisterExternalTexture(GraphBuilder, MegaLightsViewState.SceneDepthHistory);
				NormalAndShadingInfoHistory = TryRegisterExternalTexture(GraphBuilder, MegaLightsViewState.SceneNormalHistory);
			}
			else if (MaterialSource == EMaterialSource::FrontLayerGBuffer)
			{
				DepthHistoryTexture = TryRegisterExternalTexture(GraphBuilder, StochasticLightingViewState.FrontLayerTranslucencyDepthHistory);
				NormalAndShadingInfoHistory = TryRegisterExternalTexture(GraphBuilder, StochasticLightingViewState.FrontLayerTranslucencyNormalHistory);
			}
			else
			{
				unimplemented();
			}

			MegaLightsNumFramesAccumulatedHistory = TryRegisterExternalTexture(GraphBuilder, MegaLightsViewState.NumFramesAccumulatedHistory);
		}
	}

	const bool bTileClassifySubstrate = RunConfig.bTileClassifySubstrate && MaterialSource == EMaterialSource::GBuffer;
	const bool bHistoryRejectBasedOnNormal = RunConfig.bReprojectLumen && LumenScreenProbeGather::UseRejectBasedOnNormal() && NormalAndShadingInfoHistory;

	const FIntPoint DownsampledBufferSize2x1 = FIntPoint::DivideAndRoundUp(SceneTextures.Config.Extent, FIntPoint(2, 1));
	const FIntPoint DownsampledViewMin2x1 = FIntPoint::DivideAndRoundUp(View.ViewRect.Min, FIntPoint(2, 1));
	const FIntPoint DownsampledViewSize2x1 = FIntPoint::DivideAndRoundUp(View.ViewRect.Size(), FIntPoint(2, 1));
	const FIntPoint DownsampledBufferSize2x2 = FIntPoint::DivideAndRoundUp(SceneTextures.Config.Extent, 2);
	const FIntPoint DownsampledViewMin2x2 = FIntPoint::DivideAndRoundUp(View.ViewRect.Min, 2);
	const FIntPoint DownsampledViewSize2x2 = FIntPoint::DivideAndRoundUp(View.ViewRect.Size(), 2);
	
	const uint32 LumenStochasticSampleMode = uint32(LumenScreenProbeGather::IsUsingDownsampledDepthAndNormal(View) ? EStochasticSampleOffset::DownsampleFactor2x2 : EStochasticSampleOffset::None);
	
	uint32 MegaLightsStochasticSampleMode = (uint32)EStochasticSampleOffset::None;
	if (RunConfig.bTileClassifyMegaLights)
	{
		const FIntPoint MegaLightsDownsampleFactor = MegaLights::GetDownsampleFactorXY(MaterialSource, View.GetShaderPlatform());
		if (MegaLightsDownsampleFactor.X == 2)
		{
			if (MegaLightsDownsampleFactor.Y == 2)
			{
				MegaLightsStochasticSampleMode = (uint32)EStochasticSampleOffset::DownsampleFactor2x2;
			}
			else
			{
				MegaLightsStochasticSampleMode = (uint32)EStochasticSampleOffset::DownsampleFactor2x1;
			}
		}
		else
		{
			MegaLightsStochasticSampleMode = (uint32)EStochasticSampleOffset::None;
		}
	}

	int32 StateFrameIndex = GetStateFrameIndex(View.ViewState);
	if (RunConfig.StateFrameIndexOverride >= 0)
	{
		StateFrameIndex = RunConfig.StateFrameIndexOverride;
	}

	EStochasticSampleOffset StochasticSampleOffset = EStochasticSampleOffset::None;
	if (RunConfig.bDownsampleDepthAndNormal2x1 && RunConfig.bDownsampleDepthAndNormal2x2)
	{
		StochasticSampleOffset = EStochasticSampleOffset::Both;
	}
	else if (RunConfig.bDownsampleDepthAndNormal2x1)
	{
		StochasticSampleOffset = EStochasticSampleOffset::DownsampleFactor2x1;
	}
	else if (RunConfig.bDownsampleDepthAndNormal2x2)
	{
		StochasticSampleOffset = EStochasticSampleOffset::DownsampleFactor2x2;
	}

	if ((RunConfig.bReprojectLumen || RunConfig.bReprojectMegaLights) && !DepthHistoryTexture)
	{
		DepthHistoryTexture = GSystemTextures.GetDepthDummy(GraphBuilder);
		NormalAndShadingInfoHistory = GSystemTextures.GetBlackDummy(GraphBuilder);
	}

	if (!ClosureHistory)
	{
		ClosureHistory = GSystemTextures.GetBlackDummy(GraphBuilder);
	}

	if (RunConfig.bReprojectMegaLights && !MegaLightsNumFramesAccumulatedHistory)
	{
		MegaLightsNumFramesAccumulatedHistory = GSystemTextures.GetBlackDummy(GraphBuilder);
	}

	FStochasticLightingTileClassificationMarkCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FCopyDepthAndNormal>(RunConfig.bCopyDepthAndNormal);
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FStochasticSampleOffset>(StochasticSampleOffset);
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FTileClassifyLumen>(RunConfig.bTileClassifyLumen);
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FTileClassifyMegaLights>(RunConfig.bTileClassifyMegaLights);
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FTileClassifySubstrate>(bTileClassifySubstrate);
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FReprojectLumen>(RunConfig.bReprojectLumen);
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FReprojectMegaLights>(RunConfig.bReprojectMegaLights);
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FHistoryRejectBasedOnNormal>(bHistoryRejectBasedOnNormal);
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FMaterialSource>(MaterialSource);
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FOverflowTile>(RunConfig.bSubstrateOverflow);
	PermutationVector.Set<FStochasticLightingTileClassificationMarkCS::FTileIndirectDispatch>(RunConfig.TileDispatchParams.TileIndirectArgs != nullptr);
	PermutationVector = FStochasticLightingTileClassificationMarkCS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
	auto ComputeShader = View.ShaderMap->GetShader<FStochasticLightingTileClassificationMarkCS>(PermutationVector);

	FStochasticLightingTileClassificationMarkCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FStochasticLightingTileClassificationMarkCS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;
	if (MaterialSource == EMaterialSource::GBuffer)
	{
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures.UniformBuffer);
		PassParameters->SceneTexturesStruct = SceneTextures.UniformBuffer;
	}
	else if (MaterialSource == EMaterialSource::HairStrands)
	{
		PassParameters->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
		PassParameters->SceneTextures.GBufferVelocityTexture = (*SceneTextures.UniformBuffer)->GBufferVelocityTexture;
	}
	else if (MaterialSource == EMaterialSource::FrontLayerGBuffer)
	{
		PassParameters->FrontLayerTranslucencyGBufferParameters = FrontLayerTranslucencyGBuffer;
	}
	else
	{
		unimplemented();
	}
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	PassParameters->DepthHistoryTexture = DepthHistoryTexture;
	PassParameters->NormalAndShadingInfoHistory = NormalAndShadingInfoHistory;
	PassParameters->ClosureHistory = ClosureHistory;
	PassParameters->MegaLightsNumFramesAccumulatedHistory = MegaLightsNumFramesAccumulatedHistory;
	PassParameters->RWDepthTexture = DepthHistoryUAV;
	PassParameters->RWNormalTexture = NormalHistoryUAV;
	PassParameters->RWClosureTexture = ClosureHistoryUAV;
	PassParameters->RWDownsampledSceneDepth2x1 = DownsampledSceneDepth2x1UAV;
	PassParameters->RWDownsampledSceneDepth2x2 = DownsampledSceneDepth2x2UAV;
	PassParameters->RWDownsampledWorldNormal2x1 = DownsampledWorldNormal2x1UAV;
	PassParameters->RWDownsampledWorldNormal2x2 = DownsampledWorldNormal2x2UAV;
	PassParameters->RWLumenTileBitmask = LumenTileBitmaskUAV;
	PassParameters->RWMegaLightsTileBitmask = MegaLightsTileBitmaskUAV;
	PassParameters->RWEncodedHistoryScreenCoord = EncodedHistoryScreenCoordUAV;
	PassParameters->RWLumenPackedPixelData = LumenPackedPixelDataUAV;
	PassParameters->RWMegaLightsPackedPixelData = MegaLightsPackedPixelDataUAV;
	LumenScreenProbeGather::SetupTileClassifyParameters(View, PassParameters->ScreenProbeGatherTileClassifyParameters);
	LumenReflections::SetupCompositeParameters(View, RunConfig.ReflectionsMethod, PassParameters->ReflectionsCompositeParameters);
	MegaLights::SetupTileClassifyParameters(View, PassParameters->MegaLightsTileClassifyParameters);
	PassParameters->HistoryScreenParameters = HistoryScreenParameters;
	PassParameters->ReflectionPass = (uint32)(MaterialSource == StochasticLighting::EMaterialSource::FrontLayerGBuffer ? ELumenReflectionPass::FrontLayerTranslucency : ELumenReflectionPass::Opaque);
	PassParameters->DownsampledViewMin2x1 = DownsampledViewMin2x1;
	PassParameters->DownsampledViewSize2x1 = DownsampledViewSize2x1;
	PassParameters->DownsampledViewMin2x2 = DownsampledViewMin2x2;
	PassParameters->DownsampledViewSize2x2 = DownsampledViewSize2x2;
	PassParameters->LumenStochasticSampleMode = LumenStochasticSampleMode;
	PassParameters->MegaLightsStochasticSampleMode = MegaLightsStochasticSampleMode;
	PassParameters->StochasticLightingStateFrameIndex = StateFrameIndex;
	PassParameters->OutputDownsampledDepthAndNormal = RunConfig.bOutputDownsampledDepthAndNormal ? 1 : 0;
	PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
	PassParameters->BlueNoise = BlueNoiseUniformBuffer;

	if (bTileClassifySubstrate)
	{
		const FSubstrateViewData* SubstrateViewData = &View.SubstrateViewData;
		PassParameters->TileDrawIndirectDataBufferUAV = SubstrateViewData->ClassificationTileDrawIndirectBufferUAV;
		PassParameters->TileListBufferUAV = SubstrateViewData->ClassificationTileListBufferUAV;
		PassParameters->TileEncoding = SubstrateViewData->TileEncoding;
		PassParameters->bRectPrimitive = GRHISupportsRectTopology ? 1 : 0;
		for (uint32 TileType = 0; TileType < SUBSTRATE_TILE_TYPE_COUNT; ++TileType)
		{
			PassParameters->TileListBufferOffsets[TileType] = FUintVector4(SubstrateViewData->ClassificationTileListBufferOffset[TileType], 0, 0, 0);
		}
	}

	if (RunConfig.bSubstrateOverflow)
	{
		PassParameters->TileIndirectBuffer = View.SubstrateViewData.ClosureTileDispatchIndirectBuffer;
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("TileClassificationMark(Overflow)"),
			RunConfig.ComputePassFlags,
			ComputeShader,
			PassParameters,
			View.SubstrateViewData.ClosureTileDispatchIndirectBuffer,
			Substrate::GetClosureTileIndirectArgsOffset(/*InDownsampleFactor*/ 1));
	}
	else if (RunConfig.TileDispatchParams.TileIndirectArgs != nullptr)
	{
		PassParameters->TileIndirectDispatchData = GraphBuilder.CreateSRV(RunConfig.TileDispatchParams.TileData);
		PassParameters->TileIndirectBuffer = RunConfig.TileDispatchParams.TileIndirectArgs;
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("TileClassificationMark(Indirect)"),
			RunConfig.ComputePassFlags,
			ComputeShader,
			PassParameters,
			RunConfig.TileDispatchParams.TileIndirectArgs,
			/* offset */ 0);
	}
	else
	{
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("TileClassificationMark"),
			RunConfig.ComputePassFlags,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), FStochasticLightingTileClassificationMarkCS::GetGroupSize()));
	}

	if (bTileClassifySubstrate)
	{
		// Sanity check
		check(!RunConfig.bSubstrateOverflow);
		Substrate::AddSubstrateMaterialClassificationIndirectArgsPass(GraphBuilder, View, RunConfig.ComputePassFlags);
	}
}

/**
 * Load GBuffer data once and transform it for subsequent lighting passes
 * This includes full res depth and normal copy for opaque before it gets overwritten by water or other translucency writing depth
 */
StochasticLighting::FFrameTemporaries StochasticLighting::TileClassificationMark(
	FRDGBuilder& GraphBuilder,
	TConstArrayView<FViewInfo> Views,
	const FMinimalSceneTextures& SceneTextures,
	const FFrontLayerTranslucencyGBufferParameters& FrontLayerTranslucencyGBuffer,
	TConstArrayView<StochasticLighting::FRunConfig> RunConfigs,
	StochasticLighting::EMaterialSource MaterialSource)
{
	FFrameTemporaries FrameTemporaries;

	const FViewInfo& ReferenceView = Views[0];

	const uint32 ClosureCount = (MaterialSource == EMaterialSource::GBuffer) ? Substrate::GetSubstrateMaxClosureCount(ReferenceView) : 1;

	bool bAnyViewCopyDepthAndNormal = false;
	bool bAnyViewDownsampleDepthAndNormal2x1 = false;
	bool bAnyViewDownsampleDepthAndNormal2x2 = false;
	bool bAnyViewTileClassifyLumen = false;
	bool bAnyViewTileClassifyMegaLights = false;
	bool bAnyViewTileClassifySubstrate = false;
	bool bAnyViewReprojectLumen = false;
	bool bAnyViewReprojectMegaLights = false;
	bool bAnyViewOutputDownsampledDepthAndNormal = false;

	for (const FRunConfig& RunConfig : RunConfigs)
	{
		bAnyViewCopyDepthAndNormal |= RunConfig.bCopyDepthAndNormal;
		bAnyViewDownsampleDepthAndNormal2x1 |= RunConfig.bDownsampleDepthAndNormal2x1;
		bAnyViewDownsampleDepthAndNormal2x2 |= RunConfig.bDownsampleDepthAndNormal2x2;
		bAnyViewTileClassifyLumen |= RunConfig.bTileClassifyLumen;
		bAnyViewTileClassifyMegaLights |= RunConfig.bTileClassifyMegaLights;
		bAnyViewTileClassifySubstrate |= RunConfig.bTileClassifySubstrate;
		bAnyViewReprojectLumen |= RunConfig.bReprojectLumen;
		bAnyViewReprojectMegaLights |= RunConfig.bReprojectMegaLights;
		bAnyViewOutputDownsampledDepthAndNormal |= RunConfig.bOutputDownsampledDepthAndNormal;
	}

	// Actual extent of viewports -- useful for passing to EncloseVisualizeExtent (used by VisualizeTexture debug feature)
	FIntPoint VisualizeExtent = FIntPoint(0, 0);
	for (const FViewInfo& View : Views)
	{
		VisualizeExtent.X = FMath::Max(VisualizeExtent.X, View.ViewRect.Max.X);
		VisualizeExtent.Y = FMath::Max(VisualizeExtent.Y, View.ViewRect.Max.Y);
	}

	FRDGTextureRef DepthHistory = nullptr;
	FRDGTextureRef NormalHistory = nullptr;
	FRDGTextureRef ClosureHistory = nullptr;
	if (bAnyViewCopyDepthAndNormal)
	{
		DepthHistory = FrameTemporaries.DepthHistory.CreateSharedRT(GraphBuilder,
			FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PF_R32_FLOAT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
			VisualizeExtent,
			TEXT("StochasticLighting.DepthHistory"));

		NormalHistory = FrameTemporaries.NormalHistory.CreateSharedRT(GraphBuilder,
			FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PF_A2B10G10R10, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
			VisualizeExtent,
			TEXT("StochasticLighting.NormalAndShadingInfoHistory"));

		if (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(ReferenceView.GetShaderPlatform()))
		{
			ClosureHistory = FrameTemporaries.ClosureHistory.CreateSharedRT(GraphBuilder,
				FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PF_R8_UINT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
				VisualizeExtent,
				TEXT("StochasticLighting.ClosureHistory"));
		}
	}

	FRDGTextureRef DownsampledSceneDepth2x1 = nullptr;
	FRDGTextureRef DownsampledWorldNormal2x1 = nullptr;
	if (bAnyViewDownsampleDepthAndNormal2x1)
	{
		FIntPoint DownsampledBufferSize = bAnyViewOutputDownsampledDepthAndNormal ? FIntPoint::DivideAndRoundUp(SceneTextures.Config.Extent, FIntPoint(2, 1)) : FIntPoint(1, 1);

		DownsampledSceneDepth2x1 = FrameTemporaries.DownsampledSceneDepth2x1.CreateSharedRT(GraphBuilder,
			FRDGTextureDesc::Create2D(DownsampledBufferSize, PF_R32_FLOAT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
			VisualizeExtent,
			TEXT("StochasticLighting.DownsampledSceneDepth2x1"));

		DownsampledWorldNormal2x1 = FrameTemporaries.DownsampledWorldNormal2x1.CreateSharedRT(GraphBuilder,
			FRDGTextureDesc::Create2D(DownsampledBufferSize, PF_A2B10G10R10, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
			VisualizeExtent,
			TEXT("StochasticLighting.DownsampledWorldNormal2x1"));
	}

	FRDGTextureRef DownsampledSceneDepth2x2 = nullptr;
	FRDGTextureRef DownsampledWorldNormal2x2 = nullptr;
	if (bAnyViewDownsampleDepthAndNormal2x2)
	{
		FIntPoint DownsampledBufferSize = bAnyViewOutputDownsampledDepthAndNormal ? FIntPoint::DivideAndRoundUp(SceneTextures.Config.Extent, FIntPoint(2, 2)) : FIntPoint(1, 1);

		DownsampledSceneDepth2x2 = FrameTemporaries.DownsampledSceneDepth2x2.CreateSharedRT(GraphBuilder,
			FRDGTextureDesc::Create2D(DownsampledBufferSize, PF_R32_FLOAT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
			VisualizeExtent,
			TEXT("StochasticLighting.DownsampledSceneDepth2x2"));

		DownsampledWorldNormal2x2 = FrameTemporaries.DownsampledWorldNormal2x2.CreateSharedRT(GraphBuilder,
			FRDGTextureDesc::Create2D(DownsampledBufferSize, PF_A2B10G10R10, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
			VisualizeExtent,
			TEXT("StochasticLighting.DownsampledWorldNormal2x2"));
	}

	FRDGTextureRef LumenTileBitmask = nullptr;
	if (bAnyViewTileClassifyLumen)
	{
		const FIntPoint BufferSize = (MaterialSource == EMaterialSource::GBuffer) ? Substrate::GetSubstrateTextureResolution(ReferenceView, SceneTextures.Config.Extent) : SceneTextures.Config.Extent;
		const FIntPoint BufferSizeInTiles = FIntPoint::DivideAndRoundUp(BufferSize, TileSize);

		LumenTileBitmask = FrameTemporaries.LumenTileBitmask.CreateSharedRT(GraphBuilder,
			FRDGTextureDesc::Create2DArray(BufferSizeInTiles, PF_R8_UINT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV, ClosureCount),
			VisualizeExtent,
			TEXT("StochasticLighting.LumenTileBitmask"));
	}

	FRDGTextureRef MegaLightsTileBitmask = nullptr;
	if (bAnyViewTileClassifyMegaLights)
	{
		const FIntPoint BufferSizeInTiles = FIntPoint::DivideAndRoundUp(SceneTextures.Config.Extent, TileSize);

		MegaLightsTileBitmask = FrameTemporaries.MegaLightsTileBitmask.CreateSharedRT(GraphBuilder,
			FRDGTextureDesc::Create2D(BufferSizeInTiles, PF_R8_UINT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
			VisualizeExtent,
			TEXT("StochasticLighting.MegaLightsTileBitmask"));
	}

	FRDGTextureRef EncodedHistoryScreenCoord = nullptr;
	FRDGTextureRef LumenPackedPixelData = nullptr;
	FRDGTextureRef MegaLightsPackedPixelData = nullptr;
	if (bAnyViewReprojectLumen || bAnyViewReprojectMegaLights)
	{
		EncodedHistoryScreenCoord = FrameTemporaries.EncodedHistoryScreenCoord.CreateSharedRT(GraphBuilder,
			FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PF_R16G16_UINT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
			VisualizeExtent,
			TEXT("StochasticLighting.EncodedHistoryScreenCoord"));

		if (bAnyViewReprojectLumen)
		{
			LumenPackedPixelData = FrameTemporaries.LumenPackedPixelData.CreateSharedRT(GraphBuilder,
				FRDGTextureDesc::Create2DArray(SceneTextures.Config.Extent, PF_R8_UINT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV, ClosureCount),
				VisualizeExtent,
				TEXT("StochasticLighting.LumenPackedPixelData"));
		}

		if (bAnyViewReprojectMegaLights)
		{
			MegaLightsPackedPixelData = FrameTemporaries.MegaLightsPackedPixelData.CreateSharedRT(GraphBuilder,
				FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PF_R8_UINT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
				VisualizeExtent,
				TEXT("StochasticLighting.MegaLightsPackedPixelData"));
		}
	}

	FContext StochasticLightingContext(GraphBuilder, SceneTextures, FrontLayerTranslucencyGBuffer, MaterialSource);
	StochasticLightingContext.DepthHistoryUAV = DepthHistory ? GraphBuilder.CreateUAV(DepthHistory, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.NormalHistoryUAV = NormalHistory ? GraphBuilder.CreateUAV(NormalHistory, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.ClosureHistoryUAV = ClosureHistory ? GraphBuilder.CreateUAV(ClosureHistory, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.DownsampledSceneDepth2x1UAV = DownsampledSceneDepth2x1 ? GraphBuilder.CreateUAV(DownsampledSceneDepth2x1, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.DownsampledWorldNormal2x1UAV = DownsampledWorldNormal2x1 ? GraphBuilder.CreateUAV(DownsampledWorldNormal2x1, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.DownsampledSceneDepth2x2UAV = DownsampledSceneDepth2x2 ? GraphBuilder.CreateUAV(DownsampledSceneDepth2x2, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.DownsampledWorldNormal2x2UAV = DownsampledWorldNormal2x2 ? GraphBuilder.CreateUAV(DownsampledWorldNormal2x2, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.LumenTileBitmaskUAV = LumenTileBitmask ? GraphBuilder.CreateUAV(LumenTileBitmask, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.MegaLightsTileBitmaskUAV = MegaLightsTileBitmask ? GraphBuilder.CreateUAV(MegaLightsTileBitmask, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.EncodedHistoryScreenCoordUAV = EncodedHistoryScreenCoord ? GraphBuilder.CreateUAV(EncodedHistoryScreenCoord, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.LumenPackedPixelDataUAV = LumenPackedPixelData ? GraphBuilder.CreateUAV(LumenPackedPixelData, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	StochasticLightingContext.MegaLightsPackedPixelDataUAV = MegaLightsPackedPixelData ? GraphBuilder.CreateUAV(MegaLightsPackedPixelData, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
	// When using the compact tile list, only tiles in the list are processed by the CS.
	// Since other tiles are never written, their bitmask values are uninitialized.
	// Clear the bitmasks upfront so unprocessed tiles correctly read as 0 for downstream MegaLights/Lumen.
	const bool bAnyViewHasIndirectDispatch = RunConfigs.ContainsByPredicate(
		[](const FRunConfig& RC) { return RC.TileDispatchParams.TileIndirectArgs != nullptr; });
	if (bAnyViewHasIndirectDispatch)
	{
		if (StochasticLightingContext.LumenTileBitmaskUAV)
		{
			AddClearUAVPass(GraphBuilder, StochasticLightingContext.LumenTileBitmaskUAV, 0u);
		}
		if (StochasticLightingContext.MegaLightsTileBitmaskUAV)
		{
			AddClearUAVPass(GraphBuilder, StochasticLightingContext.MegaLightsTileBitmaskUAV, 0u);
		}
	}

	bool bNeedsClear = true;

	for (const FRunConfig& RunConfig : RunConfigs)
	{
		const FViewInfo& View = RunConfig.View;

		const bool bSupportsMultipleClosureEvaluation = (MaterialSource == EMaterialSource::GBuffer) && Lumen::SupportsMultipleClosureEvaluation(View);

		if (bSupportsMultipleClosureEvaluation)
		{
			if (LumenPackedPixelData && ClosureCount > 1 && bNeedsClear)
			{
				const uint32 LumenInvalidPackedPixelData = 0x30;

				// Initialize LumenPackedPixelData to invalid value for all pixels belonging to slice>0, i.e. closure with index > 0. This is necessary because:
				// 1) The classification is dispatched only on valid tiles. For closure>0, the LumenPackedPixelData won't be initialized otherwise
				// 2) The temporal reprojection pass uses LumenPackedPixelData to update history value (in particular NumFamesAccumulated), which are used next frame to prune invalid history data.
				// Without this, LumenScreenProbeGather will fetch invalid/uninitialized history data for closure>0, causing visual artifacts
				FRDGTextureUAVRef LumenPackedPixelDataOverflowUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(LumenPackedPixelData, 0/*InMipLevel*/, LumenPackedPixelData->Desc.Format, 1/*InFirstArraySlice*/, ClosureCount - 1u/*InNumArraySlices*/));

				// This value needs to be kept in sync with StochasticLightingCommon.ush
				AddClearUAVPass(GraphBuilder, LumenPackedPixelDataOverflowUAV, LumenInvalidPackedPixelData, RunConfig.ComputePassFlags);
				bNeedsClear = false;
			}
		}

		StochasticLightingContext.Run(RunConfig);

		if (bSupportsMultipleClosureEvaluation)
		{
			FRunConfig OverflowTileRunConfig(View);
			OverflowTileRunConfig.ComputePassFlags = RunConfig.ComputePassFlags;
			OverflowTileRunConfig.ReflectionsMethod = RunConfig.ReflectionsMethod;
			OverflowTileRunConfig.bSubstrateOverflow = true;
			OverflowTileRunConfig.bTileClassifyLumen = RunConfig.bTileClassifyLumen;
			OverflowTileRunConfig.bReprojectLumen = RunConfig.bReprojectLumen;

			StochasticLightingContext.Run(OverflowTileRunConfig);
		}
	}

	return MoveTemp(FrameTemporaries);
}

static bool InternalRequiresStochasticLightingPass(const FViewFamilyInfo& ViewFamily, EDiffuseIndirectMethod InDiffuseIndirectMethod, EReflectionsMethod InReflectionsMethod)
{
	const ELumenFinalGatherMethod LumenFinalGatherMethod = Lumen::GetFinalGatherMethod(ViewFamily, ViewFamily.GetShaderPlatform());
	const bool bLumenScreenProbeGather = InDiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen && LumenFinalGatherMethod == ELumenFinalGatherMethod::ScreenProbeGather;
	const bool bLumenIrradianceFieldGather = InDiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen && LumenFinalGatherMethod == ELumenFinalGatherMethod::IrradianceFieldGather;

	return bLumenScreenProbeGather
		|| (bLumenIrradianceFieldGather && LumenIrradianceFieldGather::NeedsStochasticLightingDownsample())
		|| InReflectionsMethod == EReflectionsMethod::Lumen
		|| MegaLights::IsEnabled(ViewFamily)
		|| Substrate::UsesStochasticLightingClassification(ViewFamily.GetShaderPlatform());
}

bool FDeferredShadingSceneRenderer::RequiresStochasticLightingPass()
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(Views[ViewIndex]);
		if (InternalRequiresStochasticLightingPass(ViewFamily, ViewPipelineState.DiffuseIndirectMethod, ViewPipelineState.ReflectionsMethod))
		{
			return true;
		}
	}

	return false;
}

/**
 * Load GBuffer data once and transform it for subsequent lighting passes
 * This includes full res depth and normal copy for opaque before it gets overwritten by water or other translucency writing depth
 */
void FDeferredShadingSceneRenderer::StochasticLightingTileClassificationMark(FRDGBuilder& GraphBuilder, FLumenSceneFrameTemporaries& FrameTemporaries, const FSceneTextures& SceneTextures, bool bAsyncMegaLightsGenerateSamples)
{
	if (Views.IsEmpty())
	{
		return;
	}

	TArray<StochasticLighting::FRunConfig, TInlineAllocator<LUMEN_MAX_VIEWS>> RunConfigs;
	RunConfigs.Reserve(Views.Num());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];
		const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);

		if (InternalRequiresStochasticLightingPass(ViewFamily, ViewPipelineState.DiffuseIndirectMethod, ViewPipelineState.ReflectionsMethod))
		{
			const FSceneTextureParameters& SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, SceneTextures);
			const FIntPoint MegaLightsDownsampleFactor = MegaLights::GetDownsampleFactorXY(StochasticLighting::EMaterialSource::GBuffer, View.GetShaderPlatform());
			const ELumenFinalGatherMethod LumenFinalGatherMethod = Lumen::GetFinalGatherMethod(ViewFamily, ShaderPlatform);
			const bool bLumenScreenProbeGather = ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen && LumenFinalGatherMethod == ELumenFinalGatherMethod::ScreenProbeGather;
			const bool bLumenIrradianceFieldGather = ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen && LumenFinalGatherMethod == ELumenFinalGatherMethod::IrradianceFieldGather;
			const bool bTileClassifyLumen = bLumenScreenProbeGather || ViewPipelineState.ReflectionsMethod == EReflectionsMethod::Lumen;
			const bool bTileClassifyMegaLights = MegaLights::IsEnabled(ViewFamily);
			const bool bTileClassifySubstrate = Substrate::UsesStochasticLightingClassification(View.GetShaderPlatform());
			const bool bNeedsReprojection = bLumenScreenProbeGather || bTileClassifyMegaLights;
			// Always copy if MegaLights is enabled. The spatial denoiser needs the normal copy
			const bool bCopyDepthAndNormal = ((View.ViewState && !View.bStatePrevViewInfoIsReadOnly) && bTileClassifyLumen) || bTileClassifyMegaLights;
			const bool bDownsampleDepthAndNormal2x1 = bTileClassifyMegaLights && MegaLightsDownsampleFactor == FIntPoint(2, 1);
			const bool bDownsampleDepthAndNormal2x2 = (bLumenScreenProbeGather && LumenScreenProbeGather::IsUsingDownsampledDepthAndNormal(View))
				|| bLumenIrradianceFieldGather
				|| (bTileClassifyMegaLights && MegaLightsDownsampleFactor == FIntPoint(2, 2));

			StochasticLighting::FRunConfig& RunConfig = RunConfigs.Emplace_GetRef(View);

			ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute;
			{
				const int32 AsyncComputeMode = FMath::Clamp(CVarStochasticLightingAsyncCompute.GetValueOnRenderThread(), 0, 2);
				if (AsyncComputeMode == 1)
				{
					if (bTileClassifyMegaLights)
					{
						ComputePassFlags = bAsyncMegaLightsGenerateSamples ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute;
					}
					else if (bLumenIrradianceFieldGather)
					{
						const bool bLumenFinalGatherUseAsyncCompute = LumenDiffuseIndirect::UseAsyncCompute(ViewFamily, ViewPipelineState.DiffuseIndirectMethod);
						ComputePassFlags = bLumenFinalGatherUseAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute;
					}
				}
				else if (AsyncComputeMode == 2)
				{
					ComputePassFlags = ERDGPassFlags::AsyncCompute;
				}
			}

			RunConfig.ComputePassFlags = ComputePassFlags;
			RunConfig.ReflectionsMethod = ViewPipelineState.ReflectionsMethod;
			RunConfig.bCopyDepthAndNormal = bCopyDepthAndNormal;
			RunConfig.bDownsampleDepthAndNormal2x1 = bDownsampleDepthAndNormal2x1;
			RunConfig.bDownsampleDepthAndNormal2x2 = bDownsampleDepthAndNormal2x2;
			RunConfig.bTileClassifyLumen = bTileClassifyLumen;
			RunConfig.bTileClassifyMegaLights = bTileClassifyMegaLights;
			RunConfig.bTileClassifySubstrate = bTileClassifySubstrate;
			RunConfig.bReprojectLumen = bNeedsReprojection && bLumenScreenProbeGather;
			RunConfig.bReprojectMegaLights = bNeedsReprojection && bTileClassifyMegaLights;
		}
	}

	FFrontLayerTranslucencyGBufferParameters FrontLayerTranslucencyGBuffer;
	FrontLayerTranslucencyGBuffer.FrontLayerTranslucencyGBufferA = nullptr;
	FrontLayerTranslucencyGBuffer.FrontLayerTranslucencyGBufferB = nullptr;
	FrontLayerTranslucencyGBuffer.FrontLayerTranslucencyGBufferC = nullptr;
	FrontLayerTranslucencyGBuffer.FrontLayerTranslucencySceneDepth = nullptr;

	FrameTemporaries.StochasticLighting = StochasticLighting::TileClassificationMark(
		GraphBuilder,
		Views,
		SceneTextures,
		FrontLayerTranslucencyGBuffer,
		RunConfigs,
		StochasticLighting::EMaterialSource::GBuffer);
}

namespace Lumen
{
	bool UseFrontLayerTranslucencyReflections(const FViewInfo& View);
}

void FDeferredShadingSceneRenderer::StochasticLightingFrontLayerTranslucencyTileClassificationMark(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures, FFrontLayerTranslucencyData& FrontLayerTranslucencyData, ERDGPassFlags ComputePassFlags)
{
	const FFrontLayerTranslucencyGBufferParameters FrontLayerTranslucencyGBuffer = GetFrontLayerTranslucencyGBufferParameters(FrontLayerTranslucencyData);

	if (Views.IsEmpty())
	{
		return;
	}

	TArray<StochasticLighting::FRunConfig, TInlineAllocator<LUMEN_MAX_VIEWS>> RunConfigs;
	RunConfigs.Reserve(Views.Num());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];
		const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);

		const FIntPoint MegaLightsDownsampleFactor = MegaLights::GetDownsampleFactorXY(StochasticLighting::EMaterialSource::FrontLayerGBuffer, View.GetShaderPlatform());

		const bool bLumenReflectionsFrontLayer = (ViewPipelineState.ReflectionsMethod == EReflectionsMethod::Lumen && Lumen::UseFrontLayerTranslucencyReflections(View));
		const bool bTileClassifyLumen = View.bTranslucentSurfaceLighting && (bLumenReflectionsFrontLayer || RayTracedTranslucency::IsEnabled(View));
		const bool bTileClassifyMegaLights = MegaLights::UseFrontLayerTranslucencyDirectLighting(View);

		if (!bTileClassifyLumen && !bTileClassifyMegaLights)
		{
			continue;
		}

		const bool bCopyDepthAndNormal = false;
		const bool bDownsampleDepthAndNormal2x1 = bTileClassifyMegaLights && MegaLightsDownsampleFactor == FIntPoint(2, 1);
		const bool bDownsampleDepthAndNormal2x2 = bTileClassifyMegaLights && MegaLightsDownsampleFactor == FIntPoint(2, 2);

		EReflectionsMethod ReflectionsMethod = EReflectionsMethod::Disabled;

		if (bTileClassifyLumen)
		{
			ReflectionsMethod = EReflectionsMethod::Lumen;
		}

		StochasticLighting::FRunConfig& RunConfig = RunConfigs.Emplace_GetRef(View);
		RunConfig.ComputePassFlags = ComputePassFlags;
		RunConfig.ReflectionsMethod = ReflectionsMethod;
		RunConfig.bCopyDepthAndNormal = bCopyDepthAndNormal;
		RunConfig.bDownsampleDepthAndNormal2x1 = bDownsampleDepthAndNormal2x1;
		RunConfig.bDownsampleDepthAndNormal2x2 = bDownsampleDepthAndNormal2x2;
		RunConfig.bTileClassifyLumen = bTileClassifyLumen;
		RunConfig.bTileClassifyMegaLights = bTileClassifyMegaLights;
		RunConfig.bTileClassifySubstrate = false;
		RunConfig.bReprojectLumen = false;
		RunConfig.bReprojectMegaLights = bTileClassifyMegaLights;

		if (!bTileClassifyLumen && bTileClassifyMegaLights && ViewIndex < FrontLayerTranslucencyData.PerViewMegaLightsTileLists.Num())
		{
			const FMegaLightsTileList& ViewTileList = FrontLayerTranslucencyData.PerViewMegaLightsTileLists[ViewIndex];
			RunConfig.TileDispatchParams.TileData = ViewTileList.TileData;
			RunConfig.TileDispatchParams.TileIndirectArgs = ViewTileList.TileIndirectArgs;
		}
	}

	FrontLayerTranslucencyData.StochasticLighting = StochasticLighting::TileClassificationMark(
		GraphBuilder,
		Views,
		SceneTextures,
		FrontLayerTranslucencyGBuffer,
		RunConfigs,
		StochasticLighting::EMaterialSource::FrontLayerGBuffer);
}

void FDeferredShadingSceneRenderer::QueueExtractStochasticLighting(FRDGBuilder& GraphBuilder, FLumenSceneFrameTemporaries& FrameTemporaries, const FFrontLayerTranslucencyData& FrontLayerTranslucencyData, const FMinimalSceneTextures& SceneTextures)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];

		if (View.ViewState && !View.bStatePrevViewInfoIsReadOnly)
		{
			FStochasticLightingViewState& ViewState = View.ViewState->StochasticLighting;

			if (FrameTemporaries.StochasticLighting.DepthHistory.GetRenderTarget())
			{
				GraphBuilder.QueueTextureExtraction(FrameTemporaries.StochasticLighting.DepthHistory.GetRenderTarget(), &ViewState.SceneDepthHistory);
			}
			else
			{
				ViewState.SceneDepthHistory = nullptr;
			}

			if (FrameTemporaries.StochasticLighting.NormalHistory.GetRenderTarget())
			{
				GraphBuilder.QueueTextureExtraction(FrameTemporaries.StochasticLighting.NormalHistory.GetRenderTarget(), &ViewState.SceneNormalHistory);
			}
			else
			{
				ViewState.SceneNormalHistory = nullptr;
			}

			if (FrameTemporaries.StochasticLighting.ClosureHistory.GetRenderTarget())
			{
				GraphBuilder.QueueTextureExtraction(FrameTemporaries.StochasticLighting.ClosureHistory.GetRenderTarget(), &ViewState.SceneClosureHistory);
			}
			else
			{
				ViewState.SceneClosureHistory = nullptr;
			}

			if (FrontLayerTranslucencyData.SceneDepth)
			{
				GraphBuilder.QueueTextureExtraction(FrontLayerTranslucencyData.SceneDepth, &ViewState.FrontLayerTranslucencyDepthHistory);
			}
			else
			{
				ViewState.FrontLayerTranslucencyDepthHistory = nullptr;
			}

			if (FrontLayerTranslucencyData.GBufferA)
			{
				GraphBuilder.QueueTextureExtraction(FrontLayerTranslucencyData.GBufferA, &ViewState.FrontLayerTranslucencyNormalHistory);
			}
			else
			{
				ViewState.FrontLayerTranslucencyNormalHistory = nullptr;
			}

			ViewState.HistoryFrameIndex = View.ViewState->PendingPrevFrameNumber;
			ViewState.HistoryViewRect = View.ViewRect;

			ViewState.HistoryScreenPositionScaleBias = View.GetScreenPositionScaleBias(SceneTextures.Config.Extent, View.ViewRect);

			const FVector2f InvBufferSize(1.0f / SceneTextures.Config.Extent.X, 1.0f / SceneTextures.Config.Extent.Y);

			ViewState.HistoryUVMinMax = FVector4f(
				View.ViewRect.Min.X * InvBufferSize.X,
				View.ViewRect.Min.Y * InvBufferSize.Y,
				View.ViewRect.Max.X * InvBufferSize.X,
				View.ViewRect.Max.Y * InvBufferSize.Y);

			// Clamp gather4 to a valid bilinear footprint in order to avoid sampling outside of valid bounds.
			// Expand the guard band according to encoding precision
			float GuardBandSizeX = 0.5f;
			float GuardBandSizeY = 0.5f;
			{
				FVector4f SubPixelGridSizeAndInvSize;
				FIntPoint DecodeShift;
				StochasticLighting::GetHistoryScreenCoordCodec(SceneTextures.Config.Extent, SubPixelGridSizeAndInvSize, DecodeShift);

				GuardBandSizeX += FMath::Max(SubPixelGridSizeAndInvSize.Z, 1.0f / 512.0f);
				GuardBandSizeY += FMath::Max(SubPixelGridSizeAndInvSize.W, 1.0f / 512.0f);
			}
			ViewState.HistoryGatherUVMinMax = FVector4f(
				(View.ViewRect.Min.X + GuardBandSizeX) * InvBufferSize.X,
				(View.ViewRect.Min.Y + GuardBandSizeY) * InvBufferSize.Y,
				(View.ViewRect.Max.X - GuardBandSizeX) * InvBufferSize.X,
				(View.ViewRect.Max.Y - GuardBandSizeY) * InvBufferSize.Y);

			ViewState.HistoryBufferSizeAndInvSize = FVector4f(
				SceneTextures.Config.Extent.X,
				SceneTextures.Config.Extent.Y,
				1.0f / SceneTextures.Config.Extent.X,
				1.0f / SceneTextures.Config.Extent.Y);
		}
	}
}

FRDGTextureRef FLumenSharedRT::CreateSharedRT(
	FRDGBuilder& Builder,
	const FRDGTextureDesc& Desc,
	FIntPoint VisibleExtent,
	const TCHAR* Name,
	ERDGTextureFlags Flags)
{
	if (RenderTarget)
	{
		check(Desc.Extent == RenderTarget->Desc.Extent);
		return RenderTarget;
	}

	RenderTarget = Builder.CreateTexture(Desc, Name, Flags);
	RenderTarget->EncloseVisualizeExtent(VisibleExtent);

	return RenderTarget;
}
