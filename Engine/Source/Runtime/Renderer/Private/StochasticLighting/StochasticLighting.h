// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "RenderGraphFwd.h"
#include "RenderGraphDefinitions.h"
#include "ShaderParameterMacros.h"
#include "Math/MathFwd.h"
#include "DeferredShadingRendererTypes.h"
#include "LumenDefinitions.h"

class FViewInfo;
class FSceneViewState;
class FFrontLayerTranslucencyGBufferParameters;
struct FMinimalSceneTextures;
enum class EReflectionsMethod;

class FLumenSharedRT
{
public:
	FRDGTextureRef CreateSharedRT(FRDGBuilder& Builder, const FRDGTextureDesc& Desc, FIntPoint VisibleExtent, const TCHAR* Name, ERDGTextureFlags Flags = ERDGTextureFlags::None);

	FRDGTextureRef GetRenderTarget() const
	{
		return RenderTarget;
	}

private:
	FRDGTextureRef RenderTarget = nullptr;
};

namespace StochasticLighting
{
	enum class EMaterialSource
	{
		GBuffer,
		FrontLayerGBuffer,
		HairStrands,
		MAX
	};

	enum class EStochasticSampleOffset
	{
		None,
		DownsampleFactor2x1,
		DownsampleFactor2x2,
		Both,
		MAX
	};

	BEGIN_SHADER_PARAMETER_STRUCT(FHistoryScreenParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint2>, EncodedHistoryScreenCoordTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, PackedPixelDataTexture)
		SHADER_PARAMETER(FVector4f, HistoryScreenPositionScaleBias)
		SHADER_PARAMETER(FVector4f, HistoryUVMinMax)
		SHADER_PARAMETER(FVector4f, HistoryGatherUVMinMax)
		SHADER_PARAMETER(FVector4f, HistoryBufferSizeAndInvSize)
		SHADER_PARAMETER(FVector4f, HistorySubPixelGridSizeAndInvSize)
		SHADER_PARAMETER(FIntPoint, HistoryScreenCoordDecodeShift)
	END_SHADER_PARAMETER_STRUCT()

	int32 GetStateFrameIndex(const FSceneViewState* ViewState);

	bool IsStateFrameIndexOverridden();

	void InitDefaultHistoryScreenParameters(FHistoryScreenParameters& OutHistoryScreenParameters);

	// Only non-resource parameters are populated
	FHistoryScreenParameters GetHistoryScreenParameters(const FViewInfo& View);

	// Temporaries valid only in a single frame
	struct FFrameTemporaries
	{
		FLumenSharedRT DepthHistory;
		FLumenSharedRT NormalHistory;
		FLumenSharedRT ClosureHistory;

		FLumenSharedRT DownsampledSceneDepth2x1;
		FLumenSharedRT DownsampledWorldNormal2x1;
		FLumenSharedRT DownsampledSceneDepth2x2;
		FLumenSharedRT DownsampledWorldNormal2x2;

		FLumenSharedRT LumenTileBitmask;
		FLumenSharedRT MegaLightsTileBitmask;

		FLumenSharedRT EncodedHistoryScreenCoord;
		FLumenSharedRT LumenPackedPixelData;
		FLumenSharedRT MegaLightsPackedPixelData;
	};

	// Optional compact tile list buffers used for indirect dispatch
	struct FTileDispatchParameters
	{
		FRDGBufferRef TileData = nullptr;
		FRDGBufferRef TileIndirectArgs = nullptr;
	};

	struct FRunConfig
	{
		const FViewInfo& View;
		ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute;
		EReflectionsMethod ReflectionsMethod = EReflectionsMethod::Disabled;
		int32 StateFrameIndexOverride = -1;
		bool bSubstrateOverflow = false;
		bool bCopyDepthAndNormal = false;
		bool bDownsampleDepthAndNormal2x1 = false;
		bool bDownsampleDepthAndNormal2x2 = false;
		bool bTileClassifyLumen = false;
		bool bTileClassifyMegaLights = false;
		bool bTileClassifySubstrate = false;
		bool bReprojectLumen = false;
		bool bReprojectMegaLights = false;
		bool bOutputDownsampledDepthAndNormal = true;
		FTileDispatchParameters TileDispatchParams;

		FRunConfig(const FViewInfo& InView)
			: View(InView)
		{
		}
	};

	struct FContext
	{
		FRDGBuilder& GraphBuilder;
		const FMinimalSceneTextures& SceneTextures;
		const FFrontLayerTranslucencyGBufferParameters& FrontLayerTranslucencyGBuffer;
		EMaterialSource MaterialSource;
		FRDGTextureUAVRef DepthHistoryUAV = nullptr;
		FRDGTextureUAVRef NormalHistoryUAV = nullptr;
		FRDGTextureUAVRef ClosureHistoryUAV = nullptr;
		FRDGTextureUAVRef DownsampledSceneDepth2x1UAV = nullptr;
		FRDGTextureUAVRef DownsampledWorldNormal2x1UAV = nullptr;
		FRDGTextureUAVRef DownsampledSceneDepth2x2UAV = nullptr;
		FRDGTextureUAVRef DownsampledWorldNormal2x2UAV = nullptr;
		FRDGTextureUAVRef LumenTileBitmaskUAV = nullptr;
		FRDGTextureUAVRef MegaLightsTileBitmaskUAV = nullptr;
		FRDGTextureUAVRef EncodedHistoryScreenCoordUAV = nullptr;
		FRDGTextureUAVRef LumenPackedPixelDataUAV = nullptr;
		FRDGTextureUAVRef MegaLightsPackedPixelDataUAV = nullptr;

		FContext(
			FRDGBuilder& InGraphBuilder,
			const FMinimalSceneTextures& InSceneTextures,
			const FFrontLayerTranslucencyGBufferParameters& InFrontLayerTranslucencyGBuffer,
			EMaterialSource InMaterialSource);

		void Validate(const FRunConfig& RunConfig) const;

		void Run(const FRunConfig& RunConfig);
	};

	FFrameTemporaries TileClassificationMark(
		FRDGBuilder& GraphBuilder,
		TConstArrayView<FViewInfo> Views,
		const FMinimalSceneTextures& SceneTextures,
		const FFrontLayerTranslucencyGBufferParameters& FrontLayerTranslucencyGBuffer,
		TConstArrayView<FRunConfig> RunConfigs,
		EMaterialSource MaterialSource);
}
