// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RendererInterface.h"

class FStochasticLightingViewState
{
public:
	TRefCountPtr<IPooledRenderTarget> SceneDepthHistory;
	TRefCountPtr<IPooledRenderTarget> SceneNormalHistory;
	TRefCountPtr<IPooledRenderTarget> SceneClosureHistory;

	TRefCountPtr<IPooledRenderTarget> FrontLayerTranslucencyDepthHistory;
	TRefCountPtr<IPooledRenderTarget> FrontLayerTranslucencyNormalHistory;

	uint32 HistoryFrameIndex = 0;
	FIntRect HistoryViewRect = FIntRect(0, 0, 0, 0);
	FVector4f HistoryScreenPositionScaleBias = FVector4f(1.0f, 1.0f, 0.0f, 0.0f);
	FVector4f HistoryUVMinMax = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
	FVector4f HistoryGatherUVMinMax = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
	FVector4f HistoryBufferSizeAndInvSize = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);

	void SafeRelease()
	{
		SceneDepthHistory.SafeRelease();
		SceneNormalHistory.SafeRelease();
		SceneClosureHistory.SafeRelease();

		FrontLayerTranslucencyDepthHistory.SafeRelease();
		FrontLayerTranslucencyNormalHistory.SafeRelease();
	}

	uint64 GetGPUSizeBytes(bool bLogSizes) const;
};
