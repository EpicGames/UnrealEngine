// Copyright Epic Games, Inc. All Rights Reserved.

#include "StochasticLightingVisualize.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"

StochasticLightingVisualize::FTonemappingParameters StochasticLightingVisualize::GetTonemappingParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	bool bAllowTonemapping)
{
	FTonemappingParameters TonemappingParameters;

	FRDGTextureRef ColorGradingTexture = TryRegisterExternalTexture(GraphBuilder, View.GetTonemappingLUT());
	FRDGBufferRef EyeAdaptationBuffer = GetEyeAdaptationBuffer(GraphBuilder, View);

	TonemappingParameters.Tonemap = 0;
	if (EyeAdaptationBuffer != nullptr
		&& ColorGradingTexture != nullptr
		&& View.Family->EngineShowFlags.Tonemapper != 0
		&& bAllowTonemapping)
	{
		TonemappingParameters.Tonemap = 1;
	}

	TonemappingParameters.ColorGradingLUT = ColorGradingTexture;
	TonemappingParameters.ColorGradingLUTSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	TonemappingParameters.EyeAdaptationBuffer = GraphBuilder.CreateSRV(EyeAdaptationBuffer);

	if (!TonemappingParameters.ColorGradingLUT)
	{
		TonemappingParameters.ColorGradingLUT = FRDGSystemTextures::Get(GraphBuilder).VolumetricBlack;
	}

	return TonemappingParameters;
}

void StochasticLightingVisualize::GetTileOutputView(const FIntRect& ViewRect, int32 LinearTileIndex, FIntPoint& OutputViewOffset, FIntPoint& OutputViewSize, uint32 NumOverviewTilesPerRow)
{
	if (LinearTileIndex >= 0)
	{
		const FIntPoint TileCoord(LinearTileIndex % NumOverviewTilesPerRow, LinearTileIndex / NumOverviewTilesPerRow);
		const FIntPoint TileSize = FMath::DivideAndRoundDown<FIntPoint>(ViewRect.Size() - OverviewTileMargin * (NumOverviewTilesPerRow + 1), NumOverviewTilesPerRow);

		OutputViewSize = TileSize;
		OutputViewOffset = ViewRect.Min + TileSize * TileCoord + FIntPoint(OverviewTileMargin) * (TileCoord + 1);
	}
	else
	{
		OutputViewOffset = ViewRect.Min;
		OutputViewSize = ViewRect.Size();
	}
}

