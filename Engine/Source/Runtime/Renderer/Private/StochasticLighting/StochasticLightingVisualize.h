// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphFwd.h"
#include "ShaderParameterMacros.h"

class FViewInfo;

namespace StochasticLightingVisualize
{
	BEGIN_SHADER_PARAMETER_STRUCT(FTonemappingParameters, )
		SHADER_PARAMETER(int32, Tonemap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, ColorGradingLUT)
		SHADER_PARAMETER_SAMPLER(SamplerState, ColorGradingLUTSampler)
	END_SHADER_PARAMETER_STRUCT()

	FTonemappingParameters GetTonemappingParameters(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		bool bAllowTonemapping);

	constexpr int32 OverviewTileMargin = 4;

	void GetTileOutputView(const FIntRect& ViewRect, int32 LinearTileIndex, FIntPoint& OutputViewOffset, FIntPoint& OutputViewSize, uint32 NumOverviewTilesPerRow);
}