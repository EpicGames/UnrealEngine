// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GlintShadingLUTs.cpp
=============================================================================*/

#include "GlintShadingLUTs.h"
#include "SceneRendering.h"
#include "Engine/Engine.h"
#include "RenderUtils.h"
#include "TextureResource.h"
#include "Engine/Texture2DArray.h"
#include "SceneViewState.h"

#include "GlintShadingLUTsData.h"


#if OVERRIDE_GLINT_LUT_ASSSET_WITH_EMBEDDED_DATA
#include "RenderTargetPool.h"
static FGlintShadingLUTsStateData GlintShadingLUTsDataOverride; // Only for local debugging purpose.
#endif

void FGlintShadingLUTsStateData::Init(FRDGBuilder& GraphBuilder, FViewInfo& View)
{
#if OVERRIDE_GLINT_LUT_ASSSET_WITH_EMBEDDED_DATA

	if (Substrate::IsGlintEnabled(View.GetShaderPlatform()))
	{
		if (GlintShadingLUTsDataOverride.GlintShadingLUTs == nullptr)
		{
			const uint32 MipCount = 7;
			const EPixelFormat PixelFormat = PF_FloatR11G11B10;

			GlintShadingLUTsDataOverride.GlintShadingLUTs = GRenderTargetPool.FindFreeElement(
				FRDGTextureDesc::Create2DArray(FIntPoint(GlintLut1DArrayMip0Width, 1), PixelFormat, FClearValueBinding::None, TexCreate_ShaderResource, GlintLut1DArrayMip0Height, MipCount), TEXT("Material.Glints"));

			auto UploadGlintTextureArrayMipData = [&](uint32 LutArrayIndex, uint32 LutMipLevel, uint32 LutMipWidth, uint8* SrcData)
			{
			#if 0
				check(PixelFormat == PF_A32B32G32R32F);
				const uint32 SrcBytesPerPixel = sizeof(float) * 4;
			#elif 1
				check(PixelFormat == PF_FloatR11G11B10);
				const uint32 SrcBytesPerPixel = sizeof(uint32) * 1;
				// If we have a texture2darray of 64x1 texel with 1024 slices, then all the source arrays are 1024*(64+32+16+8+4+2+1) = 130048 = 127KB. 
				// Loaded on GPU with this format, it seems to be 3MB as reported by dx12.
			#elif 0
				check(PixelFormat == PF_FloatRGBA);
				const uint32 SrcBytesPerPixel = sizeof(uint32) * 2;
			#else
			#error A valid branch must be taken
			#endif

				uint32 SrcStride = SrcBytesPerPixel * LutMipWidth;

				uint32 DstStride;
				uint8* Dst = (uint8*)GraphBuilder.RHICmdList.LockTexture2DArray(GlintShadingLUTsDataOverride.GlintShadingLUTs->GetRHI()->GetTexture2DArray(), LutArrayIndex, LutMipLevel, RLM_WriteOnly, DstStride, false);

				const uint8* Src = SrcData +LutArrayIndex * SrcStride;

				FMemory::Memcpy(Dst, Src, SrcStride);

				GraphBuilder.RHICmdList.UnlockTexture2DArray(GlintShadingLUTsDataOverride.GlintShadingLUTs->GetRHI()->GetTexture2DArray(), LutArrayIndex, LutMipLevel, false);
			};

			const uint32 GlintLut1DArraySize = GlintLut1DArrayMip0Height;
			for (int LutArrayIndex = 0; LutArrayIndex < GlintLut1DArraySize; ++LutArrayIndex)
			{
				UploadGlintTextureArrayMipData(LutArrayIndex, 0, GlintLut1DArrayMip0Width, (uint8*)GlintLut1DArrayMip0RGBA);
			}
			for (int LutArrayIndex = 0; LutArrayIndex < GlintLut1DArraySize; ++LutArrayIndex)
			{
				UploadGlintTextureArrayMipData(LutArrayIndex, 1, GlintLut1DArrayMip1Width, (uint8*)GlintLut1DArrayMip1RGBA);
			}
			for (int LutArrayIndex = 0; LutArrayIndex < GlintLut1DArraySize; ++LutArrayIndex)
			{
				UploadGlintTextureArrayMipData(LutArrayIndex, 2, GlintLut1DArrayMip2Width, (uint8*)GlintLut1DArrayMip2RGBA);
			}
			for (int LutArrayIndex = 0; LutArrayIndex < GlintLut1DArraySize; ++LutArrayIndex)
			{
				UploadGlintTextureArrayMipData(LutArrayIndex, 3, GlintLut1DArrayMip3Width, (uint8*)GlintLut1DArrayMip3RGBA);
			}
			for (int LutArrayIndex = 0; LutArrayIndex < GlintLut1DArraySize; ++LutArrayIndex)
			{
				UploadGlintTextureArrayMipData(LutArrayIndex, 4, GlintLut1DArrayMip4Width, (uint8*)GlintLut1DArrayMip4RGBA);
			}
			for (int LutArrayIndex = 0; LutArrayIndex < GlintLut1DArraySize; ++LutArrayIndex)
			{
				UploadGlintTextureArrayMipData(LutArrayIndex, 5, GlintLut1DArrayMip5Width, (uint8*)GlintLut1DArrayMip5RGBA);
			}
			for (int LutArrayIndex = 0; LutArrayIndex < GlintLut1DArraySize; ++LutArrayIndex)
			{
				UploadGlintTextureArrayMipData(LutArrayIndex, 6, GlintLut1DArrayMip6Width, (uint8*)GlintLut1DArrayMip6RGBA);
			}

			GlintShadingLUTsDataOverride.SetDictionaryParameter(16, 64, 0.5);	// Match the embedded LUT
		}
	}
	else
	{
		if (View.ViewState)
		{
			GlintShadingLUTsDataOverride.GlintShadingLUTs = nullptr;
			GlintShadingLUTsDataOverride.RHIGlintShadingLUTs = nullptr;
		}
	}

	// Assign the RHI texture is is possible
	if (GlintShadingLUTsDataOverride.RHIGlintShadingLUTs == nullptr && GlintShadingLUTsDataOverride.GlintShadingLUTs != nullptr)
	{
		GlintShadingLUTsDataOverride.RHIGlintShadingLUTs = GlintShadingLUTsDataOverride.GlintShadingLUTs->GetRHI()->GetTexture2DArray();
	}

#else

	// No more texture pointer caching here as we could then end up using dangling texture's RHI resource.
	// This was noticed when switching editor preview modes with passes (in this case hit proxy pass) that builds the view uniform buffer without first calling Init (such as the hit proxy pass) See UE-383643.

#endif
}

FGlintShadingLUTsStateData FGlintShadingLUTsStateData::GetParameters(const FViewInfo& View)
{
#if OVERRIDE_GLINT_LUT_ASSSET_WITH_EMBEDDED_DATA

	// The single data used when debugging and iterating locally on LUTs.
	return GlintShadingLUTsDataOverride;

#else

	FGlintShadingLUTsStateData Out;

	// Resolve the texture on demand.
	auto ResolveRHI = [](UTexture2DArray* Texture) -> FRHITexture*
	{
		const FTextureResource* Resource = Texture ? Texture->GetResource() : nullptr;
		return (Resource && Resource->TextureRHI) ? Resource->TextureRHI->GetTexture2DArray() : nullptr;
	};

	if (Substrate::GlintLUTIndex() == 0)
	{
		Out.RHIGlintShadingLUTs = ResolveRHI(GEngine->GlintTexture);
		Out.SetDictionaryParameter(16, 64, 0.5);	// Match the embedded LUT
	}
	else
	{
		Out.RHIGlintShadingLUTs = ResolveRHI(GEngine->GlintTexture2);
		Out.SetDictionaryParameter(8, 256, 0.5);	// Match the embedded LUT
	}

	return Out;

#endif
}

void FGlintShadingLUTsStateData::SetDictionaryParameter(int32 InNumberOfLevels, int32 InNumberOfDistributionsPerChannel, float InDictionaryAlpha)
{
	Dictionary_Alpha					= InDictionaryAlpha;
	Dictionary_NLevels					= InNumberOfLevels;
	Dictionary_NDistributionsPerChannel = InNumberOfDistributionsPerChannel;
	Dictionary_N						= Dictionary_NDistributionsPerChannel * 3;
	Dictionary_Pyramid0Size				= 1u << (Dictionary_NLevels - 1u);
}

