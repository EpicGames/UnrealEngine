// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GlintShadingLUTs.h: look up table to be ablew to render luts.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"

class FViewInfo;

struct FGlintShadingLUTsStateData
{
	TRefCountPtr<IPooledRenderTarget> GlintShadingLUTs = nullptr;
	FRHITexture* RHIGlintShadingLUTs = nullptr;

	float Dictionary_Alpha = 0.0f;
	int32 Dictionary_NDistributionsPerChannel = 0;
	int32 Dictionary_N = 0;
	int32 Dictionary_NLevels = 0;
	int32 Dictionary_Pyramid0Size = 0;

	uint64 GetGPUSizeBytes(bool bLogSizes) const;

	static void Init(FRDGBuilder& GraphBuilder, FViewInfo& View);

	// Resolves the glint LUT texture and its parameters to use this frame.
	static FGlintShadingLUTsStateData GetParameters(const FViewInfo& View);

private:
	void SetDictionaryParameter(int32 NumberOfLevels, int32 NumberOfDistributionsPerChannel, float Dictionary_Alpha);
};
