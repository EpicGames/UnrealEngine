// Copyright Epic Games, Inc. All Rights Reserved.

#include "Substrate.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HAL/IConsoleManager.h"
#include "PixelShaderUtils.h"
#include "SceneView.h"
#include "SceneRendering.h"
#include "RendererInterface.h"
#include "UniformBuffer.h"
#include "PostProcess/SceneRenderTargets.h"
#include "SceneTextureParameters.h"
#include "ShaderCompiler.h"
#include "Lumen/Lumen.h"
#include "RendererUtils.h"
#include "EngineAnalytics.h"
#include "SystemTextures.h"
#include "DBufferTextures.h"
#include "DecalRenderingShared.h"
#include "CompositionLighting/PostProcessDeferredDecals.h"
#include "MegaLights/MegaLights.h"


// The project setting for Substrate
static TAutoConsoleVariable<int32> CVarUseCmaskClear(
	TEXT("r.Substrate.UseCmaskClear"),
	0,
	TEXT("TEST."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateUseClosureCountFromMaterial(
	TEXT("r.Substrate.UseClosureCountFromMaterial"),
	1,
	TEXT("When enable, scale the number of Lumen's layers for multi-closures pixels based on material data. Otherwise use r.Substrate.ClosuresPerPixel."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateDebugPeelLayersAboveDepth(
	TEXT("r.Substrate.Debug.PeelLayersAboveDepth"),
	0,
	TEXT("Substrate debug control to progressively peel off materials layer by layer."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateDebugRoughnessTracking(
	TEXT("r.Substrate.Debug.RoughnessTracking"),
	1,
	TEXT("Substrate debug control to disable roughness tracking, e.g. top layer roughness affecting bottom layer roughness to simulate light scattering."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateAsyncClassification(
	TEXT("r.Substrate.AsyncClassification"),
	1,
	TEXT("Run Substrate material classification in async (with shadow)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateDBufferPassDedicatedTiles(
	TEXT("r.Substrate.DBufferPass.DedicatedTiles"),
	0,
	TEXT("Use dedicated tile for DBuffer application when DBuffer pass is enabled."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateAllocationMode(
	TEXT("r.Substrate.AllocationMode"),
	1,
	TEXT("Substrate resource allocation mode. \n 0: Allocate resources based on view requirement, \n 1: Allocate resources based on view requirement, but can only grow over frame to minimize resources reallocation and hitches, \n 2: Allocate resources based on platform settings."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateTileCoord8Bits(
	TEXT("r.Substrate.TileCoord8bits"),
	1, // Substrate 60fps by default
	TEXT("Format of tile coord. This variable is read-only."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateStochasticLightingActive(
	TEXT("r.Substrate.StochasticLighting.Active"),
	0,
	TEXT("Activate stochastic lighting to get better performance (runtime toggle for debugging). Requires r.Substrate.StochasticLighting to be enabled (which is read-only)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateClearMaterialBuffer(
	TEXT("r.Substrate.Debug.ClearMaterialBuffer"),
	0,
	TEXT("Clear Substrate material buffer before writing to it for debugging purpose"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateUseStochasticLightingClassification(
	TEXT("r.Substrate.UseStochasticLightingClassification"),
	1,
	TEXT("Run Substrate tile classification within the Stochastic lighting classification. Only available when using Substrate Blendable GBuffer."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSubstrateStencilPassStage(
	TEXT("r.Substrate.StencilPassStage"),
	1,
	TEXT("Run Substrate tile stencil writing: \n 0: During the indirect compose pass \n 1: Before lighting pass \n 2: During the first need (lighting pass or sky lighting pass)"),
	ECVF_RenderThreadSafe);

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSubstrateGlobalUniformParameters, "Substrate");

void FSubstrateViewData::Reset()
{
	// When tracking the MaxClosurePerPixel per view, we use a bit mask stored onto 8bit. 
	// If SUBSTRATE_MAX_CLOSURE_COUNT>8u, it will overflow. Hence the static assert here
	// Variables to verify when increasing the max. closure count:
	// * MaxClosurePerPixel
	// * SubstrateClosureCountMask
	static_assert(SUBSTRATE_MAX_CLOSURE_COUNT <= 8u);

	// Propagate values after reset as we use the per-view (vs. the per-scene) 
	// value to know if a view needs special complex path or not
	const uint8 OldUsesTileTypeMask = UsesTileTypeMask;
	const bool OldUsesAnistropy = bUsesAnisotropy;
	*this = FSubstrateViewData();
	UsesTileTypeMask = OldUsesTileTypeMask;
	bUsesAnisotropy = OldUsesAnistropy;
}

namespace MegaLights
{
	uint32 GetStateFrameIndex(FSceneViewState*, EShaderPlatform InPlatform);
}

namespace Substrate
{

EStencilPassStage GetStencilPassStage()
{
	const uint32 Stage = FMath::Clamp(CVarSubstrateStencilPassStage.GetValueOnRenderThread(), 0, 2);
	switch (Stage)
	{
		case 0:  return EStencilPassStage::DuringIndirectLighting;
		case 1:  return EStencilPassStage::Standalone;
		case 2:  return EStencilPassStage::StandaloneAdaptive;
		default: return EStencilPassStage::None;
	}
}

uint32 GetMaxDownsampleFactor()
{
	return 2;
}

uint32 GetClosureTileIndirectArgsOffset(uint32 InDownsampleFactor)
{
	// Args buffer is arranged as follow:
	// 0 : DownsampleFactor=1 (1x1)
	// 1 : DownsampleFactor=2 (2x2)
	// 2 : DownsampleFactor=3 (4x4)
	// ...
	check(InDownsampleFactor <= GetMaxDownsampleFactor());
	const uint32 ClampedDownsampleFactor = FMath::Clamp(InDownsampleFactor, 1u, GetMaxDownsampleFactor());
	const uint32 Offset = ClampedDownsampleFactor - 1u;
	return Offset * sizeof(FRHIDispatchIndirectParameters);
}

bool IsStochasticLightingActive(EShaderPlatform InPlatform)
{
	return Substrate::IsStochasticLightingEnabled(InPlatform) && CVarSubstrateStochasticLightingActive.GetValueOnRenderThread() > 0;
}

bool UsesSubstrateMaterialBuffer(EShaderPlatform In)
{
	return IsUsingGBuffers(In);
}

uint32 GetMaterialBufferAllocationMode()
{
	return FMath::Clamp(CVarSubstrateAllocationMode.GetValueOnAnyThread(), 0, 2);
}

bool UsesSubstrateClosureCountFromMaterialData() 
{
	return CVarSubstrateUseClosureCountFromMaterial.GetValueOnRenderThread() > 0;
}

// Accessor to experimental variable static local here to not have have it exposed in public header RenderUtils.h for now.
static bool IsExperimentalToonUnifiedDiffuseEnabled()
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Substrate.Experimental.ToonUnifiedDiffuse"));
	return CVar && CVar->GetInt() > 0;
}

uint32 GetSubstrateMaxClosureCount(const FViewInfo& View)
{
	uint32 Out = 1;
	if (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(View.GetShaderPlatform()))
	{
		if (UsesSubstrateClosureCountFromMaterialData())
		{
			Out = FMath::Clamp(View.SubstrateViewData.SceneData ? View.SubstrateViewData.SceneData->EffectiveMaxClosurePerPixel : View.SubstrateViewData.MaxClosurePerPixel, 1u, SUBSTRATE_MAX_CLOSURE_COUNT);
		}
		else
		{
			Out = FMath::Clamp(uint32(GetClosurePerPixel(View.GetShaderPlatform(), GetCurrentMaterialQualityLevelChecked())), 1u, SUBSTRATE_MAX_CLOSURE_COUNT);
		}
	}
	return Out;
}

static FIntPoint GetSubstrateTextureTileResolution(const FViewInfo& View, const FIntPoint& InResolution)
{
	FIntPoint Out = InResolution;
	Out.X = FMath::DivideAndRoundUp(Out.X, SUBSTRATE_TILE_SIZE);
	Out.Y = FMath::DivideAndRoundUp(Out.Y, SUBSTRATE_TILE_SIZE);
	return Out;
}

FIntPoint GetSubstrateTextureResolution(const FViewInfo& View, const FIntPoint& InResolution)
{
	if (Substrate::IsSubstrateEnabled())
	{
		// Ensure Substrate resolution are round to SUBSTRATE_TILE_SIZE (8) 
		// This is ensured by QuantizeSceneBufferSize()
		check((uint32(InResolution.X) & 0x3) == 0 && (uint32(InResolution.Y) & 0x3) == 0);
	}
	return InResolution;
}

bool Is8bitTileCoordEnabled()
{
	return CVarSubstrateTileCoord8Bits.GetValueOnAnyThread() > 0 ? 1 : 0;
}

bool GetSubstrateUsesTileType(const FViewInfo& View, ESubstrateTileType TileType)
{
	if (Substrate::IsSubstrateEnabled())
	{
		static_assert(uint32(ESubstrateTileType::EComplexSpecial) < sizeof(uint8)*8u);
		check(TileType <= ESubstrateTileType::EComplexSpecial);

		const bool bCompatible = TileType == ESubstrateTileType::EComplexSpecial ? !Substrate::IsSubstrateBlendableGBufferEnabled(View.GetShaderPlatform()) : true;
		const uint8 Mask = GetSubstrateTileTypeAsUint8(TileType);
		return bCompatible && UsesSubstrateTileType(View.SubstrateViewData.UsesTileTypeMask, TileType);
	}
	return false;
}	

bool GetSubstrateUsesComplexSpecialPath(const FViewInfo& View)
{
	if (Substrate::IsSubstrateEnabled())
	{
		// Use the per-view value rather than the per-scene data to have more accurate dispatching of special complex tiles 
		// and avoid unecessary empty-dispatch
		return UsesSubstrateTileType(View.SubstrateViewData.UsesTileTypeMask, ESubstrateTileType::EComplexSpecial);
	}
	return false;
}

bool GetSubstrateUsesAnisotropy(const FViewInfo& View)
{
	if (Substrate::IsSubstrateEnabled())
	{
		return View.SubstrateViewData.bUsesAnisotropy;
	}
	return false;
}

static void BindSubstrateGlobalUniformParameters(FRDGBuilder& GraphBuilder, const FSubstrateViewData* SubstrateViewData, bool bNeedsMaterialBuffer, bool bBlendableGBufferEnabled, FSubstrateGlobalUniformParameters& OutSubstrateUniformParameters);

bool SupportsCMask(const FStaticShaderPlatform InPlatform)
{
	return CVarUseCmaskClear.GetValueOnRenderThread() > 0 && FDataDrivenShaderPlatformInfo::GetSupportsRenderTargetWriteMask(InPlatform);
}

bool IsClassificationAsync()
{
	return CVarSubstrateAsyncClassification.GetValueOnRenderThread() > 0;
}

bool UsesStochasticLightingClassification(EShaderPlatform InPlatform)
{
	return 
		Substrate::IsSubstrateEnabled() && 
		Substrate::IsSubstrateBlendableGBufferEnabled(InPlatform) && 
		Substrate::UsesSubstrateMaterialBuffer(InPlatform) && // Only needed for deferred lighting
		(DoesPlatformSupportLumenGI(InPlatform) || MegaLights::ShouldCompileShaders(InPlatform)) &&
		CVarSubstrateUseStochasticLightingClassification.GetValueOnRenderThread() && 
		!Substrate::UseTileSize16();	// Only supported when tile size is 8 otherwise tiles are incomplete (stochastic pass would have to gather 2x2 tiles of 8 pixel ot create a new tile of 16x16 pixels).
	;
}

static EPixelFormat GetClassificationTileFormat(const FIntPoint& InResolution, uint32 InTileEncoding)
{
	return InTileEncoding == SUBSTRATE_TILE_ENCODING_16BITS ? PF_R32_UINT : PF_R16_UINT;
}

static void InitialiseSubstrateViewData(FRDGBuilder& GraphBuilder, FViewInfo& View, const FSceneTexturesConfig& SceneTexturesConfig, bool bNeedsClosureOffets, bool bNeedsMaterialBuffer, bool bBlendableGBufferEnabled, FSubstrateSceneData& SceneData)
{
	// Sanity check: the scene data should already exist 
	if (bNeedsMaterialBuffer && !bBlendableGBufferEnabled)
	{
		check(SceneData.MaterialTextureArray != nullptr);
	}

	FSubstrateViewData& Out = View.SubstrateViewData;
	Out.Reset();
	Out.SceneData = &SceneData;

	// Allocate texture using scene render targets size so we do not reallocate every frame when dynamic resolution is used in order to avoid resources allocation hitches.
	const FIntPoint DynResIndependentViewSize = SceneTexturesConfig.Extent;
	if (IsSubstrateEnabled())
	{
		const FIntPoint TileResolution(FMath::DivideAndRoundUp(DynResIndependentViewSize.X, SUBSTRATE_TILE_SIZE), FMath::DivideAndRoundUp(DynResIndependentViewSize.Y, SUBSTRATE_TILE_SIZE));

		// Tile classification buffers
		if (bNeedsMaterialBuffer)
		{
			// Indirect draw
			Out.ClassificationTileDrawIndirectBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndirectParameters>(ESubstrateTileType::ECount), TEXT("Substrate.SubstrateTileDrawIndirectBuffer"));
			Out.ClassificationTileDrawIndirectBufferUAV = GraphBuilder.CreateUAV(Out.ClassificationTileDrawIndirectBuffer, PF_R32_UINT);
			AddClearUAVPass(GraphBuilder, Out.ClassificationTileDrawIndirectBufferUAV, 0);

			// Indirect dispatch
			Out.ClassificationTileDispatchIndirectBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(ESubstrateTileType::ECount), TEXT("Substrate.SubstrateTileDispatchIndirectBuffer"));
			Out.ClassificationTileDispatchIndirectBufferUAV = GraphBuilder.CreateUAV(Out.ClassificationTileDispatchIndirectBuffer, PF_R32_UINT);
			AddClearUAVPass(GraphBuilder, Out.ClassificationTileDispatchIndirectBufferUAV, 0);

			// Separated subsurface & rough refraction textures (tile data)
			const uint32 RoughTileCount = IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()) ? TileResolution.X * TileResolution.Y : 4;
			const uint32 DecalTileCount = IsDBufferPassEnabled(View.GetShaderPlatform()) ? TileResolution.X * TileResolution.Y : 4;
			const uint32 RegularTileCount = TileResolution.X * TileResolution.Y;

			// For platforms whose resolution is never above 1080p, use 8bit tile format for performance, if possible
			const bool bRequest8bit = Substrate::Is8bitTileCoordEnabled() && (TileResolution.X <= 256 && TileResolution.Y <= 256);
			Out.TileEncoding = bRequest8bit ? SUBSTRATE_TILE_ENCODING_8BITS : SUBSTRATE_TILE_ENCODING_16BITS;

			Out.ClassificationTileListBufferOffset[ESubstrateTileType::ESimple]							= 0;
			Out.ClassificationTileListBufferOffset[ESubstrateTileType::ESingle]							= Out.ClassificationTileListBufferOffset[ESubstrateTileType::ESimple]							+ RegularTileCount;
			Out.ClassificationTileListBufferOffset[ESubstrateTileType::EComplex]						= Out.ClassificationTileListBufferOffset[ESubstrateTileType::ESingle]							+ RegularTileCount;
			Out.ClassificationTileListBufferOffset[ESubstrateTileType::EComplexSpecial]					= Out.ClassificationTileListBufferOffset[ESubstrateTileType::EComplex]							+ RegularTileCount;
			Out.ClassificationTileListBufferOffset[ESubstrateTileType::EOpaqueRoughRefraction]			= Out.ClassificationTileListBufferOffset[ESubstrateTileType::EComplexSpecial]					+ (UsesSubstrateTileType(SceneData.UsesTileTypeMask, ESubstrateTileType::EComplexSpecial) ? RegularTileCount : 4);
			Out.ClassificationTileListBufferOffset[ESubstrateTileType::EOpaqueRoughRefractionSSSWithout]= Out.ClassificationTileListBufferOffset[ESubstrateTileType::EOpaqueRoughRefraction]			+ RoughTileCount;
			Out.ClassificationTileListBufferOffset[ESubstrateTileType::EDecalSimple]					= Out.ClassificationTileListBufferOffset[ESubstrateTileType::EOpaqueRoughRefractionSSSWithout]	+ RoughTileCount;
			Out.ClassificationTileListBufferOffset[ESubstrateTileType::EDecalSingle]					= Out.ClassificationTileListBufferOffset[ESubstrateTileType::EDecalSimple]						+ DecalTileCount;
			Out.ClassificationTileListBufferOffset[ESubstrateTileType::EDecalComplex]					= Out.ClassificationTileListBufferOffset[ESubstrateTileType::EDecalSingle]						+ DecalTileCount;
			uint32 TotalTileCount										 								= Out.ClassificationTileListBufferOffset[ESubstrateTileType::EDecalComplex]						+ DecalTileCount;

			check(TotalTileCount > 0);

			const EPixelFormat ClassificationTileFormat = GetClassificationTileFormat(DynResIndependentViewSize, Out.TileEncoding);
			const uint32 FormatBytes = ClassificationTileFormat == PF_R16_UINT ? sizeof(uint16) : sizeof(uint32);

			Out.ClassificationTileListBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(FormatBytes, TotalTileCount), TEXT("Substrate.TileListBuffer"));
			Out.ClassificationTileListBufferSRV = GraphBuilder.CreateSRV(Out.ClassificationTileListBuffer, ClassificationTileFormat);
			Out.ClassificationTileListBufferUAV = GraphBuilder.CreateUAV(Out.ClassificationTileListBuffer, ClassificationTileFormat);
		}

		// Closure tiles
		if (bNeedsClosureOffets)
		{
			const FIntPoint TileCount = GetSubstrateTextureTileResolution(View, DynResIndependentViewSize);
			const uint32 LayerCount = GetSubstrateMaxClosureCount(View);
			const uint32 MaxTileCount = TileCount.X * TileCount.Y * LayerCount;

			Out.TileCount	= TileCount;
			Out.LayerCount  = LayerCount;
			Out.ClosureTilePerThreadDispatchIndirectBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(GetMaxDownsampleFactor()+1), TEXT("Substrate.SubstrateClosureTilePerThreadDispatchIndirectBuffer"));
			Out.ClosureTileDispatchIndirectBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(GetMaxDownsampleFactor()+1), TEXT("Substrate.SubstrateClosureTileDispatchIndirectBuffer"));
			Out.ClosureTileRaytracingIndirectBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(GetMaxDownsampleFactor()+1), TEXT("Substrate.SubstrateClosureTileRaytracingIndirectBuffer"));
			Out.ClosureTileCountBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, 1), TEXT("Substrate.ClosureTileCount"));
			Out.ClosureTileBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, MaxTileCount), TEXT("Substrate.ClosureTileBuffer"));
		}
		else
		{
			Out.TileCount = GetSubstrateTextureTileResolution(View, FIntPoint(2,2));
			Out.LayerCount = 1;
			Out.ClosureTilePerThreadDispatchIndirectBuffer = nullptr;
			Out.ClosureTileDispatchIndirectBuffer = nullptr;
			Out.ClosureTileRaytracingIndirectBuffer = nullptr;
			Out.ClosureTileCountBuffer = nullptr;
			Out.ClosureTileBuffer = nullptr;
		}

		// Create the readable uniform buffers
		{
			FSubstrateGlobalUniformParameters* SubstrateUniformParameters = GraphBuilder.AllocParameters<FSubstrateGlobalUniformParameters>();

			BindSubstrateGlobalUniformParameters(GraphBuilder, &Out, bNeedsMaterialBuffer, bBlendableGBufferEnabled, *SubstrateUniformParameters);
			Out.SubstrateGlobalUniformParameters = GraphBuilder.CreateUniformBuffer(SubstrateUniformParameters);
		}

		// Detect the complexity and mark less-compled tile types as available.
		// The CPU complexity is conservative, and indicate the max complexity that an material can have. However in order to dispatch all relevant 
		// tile types, we need to dispatch tile type with lower complexity as some part of the object might have lower complexity. 
		{
			const uint32 FirstSetBit = 8u - FMath::CountLeadingZeros8(View.SubstrateViewData.UsesTileTypeMask);
			for (uint32 Bit = 0; Bit < FirstSetBit; ++Bit)
			{
				View.SubstrateViewData.UsesTileTypeMask |= 1u<<Bit;
			}
		}
	}
}

static bool NeedsSampledMaterials(const FScene* Scene, const FSceneViewFamily& ViewFamily)
{
	bool bNeedSampledMaterial = false;
	if (IsStochasticLightingActive(ViewFamily.GetShaderPlatform()))
	{
		bNeedSampledMaterial = MegaLights::IsEnabled(ViewFamily);
		if (!bNeedSampledMaterial)
		{
			for (const FSceneView* View : ViewFamily.Views)
			{
				// For now, we only used sampled material for Lumen Reflections, not for Lumen Screen Probe
				// If later we need support for Lumen Screen Probe, we will need to add ShouldRenderLumenDiffuseGI(Scene, View)
				if (ShouldRenderLumenReflections(*View))
				{
					bNeedSampledMaterial = true;
					break;
				}
			}
		}
	}
	return bNeedSampledMaterial;
}

static bool NeedsSampledMaterials(const FScene* Scene, const FViewInfo& View)
{
	// For now, we only used sampled material for Lumen Reflections, not for Lumen Screen Probe
	// If later we need support for Lumen Screen Probe, we will need to add ShouldRenderLumenDiffuseGI(Scene, View)
	return IsStochasticLightingActive(View.GetShaderPlatform()) && (MegaLights::IsEnabled(*View.Family) || ShouldRenderLumenReflections(View));
}

static bool NeedsClosureOffsets(const FScene* Scene, const FViewInfo& View)
{
	// No need for closure index when either BlendableGBuffer is enabled or if ClosureCount == 1
	return  (ShouldRenderLumenDiffuseGI(Scene, View) || ShouldRenderLumenReflections(View) || MegaLights::IsEnabled(*View.Family) || NeedsSampledMaterials(Scene, *View.Family) || Substrate::ShouldRenderSubstrateDebugPasses(View))
		&& !Substrate::IsSubstrateBlendableGBufferEnabled(View.GetShaderPlatform())
		&& View.SubstrateViewData.MaxClosurePerPixel > 1;
}

static void RecordSubstrateAnalytics(EShaderPlatform InPlatform)
{
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Enabled"), 1));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("BytesPerPixel"), GetBytePerPixel(InPlatform)));

		FString OutStr("Substrate.Usage.ProjectSettings");
		FEngineAnalytics::GetProvider().RecordEvent(OutStr, EventAttributes);
	}
}

static EPixelFormat GetTopLayerTextureFormat(bool bUseDBufferPass)
{
	const bool bSubstrateHighQualityNormal = GetNormalQuality() > 0;

	// High quality normal is not supported on platforms that do not support R32G32 UAV load.
	// This is dues to the way Substrate account for decals. See FSubstrateDBufferPassCS, updating TopLayerTexture this way.
	// If you encounter this check, you must disable high quality normal for Substrate (material shaders must be recompiled to account for that).
	if (bUseDBufferPass)
	{
		check(!bSubstrateHighQualityNormal || (bSubstrateHighQualityNormal && UE::PixelFormat::HasCapabilities(PF_R32G32_UINT, EPixelFormatCapabilities::TypedUAVLoad)));
	}

	return bSubstrateHighQualityNormal ? PF_R32G32_UINT : PF_R32_UINT;
}

void InitialiseSubstrateFrameSceneData(FRDGBuilder& GraphBuilder, FScene& Scene, FViewFamilyInfo& ViewFamily, TConstArrayView<FViewInfo*> AllViews, EShaderPlatform ShaderPlatform)
{
	FSubstrateSceneData& Out = *ViewFamily.GetSceneTextures().SubstrateSceneData;

	// Reset Substrate scene data
	{
		const uint32 MinBytesPerPixel	= Out.PersistentMaxBytesPerPixel;
		const uint32 MaxClosureCount	= Out.PersistentMaxClosurePerPixel;
		const uint8 UsesTileTypeMask	= Out.UsesTileTypeMask;
		const bool bUsesAnisotropy		= Out.bUsesAnisotropy;
		Out = FSubstrateSceneData();
		Out.PersistentMaxBytesPerPixel	= MinBytesPerPixel;
		Out.PersistentMaxClosurePerPixel= MaxClosureCount;
		Out.UsesTileTypeMask 			= UsesTileTypeMask;
		Out.bUsesAnisotropy				= bUsesAnisotropy;
	}

	auto UpdateMaterialBufferToTiledResolution = [](FIntPoint InBufferSizeXY, FIntPoint& OutMaterialBufferSizeXY)
	{
		// We need to allocate enough for the tiled memory addressing to always work
		OutMaterialBufferSizeXY.X = FMath::DivideAndRoundUp(InBufferSizeXY.X, SUBSTRATE_TILE_SIZE) * SUBSTRATE_TILE_SIZE;
		OutMaterialBufferSizeXY.Y = FMath::DivideAndRoundUp(InBufferSizeXY.Y, SUBSTRATE_TILE_SIZE) * SUBSTRATE_TILE_SIZE;
	};

	// Compute the max byte per pixels required by the views
	bool bNeedsMaterialBuffer = UsesSubstrateMaterialBuffer(ShaderPlatform);
	bool bBlendableGBufferEnabled = Substrate::IsSubstrateBlendableGBufferEnabled(ShaderPlatform);
	bool bNeedsClosureOffsets = false;
	bool bNeedsUAV = false;
	bool bUseDBufferPass = false;
	
	FIntPoint SceneTextureExtent = ViewFamily.SceneTexturesConfig.Extent;
	if (!IsSubstrateEnabled() || !bNeedsMaterialBuffer)
	{
		SceneTextureExtent = FIntPoint(2, 2);
	}

	FIntPoint MaterialBufferSizeXY;
	UpdateMaterialBufferToTiledResolution(FIntPoint(1, 1), MaterialBufferSizeXY);
	if (IsSubstrateEnabled())
	{
		// Analytics for tracking Substrate usage
		static bool bAnalyticsInitialized = false;
		if (!bAnalyticsInitialized)
		{
			RecordSubstrateAnalytics(ShaderPlatform);
			bAnalyticsInitialized = true;
		}

		// Gather views' requirements
		Out.ViewsMaxBytesPerPixel = 0;
		Out.ViewsMaxClosurePerPixel = 0;
		for (const FViewInfo* View : AllViews)
		{
			// Custom render passes may have separate scene textures and Substrate frame scene data.  Only initialise views using the same
			// scene textures as the view family.
			if (View->GetSceneTexturesChecked() == &ViewFamily.GetSceneTextures())
			{
				bNeedsClosureOffsets = bNeedsClosureOffsets || NeedsClosureOffsets(&Scene, *View);
				bNeedsUAV = bNeedsUAV || IsDBufferPassEnabled(View->GetShaderPlatform()) || DoesPlatformSupportNanite(ShaderPlatform, true);
				Out.ViewsMaxBytesPerPixel = FMath::Max(Out.ViewsMaxBytesPerPixel, View->SubstrateViewData.MaxBytesPerPixel);
				Out.ViewsMaxClosurePerPixel = FMath::Max(Out.ViewsMaxClosurePerPixel, View->SubstrateViewData.MaxClosurePerPixel);
				bUseDBufferPass = bUseDBufferPass || IsDBufferPassEnabled(View->GetShaderPlatform());

				// Only use primary views max. byte per pixel as reflection/capture views can bias allocation requirement when using growing-only mode
				if (!View->bIsPlanarReflection && !View->bIsReflectionCapture && !View->bIsSceneCapture)
				{
					Out.PersistentMaxBytesPerPixel = FMath::Max(Out.PersistentMaxBytesPerPixel, View->SubstrateViewData.MaxBytesPerPixel);
					Out.PersistentMaxClosurePerPixel = FMath::Max(Out.PersistentMaxClosurePerPixel, View->SubstrateViewData.MaxClosurePerPixel);
					Out.UsesTileTypeMask |= View->SubstrateViewData.UsesTileTypeMask;
					Out.bUsesAnisotropy |= View->SubstrateViewData.bUsesAnisotropy;
				}
			}
		}

		// Material buffer allocation can use different modes:
		const uint32 PlatformSettingsBytesPerPixel = GetBytePerPixel(ShaderPlatform);
		const uint32 PlatformSettingsClosurePerPixel = GetClosurePerPixel(ShaderPlatform, GetCurrentMaterialQualityLevelChecked());
		uint32 CurrentMaxBytesPerPixel = 0;
		uint32 CurrentMaxClosurePerPixel = 0;
		switch (GetMaterialBufferAllocationMode())
		{
			// Allocate material buffer based on view requirement,
			case 0:
			{
				CurrentMaxBytesPerPixel = Out.ViewsMaxBytesPerPixel; 
				CurrentMaxClosurePerPixel  = Out.ViewsMaxClosurePerPixel;
			}
			break;
			// Allocate material buffer based on view requirement, but can only grow over frame to minimize buffer reallocation and hitches,
			case 1:
			{
				CurrentMaxBytesPerPixel = FMath::Max(Out.ViewsMaxBytesPerPixel, Out.PersistentMaxBytesPerPixel); 
				CurrentMaxClosurePerPixel  = FMath::Max(Out.ViewsMaxClosurePerPixel,  Out.PersistentMaxClosurePerPixel);
			}
			break;
			// Allocate material buffer based on platform settings.
			case 2:
			{
				CurrentMaxBytesPerPixel = PlatformSettingsBytesPerPixel; 
				CurrentMaxClosurePerPixel  = PlatformSettingsClosurePerPixel;
			}
			break;
		}

		// If this happens, it means there is probably a shader compilation mismatch issue (the compiler has not correctly accounted for the byte per pixel limitation for the platform).
		check(CurrentMaxBytesPerPixel <= PlatformSettingsBytesPerPixel);
		check(CurrentMaxClosurePerPixel <= PlatformSettingsClosurePerPixel);

		const uint32 RoundToValue = 4u;
		CurrentMaxBytesPerPixel = FMath::Clamp(CurrentMaxBytesPerPixel, 4u * SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT, PlatformSettingsBytesPerPixel);
		Out.EffectiveMaxBytesPerPixel = FMath::DivideAndRoundUp(CurrentMaxBytesPerPixel, RoundToValue) * RoundToValue;
		Out.EffectiveMaxClosurePerPixel = CurrentMaxClosurePerPixel;

		
		// We need to allocate enough for the tiled memory addressing of material data to always work
		UpdateMaterialBufferToTiledResolution(SceneTextureExtent, MaterialBufferSizeXY);

		// Top layer texture
		if (bNeedsMaterialBuffer && !bBlendableGBufferEnabled)
		{
			Out.TopLayerTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(SceneTextureExtent, GetTopLayerTextureFormat(bUseDBufferPass), FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_FastVRAM | (bNeedsUAV ? TexCreate_UAV : TexCreate_None)), TEXT("Substrate.TopLayerTexture"));
			if (bNeedsUAV)
			{
				Out.TopLayerTextureUAV = GraphBuilder.CreateUAV(Out.TopLayerTexture);
			}
		}

		// Separated subsurface and rough refraction textures
		if (bNeedsMaterialBuffer && !bBlendableGBufferEnabled)
		{
			const bool bIsSubstrateOpaqueMaterialRoughRefractionEnabled = IsOpaqueRoughRefractionEnabled(ShaderPlatform);
			const FIntPoint OpaqueRoughRefractionSceneExtent		 = bIsSubstrateOpaqueMaterialRoughRefractionEnabled ? SceneTextureExtent : FIntPoint(4, 4);
			
			Out.OpaqueRoughRefractionTexture = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(OpaqueRoughRefractionSceneExtent, PF_FloatR11G11B10, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable), TEXT("Substrate.OpaqueRoughRefractionTexture"));
			Out.OpaqueRoughRefractionTextureUAV = GraphBuilder.CreateUAV(Out.OpaqueRoughRefractionTexture);
			
			Out.SeparatedSubSurfaceSceneColor			= GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(OpaqueRoughRefractionSceneExtent, PF_FloatR11G11B10, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable), TEXT("Substrate.SeparatedSubSurfaceSceneColor"));
			Out.SeparatedOpaqueRoughRefractionSceneColor= GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(OpaqueRoughRefractionSceneExtent, PF_FloatR11G11B10, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable), TEXT("Substrate.SeparatedOpaqueRoughRefractionSceneColor"));

			if (bIsSubstrateOpaqueMaterialRoughRefractionEnabled)
			{
				// Fast clears
				AddClearRenderTargetPass(GraphBuilder, Out.OpaqueRoughRefractionTexture, Out.OpaqueRoughRefractionTexture->Desc.ClearValue.GetClearColor());
				AddClearRenderTargetPass(GraphBuilder, Out.SeparatedSubSurfaceSceneColor, Out.SeparatedSubSurfaceSceneColor->Desc.ClearValue.GetClearColor());
				AddClearRenderTargetPass(GraphBuilder, Out.SeparatedOpaqueRoughRefractionSceneColor, Out.SeparatedOpaqueRoughRefractionSceneColor->Desc.ClearValue.GetClearColor());
			}
		}

		Out.SubstrateSceneToonData0 = nullptr;
		Out.SubstrateSceneToonData1 = nullptr;
		if (IsExperimentalToonUnifiedDiffuseEnabled())
		{
			// Prefer 32bits format but fallback more common format if necessary. However, those format must be HDR.
			EPixelFormat ToonDataFormat0 = GPixelFormats[PF_FloatR11G11B10].Supported	? PF_FloatR11G11B10	: PF_FloatRGBA;
			EPixelFormat ToonDataFormat1 = GPixelFormats[PF_G16R16F].Supported			? PF_G16R16F		: PF_FloatRGBA;

			Out.SubstrateSceneToonData0 = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(SceneTextureExtent, ToonDataFormat0, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f)), TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable), TEXT("Substrate.SubstrateSceneToonData0"));
			AddClearRenderTargetPass(GraphBuilder, Out.SubstrateSceneToonData0, Out.SubstrateSceneToonData0->Desc.ClearValue.GetClearColor());
			Out.SubstrateSceneToonData1 = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(SceneTextureExtent, ToonDataFormat1, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f)), TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable), TEXT("Substrate.SubstrateSceneToonData1"));
			AddClearRenderTargetPass(GraphBuilder, Out.SubstrateSceneToonData1, Out.SubstrateSceneToonData1->Desc.ClearValue.GetClearColor());
		}

		// Closure offsets
		if (bNeedsMaterialBuffer && bNeedsClosureOffsets)
		{
			Out.ClosureOffsetTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(SceneTextureExtent, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource), TEXT("Substrate.ClosureOffsets"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Out.ClosureOffsetTexture), 0u);
		}

		if (NeedsSampledMaterials(&Scene, ViewFamily))
		{
			Out.SampledMaterialTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(SceneTextureExtent, PF_R32G32B32A32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource), TEXT("Substrate.SampledMaterial"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Out.SampledMaterialTexture), 0u); // Needed?
		}
	}
	else
	{
		Out.EffectiveMaxBytesPerPixel = 4u * SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT;
	}

	// Create the material data container
	const uint32 SliceCountSSS = SUBSTRATE_SSS_DATA_UINT_COUNT;
	const uint32 SliceCountAdvDebug = IsAdvancedVisualizationEnabled(ShaderPlatform) ? 1 : 0;
	const uint32 SliceCount = bNeedsMaterialBuffer ? FMath::DivideAndRoundUp(Out.EffectiveMaxBytesPerPixel, 4u) + SliceCountSSS + SliceCountAdvDebug : 1u;
	if (bNeedsMaterialBuffer && !bBlendableGBufferEnabled)
	{
		FRDGTextureDesc MaterialTextureDesc = FRDGTextureDesc::Create2DArray(SceneTextureExtent, PF_R32_UINT, FClearValueBinding::Transparent, TexCreate_TargetArraySlicesIndependently | TexCreate_DisableDCC | TexCreate_NoFastClear | TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV | TexCreate_FastVRAM, SliceCount, 1, 1);
		MaterialTextureDesc.FastVRAMPercentage = (1.0f / SliceCount) * 0xFF; // Only allocate the first slice into ESRAM
		Out.MaterialTextureArray = GraphBuilder.CreateTexture(MaterialTextureDesc, TEXT("Substrate.Material"));
		Out.MaterialTextureArraySRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Out.MaterialTextureArray));
		Out.MaterialTextureArrayUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Out.MaterialTextureArray, 0));

		// See AppendSubstrateMRTs
		check(SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT <= (SliceCount - SliceCountSSS - SliceCountAdvDebug)); // We want enough slice for MRTs but also do not want the SSSData to be a MRT.
		Out.MaterialTextureArrayUAVWithoutRTs = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Out.MaterialTextureArray, 0, PF_Unknown, SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT, SliceCount - SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT));

		if (CVarSubstrateClearMaterialBuffer.GetValueOnRenderThread() > 0)
		{
			for (uint32 SliceIt=0; SliceIt<SliceCount; ++SliceIt)
			{
				AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Out.MaterialTextureArray, 0, PF_R32_UINT, SliceIt, 1)), 0u);
			}
		}
	}

	// Rough diffuse model
	Out.bRoughDiffuse = IsRoughDiffuseEnabled();
	Out.PeelLayersAboveDepth = FMath::Max(CVarSubstrateDebugPeelLayersAboveDepth.GetValueOnRenderThread(), 0);
	Out.bRoughnessTracking = CVarSubstrateDebugRoughnessTracking.GetValueOnRenderThread() > 0 ? 1 : 0;
	Out.bStochasticLighting = IsStochasticLightingActive(ShaderPlatform);

	// Mark stencil with a standalone pass 
	const EStencilPassStage StencilPassStage = GetStencilPassStage();
	Out.bWriteStencil = StencilPassStage == EStencilPassStage::Standalone || StencilPassStage == EStencilPassStage::StandaloneAdaptive;

	if (bNeedsMaterialBuffer)
	{
		// SUBSTRATE_TODO allocate a slice for StoringDebugSubstrate only if SUBSTRATE_ADVANCED_DEBUG_ENABLED is enabled 
		Out.SliceStoringDebugSubstrateTreeData				= SliceCount - SliceCountAdvDebug;										// When we read, there is no slices excluded
		Out.SliceStoringDebugSubstrateTreeDataWithoutMRT	= SliceCount - SliceCountAdvDebug - SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT;	// The UAV skips the first slices set as render target

		Out.FirstSliceStoringSubstrateSSSData				= SliceCount - SliceCountSSS - SliceCountAdvDebug;										// When we read, there is no slices excluded
		Out.FirstSliceStoringSubstrateSSSDataWithoutMRT 	= SliceCount - SliceCountSSS - SliceCountAdvDebug - SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT;	// The UAV skips the first slices set as render target
	}
	else
	{
		Out.SliceStoringDebugSubstrateTreeData 				= -1;
		Out.SliceStoringDebugSubstrateTreeDataWithoutMRT	= -1;
		Out.FirstSliceStoringSubstrateSSSData				= -1;
		Out.FirstSliceStoringSubstrateSSSDataWithoutMRT		= -1;
	}

	// Initialized view data
	for (int32 ViewIndex = 0; ViewIndex < AllViews.Num(); ViewIndex++)
	{
		// Custom render passes may have separate scene textures and Substrate frame scene data.  Only initialise views using the same
		// scene textures as the view family.
		if (AllViews[ViewIndex]->GetSceneTexturesChecked() == &ViewFamily.GetSceneTextures())
		{
			Substrate::InitialiseSubstrateViewData(GraphBuilder, *AllViews[ViewIndex], AllViews[ViewIndex]->GetSceneTexturesConfig(), bNeedsClosureOffsets, bNeedsMaterialBuffer, bBlendableGBufferEnabled, Out);
		}
	}

	if (IsSubstrateEnabled())
	{
		Out.SubstratePublicGlobalUniformParameters = ::Substrate::CreatePublicGlobalUniformBuffer(GraphBuilder, &Out);
	}
}

static FSubstrateCommonParameters GetSubstrateCommonParameter()
{
	FSubstrateCommonParameters Out;
	Out.bRoughDiffuse 		= 0u;
	Out.MaxBytesPerPixel 	= 0u;
	Out.MaxClosurePerPixel	= 0u;
	Out.PeelLayersAboveDepth= 0u;
	Out.bRoughnessTracking 	= 0u;
	Out.bStochasticLighting	= 0u;
	return Out;
}

FSubstrateCommonParameters GetSubstrateCommonParameter(const FSubstrateSceneData& In)
{
	FSubstrateCommonParameters Out;
	Out.bRoughDiffuse 		= In.bRoughDiffuse ? 1u : 0u;
	Out.MaxBytesPerPixel 	= In.EffectiveMaxBytesPerPixel;
	Out.MaxClosurePerPixel 	= In.EffectiveMaxClosurePerPixel;
	Out.PeelLayersAboveDepth= In.PeelLayersAboveDepth;
	Out.bRoughnessTracking 	= In.bRoughnessTracking ? 1u : 0u;
	Out.bStochasticLighting	= In.bStochasticLighting ? 1u : 0u;
	return Out;
}

void BindSubstrateBasePassUniformParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSubstrateBasePassUniformParameters& OutSubstrateUniformParameters)
{
	bool bCreateDummyResources = false;

	const FSubstrateSceneData* SubstrateSceneData = View.SubstrateViewData.SceneData;
	if (IsSubstrateEnabled() && SubstrateSceneData)
	{
		OutSubstrateUniformParameters.Common = GetSubstrateCommonParameter(*SubstrateSceneData);
		if (SubstrateSceneData->MaterialTextureArrayUAVWithoutRTs)
		{
			OutSubstrateUniformParameters.SliceStoringDebugSubstrateTreeDataWithoutMRT = SubstrateSceneData->SliceStoringDebugSubstrateTreeDataWithoutMRT;
			OutSubstrateUniformParameters.FirstSliceStoringSubstrateSSSDataWithoutMRT = SubstrateSceneData->FirstSliceStoringSubstrateSSSDataWithoutMRT;
			OutSubstrateUniformParameters.MaterialTextureArrayUAVWithoutRTs = SubstrateSceneData->MaterialTextureArrayUAVWithoutRTs;
			OutSubstrateUniformParameters.OpaqueRoughRefractionTextureUAV = SubstrateSceneData->OpaqueRoughRefractionTextureUAV;
		}
		else
		{
			bCreateDummyResources = true;
		}
	}
	else
	{
		OutSubstrateUniformParameters.Common = GetSubstrateCommonParameter();
		bCreateDummyResources = true;
	}

	if (bCreateDummyResources)
	{
		FRDGTextureRef DummyWritableRefracTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_R8, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV), TEXT("Substrate.DummyWritableTexture"));
		FRDGTextureUAVRef DummyWritableRefracTextureUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(DummyWritableRefracTexture));

		FRDGTextureRef DummyWritableTextureArray = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2DArray(FIntPoint(1, 1), PF_R32_UINT, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV, 1), TEXT("Substrate.DummyWritableTexture"));
		FRDGTextureUAVRef DummyWritableTextureArrayUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(DummyWritableTextureArray));

		const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
		OutSubstrateUniformParameters.SliceStoringDebugSubstrateTreeDataWithoutMRT = -1;
		OutSubstrateUniformParameters.FirstSliceStoringSubstrateSSSDataWithoutMRT = -1;
		OutSubstrateUniformParameters.MaterialTextureArrayUAVWithoutRTs = DummyWritableTextureArrayUAV;
		OutSubstrateUniformParameters.OpaqueRoughRefractionTextureUAV = DummyWritableRefracTextureUAV;
	}
}

static FRDGTextureRef GetDefaultSubstrateMaterialTextureArray(FRDGBuilder& GraphBuilder)
{
	FRDGTextureRef DefaultSubstrateMaterialTextureArray = GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture2DArray, EPixelFormat::PF_R32_UINT, FClearValueBinding::Transparent);
	return DefaultSubstrateMaterialTextureArray;
}

static void BindSubstrateGlobalUniformParameters(FRDGBuilder& GraphBuilder, const FSubstrateViewData* SubstrateViewData, bool bNeedsMaterialBuffer, bool bBlendableGBufferEnabled, FSubstrateGlobalUniformParameters& OutSubstrateUniformParameters)
{
	FSubstrateSceneData* SubstrateSceneData = SubstrateViewData->SceneData;
	if (IsSubstrateEnabled() && SubstrateSceneData)
	{
		OutSubstrateUniformParameters.Common = GetSubstrateCommonParameter(*SubstrateSceneData);
		OutSubstrateUniformParameters.SliceStoringDebugSubstrateTreeData = SubstrateSceneData->SliceStoringDebugSubstrateTreeData;
		OutSubstrateUniformParameters.FirstSliceStoringSubstrateSSSData = SubstrateSceneData->FirstSliceStoringSubstrateSSSData;
		OutSubstrateUniformParameters.TileSize = SUBSTRATE_TILE_SIZE;
		OutSubstrateUniformParameters.TileSizeLog2 = SUBSTRATE_TILE_SIZE_DIV_AS_SHIFT;
		OutSubstrateUniformParameters.TileCount = SubstrateViewData->TileCount;
		OutSubstrateUniformParameters.MaterialTextureArray = SubstrateSceneData->MaterialTextureArray;
		OutSubstrateUniformParameters.TopLayerTexture = SubstrateSceneData->TopLayerTexture;
		OutSubstrateUniformParameters.OpaqueRoughRefractionTexture = SubstrateSceneData->OpaqueRoughRefractionTexture;
		OutSubstrateUniformParameters.ClosureOffsetTexture = SubstrateSceneData->ClosureOffsetTexture;
		OutSubstrateUniformParameters.ClosureTileCountBuffer = SubstrateViewData->ClosureTileCountBuffer ? GraphBuilder.CreateSRV(SubstrateViewData->ClosureTileCountBuffer, PF_R32_UINT) : nullptr;
		OutSubstrateUniformParameters.ClosureTileBuffer = SubstrateViewData->ClosureTileBuffer ? GraphBuilder.CreateSRV(SubstrateViewData->ClosureTileBuffer, PF_R32_UINT) : nullptr;
		OutSubstrateUniformParameters.SampledMaterialTexture = SubstrateSceneData->SampledMaterialTexture;

		if (OutSubstrateUniformParameters.ClosureOffsetTexture == nullptr)
		{
			const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
			FRDGBufferSRVRef DefaultBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 4, 0u), PF_R32_UINT);
			OutSubstrateUniformParameters.ClosureOffsetTexture = GSystemTextures.GetZeroUIntDummy(GraphBuilder);
			OutSubstrateUniformParameters.ClosureTileCountBuffer = DefaultBuffer;
			OutSubstrateUniformParameters.ClosureTileBuffer = DefaultBuffer;
		}

		if (!bNeedsMaterialBuffer || bBlendableGBufferEnabled)
		{
			check(OutSubstrateUniformParameters.MaterialTextureArray == nullptr);
			check(SubstrateSceneData->TopLayerTexture == nullptr);
			check(SubstrateSceneData->OpaqueRoughRefractionTexture == nullptr);
			const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
			FRDGTextureRef DefaultTextureArray = GetDefaultSubstrateMaterialTextureArray(GraphBuilder);
			OutSubstrateUniformParameters.MaterialTextureArray = DefaultTextureArray;
			OutSubstrateUniformParameters.TopLayerTexture = SystemTextures.DefaultNormal8Bit;
			OutSubstrateUniformParameters.OpaqueRoughRefractionTexture = SystemTextures.Black;
		}

		if (SubstrateSceneData->SampledMaterialTexture == nullptr)
		{
			const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
			OutSubstrateUniformParameters.SampledMaterialTexture = SystemTextures.Black;
		}
	}
	else
	{
		const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
		FRDGTextureRef DefaultTextureArray = GetDefaultSubstrateMaterialTextureArray(GraphBuilder);
		FRDGBufferSRVRef DefaultBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 4, 0u), PF_R32_UINT);
		OutSubstrateUniformParameters.Common = GetSubstrateCommonParameter();
		OutSubstrateUniformParameters.SliceStoringDebugSubstrateTreeData = -1;
		OutSubstrateUniformParameters.FirstSliceStoringSubstrateSSSData = -1;
		OutSubstrateUniformParameters.TileSize = 0;
		OutSubstrateUniformParameters.TileSizeLog2 = 0;
		OutSubstrateUniformParameters.TileCount = 0;
		OutSubstrateUniformParameters.MaterialTextureArray = DefaultTextureArray;
		OutSubstrateUniformParameters.TopLayerTexture = SystemTextures.DefaultNormal8Bit;
		OutSubstrateUniformParameters.OpaqueRoughRefractionTexture = SystemTextures.Black;
		OutSubstrateUniformParameters.ClosureOffsetTexture = GSystemTextures.GetZeroUIntDummy(GraphBuilder);
		OutSubstrateUniformParameters.ClosureTileCountBuffer = DefaultBuffer;
		OutSubstrateUniformParameters.ClosureTileBuffer = DefaultBuffer;
		OutSubstrateUniformParameters.SampledMaterialTexture = SystemTextures.Black;
	}
}

void BindSubstrateForwardPasslUniformParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSubstrateForwardPassUniformParameters& OutSubstrateUniformParameters)
{
	FSubstrateSceneData* SubstrateSceneData = View.SubstrateViewData.SceneData;
	bool bCreateDummyResources = false;
	if (IsSubstrateEnabled() && SubstrateSceneData)
	{
		OutSubstrateUniformParameters.Common = GetSubstrateCommonParameter(*SubstrateSceneData);
		if (SubstrateSceneData->MaterialTextureArray)
		{
			OutSubstrateUniformParameters.FirstSliceStoringSubstrateSSSData = SubstrateSceneData->FirstSliceStoringSubstrateSSSData;
			OutSubstrateUniformParameters.MaterialTextureArray = SubstrateSceneData->MaterialTextureArray;
			OutSubstrateUniformParameters.TopLayerTexture = SubstrateSceneData->TopLayerTexture;
		}
		else
		{
			bCreateDummyResources = true;
		}
	}
	else
	{
		OutSubstrateUniformParameters.Common = GetSubstrateCommonParameter();
		bCreateDummyResources = true;
	}

	if (bCreateDummyResources)
	{
		const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
		OutSubstrateUniformParameters.FirstSliceStoringSubstrateSSSData = -1;
		OutSubstrateUniformParameters.MaterialTextureArray = GetDefaultSubstrateMaterialTextureArray(GraphBuilder);
		OutSubstrateUniformParameters.TopLayerTexture = SystemTextures.DefaultNormal8Bit;
	}
}

void BindSubstrateMobileForwardPasslUniformParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSubstrateMobileForwardPassUniformParameters& OutSubstrateUniformParameters)
{
	FSubstrateSceneData* SubstrateSceneData = View.SubstrateViewData.SceneData;
	if (IsSubstrateEnabled() && SubstrateSceneData)
	{
		OutSubstrateUniformParameters.Common = GetSubstrateCommonParameter(*SubstrateSceneData);
	}
	else
	{
		OutSubstrateUniformParameters.Common = GetSubstrateCommonParameter();
	}
}

TRDGUniformBufferRef<FSubstrateGlobalUniformParameters> BindSubstrateGlobalUniformParameters(const FViewInfo& View)
{
	check(View.SubstrateViewData.SubstrateGlobalUniformParameters != nullptr || !IsSubstrateEnabled());
	return View.SubstrateViewData.SubstrateGlobalUniformParameters;
}

static ERHIFeatureSupport SubstrateSupportsWaveOps(EShaderPlatform Platform)
{
	// D3D11 / SM5 or preview do not support, or work well with, wave-ops by default (or SM5 preview has issues with wave intrinsics too), that fixes classification and black/wrong tiling.
	if (Platform == SP_PCD3D_SM5 || FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(Platform))
	{
		return ERHIFeatureSupport::Unsupported;
	}

	return FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(Platform);
}

void BindSubstratePublicGlobalUniformParameters(FRDGBuilder& GraphBuilder, const FSubstrateSceneData* SubstrateSceneData, FSubstratePublicParameters& OutSubstrateParameters)
{
	if (SubstrateSceneData && SubstrateSceneData->TopLayerTexture)
	{
		OutSubstrateParameters.Common = GetSubstrateCommonParameter(*SubstrateSceneData);
		OutSubstrateParameters.FirstSliceStoringSubstrateSSSData = SubstrateSceneData->FirstSliceStoringSubstrateSSSData;
		OutSubstrateParameters.MaterialTextureArray = SubstrateSceneData->MaterialTextureArray;
		OutSubstrateParameters.TopLayerTexture = SubstrateSceneData->TopLayerTexture;
	}
	else
	{
		const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
		OutSubstrateParameters.Common = GetSubstrateCommonParameter();
		OutSubstrateParameters.FirstSliceStoringSubstrateSSSData = -1;
		OutSubstrateParameters.MaterialTextureArray = GetDefaultSubstrateMaterialTextureArray(GraphBuilder);
		OutSubstrateParameters.TopLayerTexture = SystemTextures.Black;
	}
}

TRDGUniformBufferRef<FSubstratePublicGlobalUniformParameters> CreatePublicGlobalUniformBuffer(FRDGBuilder& GraphBuilder, FSubstrateSceneData* SubstrateScene)
{
	FSubstratePublicGlobalUniformParameters* SubstratePublicUniformParameters = GraphBuilder.AllocParameters<FSubstratePublicGlobalUniformParameters>();
	check(SubstratePublicUniformParameters);
	BindSubstratePublicGlobalUniformParameters(GraphBuilder, SubstrateScene, SubstratePublicUniformParameters->Public);
	return GraphBuilder.CreateUniformBuffer(SubstratePublicUniformParameters);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static bool DoesRuntimeSupportWave64()
{
	return GRHISupportsWaveOperations && (GRHIMinimumWaveSize <= 64 && GRHIMaximumWaveSize >= 64);
}

class FSubstrateClosureTilePassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubstrateClosureTilePassCS);
	SHADER_USE_PARAMETER_STRUCT(FSubstrateClosureTilePassCS, FGlobalShader);

	class FWaveOps : SHADER_PERMUTATION_BOOL("PERMUTATION_WAVE_OPS");
	using FPermutationDomain = TShaderPermutationDomain<FWaveOps>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(int32, bRectPrimitive)
		SHADER_PARAMETER(int32, TileSizeLog2)
		SHADER_PARAMETER(FIntPoint, TileCount_Primary)
		SHADER_PARAMETER(FIntPoint, ViewResolution)
		SHADER_PARAMETER(uint32, MaxBytesPerPixel)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<SUBSTRATE_TOP_LAYER_TYPE>, TopLayerTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<uint>, MaterialTextureArray)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWClosureOffsetTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWClosureTileCountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWClosureTileBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileListBuffer)
		SHADER_PARAMETER(uint32, TileListBufferOffset)
		SHADER_PARAMETER(uint32, TileEncoding)
		RDG_BUFFER_ACCESS(TileIndirectBuffer, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (Substrate::IsSubstrateBlendableGBufferEnabled(Parameters.Platform))
		{
			return false;
		}

		const bool bUseWaveIntrinsics = SubstrateSupportsWaveOps(Parameters.Platform) != ERHIFeatureSupport::Unsupported;
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FWaveOps>() && !bUseWaveIntrinsics)
		{
			return false;
		}
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5 && Substrate::IsSubstrateEnabled();
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		bool bUsed = ShouldCompilePermutation(Parameters);
		if (bUsed)
		{
			FPermutationDomain PermutationVector(Parameters.PermutationId);
			if (PermutationVector.Get<FWaveOps>() && !DoesRuntimeSupportWave64())
			{
				bUsed = false;
			}
		}
		return bUsed ? EShaderPermutationPrecacheRequest::Precached : EShaderPermutationPrecacheRequest::NotUsed;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CLOSURE_TILE"), 1);

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FWaveOps>())
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_WaveOperations);
		}
	}
};
IMPLEMENT_GLOBAL_SHADER(FSubstrateClosureTilePassCS, "/Engine/Private/Substrate/SubstrateMaterialClassification.usf", "ClosureTileMainCS", SF_Compute);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FSubstrateMaterialTileClassificationPassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubstrateMaterialTileClassificationPassCS);
	SHADER_USE_PARAMETER_STRUCT(FSubstrateMaterialTileClassificationPassCS, FGlobalShader);

	class FCmask : SHADER_PERMUTATION_BOOL("PERMUTATION_CMASK");
	class FWaveOps : SHADER_PERMUTATION_BOOL("PERMUTATION_WAVE_OPS");
	class FDecal : SHADER_PERMUTATION_BOOL("PERMUTATION_DECAL");
	class FClassificationTileSize16 : SHADER_PERMUTATION_BOOL("SUBSTRATE_CLASSIFICATION_TILE_SIZE_16");
	using FPermutationDomain = TShaderPermutationDomain<FCmask, FWaveOps, FDecal, FClassificationTileSize16>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(int32, FirstSliceStoringSubstrateSSSData)
		SHADER_PARAMETER(int32, bRectPrimitive)
		SHADER_PARAMETER(FIntPoint, ViewResolution)
		SHADER_PARAMETER(uint32, MaxBytesPerPixel)
		SHADER_PARAMETER(uint32, TileEncoding)
		SHADER_PARAMETER_ARRAY(FUintVector4, TileListBufferOffsets, [SUBSTRATE_TILE_TYPE_COUNT])
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<SUBSTRATE_TOP_LAYER_TYPE>, TopLayerTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, TopLayerCmaskTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileDrawIndirectDataBufferUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileListBufferUAV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, MaterialTextureArrayUAV)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, OpaqueRoughRefractionTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDBufferParameters, DBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, SceneStencilTexture)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const bool bUseWaveIntrinsics = SubstrateSupportsWaveOps(Parameters.Platform) != ERHIFeatureSupport::Unsupported;
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FWaveOps>() && !bUseWaveIntrinsics)
		{
			return false;
		}
		if (PermutationVector.Get<FDecal>() && !IsConsolePlatform(Parameters.Platform))
		{
			return false;
		}		
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5 && Substrate::IsSubstrateEnabled();
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return ShouldCompilePermutation(Parameters) ? EShaderPermutationPrecacheRequest::Precached : EShaderPermutationPrecacheRequest::NotUsed;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TILE_CATEGORIZATION"), 1);

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FWaveOps>())
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_WaveOperations);
		}
	}
};
IMPLEMENT_GLOBAL_SHADER(FSubstrateMaterialTileClassificationPassCS, "/Engine/Private/Substrate/SubstrateMaterialClassification.usf", "TileMainCS", SF_Compute);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FSubstrateDBufferPassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubstrateDBufferPassCS);
	SHADER_USE_PARAMETER_STRUCT(FSubstrateDBufferPassCS, FGlobalShader);

	class FTileType : SHADER_PERMUTATION_INT("PERMUTATION_TILETYPE", 3);
	using FPermutationDomain = TShaderPermutationDomain<FTileType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, ViewResolution)
		SHADER_PARAMETER(uint32, MaxBytesPerPixel)
		SHADER_PARAMETER(uint32, FirstSliceStoringSubstrateSSSData)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDBufferParameters, DBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<SUBSTRATE_TOP_LAYER_TYPE>, TopLayerTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, MaterialTextureArrayUAV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileListBuffer)
		SHADER_PARAMETER(uint32, TileListBufferOffset)
		SHADER_PARAMETER(uint32, TileEncoding)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, SceneStencilTexture)
		RDG_BUFFER_ACCESS(TileIndirectBuffer, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5 && Substrate::IsSubstrateEnabled() && IsUsingDBuffers(Parameters.Platform) && !Substrate::IsSubstrateBlendableGBufferEnabled(Parameters.Platform);
	}

	static void SetDBufferCompilationEnvironment(FShaderCompilerEnvironment& OutEnvironment)
	{
		const uint32 SubstrateStencilDbufferMask 
			= GET_STENCIL_BIT_MASK(SUBSTRATE_RECEIVE_DBUFFER_NORMAL, 1) 
			| GET_STENCIL_BIT_MASK(SUBSTRATE_RECEIVE_DBUFFER_DIFFUSE, 1) 
			| GET_STENCIL_BIT_MASK(SUBSTRATE_RECEIVE_DBUFFER_ROUGHNESS, 1);

		OutEnvironment.SetDefine(TEXT("SUBSTRATE_STENCIL_DBUFFER_MASK"), SubstrateStencilDbufferMask);
		OutEnvironment.SetDefine(TEXT("STENCIL_SUBSTRATE_RECEIVE_DBUFFER_NORMAL_BIT_ID"), STENCIL_SUBSTRATE_RECEIVE_DBUFFER_NORMAL_BIT_ID);
		OutEnvironment.SetDefine(TEXT("STENCIL_SUBSTRATE_RECEIVE_DBUFFER_DIFFUSE_BIT_ID"), STENCIL_SUBSTRATE_RECEIVE_DBUFFER_DIFFUSE_BIT_ID);
		OutEnvironment.SetDefine(TEXT("STENCIL_SUBSTRATE_RECEIVE_DBUFFER_ROUGHNESS_BIT_ID"), STENCIL_SUBSTRATE_RECEIVE_DBUFFER_ROUGHNESS_BIT_ID);	
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		SetDBufferCompilationEnvironment(OutEnvironment);

		// Needed as top layer texture can be a uint2
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};
IMPLEMENT_GLOBAL_SHADER(FSubstrateDBufferPassCS, "/Engine/Private/Substrate/SubstrateDBuffer.usf", "DBufferPassCS", SF_Compute);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FSubstrateDBufferPassPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubstrateDBufferPassPS);
	SHADER_USE_PARAMETER_STRUCT(FSubstrateDBufferPassPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, ViewResolution)
		SHADER_PARAMETER(uint32, MaxBytesPerPixel)
		SHADER_PARAMETER(uint32, FirstSliceStoringSubstrateSSSData)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDBufferParameters, DBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, SceneStencilTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5 && Substrate::IsSubstrateEnabled() && IsUsingDBuffers(Parameters.Platform) && Substrate::IsSubstrateBlendableGBufferEnabled(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FSubstrateDBufferPassCS::SetDBufferCompilationEnvironment(OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FSubstrateDBufferPassPS, "/Engine/Private/Substrate/SubstrateDBuffer.usf", "DBufferPassPS", SF_Pixel);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FSubstrateMaterialTilePrepareArgsPassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubstrateMaterialTilePrepareArgsPassCS);
	SHADER_USE_PARAMETER_STRUCT(FSubstrateMaterialTilePrepareArgsPassCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer,   TileDrawIndirectDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileDispatchIndirectDataBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5 && Substrate::IsSubstrateEnabled();
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_MATERIAL_TILE_PREPARE_ARGS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FSubstrateMaterialTilePrepareArgsPassCS, "/Engine/Private/Substrate/SubstrateMaterialClassification.usf", "ArgsMainCS", SF_Compute);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FSubstrateClosureTilePrepareArgsPassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubstrateClosureTilePrepareArgsPassCS);
	SHADER_USE_PARAMETER_STRUCT(FSubstrateClosureTilePrepareArgsPassCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, TileCount_Primary)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, TileDrawIndirectDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileDispatchIndirectDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileDispatchPerThreadIndirectDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileRaytracingIndirectDataBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5 && Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CLOSURE_TILE_PREPARE_ARGS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FSubstrateClosureTilePrepareArgsPassCS, "/Engine/Private/Substrate/SubstrateMaterialClassification.usf", "ArgsMainCS", SF_Compute);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool FSubstrateTilePassVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5; // We do not skip the compilation because we have some conditional when tiling a pass and the shader must be fetch once before hand.
}

void FSubstrateTilePassVS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("SHADER_TILE_VS"), 1);
}

class FSubstrateMaterialStencilTaggingPassPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubstrateMaterialStencilTaggingPassPS);
	SHADER_USE_PARAMETER_STRUCT(FSubstrateMaterialStencilTaggingPassPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(Substrate::FSubstrateTilePassVS::FParameters, VS)
		SHADER_PARAMETER(FVector4f, DebugTileColor)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5 && Substrate::IsSubstrateEnabled();
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_STENCIL_TAGGING_PS"), 1); 
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubstrateTilePassVS, "/Engine/Private/Substrate/SubstrateTile.usf", "SubstrateTilePassVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FSubstrateMaterialStencilTaggingPassPS, "/Engine/Private/Substrate/SubstrateTile.usf", "StencilTaggingMainPS", SF_Pixel);

static FSubstrateTileParameter InternalSetTileParameters(FRDGBuilder* GraphBuilder, const FViewInfo& View, const ESubstrateTileType TileType)
{
	FSubstrateTileParameter Out;
	if (TileType != ESubstrateTileType::ECount)
	{
		Out.TileListBuffer = View.SubstrateViewData.ClassificationTileListBufferSRV;
		Out.TileListBufferOffset = View.SubstrateViewData.ClassificationTileListBufferOffset[TileType];
		Out.TileEncoding = View.SubstrateViewData.TileEncoding;
		Out.TileIndirectBuffer = View.SubstrateViewData.ClassificationTileDrawIndirectBuffer;
	}
	else if (GraphBuilder)
	{
		FRDGBufferRef BufferDummy = GSystemTextures.GetDefaultBuffer(*GraphBuilder, 4, 0u);
		FRDGBufferSRVRef BufferDummySRV = GraphBuilder->CreateSRV(BufferDummy, PF_R32_UINT);
		Out.TileListBuffer = BufferDummySRV;
		Out.TileListBufferOffset = 0;
		Out.TileEncoding = SUBSTRATE_TILE_ENCODING_16BITS;
		Out.TileIndirectBuffer = BufferDummy;
	}
	return Out;
}

static FSubstrateTilePassVS::FParameters SetTileParametersCommon(FRDGBuilder* GraphBuilder,
		const FViewInfo& View, 
		const ESubstrateTileType TileType, 
		EPrimitiveType& PrimitiveType)
{
	FSubstrateTileParameter Temp = InternalSetTileParameters(GraphBuilder, View, TileType);
	PrimitiveType = GRHISupportsRectTopology ? PT_RectList : PT_TriangleList;

	FSubstrateTilePassVS::FParameters Out;
	Out.View = View.ViewUniformBuffer;
	Out.TileListBuffer = Temp.TileListBuffer;
	Out.TileListBufferOffset = Temp.TileListBufferOffset;
	Out.TileEncoding = Temp.TileEncoding;
	Out.TileIndirectBuffer = Temp.TileIndirectBuffer;
	return Out;
}


FSubstrateTilePassVS::FParameters SetTileParameters(
	const FViewInfo& View,
	const ESubstrateTileType TileType,
	EPrimitiveType& PrimitiveType)
{
	return SetTileParametersCommon(nullptr, View, TileType, PrimitiveType);
}

FSubstrateTilePassVS::FParameters SetTileParameters(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View, 
	const ESubstrateTileType TileType,
	EPrimitiveType& PrimitiveType)
{
	return SetTileParametersCommon(&GraphBuilder, View, TileType, PrimitiveType);
}

FSubstrateTileParameter SetTileParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, const ESubstrateTileType TileType)
{
	return InternalSetTileParameters(&GraphBuilder, View, TileType);
}

uint32 TileTypeDrawIndirectArgOffset(const ESubstrateTileType Type)
{
	check(Type >= 0 && Type < ESubstrateTileType::ECount);
	return GetSubstrateTileTypeDrawIndirectArgOffset_Byte(Type);
}

uint32 TileTypeDispatchIndirectArgOffset(const ESubstrateTileType Type)
{
	check(Type >= 0 && Type < ESubstrateTileType::ECount);
	return GetSubstrateTileTypeDispatchIndirectArgOffset_Byte(Type);
}

void AddSubstrateInternalClassificationTilePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureRef* DepthTexture,
	const FRDGTextureRef* ColorTexture,
	ESubstrateTileType TileMaterialType,
	const bool bDebug = false)
{
	EPrimitiveType SubstrateTilePrimitiveType = PT_TriangleList;
	FIntPoint DebugOutputResolution = FIntPoint(View.UnscaledViewRect.Width(), View.UnscaledViewRect.Height());
	const FIntRect ViewRect = View.ViewRect;

	FSubstrateMaterialStencilTaggingPassPS::FParameters* ParametersPS = GraphBuilder.AllocParameters<FSubstrateMaterialStencilTaggingPassPS::FParameters>();
	ParametersPS->VS = Substrate::SetTileParameters(GraphBuilder, View, TileMaterialType, SubstrateTilePrimitiveType);

	FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
	VSPermutationVector.Set< FSubstrateTilePassVS::FEnableDebug >(bDebug);
	VSPermutationVector.Set< FSubstrateTilePassVS::FEnableTexCoordScreenVector >(false);
	VSPermutationVector.Set< FSubstrateTilePassVS::FClassificationTileSize16 >(Substrate::UseTileSize16());
	TShaderMapRef<FSubstrateTilePassVS> VertexShader(View.ShaderMap, VSPermutationVector);
	TShaderMapRef<FSubstrateMaterialStencilTaggingPassPS> PixelShader(View.ShaderMap);

	// For debug purpose
	if (bDebug)
	{
		// ViewRect contains the scaled resolution according to TSR screen percentage.
		// The ColorTexture can be larger than the screen resolution if the screen percentage has be manipulated to be >100%.
		// So we simply re-use the previously computed ViewResolutionFraction to recover the targeted resolution in the editor.
		// TODO fix this for split screen.
		const float InvViewResolutionFraction = View.Family->bRealtimeUpdate ? 1.0f / View.CachedViewUniformShaderParameters->ViewResolutionFraction : 1.0f;
		DebugOutputResolution = FIntPoint(float(ViewRect.Width()) * InvViewResolutionFraction, float(ViewRect.Height()) * InvViewResolutionFraction);

		check(ColorTexture);
		ParametersPS->RenderTargets[0] = FRenderTargetBinding(*ColorTexture, ERenderTargetLoadAction::ELoad);
		switch (TileMaterialType)
		{
		case ESubstrateTileType::ESimple:							ParametersPS->DebugTileColor = FVector4f(0.0f, 1.0f, 0.0f, 1.0); break;
		case ESubstrateTileType::ESingle:							ParametersPS->DebugTileColor = FVector4f(1.0f, 1.0f, 0.0f, 1.0); break;
		case ESubstrateTileType::EComplex:							ParametersPS->DebugTileColor = FVector4f(1.0f, 0.0f, 0.0f, 1.0); break;
		case ESubstrateTileType::EComplexSpecial:					ParametersPS->DebugTileColor = FVector4f(0.3f, 0.0f, 0.3f, 1.0); break;

		case ESubstrateTileType::EOpaqueRoughRefraction:			ParametersPS->DebugTileColor = FVector4f(0.0f, 1.0f, 1.0f, 1.0); break;
		case ESubstrateTileType::EOpaqueRoughRefractionSSSWithout:	ParametersPS->DebugTileColor = FVector4f(0.0f, 0.0f, 1.0f, 1.0); break;

		case ESubstrateTileType::EDecalSingle:						ParametersPS->DebugTileColor = FVector4f(0.0f, 1.0f, 0.0f, 1.0); break;
		case ESubstrateTileType::EDecalSimple:						ParametersPS->DebugTileColor = FVector4f(1.0f, 1.0f, 0.0f, 1.0); break;
		case ESubstrateTileType::EDecalComplex:						ParametersPS->DebugTileColor = FVector4f(1.0f, 0.0f, 0.0f, 1.0); break;
		default: check(false);
		}
	}
	else
	{
		check(DepthTexture);
		ParametersPS->RenderTargets.DepthStencil = FDepthStencilBinding(
			*DepthTexture,
			ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad,
			FExclusiveDepthStencil::DepthRead_StencilWrite);
		ParametersPS->DebugTileColor = FVector4f(ForceInitToZero);
	}
	
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Substrate::%sClassificationPass(%s)", bDebug ? TEXT("Debug") : TEXT("Stencil"), ToString(TileMaterialType)),
		ParametersPS,
		ERDGPassFlags::Raster,
		[ParametersPS, VertexShader, PixelShader, ViewRect, DebugOutputResolution, SubstrateTilePrimitiveType, TileMaterialType, bDebug](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			uint32 StencilRef = 0xFF;
			if (bDebug)
			{
				// Use premultiplied alpha blending, pixel shader and depth/stencil is off
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			}
			else
			{
				check(TileMaterialType != ESubstrateTileType::ECount && TileMaterialType != ESubstrateTileType::EOpaqueRoughRefraction && TileMaterialType != ESubstrateTileType::EOpaqueRoughRefractionSSSWithout);

				// No blending and no pixel shader required. Stencil will be written to.
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = nullptr;
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
				switch (TileMaterialType)
				{
				case ESubstrateTileType::ESimple:
				{
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
						false, CF_Always,
						true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
						false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
						0xFF, StencilBit_Fast>::GetRHI();
					StencilRef = StencilBit_Fast;
				}
				break;
				case ESubstrateTileType::ESingle:
				{
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
						false, CF_Always,
						true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
						false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
						0xFF, StencilBit_Single>::GetRHI();
					StencilRef = StencilBit_Single;
				}
				break;
				case ESubstrateTileType::EComplex:
				{
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
						false, CF_Always,
						true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
						false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
						0xFF, StencilBit_Complex>::GetRHI();
					StencilRef = StencilBit_Complex;
				}
				break;
				case ESubstrateTileType::EComplexSpecial:
				{
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
						false, CF_Always,
						true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
						false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
						0xFF, StencilBit_ComplexSpecial>::GetRHI();
					StencilRef = StencilBit_ComplexSpecial;
				}
				break;
				}
			}
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.PrimitiveType = SubstrateTilePrimitiveType;
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersPS->VS);
			if (bDebug)
			{
				// Debug rendering is aways done during the post-processing stage, which has an ViewMinRect set to (0,0)
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *ParametersPS);
				RHICmdList.SetViewport(0, 0, 0.0f, DebugOutputResolution.X, DebugOutputResolution.Y, 1.0f);
			}
			else
			{
				RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);
			}
			RHICmdList.SetStreamSource(0, nullptr, 0);
			RHICmdList.DrawPrimitiveIndirect(ParametersPS->VS.TileIndirectBuffer->GetIndirectRHICallBuffer(), TileTypeDrawIndirectArgOffset(TileMaterialType));
		});
}

void AddSubstrateStencilPass(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const TArray<FViewInfo>& Views,
	const FMinimalSceneTextures& SceneTextures,
	FSubstrateSceneData& SubstrateSceneData)
{
	if (!SubstrateSceneData.bWriteStencil)
	{
		return;
	}

	// Mark Stencil as written
	SubstrateSceneData.bWriteStencil = false;

	for (int32 i = 0; i < Views.Num(); ++i)
	{
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", i);

		const FViewInfo& View = Views[i];
		if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplexSpecial))
		{
			AddSubstrateInternalClassificationTilePass(GraphBuilder, View, &SceneTextures.Depth.Target, nullptr, ESubstrateTileType::EComplexSpecial);
		}
		if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplex))
		{
			AddSubstrateInternalClassificationTilePass(GraphBuilder, View, &SceneTextures.Depth.Target, nullptr, ESubstrateTileType::EComplex);
		}
		if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESingle))
		{
			AddSubstrateInternalClassificationTilePass(GraphBuilder, View, &SceneTextures.Depth.Target, nullptr, ESubstrateTileType::ESingle);
		}
		if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESimple))
		{
			AddSubstrateInternalClassificationTilePass(GraphBuilder, View, &SceneTextures.Depth.Target, nullptr, ESubstrateTileType::ESimple);
		}
	}
}

class FSubstrateSampleMaterialPassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubstrateSampleMaterialPassCS);
	SHADER_USE_PARAMETER_STRUCT(FSubstrateSampleMaterialPassCS, FGlobalShader);

	class FWaveOps : SHADER_PERMUTATION_BOOL("PERMUTATION_WAVE_OPS");
	using FPermutationDomain = TShaderPermutationDomain<FWaveOps>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, MegaLightsStateFrameIndex)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateCommonParameters, Substrate)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<SUBSTRATE_TOP_LAYER_TYPE>, TopLayerTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<uint>, MaterialTextureArray)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, ClosureOffsetTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint4>, RWMaterialData)
	END_SHADER_PARAMETER_STRUCT()

	static uint32 GetGroupSize()
	{
		return SUBSTRATE_TILE_SIZE;
	}	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5 && Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_SAMPLE_MATERIAL"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FSubstrateSampleMaterialPassCS, "/Engine/Private/Substrate/SubstrateMaterialSampling.usf", "SubstrateMaterialSamplingCS", SF_Compute);

static void AddSubstrateInternalSampleMaterialPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMinimalSceneTextures& SceneTextures, const FSubstrateSceneData& SubstrateSceneData, FRDGTextureUAVRef Out)
{
	FRDGTextureRef MaterialData = nullptr;
	FSubstrateSampleMaterialPassCS::FPermutationDomain PermutationVector;
	TShaderMapRef<FSubstrateSampleMaterialPassCS> ComputeShader(View.ShaderMap, PermutationVector);
	FSubstrateSampleMaterialPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubstrateSampleMaterialPassCS::FParameters>();

	FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
	TUniformBufferRef<FBlueNoise> BlueNoiseUniformBuffer = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

	PassParameters->SceneTexturesStruct = SceneTextures.UniformBuffer;
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->Substrate = GetSubstrateCommonParameter(SubstrateSceneData);
	PassParameters->ClosureOffsetTexture = SubstrateSceneData.ClosureOffsetTexture;
	PassParameters->TopLayerTexture = SubstrateSceneData.TopLayerTexture;
	PassParameters->MaterialTextureArray = SubstrateSceneData.MaterialTextureArraySRV;
	PassParameters->RWMaterialData = Out;
	PassParameters->MegaLightsStateFrameIndex = MegaLights::GetStateFrameIndex(View.ViewState, View.GetShaderPlatform());
	PassParameters->BlueNoise = BlueNoiseUniformBuffer;
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintUniformBuffer);

	const FIntVector DispatchCount = FIntVector(
		FMath::DivideAndRoundUp(uint32(View.ViewRect.Size().X), FSubstrateSampleMaterialPassCS::GetGroupSize()), 
		FMath::DivideAndRoundUp(uint32(View.ViewRect.Size().Y), FSubstrateSampleMaterialPassCS::GetGroupSize()), 
		1);
	
	// TODO add tiles type
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("Substrate::MaterialSampling"),
		ComputeShader,
		PassParameters, 
		DispatchCount);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void AppendSubstrateMRTs(const FSceneRenderer& SceneRenderer, uint32& RenderTargetCount, TArrayView<FTextureRenderTargetBinding> RenderTargets)
{
	const bool bUsesMaterialBuffer = UsesSubstrateMaterialBuffer(SceneRenderer.ShaderPlatform);
	if (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(SceneRenderer.ShaderPlatform) && SceneRenderer.Scene && bUsesMaterialBuffer)
	{
		const FSubstrateSceneData& SubstrateSceneData = *SceneRenderer.ViewFamily.GetSceneTextures().SubstrateSceneData;

		// If this function changes, update Substrate::SetBasePassRenderTargetOutputFormat()
		 
		// Add 2 uint for Substrate fast path. 
		// - We must clear the first uint to 0 to identify pixels that have not been written to.
		// - We must never clear the second uint, it will only be written/read if needed.
		auto AddSubstrateOutputTarget = [&](int16 SubstrateMaterialArraySlice, bool bNeverClear = false)
		{
			RenderTargets[RenderTargetCount] = FTextureRenderTargetBinding(SubstrateSceneData.MaterialTextureArray, SubstrateMaterialArraySlice, bNeverClear);
			RenderTargetCount++;
		};
		const bool bSupportCMask = SupportsCMask(GMaxRHIShaderPlatform);
		for (int i = 0; i < SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT; ++i)
		{
			const bool bNeverClear = bSupportCMask || i != 0; // Only allow clearing the first slice containing the header
			AddSubstrateOutputTarget(i, bNeverClear);
		}

		// Add another MRT for Substrate top layer information. We want to follow the usual clear process which can leverage fast clear.
		{
			RenderTargets[RenderTargetCount] = FTextureRenderTargetBinding(SubstrateSceneData.TopLayerTexture);
			RenderTargetCount++;
		};
	}
}

void SetBasePassRenderTargetOutputFormat(const EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, FShaderCompilerEnvironment& OutEnvironment, EGBufferLayout GBufferLayout)
{
	if (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(Platform))
	{
		FGBufferParams GBufferParams = FShaderCompileUtilities::FetchGBufferParamsRuntime(Platform, GBufferLayout);

		// If it is not a water material, we force bHasSingleLayerWaterSeparatedMainLight to false, in order to 
		// ensure non-used MRTs are not inserted in BufferInfo. Otherwise this would offset Substrate MRTs, causing 
		// MRTs' format to be incorrect
		const bool bIsSingleLayerWater = MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater);
		const bool bNeedsSeparateMainDirLightTexture = IsWaterSeparateMainDirLightEnabled(Platform);
		if (!bIsSingleLayerWater || !bNeedsSeparateMainDirLightTexture)
		{
			GBufferParams.bHasSingleLayerWaterSeparatedMainLight = false;
		}
		const FGBufferInfo BufferInfo = FetchFullGBufferInfo(GBufferParams);

		// Translucent blend mode do not write material data, and thus don't need output format (default to RGBA16f). 
		// Dual source blending requires to have both target format set to RGBA16f
		const bool bIsTranslucent = IsTranslucentBlendMode(MaterialParameters.BlendMode);
		if (!bIsTranslucent)
		{
			// Add N uint for Substrate fast path
			for (int i = 0; i < SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT; ++i)
			{
				OutEnvironment.SetRenderTargetOutputFormat(BufferInfo.NumTargets + i, PF_R32_UINT);
			}
		}

		// Add another MRT for Substrate top layer information
		OutEnvironment.SetRenderTargetOutputFormat(BufferInfo.NumTargets + SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT, GetTopLayerTextureFormat(IsDBufferPassEnabled(Platform)));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void AddSubstrateMaterialClassificationIndirectArgsPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, ERDGPassFlags ComputePassFlags)
{
	const EShaderPlatform Platform = View.GetShaderPlatform();
	check(UsesSubstrateMaterialBuffer(Platform));

	const FSubstrateViewData* SubstrateViewData = &View.SubstrateViewData;

	TShaderMapRef<FSubstrateMaterialTilePrepareArgsPassCS> ComputeShader(View.ShaderMap);
	FSubstrateMaterialTilePrepareArgsPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubstrateMaterialTilePrepareArgsPassCS::FParameters>();
	PassParameters->TileDrawIndirectDataBuffer = GraphBuilder.CreateSRV(SubstrateViewData->ClassificationTileDrawIndirectBuffer, PF_R32_UINT);
	PassParameters->TileDispatchIndirectDataBuffer = SubstrateViewData->ClassificationTileDispatchIndirectBufferUAV;

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("Substrate::MaterialTilePrepareArgs"),
		ComputePassFlags,
		ComputeShader,
		PassParameters,
		FIntVector(1,1,1));
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void AddSubstrateMaterialClassificationPass(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures, const TArray<FViewInfo>& Views)
{
	RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, IsSubstrateEnabled() && Views.Num() > 0, "Substrate::MaterialClassification");
	if (!IsSubstrateEnabled())
	{
		return;
	}

	// Optionally run tile classification in async compute
	const ERDGPassFlags PassFlags = IsClassificationAsync() ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute;

	for (int32 i = 0; i < Views.Num(); ++i)
	{
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", i);

		const FViewInfo& View = Views[i];
		const EShaderPlatform Platform = View.GetShaderPlatform();
		if (!UsesSubstrateMaterialBuffer(Platform))
		{
			continue;
		}

		// Our current classification require 64 waves
		const bool bWaveOps = SubstrateSupportsWaveOps(Platform) != ERHIFeatureSupport::Unsupported;
		const bool bWaveOps64 = DoesRuntimeSupportWave64() && bWaveOps ;
		
		const FSubstrateViewData* SubstrateViewData = &View.SubstrateViewData;
		const FSubstrateSceneData* SubstrateSceneData = View.SubstrateViewData.SceneData;

		// Tile reduction
		{
			// When the platform support explicit CMask texture, we disable material data bufferclear. Material buffer buffer clear (the header part) is done during the classification pass.  
			// To reduce the reading bandwidth, we rely on TopLayerData CMask to 'drive' the clearing process. This allows to clear quickly empty tiles.
			const bool bSupportCMask = SupportsCMask(Platform);
			FRDGTextureRef TopLayerCmaskTexture = SubstrateSceneData->TopLayerTexture;			
			if (bSupportCMask)
			{
				// Combine DBuffer RTWriteMasks; will end up in one texture we can load from in the base pass PS and decide whether to do the actual work or not.
				FRDGTextureRef SourceCMaskTextures[] = { SubstrateSceneData->TopLayerTexture };
				FRenderTargetWriteMask::Decode(GraphBuilder, View.ShaderMap, MakeArrayView(SourceCMaskTextures), TopLayerCmaskTexture, GFastVRamConfig.DBufferMask, TEXT("Substrate::TopLayerCmask"));
			}

			// If Dbuffer pass (i.e. apply DBuffer data after the base-pass) is enabled, run special classification for outputing tile with/without tiles
			const bool bDBufferTiles = IsDBufferPassEnabled(Platform) && CVarSubstrateDBufferPassDedicatedTiles.GetValueOnRenderThread() > 0 && SceneTextures.DBufferTextures.IsValid() && IsConsolePlatform(Platform);

			const bool bUseTileSize16 = Substrate::UseTileSize16();
			FSubstrateMaterialTileClassificationPassCS::FPermutationDomain PermutationVector;
			PermutationVector.Set< FSubstrateMaterialTileClassificationPassCS::FCmask >(bSupportCMask);
			PermutationVector.Set< FSubstrateMaterialTileClassificationPassCS::FWaveOps >(bWaveOps);
			PermutationVector.Set< FSubstrateMaterialTileClassificationPassCS::FDecal>(bDBufferTiles);
			PermutationVector.Set< FSubstrateMaterialTileClassificationPassCS::FClassificationTileSize16 >(bUseTileSize16);
			TShaderMapRef<FSubstrateMaterialTileClassificationPassCS> ComputeShader(View.ShaderMap, PermutationVector);
			FSubstrateMaterialTileClassificationPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubstrateMaterialTileClassificationPassCS::FParameters>();
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->SceneTexturesStruct = SceneTextures.UniformBuffer;
			PassParameters->bRectPrimitive = GRHISupportsRectTopology ? 1 : 0;
			PassParameters->ViewResolution = View.ViewRect.Size();
			PassParameters->MaxBytesPerPixel = SubstrateSceneData->EffectiveMaxBytesPerPixel;
			PassParameters->FirstSliceStoringSubstrateSSSData = SubstrateSceneData->FirstSliceStoringSubstrateSSSData;
			PassParameters->TopLayerTexture = SubstrateSceneData->TopLayerTexture;
			PassParameters->TopLayerCmaskTexture = TopLayerCmaskTexture;
			PassParameters->MaterialTextureArrayUAV = SubstrateSceneData->MaterialTextureArrayUAV;
			PassParameters->OpaqueRoughRefractionTexture = SubstrateSceneData->OpaqueRoughRefractionTexture;
			PassParameters->TileDrawIndirectDataBufferUAV = SubstrateViewData->ClassificationTileDrawIndirectBufferUAV;
			PassParameters->DBuffer = GetDBufferParameters(GraphBuilder, *SceneTextures.DBufferTextures, Platform);
			PassParameters->SceneStencilTexture = SceneTextures.Stencil;
			PassParameters->TileListBufferUAV = SubstrateViewData->ClassificationTileListBufferUAV;
			PassParameters->TileEncoding = SubstrateViewData->TileEncoding;
			for (uint32 TileType = 0; TileType < SUBSTRATE_TILE_TYPE_COUNT; ++TileType)
			{
				PassParameters->TileListBufferOffsets[TileType] = FUintVector4(SubstrateViewData->ClassificationTileListBufferOffset[TileType], 0, 0, 0);
			}

			const uint32 GroupSize = bUseTileSize16 ? 16 : 8;
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("Substrate::MaterialTileClassification(%s%s)", bWaveOps ? TEXT("Wave") : TEXT("SharedMemory"), bSupportCMask ? TEXT(", CMask") : TEXT("")),
				PassFlags,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(PassParameters->ViewResolution, GroupSize));
		}

		// Tile indirect dispatch args conversion
		AddSubstrateMaterialClassificationIndirectArgsPass(GraphBuilder, View, PassFlags);

		// Compute closure tile index and material read offset
		if (SubstrateSceneData->ClosureOffsetTexture)
		{
			FRDGBufferUAVRef RWClosureTileCountBuffer = GraphBuilder.CreateUAV(SubstrateViewData->ClosureTileCountBuffer, PF_R32_UINT);
			AddClearUAVPass(GraphBuilder, RWClosureTileCountBuffer, 0u);

			auto MarkClosureTilePass = [&](ESubstrateTileType TileType)
			{
				FSubstrateClosureTilePassCS::FPermutationDomain PermutationVector;
				PermutationVector.Set< FSubstrateClosureTilePassCS::FWaveOps >(bWaveOps64);
				TShaderMapRef<FSubstrateClosureTilePassCS> ComputeShader(View.ShaderMap, PermutationVector);
				FSubstrateClosureTilePassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubstrateClosureTilePassCS::FParameters>();
				PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
				PassParameters->TileSizeLog2 = SUBSTRATE_TILE_SIZE_DIV_AS_SHIFT;
				PassParameters->TileCount_Primary = SubstrateViewData->TileCount;
				PassParameters->ViewResolution = View.ViewRect.Size();
				PassParameters->MaxBytesPerPixel = SubstrateSceneData->EffectiveMaxBytesPerPixel;
				PassParameters->TopLayerTexture = SubstrateSceneData->TopLayerTexture;
				PassParameters->MaterialTextureArray = SubstrateSceneData->MaterialTextureArraySRV;
				PassParameters->TileListBuffer = SubstrateViewData->ClassificationTileListBufferSRV;
				PassParameters->TileListBufferOffset = SubstrateViewData->ClassificationTileListBufferOffset[TileType];
				PassParameters->TileEncoding = SubstrateViewData->TileEncoding;
				PassParameters->TileIndirectBuffer = SubstrateViewData->ClassificationTileDispatchIndirectBuffer;

				PassParameters->RWClosureOffsetTexture = GraphBuilder.CreateUAV(SubstrateSceneData->ClosureOffsetTexture);
				PassParameters->RWClosureTileCountBuffer = RWClosureTileCountBuffer;
				PassParameters->RWClosureTileBuffer = GraphBuilder.CreateUAV(SubstrateViewData->ClosureTileBuffer, PF_R32_UINT);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("Substrate::ClosureTileAndOffsets(%s - %s)", ToString(TileType), bWaveOps64 ? TEXT("Wave") : TEXT("SharedMemory")),
					PassFlags,
					ComputeShader,
					PassParameters,
					PassParameters->TileIndirectBuffer,
					TileTypeDispatchIndirectArgOffset(TileType));
			};

			if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplexSpecial))
			{
				MarkClosureTilePass(ESubstrateTileType::EComplexSpecial);
			}
			if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplex))
			{
				MarkClosureTilePass(ESubstrateTileType::EComplex);
			}
		}

		// Tile indirect dispatch args conversion
		if (SubstrateSceneData->ClosureOffsetTexture)
		{
			TShaderMapRef<FSubstrateClosureTilePrepareArgsPassCS> ComputeShader(View.ShaderMap);
			FSubstrateClosureTilePrepareArgsPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubstrateClosureTilePrepareArgsPassCS::FParameters>();
			PassParameters->TileCount_Primary = SubstrateViewData->TileCount;
			PassParameters->TileDrawIndirectDataBuffer = GraphBuilder.CreateSRV(SubstrateViewData->ClosureTileCountBuffer, PF_R32_UINT);
			PassParameters->TileDispatchIndirectDataBuffer = GraphBuilder.CreateUAV(SubstrateViewData->ClosureTileDispatchIndirectBuffer, PF_R32_UINT);
			PassParameters->TileDispatchPerThreadIndirectDataBuffer = GraphBuilder.CreateUAV(SubstrateViewData->ClosureTilePerThreadDispatchIndirectBuffer, PF_R32_UINT);
			PassParameters->TileRaytracingIndirectDataBuffer = GraphBuilder.CreateUAV(SubstrateViewData->ClosureTileRaytracingIndirectBuffer, PF_R32_UINT);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("Substrate::ClosureTilePrepareArgs"),
				PassFlags,
				ComputeShader,
				PassParameters,
				FIntVector(1, 1, 1));
		}
	}
}

void AddSubstrateDBufferPass(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures, const TArray<FViewInfo>& Views)
{
	RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, IsSubstrateEnabled() && Views.Num() > 0, "Substrate::DBuffer");
	if (!IsSubstrateEnabled() || !SceneTextures.DBufferTextures.IsValid())
	{
		return;
	}

	for (int32 i = 0; i < Views.Num(); ++i)
	{
		const FViewInfo& View = Views[i];
		const bool bIsDBufferPassEnabled = IsDBufferPassEnabled(View.GetShaderPlatform());

		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1 && bIsDBufferPassEnabled, "View%d", i);

		if (!IsUsingDBuffers(View.GetShaderPlatform()) || View.Family->EngineShowFlags.Decals == 0 || !bIsDBufferPassEnabled)
		{
			continue;
		}

		const FSubstrateViewData* SubstrateViewData = &View.SubstrateViewData;
		const FSubstrateSceneData* SubstrateSceneData = View.SubstrateViewData.SceneData;

		if (IsSubstrateBlendableGBufferEnabled(View.GetShaderPlatform()))
		{
			const FIntRect ViewRect = View.ViewRect;

			FSubstrateDBufferPassPS::FParameters* ParametersPS = GraphBuilder.AllocParameters<FSubstrateDBufferPassPS::FParameters>();
			TShaderMapRef<FSubstrateDBufferPassPS> PixelShader(View.ShaderMap);
			ParametersPS->ViewUniformBuffer = View.ViewUniformBuffer;
			ParametersPS->DBuffer = GetDBufferParameters(GraphBuilder, *SceneTextures.DBufferTextures, View.GetShaderPlatform());
			ParametersPS->SceneStencilTexture = SceneTextures.Stencil;
			ParametersPS->RenderTargets[0] = FRenderTargetBinding(SceneTextures.GBufferA, ERenderTargetLoadAction::ELoad);
			ParametersPS->RenderTargets[1] = FRenderTargetBinding(SceneTextures.GBufferB, ERenderTargetLoadAction::ELoad);
			ParametersPS->RenderTargets[2] = FRenderTargetBinding(SceneTextures.GBufferC, ERenderTargetLoadAction::ELoad);
			ParametersPS->RenderTargets.DepthStencil = FDepthStencilBinding(
				SceneTextures.Depth.Target,
				ERenderTargetLoadAction::ELoad,
				ERenderTargetLoadAction::ELoad,
				FExclusiveDepthStencil::DepthRead_StencilNop);	

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("Substrate::DBufferPass(Blendable)"),
				ParametersPS,
				ERDGPassFlags::Raster,
				[ParametersPS, &View, PixelShader](FRHICommandList& InRHICmdList)
			{
				InRHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				FPixelShaderUtils::InitFullscreenPipelineState(InRHICmdList, View.ShaderMap, PixelShader, GraphicsPSOInit);
				GraphicsPSOInit.BlendState = TStaticBlendState<
					CW_RGB, BO_Add, BF_InverseSourceAlpha, BF_SourceAlpha, BO_Add, BF_Zero, BF_One,// Normal
					CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One,// BaseColor
					CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One	// Metallic, Specular, Roughness
					>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

				SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, 0);
				SetShaderParameters(InRHICmdList, PixelShader, PixelShader.GetPixelShader(), *ParametersPS);
				FPixelShaderUtils::DrawFullscreenTriangle(InRHICmdList);
			});
		}
		else
		{
			FRDGTextureUAVRef RWMaterialTexture = GraphBuilder.CreateUAV(SubstrateSceneData->MaterialTextureArray, ERDGUnorderedAccessViewFlags::SkipBarrier);
			FRDGTextureUAVRef RWTopLayerTexture = GraphBuilder.CreateUAV(SubstrateSceneData->TopLayerTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);

			auto DBufferPass = [&](ESubstrateTileType TileType)
			{
				// Only simple & single material are support but also dispatch complex tiles, 
				// as they can contain simple/single material pixels

				uint32 TilePermutation = 0;
				switch(TileType)
				{
				case ESubstrateTileType::EComplex:
				case ESubstrateTileType::EDecalComplex:
					TilePermutation = 2;
					break;
				case ESubstrateTileType::ESingle:
				case ESubstrateTileType::EDecalSingle:
					TilePermutation = 1;
					break;
				case ESubstrateTileType::ESimple:
				case ESubstrateTileType::EDecalSimple:
					TilePermutation = 0;
					break;
				}

				FSubstrateDBufferPassCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FSubstrateDBufferPassCS::FTileType>(TilePermutation);

				TShaderMapRef<FSubstrateDBufferPassCS> ComputeShader(View.ShaderMap, PermutationVector);
				FSubstrateDBufferPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubstrateDBufferPassCS::FParameters>();

				PassParameters->DBuffer = GetDBufferParameters(GraphBuilder, *SceneTextures.DBufferTextures, View.GetShaderPlatform());
				PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
				PassParameters->ViewResolution = View.ViewRect.Size();
				PassParameters->MaxBytesPerPixel = SubstrateSceneData->EffectiveMaxBytesPerPixel;
				PassParameters->TopLayerTexture = RWTopLayerTexture;
				PassParameters->MaterialTextureArrayUAV = RWMaterialTexture;
				PassParameters->FirstSliceStoringSubstrateSSSData = SubstrateSceneData->FirstSliceStoringSubstrateSSSData;
				PassParameters->SceneStencilTexture = SceneTextures.Stencil;

				PassParameters->TileListBuffer = SubstrateViewData->ClassificationTileListBufferSRV;
				PassParameters->TileListBufferOffset = SubstrateViewData->ClassificationTileListBufferOffset[TileType];
				PassParameters->TileEncoding = SubstrateViewData->TileEncoding;
				PassParameters->TileIndirectBuffer = SubstrateViewData->ClassificationTileDispatchIndirectBuffer;

				// Dispatch with tile data
				const uint32 GroupSize = 8;
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("Substrate::DbufferPass(%s)", ToString(TileType)),
					ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					PassParameters->TileIndirectBuffer,
					TileTypeDispatchIndirectArgOffset(TileType));
			};

			const bool bDbufferTiles = CVarSubstrateDBufferPassDedicatedTiles.GetValueOnRenderThread() > 0;
			DBufferPass(bDbufferTiles ? ESubstrateTileType::EDecalComplex : ESubstrateTileType::EComplex);
			DBufferPass(bDbufferTiles ? ESubstrateTileType::EDecalSingle : ESubstrateTileType::ESingle);
			DBufferPass(bDbufferTiles ? ESubstrateTileType::EDecalSimple : ESubstrateTileType::ESimple);
		}
	}
}

void AddSubstrateDBufferBasePass(
	FRDGBuilder& GraphBuilder, 
	TArray<FViewInfo>& Views, 
	const FSceneTextures& InSceneTextures,
	FDBufferTextures& DBufferTextures, 
	FDecalVisibilityTaskData* DecalVisibility, 
	FInstanceCullingManager& InstanceCullingManager, 
	const FSubstrateSceneData& SubstrateSceneData)
{
	if (!IsSubstrateEnabled() || DecalVisibility == nullptr)
	{
		return;
	}

	int32 DecalViewOffset = DecalVisibility->GetViewOffset(Views);
	if (DecalViewOffset == INDEX_NONE)
	{
		return;
	}

	RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, IsSubstrateEnabled() && Views.Num() > 0, "Substrate::DBufferBasePass");

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];

		const bool bEnableDecals = IsDBufferPassEnabled(View.GetShaderPlatform()) && DecalVisibility->HasStage(DecalViewOffset, ViewIndex, EDecalRenderStage::BeforeBasePass);
		if (bEnableDecals)
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

			View.BeginRenderView();
			FDeferredDecalPassTextures DecalPassTextures = GetDeferredDecalPassTextures(GraphBuilder, View, InSceneTextures, &DBufferTextures, EDecalRenderStage::BeforeBasePass);
			TConstArrayView<const FVisibleDecal*> SortedDecals = DecalVisibility->FinishRelevantDecals(DecalViewOffset, ViewIndex, EDecalRenderStage::BeforeBasePass);
			AddDeferredDecalPass(GraphBuilder, View, SortedDecals, DecalPassTextures, InstanceCullingManager, EDecalRenderStage::BeforeBasePass);
		}
	}
}

void AddSubstrateSampleMaterialPass(FRDGBuilder& GraphBuilder, const FScene* Scene, const FMinimalSceneTextures& SceneTextures, const TArray<FViewInfo>& Views)
{
	if (Substrate::IsSubstrateEnabled())
	{
		FRDGTextureUAVRef RWSampledMaterialTexture = nullptr;
		bool bNeedSampleMaterial = false;
		for (const FViewInfo& View : Views)
		{
			if (NeedsSampledMaterials(Scene, *View.Family))
			{
				if (const FSubstrateSceneData* SubstrateSceneData = View.SubstrateViewData.SceneData)
				{
					bNeedSampleMaterial = true;
					RWSampledMaterialTexture = GraphBuilder.CreateUAV(SubstrateSceneData->SampledMaterialTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
					break;
				}
			}
		}

		if (bNeedSampleMaterial)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "Substrate::SampleMaterial");
			for (const FViewInfo& View : Views)
			{
				if (NeedsSampledMaterials(Scene, View))
				{
					if (const FSubstrateSceneData* SubstrateSceneData = View.SubstrateViewData.SceneData)
					{
						AddSubstrateInternalSampleMaterialPass(GraphBuilder, View, SceneTextures, *SubstrateSceneData, RWSampledMaterialTexture);
					}
				}
			}
		}
	}
}

bool ShouldCompileSubstrateTileTypePermutations(const int32 SubstrateTileType, const EShaderPlatform Platform)
{
	const bool bIsSubstrateEnabled = Substrate::IsSubstrateEnabled();

	if (!bIsSubstrateEnabled && SubstrateTileType != 0)
	{
		return false; // When Substrate is disabled, only tile type 0 is used.
	}

	if (bIsSubstrateEnabled && SubstrateTileType == SUBSTRATE_TILE_TYPE_COMPLEX_SPECIAL && IsSubstrateBlendableGBufferEnabled(Platform))
	{
		return false; // When Substrate is enabled and uses blendable GBuffer, ComplexSpecial tile types are never used (e.g. used for glints).
	}

	return true;
}

template<ECompareFunction CompareFunction>
void InternalGetSubstrateTiledDepthStencilStateRead(ESubstrateTileType InTileType, FGraphicsPipelineStateInitializer& GraphicsPSOInit, uint32& OutStencilRef)
{
	OutStencilRef = 0;
	check(Substrate::IsSubstrateEnabled());
	switch (InTileType)
	{
		case ESubstrateTileType::ESimple:			OutStencilRef = Substrate::StencilBit_Fast;				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CompareFunction, true, CF_Equal, SO_Keep, SO_Keep, SO_Keep, true, CF_Equal, SO_Keep, SO_Keep, SO_Keep, Substrate::StencilBit_Fast, 0x0>::GetRHI(); break;
		case ESubstrateTileType::ESingle:			OutStencilRef = Substrate::StencilBit_Single;			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CompareFunction, true, CF_Equal, SO_Keep, SO_Keep, SO_Keep, true, CF_Equal, SO_Keep, SO_Keep, SO_Keep, Substrate::StencilBit_Single, 0x0>::GetRHI(); break;
		case ESubstrateTileType::EComplex:			OutStencilRef = Substrate::StencilBit_Complex;			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CompareFunction, true, CF_Equal, SO_Keep, SO_Keep, SO_Keep, true, CF_Equal, SO_Keep, SO_Keep, SO_Keep, Substrate::StencilBit_Complex, 0x0>::GetRHI(); break;
		case ESubstrateTileType::EComplexSpecial:	OutStencilRef = Substrate::StencilBit_ComplexSpecial; 	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CompareFunction, true, CF_Equal, SO_Keep, SO_Keep, SO_Keep, true, CF_Equal, SO_Keep, SO_Keep, SO_Keep, Substrate::StencilBit_ComplexSpecial, 0x0>::GetRHI(); break;
		default: check(false);
	}
}

void GetSubstrateTiledDepthStencilStateWrite(ESubstrateTileType InTileType, FGraphicsPipelineStateInitializer& GraphicsPSOInit, uint32& OutStencilRef)
{
	OutStencilRef = 0;
	check(Substrate::IsSubstrateEnabled());
	switch (InTileType)
	{
		case ESubstrateTileType::ESimple:			OutStencilRef = Substrate::StencilBit_Fast;				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always, true, CF_Always, SO_Keep, SO_Keep, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 0xFF, Substrate::StencilBit_Fast>::GetRHI(); break;
		case ESubstrateTileType::ESingle:			OutStencilRef = Substrate::StencilBit_Single;			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always, true, CF_Always, SO_Keep, SO_Keep, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 0xFF, Substrate::StencilBit_Single>::GetRHI(); break;
		case ESubstrateTileType::EComplex:			OutStencilRef = Substrate::StencilBit_Complex;			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always, true, CF_Always, SO_Keep, SO_Keep, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 0xFF, Substrate::StencilBit_Complex>::GetRHI(); break;
		case ESubstrateTileType::EComplexSpecial:	OutStencilRef = Substrate::StencilBit_ComplexSpecial; 	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always, true, CF_Always, SO_Keep, SO_Keep, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 0xFF, Substrate::StencilBit_ComplexSpecial>::GetRHI(); break;
		default: check(false);
	}
}

void GetSubstrateTiledDepthStencilStateRead(ESubstrateTileType InTileType, ECompareFunction InCompareFunction, FGraphicsPipelineStateInitializer& OutGraphicsPSOInit, uint32& OutStencilRef)
{
	switch (InCompareFunction)
	{
		case CF_DepthNearOrEqual: 	InternalGetSubstrateTiledDepthStencilStateRead<CF_DepthNearOrEqual>(InTileType, OutGraphicsPSOInit, OutStencilRef); break;
		case CF_Always: 			InternalGetSubstrateTiledDepthStencilStateRead<CF_Always>(InTileType, OutGraphicsPSOInit, OutStencilRef); break;
		default: check(false); // Not implemented
	}
}

} // namespace Substrate
