// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShaderParameterMacros.h"
#include "RenderGraphResources.h"
#include "MeshPassProcessor.h"
#include "UnifiedBuffer.h"
#include "RHIUtilities.h"
#include "SubstrateDefinitions.h"
#include "GBufferInfo.h"
#include "RendererUtils.h"
#include "ShaderParameterStruct.h"
#include "SceneView.h"
#include "Rendering/SubstrateMaterialShared.h"

// Forward declarations.
class FScene;
class FSceneRenderer;
class FRDGBuilder;
struct FDBufferTextures;
struct FMinimalSceneTextures;
struct FScreenPassTexture;
struct FTextureRenderTargetBinding;
class FDecalVisibilityTaskData;
class FInstanceCullingManager;
class FViewFamilyInfo;

BEGIN_SHADER_PARAMETER_STRUCT(FSubstrateCommonParameters, )
	SHADER_PARAMETER(uint32, MaxBytesPerPixel)
	SHADER_PARAMETER(uint32, MaxClosurePerPixel)
	SHADER_PARAMETER(uint32, bRoughDiffuse)
	SHADER_PARAMETER(uint32, PeelLayersAboveDepth)
	SHADER_PARAMETER(uint32, bRoughnessTracking)
	SHADER_PARAMETER(uint32, bStochasticLighting)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSubstrateBasePassUniformParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateCommonParameters, Common)
	SHADER_PARAMETER(int32, SliceStoringDebugSubstrateTreeDataWithoutMRT)
	SHADER_PARAMETER(int32, FirstSliceStoringSubstrateSSSDataWithoutMRT)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, MaterialTextureArrayUAVWithoutRTs)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, OpaqueRoughRefractionTextureUAV)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSubstrateForwardPassUniformParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateCommonParameters, Common)
	SHADER_PARAMETER(int32, FirstSliceStoringSubstrateSSSData)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, MaterialTextureArray)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<SUBSTRATE_TOP_LAYER_TYPE>, TopLayerTexture)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSubstrateMobileForwardPassUniformParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateCommonParameters, Common)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSubstrateTileParameter, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileListBuffer)
	SHADER_PARAMETER(uint32, TileListBufferOffset)
	SHADER_PARAMETER(uint32, TileEncoding)
	RDG_BUFFER_ACCESS(TileIndirectBuffer, ERHIAccess::IndirectArgs)
END_SHADER_PARAMETER_STRUCT()

// This parameter struct is declared with RENDERER_API even though it is not public. This is
// to workaround other modules doing 'private include' of the Renderer module
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSubstrateGlobalUniformParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateCommonParameters, Common)
	SHADER_PARAMETER(int32,  SliceStoringDebugSubstrateTreeData)
	SHADER_PARAMETER(int32,  FirstSliceStoringSubstrateSSSData)
	SHADER_PARAMETER(uint32, TileSize)
	SHADER_PARAMETER(uint32, TileSizeLog2)
	SHADER_PARAMETER(FIntPoint, TileCount)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, MaterialTextureArray)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<SUBSTRATE_TOP_LAYER_TYPE>, TopLayerTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, OpaqueRoughRefractionTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, ClosureOffsetTexture)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ClosureTileBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ClosureTileCountBuffer)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint4>, SampledMaterialTexture)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSubstratePublicParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateCommonParameters, Common)
	SHADER_PARAMETER(int32, FirstSliceStoringSubstrateSSSData)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<SUBSTRATE_TOP_LAYER_TYPE>, TopLayerTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, MaterialTextureArray)
END_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSubstratePublicGlobalUniformParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSubstratePublicParameters, Public)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

struct FSubstrateSceneData
{
	// Track max BytesPerPixel / ClosurePerPixel amoung all views
	uint32 ViewsMaxBytesPerPixel = 0;
	uint32 ViewsMaxClosurePerPixel = 0;

	// Track max BytesPerPixel / ClosurePerPixel ever encountered since the scene was created
	uint32 PersistentMaxBytesPerPixel = 0;
	uint32 PersistentMaxClosurePerPixel = 0;
	uint8 UsesTileTypeMask = 0;
	bool bUsesAnisotropy = false;

	// Track if stencil needs to be written
	bool bWriteStencil = false;

	// Current max BytesPerPixel / ClosurePerPixel
	uint32 EffectiveMaxBytesPerPixel = 0;
	uint32 EffectiveMaxClosurePerPixel = 0;

	int32 PeelLayersAboveDepth = -1;
	bool bRoughDiffuse = false;
	bool bRoughnessTracking = false;
	bool bStochasticLighting = false;

	int32 SliceStoringDebugSubstrateTreeDataWithoutMRT = -1;
	int32 SliceStoringDebugSubstrateTreeData = -1;
	int32 FirstSliceStoringSubstrateSSSDataWithoutMRT = -1;
	int32 FirstSliceStoringSubstrateSSSData = -1;

	// Resources allocated and updated each frame

	FRDGTextureRef MaterialTextureArray = nullptr;
	FRDGTextureUAVRef MaterialTextureArrayUAVWithoutRTs = nullptr;
	FRDGTextureUAVRef MaterialTextureArrayUAV = nullptr;
	FRDGTextureSRVRef MaterialTextureArraySRV = nullptr;

	FRDGTextureRef TopLayerTexture = nullptr;
	FRDGTextureRef OpaqueRoughRefractionTexture = nullptr;

	FRDGTextureUAVRef TopLayerTextureUAV = nullptr;
	FRDGTextureUAVRef OpaqueRoughRefractionTextureUAV = nullptr;

	FRDGTextureRef ClosureOffsetTexture = nullptr;

	FRDGTextureRef SampledMaterialTexture = nullptr;

	// Used when the subsurface luminance is separated from the scene color
	FRDGTextureRef SeparatedSubSurfaceSceneColor = nullptr;

	// Used for Luminance that should go through opaque rough refraction (when under a top layer interface)
	FRDGTextureRef SeparatedOpaqueRoughRefractionSceneColor = nullptr;

	// Used for Toon unified diffuse.
	FRDGTextureRef SubstrateSceneToonData0 = nullptr;
	FRDGTextureRef SubstrateSceneToonData1 = nullptr;

	//Public facing minimal uniform data.
	TRDGUniformBufferRef<FSubstratePublicGlobalUniformParameters> SubstratePublicGlobalUniformParameters{};
};

struct FSubstrateViewData
{
	// Max BytePerPixel & ClosurePerPixel count among all visible materials
	uint32 MaxClosurePerPixel = 0;
	uint32 MaxBytesPerPixel = 0;

	// Visible/Active tile type mask (1 bit per tile type)
	uint8 UsesTileTypeMask = 0;
	bool bUsesAnisotropy = false;

	FIntPoint TileCount  = FIntPoint(0, 0);
	uint32    TileEncoding = SUBSTRATE_TILE_ENCODING_16BITS;
	uint32    LayerCount = 0;

	FRDGBufferRef    ClassificationTileListBuffer = nullptr;
	FRDGBufferSRVRef ClassificationTileListBufferSRV = nullptr;
	FRDGBufferUAVRef ClassificationTileListBufferUAV = nullptr;
	uint32			 ClassificationTileListBufferOffset[SUBSTRATE_TILE_TYPE_COUNT];

	FRDGBufferRef    ClassificationTileDrawIndirectBuffer = nullptr;
	FRDGBufferUAVRef ClassificationTileDrawIndirectBufferUAV = nullptr;

	FRDGBufferRef    ClassificationTileDispatchIndirectBuffer = nullptr;
	FRDGBufferUAVRef ClassificationTileDispatchIndirectBufferUAV = nullptr;

	FRDGBufferRef  ClosureTileBuffer = nullptr;
	FRDGBufferRef  ClosureTileCountBuffer = nullptr;
	FRDGBufferRef  ClosureTileDispatchIndirectBuffer = nullptr;
	FRDGBufferRef  ClosureTileRaytracingIndirectBuffer = nullptr;
	FRDGBufferRef  ClosureTilePerThreadDispatchIndirectBuffer = nullptr;

	FSubstrateSceneData* SceneData = nullptr;

	TRDGUniformBufferRef<FSubstrateGlobalUniformParameters> SubstrateGlobalUniformParameters{};

	void Reset();
};

// The substrate debug data for each view
struct FSubstrateViewDebugData
{
	uint32 PixelMaterialDebugDataSizeBytes = 0;
	TQueue<TSharedPtr<FRHIGPUBufferReadback>> PixelMaterialDebugDataReadbackQueries;

	uint32 SystemInfoDebugDataSizeBytes = 0;
	TQueue<TSharedPtr<FRHIGPUBufferReadback>> SystemInfoDebugDataReadbackQueries;

	FSubstrateViewDebugData();

	struct FTransientDebugBuffer
	{
		uint32 DebugDataSizeInUints = 0;
		FRDGBufferRef DebugData = nullptr;
		FRDGBufferUAVRef DebugDataUAV = nullptr;
	};

	struct FTransientPixelDebugBuffer : FTransientDebugBuffer {};
	FTransientPixelDebugBuffer CreateTransientPixelDebugBuffer(FRDGBuilder& GraphBuilder);
	static FTransientPixelDebugBuffer CreateDummyPixelDebugBuffer(FRDGBuilder& GraphBuilder);

	struct FTransientSystemInfoDebugBuffer : FTransientDebugBuffer {};
	FTransientSystemInfoDebugBuffer CreateTransientSystemInfoDebugBuffer(FRDGBuilder& GraphBuilder);

	void SafeRelease();
};

namespace Substrate
{

enum class EStencilPassStage : uint8
{
	None,
	Standalone,
	StandaloneAdaptive,
	DuringIndirectLighting
};

// Bits for filling/clearing stencil during shadow/lighting passes. 
// Those are designed to not conflicts with other stencil shadow bits, when generating shadow mask. 
// Withouth these 'trailing' bits, the incr./decr. operation would change/corrupt the 'Substrate' bits.
constexpr uint32 StencilBit_Fast			= 0x10; // In sync with SceneRenderTargets.h - GET_STENCIL_BIT_MASK(STENCIL_SUBSTRATE_FASTPATH)
constexpr uint32 StencilBit_Single			= 0x20; // In sync with SceneRenderTargets.h - GET_STENCIL_BIT_MASK(STENCIL_SUBSTRATE_SINGLEPATH)
constexpr uint32 StencilBit_Complex			= 0x40; // In sync with SceneRenderTargets.h - GET_STENCIL_BIT_MASK(STENCIL_SUBSTRATE_COMPLEX)
constexpr uint32 StencilBit_ComplexSpecial	= 0x80; // In sync with SceneRenderTargets.h - GET_STENCIL_BIT_MASK(STENCIL_SUBSTRATE_COMPLEX_SPECIAL)	

FIntPoint GetSubstrateTextureResolution(const FViewInfo& View, const FIntPoint& InResolution);
uint32 GetSubstrateMaxClosureCount(const FViewInfo& View);
UE_DEPRECATED(5.7, "Please, use the GetSubstrateUsesTileType() function instead")	
bool GetSubstrateUsesComplexSpecialPath(const FViewInfo& View);
bool GetSubstrateUsesAnisotropy(const FViewInfo& View);
bool UsesSubstrateMaterialBuffer(EShaderPlatform In);
bool UsesStochasticLightingClassification(EShaderPlatform InPlatform);
bool GetSubstrateUsesTileType(const FViewInfo& View, ESubstrateTileType TileType);
EStencilPassStage GetStencilPassStage();

void InitialiseSubstrateFrameSceneData(FRDGBuilder& GraphBuilder, FScene& Scene, FViewFamilyInfo& ViewFamily, TConstArrayView<FViewInfo*> AllViews, EShaderPlatform ShaderPlatform);

void BindSubstrateBasePassUniformParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSubstrateBasePassUniformParameters& OutSubstrateUniformParameters);
void BindSubstrateForwardPasslUniformParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSubstrateForwardPassUniformParameters& OutSubstrateUniformParameters);
void BindSubstrateMobileForwardPasslUniformParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSubstrateMobileForwardPassUniformParameters& OutSubstrateUniformParameters);
TRDGUniformBufferRef<FSubstrateGlobalUniformParameters> BindSubstrateGlobalUniformParameters(const FViewInfo& View);

void AppendSubstrateMRTs(const FSceneRenderer& SceneRenderer, uint32& BasePassTextureCount, TArrayView<FTextureRenderTargetBinding> BasePassTextures);
void SetBasePassRenderTargetOutputFormat(const EShaderPlatform Platform, const FMaterialShaderParameters& MaterialParameters, FShaderCompilerEnvironment& OutEnvironment, EGBufferLayout GBufferLayout);

void BindSubstratePublicGlobalUniformParameters(FRDGBuilder& GraphBuilder, const FSubstrateSceneData* SubstrateSceneData, FSubstratePublicParameters& OutSubstrateUniformParameters);
TRDGUniformBufferRef<FSubstratePublicGlobalUniformParameters> CreatePublicGlobalUniformBuffer(FRDGBuilder& GraphBuilder, FSubstrateSceneData* SubstrateScene);

void AddSubstrateMaterialClassificationPass(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures, const TArray<FViewInfo>& Views);
void AddSubstrateDBufferPass(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures, const TArray<FViewInfo>& Views);

void AddSubstrateStencilPass(FRDGBuilder& GraphBuilder, FScene* Scene, const TArray<FViewInfo>& Views, const FMinimalSceneTextures& SceneTextures, FSubstrateSceneData& SubstrateSceneData);
void AddSubstrateSampleMaterialPass(FRDGBuilder& GraphBuilder, const FScene* Scene, const FMinimalSceneTextures& SceneTextures, const TArray<FViewInfo>& Views);
void AddSubstrateMaterialClassificationIndirectArgsPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, ERDGPassFlags ComputePassFlags);

void AddSubstrateOpaqueRoughRefractionPasses(
	FRDGBuilder& GraphBuilder,
	FSceneTextures& SceneTextures,
	TArrayView<const FViewInfo> Views);

bool ShouldRenderSubstrateDebugPasses(const FViewInfo& View);
FScreenPassTexture AddSubstrateDebugPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor);

void AddProcessAndPrintSubstrateMaterialPropertiesPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SceneColorTexture, EShaderPlatform Platform, FSubstrateViewDebugData::FTransientPixelDebugBuffer& NewSubstratePixelDebugBuffer);

class FSubstrateTilePassVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubstrateTilePassVS);
	SHADER_USE_PARAMETER_STRUCT(FSubstrateTilePassVS, FGlobalShader);

	class FEnableDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_ENABLE_DEBUG");
	class FEnableTexCoordScreenVector : SHADER_PERMUTATION_BOOL("PERMUTATION_ENABLE_TEXCOORD_SCREENVECTOR");
	class FClassificationTileSize16 : SHADER_PERMUTATION_BOOL("SUBSTRATE_CLASSIFICATION_TILE_SIZE_16");
	using FPermutationDomain = TShaderPermutationDomain<FEnableDebug, FEnableTexCoordScreenVector, FClassificationTileSize16>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileListBuffer)
		SHADER_PARAMETER(uint32, TileListBufferOffset)
		SHADER_PARAMETER(uint32, TileEncoding)
		RDG_BUFFER_ACCESS(TileIndirectBuffer, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

FSubstrateTileParameter SetTileParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, const ESubstrateTileType Type);
FSubstrateTilePassVS::FParameters SetTileParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, const ESubstrateTileType Type, EPrimitiveType& PrimitiveType);
FSubstrateTilePassVS::FParameters SetTileParameters(const FViewInfo& View, const ESubstrateTileType Type, EPrimitiveType& PrimitiveType);
uint32 TileTypeDrawIndirectArgOffset(const ESubstrateTileType Type);
uint32 TileTypeDispatchIndirectArgOffset(const ESubstrateTileType Type);

bool IsStochasticLightingActive(EShaderPlatform In);

uint32 GetClosureTileIndirectArgsOffset(uint32 InDownsampleFactor);

void AddSubstrateDBufferBasePass(
	FRDGBuilder& GraphBuilder, 
	TArray<FViewInfo>& Views, 
	const FSceneTextures& InSceneTextures,
	FDBufferTextures& DBufferTextures, 
	FDecalVisibilityTaskData* DecalVisibility, 
	FInstanceCullingManager& InstanceCullingManager, 
	const FSubstrateSceneData& SubstrateSceneData);


class FSubstrateTileType : SHADER_PERMUTATION_INT("SUBSTRATE_TILETYPE", 4);
bool ShouldCompileSubstrateTileTypePermutations(const int32 SubstrateTileType, const EShaderPlatform Platform);

void GetSubstrateTiledDepthStencilStateRead(ESubstrateTileType InTileType, ECompareFunction InCompareFunction, FGraphicsPipelineStateInitializer& OutGraphicsPSOInit, uint32& OutStencilRef);
void GetSubstrateTiledDepthStencilStateWrite(ESubstrateTileType InTileType, FGraphicsPipelineStateInitializer& GraphicsPSOInit, uint32& OutStencilRef);

}; // namespace Substrate
