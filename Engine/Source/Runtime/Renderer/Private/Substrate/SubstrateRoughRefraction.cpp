// Copyright Epic Games, Inc. All Rights Reserved.

#include "Substrate.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HAL/IConsoleManager.h"
#include "PixelShaderUtils.h"
#include "SceneView.h"
#include "SceneRendering.h"
#include "RendererInterface.h"
#include "UniformBuffer.h"
#include "SceneTextureParameters.h"
#include "ScreenPass.h"
#include "ShaderCompiler.h"
#include "LogRenderer.h"
#include "MegaLights/MegaLights.h"


static TAutoConsoleVariable<float> CVarSubstrateOpaqueMaterialRoughRefractionBlurScale(
	TEXT("r.Substrate.OpaqueMaterialRoughRefraction.BlurScale"),
	1.0f,
	TEXT("Scale opaque rough refraction strengh for debug purposes."),
	ECVF_RenderThreadSafe);

namespace Substrate
{

class FOpaqueRoughRefractionPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOpaqueRoughRefractionPS);
	SHADER_USE_PARAMETER_STRUCT(FOpaqueRoughRefractionPS, FGlobalShader);

	class FEnableBlur : SHADER_PERMUTATION_BOOL("PERMUTATION_ENABLE_BLUR");
	using FPermutationDomain = TShaderPermutationDomain<FEnableBlur>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SeparatedOpaqueRoughRefractionSceneColor)
		SHADER_PARAMETER(FVector2f, BlurDirection)
		SHADER_PARAMETER(float, BlurScale)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("OPAQUE_ROUGH_REFRACTION_PS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FOpaqueRoughRefractionPS, "/Engine/Private/Substrate/SubstrateRoughRefraction.usf", "OpaqueRoughRefractionPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FOpaqueRoughRefractionParameters, )
	SHADER_PARAMETER_STRUCT(FOpaqueRoughRefractionPS::FParameters, PS)
	SHADER_PARAMETER_STRUCT(Substrate::FSubstrateTilePassVS::FParameters, VS)
END_SHADER_PARAMETER_STRUCT()

bool IsOpaqueRoughRefractionEnabled(TArrayView<const FViewInfo> Views)
{
	for (const FViewInfo& View : Views)
	{
		if (IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()))
		{
			return true;
		}
	}
	return false;
}

void AddSubstrateOpaqueRoughRefractionPasses(
	FRDGBuilder& GraphBuilder,
	FSceneTextures& SceneTextures,
	TArrayView<const FViewInfo> Views)
{
	const uint32 ViewCount = Views.Num();

	const bool bOpaqueRoughRefractionEnabled = IsOpaqueRoughRefractionEnabled(Views);
	if (!bOpaqueRoughRefractionEnabled)
	{
		return;
	}
	RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, bOpaqueRoughRefractionEnabled, "Substrate::OpaqueRoughRefraction");

	// Opaque rough refraction support is not implemented for Megalights. Issue a warning into the log
	static bool bWarningIssued = false;
	if (ViewCount > 0 && Views[0].Family && MegaLights::IsEnabled(*Views[0].Family) && !bWarningIssued)
	{
		UE_LOGF(LogRenderer, Warning, "Substrate::OpaqueRoughRefraction is enabled at the same time as MegaLights, which is currently not supported with opaque rough refraction. The visual output might be incorrect.");
		bWarningIssued = true;
	}

	FRDGTextureRef SceneColorTexture = SceneTextures.Color.Target;
	FRDGTextureRef TempTexture = nullptr;
	ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::EClear;

	//
	// 1. First blur pass into temporary buffer. This only blurs tiles containing pixels with rough refractions.
	//
	ESubstrateTileType SubstrateTileType = ESubstrateTileType::EOpaqueRoughRefraction;
	for (uint32 ViewIndex = 0; ViewIndex < ViewCount; ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (TempTexture == nullptr)
		{
			TempTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(View.GetSceneTexturesConfig().Extent, PF_FloatRGBA, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable), TEXT("Substrate.RoughRefrac.TempTexture"));
		}

		FRDGTextureRef SeparatedOpaqueRoughRefractionSceneColor = View.SubstrateViewData.SceneData->SeparatedOpaqueRoughRefractionSceneColor;

		FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, View);
		FOpaqueRoughRefractionParameters* PassParameters = GraphBuilder.AllocParameters<FOpaqueRoughRefractionParameters>();
		FOpaqueRoughRefractionPS::FParameters* PSParameters = &PassParameters->PS;
		PSParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PSParameters->SceneTextures = GetSceneTextureShaderParameters(SceneTextures.UniformBuffer);
		PSParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PSParameters->SeparatedOpaqueRoughRefractionSceneColor = SeparatedOpaqueRoughRefractionSceneColor;
		PSParameters->RenderTargets[0] = FRenderTargetBinding(TempTexture, LoadAction);
		PSParameters->BlurDirection = FVector2f(1.0f, 0.0f);
		PSParameters->BlurScale = FMath::Max(0.0f, CVarSubstrateOpaqueMaterialRoughRefractionBlurScale.GetValueOnAnyThread());

		FOpaqueRoughRefractionPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FOpaqueRoughRefractionPS::FEnableBlur>(true);
		TShaderMapRef<FOpaqueRoughRefractionPS> PixelShader(View.ShaderMap, PermutationVector);

		Substrate::FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
		VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableDebug >(false);
		VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableTexCoordScreenVector >(false);
		TShaderMapRef<Substrate::FSubstrateTilePassVS> TileVertexShader(View.ShaderMap, VSPermutationVector);

		EPrimitiveType PrimitiveType = PT_TriangleList;
		PassParameters->VS = Substrate::SetTileParameters(GraphBuilder, View, SubstrateTileType, PrimitiveType);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("Substrate::OpaqueRoughRefraction(Blur,Horizontal)"),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, TileVertexShader, PixelShader, PassParameters, SubstrateTileType, PrimitiveType](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				// Set the device viewport for the view.
				RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI(); // no blend
				GraphicsPSOInit.PrimitiveType = PrimitiveType;
				GraphicsPSOInit.bDepthBounds = false;
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = TileVertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0x0);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);
				SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), PassParameters->VS);
				RHICmdList.DrawPrimitiveIndirect(PassParameters->VS.TileIndirectBuffer->GetIndirectRHICallBuffer(), Substrate::TileTypeDrawIndirectArgOffset(SubstrateTileType));
			});

		LoadAction = ERenderTargetLoadAction::ELoad;
	}

	//
	// 2. second blur pass into temporary buffer. This only blurs tiles containing pixels with rough refractions.
	//
	for (uint32 ViewIndex = 0; ViewIndex < ViewCount; ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, View);
		FOpaqueRoughRefractionParameters* PassParameters = GraphBuilder.AllocParameters<FOpaqueRoughRefractionParameters>();
		FOpaqueRoughRefractionPS::FParameters* PSParameters = &PassParameters->PS;
		PSParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PSParameters->SceneTextures = GetSceneTextureShaderParameters(SceneTextures.UniformBuffer);
		PSParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PSParameters->SeparatedOpaqueRoughRefractionSceneColor = TempTexture;
		PSParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);
		PSParameters->BlurDirection = FVector2f(0.0f, 1.0f);
		PSParameters->BlurScale = FMath::Max(0.0f, CVarSubstrateOpaqueMaterialRoughRefractionBlurScale.GetValueOnAnyThread());

		FOpaqueRoughRefractionPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FOpaqueRoughRefractionPS::FEnableBlur>(true);
		TShaderMapRef<FOpaqueRoughRefractionPS> PixelShader(View.ShaderMap, PermutationVector);

		Substrate::FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
		VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableDebug >(false);
		VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableTexCoordScreenVector >(false);
		TShaderMapRef<Substrate::FSubstrateTilePassVS> TileVertexShader(View.ShaderMap, VSPermutationVector);

		EPrimitiveType PrimitiveType = PT_TriangleList;
		PassParameters->VS = Substrate::SetTileParameters(GraphBuilder, View, SubstrateTileType, PrimitiveType);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("Substrate::OpaqueRoughRefraction(Blur,Vertical)"),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, TileVertexShader, PixelShader, PassParameters, SubstrateTileType, PrimitiveType](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				// Set the device viewport for the view.
				RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI(); // Additive blend
				GraphicsPSOInit.PrimitiveType = PrimitiveType;
				GraphicsPSOInit.bDepthBounds = false;
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = TileVertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0x0);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);
				SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), PassParameters->VS);
				RHICmdList.DrawPrimitiveIndirect(PassParameters->VS.TileIndirectBuffer->GetIndirectRHICallBuffer(), Substrate::TileTypeDrawIndirectArgOffset(SubstrateTileType));
			});
	}

	//
	// 3. Add remaining tiles with subsurface scattering that did not have rough refractions on them, resulting in a complete scene color texture.
	//
	SubstrateTileType = ESubstrateTileType::EOpaqueRoughRefractionSSSWithout;
	for (uint32 ViewIndex = 0; ViewIndex < ViewCount; ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		FRDGTextureRef SeparatedOpaqueRoughRefractionSceneColor = View.SubstrateViewData.SceneData->SeparatedOpaqueRoughRefractionSceneColor;

		FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, View);
		FOpaqueRoughRefractionParameters* PassParameters = GraphBuilder.AllocParameters<FOpaqueRoughRefractionParameters>();
		FOpaqueRoughRefractionPS::FParameters* PSParameters = &PassParameters->PS;
		PSParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PSParameters->SceneTextures = GetSceneTextureShaderParameters(SceneTextures.UniformBuffer);
		PSParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PSParameters->SeparatedOpaqueRoughRefractionSceneColor = SeparatedOpaqueRoughRefractionSceneColor;
		PSParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);
		PSParameters->BlurDirection = FVector2f(0.0f, 0.0f);
		PSParameters->BlurScale = FMath::Max(0.0f, CVarSubstrateOpaqueMaterialRoughRefractionBlurScale.GetValueOnAnyThread());

		FOpaqueRoughRefractionPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FOpaqueRoughRefractionPS::FEnableBlur>(false);
		TShaderMapRef<FOpaqueRoughRefractionPS> PixelShader(View.ShaderMap, PermutationVector);

		Substrate::FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
		VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableDebug >(false);
		VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableTexCoordScreenVector >(false);
		TShaderMapRef<Substrate::FSubstrateTilePassVS> TileVertexShader(View.ShaderMap, VSPermutationVector);

		EPrimitiveType PrimitiveType = PT_TriangleList;
		PassParameters->VS = Substrate::SetTileParameters(GraphBuilder, View, SubstrateTileType, PrimitiveType);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("Substrate::OpaqueRoughRefraction(SSS)"),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, TileVertexShader, PixelShader, PassParameters, SubstrateTileType, PrimitiveType](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				// Set the device viewport for the view.
				RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI(); // Additive blend
				GraphicsPSOInit.PrimitiveType = PrimitiveType;
				GraphicsPSOInit.bDepthBounds = false;
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = TileVertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0x0);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);
				SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), PassParameters->VS);
				RHICmdList.DrawPrimitiveIndirect(PassParameters->VS.TileIndirectBuffer->GetIndirectRHICallBuffer(), Substrate::TileTypeDrawIndirectArgOffset(SubstrateTileType));
			});
	}
}


//////////////////////////////////////////////////////////////////////////
// RnD shaders only used when enabled locally
//////////////////////////////////////////////////////////////////////////

// Keeping it simple: this should always be checked in with a value of 0
#define SUBSTRATE_ROUGH_REFRACTION_RND 0

#if SUBSTRATE_ROUGH_REFRACTION_RND

static TAutoConsoleVariable<int32> CVarSubstrateRoughRefractionShadersShowRoughRefractionRnD(
	TEXT("r.Substrate.ShowRoughRefractionRnD"),
	1,
	TEXT("Enable Substrate rough refraction shaders."),
	ECVF_RenderThreadSafe);

bool ShouldRenderSubstrateRoughRefractionRnD()
{
	return CVarSubstrateRoughRefractionShadersShowRoughRefractionRnD.GetValueOnAnyThread() > 0;
}

#else
bool ShouldRenderSubstrateRoughRefractionRnD() { return false; }
#endif


#if SUBSTRATE_ROUGH_REFRACTION_RND

class FEvaluateRoughRefractionLobeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FEvaluateRoughRefractionLobeCS);
	SHADER_USE_PARAMETER_STRUCT(FEvaluateRoughRefractionLobeCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, SampleCountTextureUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<>, LobStatisticsBufferUAV)
		SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER(uint32, TraceSqrtSampleCount)
	END_SHADER_PARAMETER_STRUCT()
	
	static const uint32 ThreadGroupSize = 8;

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SUBSTRATE_RND_SHADERS"), 1);
		OutEnvironment.SetDefine(TEXT("EVALUATE_ROUGH_REFRACTION_LOBE_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}
};
IMPLEMENT_GLOBAL_SHADER(FEvaluateRoughRefractionLobeCS, "/Engine/Private/Substrate/SubstrateRoughRefraction.usf", "EvaluateRoughRefractionLobeCS", SF_Compute);



class FVisualizeRoughRefractionPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeRoughRefractionPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeRoughRefractionPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint>, SampleCountTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<>, LobStatisticsBuffer)
		SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER(float, TraceDomainSize)
		SHADER_PARAMETER(uint32, SlabInterfaceLineCount)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SUBSTRATE_RND_SHADERS"), 1);
		OutEnvironment.SetDefine(TEXT("VISUALIZE_ROUGH_REFRACTION_PS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FVisualizeRoughRefractionPS, "/Engine/Private/Substrate/SubstrateRoughRefraction.usf", "VisualizeRoughRefractionPS", SF_Pixel);



class FRoughRefracDataCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRoughRefracDataCS);
	SHADER_USE_PARAMETER_STRUCT(FRoughRefracDataCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float2>, RoughRefracDataUAV)
		SHADER_PARAMETER(uint32, TraceSqrtSampleCount)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FBufferReadBackParam, )
		RDG_BUFFER_ACCESS(Buffer, ERHIAccess::CopySrc)
	END_SHADER_PARAMETER_STRUCT()

	static const uint32 ThreadGroupSize = 64;

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SUBSTRATE_RND_SHADERS"), 1);
		OutEnvironment.SetDefine(TEXT("EVALUATE_ROUGH_REFRACTION_DATA_CS"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRoughRefracDataCS, "/Engine/Private/Substrate/SubstrateRoughRefraction.usf", "RoughRefracDataCS", SF_Compute);

#endif // SUBSTRATE_ROUGH_REFRACTION_RND


void SubstrateRoughRefractionRnD(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor)
{
#if SUBSTRATE_ROUGH_REFRACTION_RND
	if (IsSubstrateEnabled() && ShouldRenderSubstrateRoughRefractionRnD())
	{
		if (!ShaderPrint::IsValid(View.ShaderPrintData))
		{
			return;
		}
		check(ShaderPrint::IsEnabled(View.ShaderPrintData));	// One must enable ShaderPrint beforehand using r.ShaderPrint=1

		//////////////////////////////////////////////////////////////////////////
		// Create resources
		
		// Texture to count
		const uint32 SampleCountTextureWidth = 64;
		const FIntPoint SampleCountTextureSize(SampleCountTextureWidth, SampleCountTextureWidth);
		FRDGTextureRef SampleCountTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(SampleCountTextureSize, PF_R32_UINT, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource), TEXT("Substrate.RoughRefrac.SampleCount"));
		FRDGTextureUAVRef SampleCountTextureUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SampleCountTexture));
		FRDGTextureSRVRef SampleCountTextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SampleCountTexture));

		FRDGBufferRef LobStatisticsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(float) * 8, 16), TEXT("Substrate.RoughRefrac.LobStat"));
		FRDGBufferUAVRef LobStatisticsBufferUAV = GraphBuilder.CreateUAV(LobStatisticsBuffer, PF_R32_UINT);
		FRDGBufferSRVRef LobStatisticsBufferSRV = GraphBuilder.CreateSRV(LobStatisticsBuffer, PF_R32_UINT);

		//////////////////////////////////////////////////////////////////////////
		// Clear resources
		AddClearUAVPass(GraphBuilder, SampleCountTextureUAV, uint32(0));

		//////////////////////////////////////////////////////////////////////////
		// Trace and update resources
		{
			FEvaluateRoughRefractionLobeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FEvaluateRoughRefractionLobeCS::FParameters>();
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->SampleCountTextureUAV = SampleCountTextureUAV;
			PassParameters->LobStatisticsBufferUAV = LobStatisticsBufferUAV;
			PassParameters->MiniFontTexture = GetMiniFontTexture();
			ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintParameters);
			PassParameters->TraceSqrtSampleCount = 128;

			FEvaluateRoughRefractionLobeCS::FPermutationDomain PermutationVector;
			TShaderMapRef<FEvaluateRoughRefractionLobeCS> ComputeShader(View.ShaderMap, PermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("Substrate::EvaluateRoughRefractionLobeCS"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(1, FEvaluateRoughRefractionLobeCS::ThreadGroupSize));
		}



		//////////////////////////////////////////////////////////////////////////
		// Debug print everything on screen
		{
			const float TraceDomainSize = 32.0f;
			const float SlabInterfaceLineCount = 16.0f;

			ShaderPrint::RequestSpaceForLines((TraceDomainSize * TraceDomainSize + SlabInterfaceLineCount * SlabInterfaceLineCount * 2) * 2); // overallocate * 2 for on the fly added debug
			ShaderPrint::RequestSpaceForCharacters(256);

			FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, View);
			FVisualizeRoughRefractionPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeRoughRefractionPS::FParameters>();
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->SampleCountTexture = SampleCountTextureSRV;
			PassParameters->LobStatisticsBuffer = LobStatisticsBufferSRV;
			PassParameters->MiniFontTexture = GetMiniFontTexture();
			ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintParameters);
			PassParameters->TraceDomainSize = TraceDomainSize;
			PassParameters->SlabInterfaceLineCount = SlabInterfaceLineCount;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenPassSceneColor.Texture, ERenderTargetLoadAction::ELoad);

			FVisualizeRoughRefractionPS::FPermutationDomain PermutationVector;
			TShaderMapRef<FVisualizeRoughRefractionPS> PixelShader(View.ShaderMap, PermutationVector);

			FRHIBlendState* PreMultipliedColorTransmittanceBlend = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();

			FPixelShaderUtils::AddFullscreenPass<FVisualizeRoughRefractionPS>(GraphBuilder, View.ShaderMap, RDG_EVENT_NAME("Substrate::VisualizeRoughRefractionPS"),
				PixelShader, PassParameters, View.ViewRect, PreMultipliedColorTransmittanceBlend);
		}

		//////////////////////////////////////////////////////////////////////////
		// Sample variance as a function of roughness
		{
			const uint32 RoughRefracDataByteSize = sizeof(float) * 2 * FRoughRefracDataCS::ThreadGroupSize;
			FRDGBufferRef RoughRefracDataBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(float) * 2, FRoughRefracDataCS::ThreadGroupSize), TEXT("Substrate.RoughRefrac.LobStat"));
			FRDGBufferUAVRef RoughRefracDataBufferUAV = GraphBuilder.CreateUAV(RoughRefracDataBuffer, PF_G32R32F);

			// Generate data in a buffer
			{
				FRoughRefracDataCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRoughRefracDataCS::FParameters>();
				PassParameters->RoughRefracDataUAV = RoughRefracDataBufferUAV;
				PassParameters->TraceSqrtSampleCount = 128;

				FRoughRefracDataCS::FPermutationDomain PermutationVector;
				TShaderMapRef<FRoughRefracDataCS> ComputeShader(View.ShaderMap, PermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("Substrate::FRoughRefracDataCS"),
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(FRoughRefracDataCS::ThreadGroupSize, FRoughRefracDataCS::ThreadGroupSize));
			}

			// Read back the data on CPU
			{
				FRoughRefracDataCS::FBufferReadBackParam* PassParameters = GraphBuilder.AllocParameters<FRoughRefracDataCS::FBufferReadBackParam>();
				PassParameters->Buffer = RoughRefracDataBuffer;

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("Substrate::FRoughRefracData ReadBack"),
					PassParameters,
					ERDGPassFlags::Readback,
					[RoughRefracDataBuffer, RoughRefracDataByteSize](FRHICommandListImmediate& RHICmdList)
				{
					check(IsInRenderingThread());
					FStagingBufferRHIRef StagingBuffer;
					StagingBuffer = RHICreateStagingBuffer();
					// Copy memory from GPU to CPU
					RHICmdList.CopyToStagingBuffer(RoughRefracDataBuffer->GetRHI(), StagingBuffer, 0, RoughRefracDataByteSize);

					// Submit all GPU work so far wait for completion.
					static const FName FenceName("DumpGPU.BufferFence");
					FGPUFenceRHIRef Fence = RHICreateGPUFence(FenceName);
					Fence->Clear();
					RHICmdList.WriteGPUFence(Fence);
					RHICmdList.SubmitAndBlockUntilGPUIdle();

					// Lock the buffer for read back
					void* RoughRefracData = RHICmdList.LockStagingBuffer(StagingBuffer, Fence.GetReference(), 0, RoughRefracDataByteSize);
					if (RoughRefracData)
					{
						// Dump to file
						RHICmdList.UnlockStagingBuffer(StagingBuffer);

						float* RoughRefracDataFloat = (float*)RoughRefracData;

						static bool bDataDumpDone = false;
						if (!bDataDumpDone)
						{
							bDataDumpDone = true;

							IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
							FString Filename = "RoughRefracData.txt";
							FString FullPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / Filename);

							FString OutString;
							const uint32 PointCount = FRoughRefracDataCS::ThreadGroupSize;
							for (int i = 0; i < PointCount; ++i)
							{
								OutString.Appendf(TEXT("%5.5f %5.5f\n"), RoughRefracDataFloat[i * 2 + 0], RoughRefracDataFloat[i * 2 + 1]);
							}

							FFileHelper::SaveStringToFile( OutString, *FullPath, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), FILEWRITE_None);

							// Example of commands to find a fit in Mathematica
							//	dataRV = Import["D:\\...\\RoughRefracData.txt", "Table"]
							//	L0 = ListPlot[dataRV]
							//	fitModel = With[{order = 4, dat = dataRV}, LinearModelFit[dat, Flatten@Outer[Times, Sequence @@	Transpose@Array[Power[{x}, #   - 1] &, order + 1]], { x }]]
							//	fitModel[x]
							//	Show[Plot[fitModel[x], {x, 0, 1}, PlotStyle -> Blue], ListPlot[dataRV, PlotStyle->Directive[Red, Opacity[0.5]]]]
							//	fitModel[0.0]
						}
					}
					//else
					//{
					//	check(false);
					//}

					StagingBuffer = nullptr;
					Fence = nullptr;
				});
			}
		}
	}
#endif
}


} // namespace Substrate
