// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Containers/Queue.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Engine.h"
#include "GlobalShader.h"
#include "IndirectLightRendering.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Misc/LargeWorldCoordinates.h"
#include "PixelShaderUtils.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "ReflectionEnvironment.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/SubstrateMaterialShared.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHIGPUReadback.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "SceneTextures.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderPrintParameters.h"
#include "ShadingEnergyConservation.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "Substrate/Substrate.h"
#include "SubstrateVisualizationData.h"
#include "SubstrateVisualizeDefinitions.h"
#include "SystemTextures.h"
#include "UnrealEngine.h"
#include "SceneViewState.h"
#include "Materials/MaterialInterface.h"

namespace Substrate
{
// Forward declarations
void AddSubstrateInternalClassificationTilePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureRef* DepthTexture,
	const FRDGTextureRef* ColorTexture,
	ESubstrateTileType TileMaterialType,
	const bool bDebug);

static bool IsSubstrateDebugVisualizationSupported(EShaderPlatform InPlatform, bool bIsEditorOnly, EShaderPermutationFlags Flags)
{
	return 
		Substrate::IsSubstrateEnabled() && 
		Substrate::UsesSubstrateMaterialBuffer(InPlatform) &&
		GetMaxSupportedFeatureLevel(InPlatform) >= ERHIFeatureLevel::SM5 &&
		(bIsEditorOnly ? (IsPCPlatform(InPlatform) || EnumHasAllFlags(Flags, EShaderPermutationFlags::HasEditorOnlyData)) : true);
}

static bool SubstrateDebugVisualizationCanRunOnPlatform(EShaderPlatform InPlatform)
{
	return IsSubstrateDebugVisualizationSupported(InPlatform, false, EShaderPermutationFlags::None);
}

class FMaterialPrintInfoCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMaterialPrintInfoCS);
	SHADER_USE_PARAMETER_STRUCT(FMaterialPrintInfoCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, bOverrideCursorPosition)
		SHADER_PARAMETER(uint32, SubstrateDebugDataSizeInUints)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int>, SubstrateDebugDataUAV)
	END_SHADER_PARAMETER_STRUCT()

	static bool IsSupported(EShaderPlatform InPlatform, EShaderPermutationFlags InFlags=EShaderPermutationFlags::None)
	{
		return IsSubstrateDebugVisualizationSupported(InPlatform, false, InFlags);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsSupported(Parameters.Platform, Parameters.Flags);
	}
	
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return EShaderPermutationPrecacheRequest::NotPrecached; 
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// Stay debug and skip optimizations to reduce compilation time on this long shader.
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_MATERIALPRINT"), 1);
		OutEnvironment.SetDefine(TEXT("SUPPORTS_ANISOTROPIC_MATERIALS"), FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Parameters.Platform));
	}
};
IMPLEMENT_GLOBAL_SHADER(FMaterialPrintInfoCS, "/Engine/Private/Substrate/SubstrateVisualize.usf", "MaterialPrintInfoCS", SF_Compute);

class FVisualizeMaterialCountPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeMaterialCountPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeMaterialCountPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, ViewMode)
		SHADER_PARAMETER(uint32, bRealTimeUpdate)
		SHADER_PARAMETER(uint32, bOverrideCursorPosition)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool IsSupported(EShaderPlatform InPlatform, EShaderPermutationFlags InFlags=EShaderPermutationFlags::None)
	{
		return IsSubstrateDebugVisualizationSupported(InPlatform, false, InFlags);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsSupported(Parameters.Platform, Parameters.Flags);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// Stay debug and skip optimizations to reduce compilation time on this long shader.
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_MATERIALCOUNT"), 1);
		OutEnvironment.SetDefine(TEXT("SUPPORTS_ANISOTROPIC_MATERIALS"), FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Parameters.Platform));
	}
};
IMPLEMENT_GLOBAL_SHADER(FVisualizeMaterialCountPS, "/Engine/Private/Substrate/SubstrateVisualize.usf", "VisualizeMaterialPS", SF_Pixel);


class FSubstrateSystemInfoCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubstrateSystemInfoCS);
	SHADER_USE_PARAMETER_STRUCT(FSubstrateSystemInfoCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, SubstrateDebugDataSizeInUints)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ClassificationTileDrawIndirectBuffer)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int>, SubstrateDebugDataUAV)
	END_SHADER_PARAMETER_STRUCT()

	static bool IsSupported(EShaderPlatform InPlatform, EShaderPermutationFlags InFlags=EShaderPermutationFlags::None)
	{
		return IsSubstrateDebugVisualizationSupported(InPlatform, false, InFlags);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsSupported(Parameters.Platform, Parameters.Flags);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// Stay debug and skip optimizations to reduce compilation time on this long shader.
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_SYSTEMINFO"), 1);
		OutEnvironment.SetDefine(TEXT("SUPPORTS_ANISOTROPIC_MATERIALS"), FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Parameters.Platform));
	}
};
IMPLEMENT_GLOBAL_SHADER(FSubstrateSystemInfoCS, "/Engine/Private/Substrate/SubstrateVisualize.usf", "MainCS", SF_Compute);

class FMaterialDebugSubstrateTreeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMaterialDebugSubstrateTreeCS);
	SHADER_USE_PARAMETER_STRUCT(FMaterialDebugSubstrateTreeCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, bOverrideCursorPosition)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()

	static bool IsSupported(EShaderPlatform InPlatform, EShaderPermutationFlags InFlags=EShaderPermutationFlags::None)
	{
		return IsSubstrateDebugVisualizationSupported(InPlatform, true, InFlags)
			&& IsAdvancedVisualizationEnabled(InPlatform)
			&& !IsSubstrateBlendableGBufferEnabled(InPlatform)
			&& !IsVulkanPlatform(InPlatform)  // SUBSTRATE_TODO Move to CPU debug visualisation and it should then work on all platforms
			&& !IsMetalPlatform(InPlatform);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsSupported(Parameters.Platform, Parameters.Flags);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// Stay debug and skip optimizations to reduce compilation time on this long shader.
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_DEBUGSUBSTRATETREE_CS"), 1);
		OutEnvironment.SetDefine(TEXT("SUPPORTS_ANISOTROPIC_MATERIALS"), FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Parameters.Platform));
	}
};
IMPLEMENT_GLOBAL_SHADER(FMaterialDebugSubstrateTreeCS, "/Engine/Private/Substrate/SubstrateVisualize.usf", "MaterialDebugSubstrateTreeCS", SF_Compute);

class FMaterialDebugSubstrateTreePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMaterialDebugSubstrateTreePS);
	SHADER_USE_PARAMETER_STRUCT(FMaterialDebugSubstrateTreePS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, bOverrideCursorPosition)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FReflectionUniformParameters, ReflectionStruct)
		SHADER_PARAMETER_STRUCT_REF(FReflectionCaptureShaderData, ReflectionCapture)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSkyDiffuseLightingParameters, SkyDiffuseLighting)
		SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGF)
		SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool IsSupported(EShaderPlatform InPlatform, EShaderPermutationFlags InFlags=EShaderPermutationFlags::None)
	{
		return IsSubstrateDebugVisualizationSupported(InPlatform, true, InFlags)
			&& IsAdvancedVisualizationEnabled(InPlatform)
			&& !IsSubstrateBlendableGBufferEnabled(InPlatform)
			&& !IsVulkanPlatform(InPlatform)  // SUBSTRATE_TODO Move to CPU debug visualisation and it should then work on all platforms
			&& !IsMetalPlatform(InPlatform);
	}
	
	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsSupported(Parameters.Platform, Parameters.Flags);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// Stay debug and skip optimizations to reduce compilation time on this long shader.
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_DEBUGSUBSTRATETREE_PS"), 1);
		OutEnvironment.SetDefine(TEXT("SUPPORTS_ANISOTROPIC_MATERIALS"), FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Parameters.Platform));
	}
};
IMPLEMENT_GLOBAL_SHADER(FMaterialDebugSubstrateTreePS, "/Engine/Private/Substrate/SubstrateVisualize.usf", "MaterialDebugSubstrateTreePS", SF_Pixel);

void AddProcessAndPrintSubstrateMaterialPropertiesPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SceneColorTexture, EShaderPlatform Platform, FSubstrateViewDebugData::FTransientPixelDebugBuffer& NewSubstratePixelDebugBuffer)
{
	if (!FMaterialPrintInfoCS::IsSupported(Platform)) return;
	FSubstrateViewDebugData& SubstrateViewDebugData = View.ViewState->GetSubstrateViewDebugData();

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Check if the latest readback query is ready and display the data on screen.
	TSharedPtr<FRHIGPUBufferReadback> AvailableReadback;
	SubstrateViewDebugData.PixelMaterialDebugDataReadbackQueries.Peek(AvailableReadback);
	if (AvailableReadback.IsValid() && AvailableReadback->IsReady())
	{
		SubstrateViewDebugData.PixelMaterialDebugDataReadbackQueries.Dequeue(AvailableReadback);

		// Access the data and copy to a frame transient buffer for rendering pass.
		void* PixelDebugData = AvailableReadback->Lock(SubstrateViewDebugData.PixelMaterialDebugDataSizeBytes);
		if (PixelDebugData == nullptr)
		{
			return;
		}
		void* PixelDebugDataToPrint = GraphBuilder.Alloc(SubstrateViewDebugData.PixelMaterialDebugDataSizeBytes);
		memcpy(PixelDebugDataToPrint, PixelDebugData, SubstrateViewDebugData.PixelMaterialDebugDataSizeBytes);

		////////////////////////////////////////////////////////////////////////////////////////////////////
		AddDrawCanvasPass(GraphBuilder, {}, View, FScreenPassRenderTarget(SceneColorTexture, View.ViewRect, ERenderTargetLoadAction::ELoad),
			[&View, PixelDebugDataToPrint](FCanvas& Canvas)
			{
				Canvas.SetScaledToRenderTarget(true);

				const FLinearColor Grey = FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);
				const FLinearColor Orange = FLinearColor(243.f / 255.f, 156.f / 255.f, 18.f / 255.f, 1.0);
				const FLinearColor CompR = FLinearColor(0.8f, 0.2f, 0.2f, 1.0);
				const FLinearColor CompG = FLinearColor(0.2f, 0.8f, 0.2f, 1.0);
				const FLinearColor CompB = FLinearColor(0.2f, 0.2f, 0.8f, 1.0);
				const float DrawPosXLeft = 40.0f;
				float DrawPosX = DrawPosXLeft;
				float DrawPosY = 50.0f;

				FSubstrateDebugDataSerializer S;
				S.SubstratePixelDebugData = (int*)PixelDebugDataToPrint;
				FSubstratePixelDebugData Data;
				SerializeSubstratePixelDebugData(S, Data);

				FString MaterialMode;
				switch (Data.MaterialMode)
				{
				case HEADER_MATERIALMODE_NONE:
					MaterialMode = TEXT("None");
					break;
				case HEADER_MATERIALMODE_SLAB_SIMPLE:
					MaterialMode = TEXT("Simple BSDF");
					break;
				case HEADER_MATERIALMODE_SLAB_SINGLE:
					MaterialMode = TEXT("Single Slab");
					break;
				case HEADER_MATERIALMODE_SLAB_COMPLEX:
					MaterialMode = TEXT("Complex Slab(s)");
					break;
				case HEADER_MATERIALMODE_SLWATER:
					MaterialMode = TEXT("Single Layer Water");
					break;
				case HEADER_MATERIALMODE_HAIR:
					MaterialMode = TEXT("Hair");
					break;
				case HEADER_MATERIALMODE_EYE:
					MaterialMode = TEXT("Eye");
					break;
				default:
					MaterialMode = TEXT("Unkown material mode, please update visualization code.");
				}
				if (Data.bIsComplexSpecialMaterial)
				{
					MaterialMode = TEXT("Complex Special Slab(s)");
				}

				FString OptimisedLegacyMode;
				switch (Data.OptimisedLegacyMode)
				{
				case SINGLE_OPTLEGACYMODE_NONE:
					break;
				case SINGLE_OPTLEGACYMODE_CLEARCOAT:
					MaterialMode += TEXT(" - Legacy Clear Coat");
					break;
				case SINGLE_OPTLEGACYMODE_CLOTH:
					MaterialMode += TEXT(" - Legacy Cloth");
					break;
				case SINGLE_OPTLEGACYMODE_SSSWRAP:
					MaterialMode += TEXT(" - Legacy SSS-Wrap");
					break;
				case SINGLE_OPTLEGACYMODE_SSSPROFILE:
					MaterialMode += TEXT(" - Legacy SSS-Profile");
					break;
				case SINGLE_OPTLEGACYMODE_TWO_SIDED_SSSWRAP:
					MaterialMode += TEXT(" - Legacy Two-Sided SSS-Wrap (Foliage)");
					break;
				default:
					MaterialMode += TEXT(" - Unkown optimised legacy material mode, please update visualization code.");
				}

				TArray<FCanvasTextItem> TextItemsBatch; // Not very efficient but this is for debug purpose
				FIntPoint BatchMinMaxY = FIntPoint(99999999, -99999999);

				auto BeginBatch = [&]()
				{
					TextItemsBatch.Empty();
					TextItemsBatch.Reserve(64);
					BatchMinMaxY = FIntPoint(99999999, -99999999);
				};
				auto ExpendBatchBounds = [&](float X, float Y)
				{
					BatchMinMaxY = FIntPoint(FMath::Min(BatchMinMaxY.X, Y - 5), FMath::Max(BatchMinMaxY.Y, Y + 20));
				};
				auto DispatchBatch = [&]()
				{
					Canvas.DrawTile(DrawPosXLeft - 10.0f, BatchMinMaxY.X, 1000.0f, BatchMinMaxY.Y - BatchMinMaxY.X, 0, 0, 1, 1, FLinearColor(0.0f, 0.0f, 0.0f, 0.3f));
					for (auto& Item : TextItemsBatch)
					{
						Canvas.DrawItem(Item);
					}
				};

				auto PrintS = [&](float X, float Y, FString Text, FLinearColor Color = FLinearColor::White)
				{
					TextItemsBatch.Push(FCanvasTextItem(FVector2D(X, Y), FText::FromString(*Text), GEngine->GetSmallFont(), Color));
					ExpendBatchBounds(X, Y);
				};
				auto PrintI = [&](float X, float Y, int Value, FLinearColor Color = FLinearColor::White)
				{
					FString String = FString::Printf(TEXT("%i"), Value);
					TextItemsBatch.Push(FCanvasTextItem(FVector2D(X, Y), FText::FromString(*String), GetStatsFont(), Color));
					ExpendBatchBounds(X, Y);
				};
				auto PrintUI = [&](float X, float Y, uint32 Value, FLinearColor Color = FLinearColor::White)
				{
					FString String = FString::Printf(TEXT("%u"), Value);
					TextItemsBatch.Push(FCanvasTextItem(FVector2D(X, Y), FText::FromString(*String), GetStatsFont(), Color));
					ExpendBatchBounds(X, Y);
				};
				auto PrintFSmall = [&](float X, float Y, float Value, FLinearColor Color = FLinearColor::White)
				{
					FString String = FString::Printf(TEXT("%1.3f"), Value);
					TextItemsBatch.Push(FCanvasTextItem(FVector2D(X, Y), FText::FromString(*String), GetStatsFont(), Color));
					ExpendBatchBounds(X, Y);
				};
				auto PrintFAdapt = [&](float X, float Y, float Value, FLinearColor Color = FLinearColor::White)
				{
					FString String;
					if (Value > 100)
					{
						String = FString::Printf(TEXT("%.0f"), Value);
					}
					else if (Value > 10)
					{
						String = FString::Printf(TEXT("%2.2f"), Value);
					}
					else if (Value > 1)
					{
						String = FString::Printf(TEXT("%1.3f"), Value);
					}
					else
					{
						String = FString::Printf(TEXT("%0.4f"), Value);
					}

					TextItemsBatch.Push(FCanvasTextItem(FVector2D(X, Y), FText::FromString(*String), GetStatsFont(), Color));
					ExpendBatchBounds(X, Y);
				};
				auto PrintFColorSmall = [&](float X, float Y, float R, float G, float B)
				{
					PrintFSmall(X + 0.0f, Y, R, CompR);
					PrintFSmall(X + 40.0f, Y, G, CompG);
					PrintFSmall(X + 80.0f, Y, B, CompB);
				};
				auto PrintFColorAdapt = [&](float X, float Y, float R, float G, float B)
				{
					PrintFAdapt(X + 0.0f, Y, R, CompR);
					PrintFAdapt(X + 50.0f, Y, G, CompG);
					PrintFAdapt(X + 100.0f, Y, B, CompB);
				};
				auto PrintF2DAdapt = [&](float X, float Y, float R, float G)
				{
					PrintFAdapt(X + 0.0f, Y, R, CompR);
					PrintFAdapt(X + 50.0f, Y, G, CompG);
				};
				auto PrintFBool = [&](float X, float Y, float Value)
				{
					FString String = Value > 0.0f ? TEXT("Yes") : TEXT("No");
					TextItemsBatch.Push(FCanvasTextItem(FVector2D(X, Y), FText::FromString(*String), GetStatsFont(), Value > 0.0f ? FLinearColor::Green : FLinearColor::Red));
					ExpendBatchBounds(X, Y);
				};
				auto NewLine = [&]()
				{
					DrawPosY += 17;
				};

				// Header 
				{
					BeginBatch();

					PrintS(DrawPosX, DrawPosY, FString::Printf(TEXT("Closure Count = %i"), Data.ClosureCount), FLinearColor(0.2f, 0.8f, 0.2f, 1.0f));
					PrintS(DrawPosX + 130.0f, DrawPosY, FString::Printf(TEXT("Material Mode = %s"), *MaterialMode), FLinearColor(0.7f, 0.7f, 0.0f, 1.0f));
					NewLine();

					PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("AO"));
					PrintS(DrawPosX + 50.0f, DrawPosY, TEXT("Ind.Irradiance"));
					PrintS(DrawPosX + 150.0f, DrawPosY, TEXT("TopRoughness"));
					PrintS(DrawPosX + 250.0f, DrawPosY, TEXT("PreShadow"));
					PrintS(DrawPosX + 350.0f, DrawPosY, TEXT("ZeroShadow"));
					PrintS(DrawPosX + 450.0f, DrawPosY, TEXT("ContactShadow"));
					PrintS(DrawPosX + 550.0f, DrawPosY, TEXT("Ind.Occluder"));
					PrintS(DrawPosX + 650.0f, DrawPosY, TEXT("HasSSS"));
					PrintS(DrawPosX + 720.0f, DrawPosY, TEXT("BasisCount"));
					NewLine();
					PrintFSmall(DrawPosX + 0.0f, DrawPosY, Data.MaterialAO, Grey);
					PrintFSmall(DrawPosX + 50.0f, DrawPosY, Data.IndirectIrradiance, Grey);
					PrintFSmall(DrawPosX + 150.0f, DrawPosY, Data.TopLayerRoughness, Grey);
					PrintFBool(DrawPosX + 250.0f, DrawPosY, Data.HasPrecShadowMask);
					PrintFBool(DrawPosX + 350.0f, DrawPosY, Data.HasZeroPrecShadowMask);
					PrintFBool(DrawPosX + 450.0f, DrawPosY, Data.DoesCastContactShadow);
					PrintFBool(DrawPosX + 550.0f, DrawPosY, Data.HasDynamicIndirectShadowCasterRepresentation);
					PrintFBool(DrawPosX + 650.0f, DrawPosY, Data.HasSubsurface);
					PrintI(DrawPosX + 720.0f, DrawPosY, Data.LocalBasesCount, Grey);

					DispatchBatch();
				}

				NewLine();
				NewLine();

				// Each Closure
				for (int i = 0; i < Data.ClosureCount; ++i)
				{
					FSubsterateDebugClosure& Closure = Data.Closures[i];
					BeginBatch();

					FString Type;
					switch (Closure.Type)
					{
					case SUBSTRATE_BSDF_TYPE_SLAB:
						Type = TEXT("Slab");
						break;
					case SUBSTRATE_BSDF_TYPE_VOLUMETRICFOGCLOUD:
						Type = TEXT("VolumetricFogCloud");
						break;
					case SUBSTRATE_BSDF_TYPE_UNLIT:
						Type = TEXT("Unlit");
						break;
					case SUBSTRATE_BSDF_TYPE_HAIR:
						Type = TEXT("Hair");
						break;
					case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
						Type = TEXT("SingleLayerWater");
						break;
					case SUBSTRATE_BSDF_TYPE_TOON:
						Type = TEXT("Toon");
						break;
					case SUBSTRATE_BSDF_TYPE_EYE:
						Type = TEXT("Eye");
						break;
					default:
						MaterialMode = TEXT("Unkown closure type, please update visualization code.");
					}

					PrintS(DrawPosX, DrawPosY, Type, FLinearColor(0.2f, 1.0f, 0.2f, 1.0f));
					PrintS(DrawPosX + 100.0f, DrawPosY, FString::Printf(TEXT("[Address=%i]"), Closure.Address), FLinearColor(0.2f, 0.5f, 0.5f, 1.0f));
					NewLine();


					PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("NormalID"));
					PrintS(DrawPosX + 100.0f, DrawPosY, TEXT("BasisType"));
					PrintS(DrawPosX + 200.0f, DrawPosY, TEXT("Anisotropy"));
					PrintS(DrawPosX + 300.0f, DrawPosY, TEXT("TopLayer"));
					PrintS(DrawPosX + 400.0f, DrawPosY, TEXT("Scattering"));
					PrintS(DrawPosX + 500.0f, DrawPosY, TEXT("IsThin"));
					PrintS(DrawPosX + 550.0f, DrawPosY, TEXT("WeightV"));
					if (Closure.bHasWeightL)
					{
						PrintS(DrawPosX + 700.0f, DrawPosY, TEXT("CoverAlongN"));
						PrintS(DrawPosX + 800.0f, DrawPosY, TEXT("TransAlongN"));
					}
					NewLine();
					PrintI(DrawPosX + 0.0f, DrawPosY, Closure.NormalID, Grey);
					PrintS(DrawPosX + 100.0f, DrawPosY, Closure.BasisType == 0 ? TEXT("Normal") : TEXT("Nor+Tan"), Grey);
					PrintFBool(DrawPosX + 200.0f, DrawPosY, Closure.bHasAnisotropy);
					PrintFBool(DrawPosX + 300.0f, DrawPosY, Closure.bIsTopLayer);
					PrintFBool(DrawPosX + 400.0f, DrawPosY, Closure.SSSType);		// TODO: print the special type
					PrintFBool(DrawPosX + 500.0f, DrawPosY, Closure.bIsThin);
					if (Closure.bHasGreyWeightV)
					{
						PrintFSmall(DrawPosX + 550.0f, DrawPosY, Closure.LuminanceWeightR, Grey);
					}
					else
					{
						PrintFColorSmall(DrawPosX + 550.0f, DrawPosY, Closure.LuminanceWeightR, Closure.LuminanceWeightG, Closure.LuminanceWeightB);
					}
					if (Closure.bHasWeightL)
					{
						PrintFSmall(DrawPosX + 700.0f, DrawPosY, Closure.CoverageAboveAlongN, Grey);
						PrintFColorSmall(DrawPosX + 800.0f, DrawPosY, Closure.TransmittanceAboveAlongNR, Closure.TransmittanceAboveAlongNG, Closure.TransmittanceAboveAlongNB);
					}
					NewLine();
					DrawPosY += 5;	// a little bit more space between the header and data

					const float DataOffset = 130.0f;
					if (Closure.Type == SUBSTRATE_BSDF_TYPE_SLAB)
					{
						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Diffuse"), Orange);
						PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.DiffuseR, Closure.DiffuseG, Closure.DiffuseB);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("F0"), Orange);
						PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.F0R, Closure.F0G, Closure.F0B);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Roughness"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.Roughness);
						NewLine();

						if (Closure.bHasF90)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("F90"), Orange);
							PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.F90R, Closure.F90G, Closure.F90B);
							NewLine();
						}

						if (Closure.bHasAnisotropy)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Anisotropy"), Orange);
							PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.Anisotropy);
							NewLine();
						}

						if (Data.Closures[i].bHasHaziness)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Haziness"), Orange);
							NewLine();

							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Weight"), Orange);
							PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].HazeWeight);
							NewLine();

							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Roughness"), Orange);
							PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].HazeRoughness);
							NewLine();

							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - LegacyClearCoat"), Orange);
							PrintFBool(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].HazeSimpleClearCoatMode);
							NewLine();

							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - HasBottomNormal"), Orange);
							PrintFBool(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].HasBottomNormal);
							NewLine();
						}

						if (Closure.SSSType != SSS_TYPE_NONE || Closure.bIsThin)
						{
							if (Closure.SSSType == SSS_TYPE_WRAP || Closure.SSSType == SSS_TYPE_TWO_SIDED_WRAP)
							{
								if (Closure.SSSType == SSS_TYPE_WRAP)
								{
									PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("SSS Wrap (Legacy Subsurface)"), Orange);
								}
								else
								{
									PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("SSS Two-Sided Wrap (Legacy Foliage)"), Orange);
								}
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - MFP"), Orange);
								PrintFColorAdapt(DrawPosX + DataOffset, DrawPosY, Closure.SSSMFPR, Closure.SSSMFPG, Closure.SSSMFPB);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Thickness"), Orange);
								PrintFAdapt(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSThickness);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Phase"), Orange);
								PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSPhase);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Opacity"), Orange);
								PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSOpacity);
								NewLine();
							}
							else if (Closure.SSSType == SSS_TYPE_SIMPLEVOLUME)
							{
								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("SSS Simple Volume"), Orange);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - MFP"), Orange);
								PrintFColorAdapt(DrawPosX + DataOffset, DrawPosY, Closure.SSSMFPR, Closure.SSSMFPG, Closure.SSSMFPB);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Thickness"), Orange);
								PrintFAdapt(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSThickness);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Phase"), Orange);
								PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSPhase);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Opacity"), Orange);
								PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSOpacity);
								NewLine();
							}
							else if (Closure.SSSType == SSS_TYPE_DIFFUSION_PROFILE)
							{
								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("SSS Profile"), Orange);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - ProfileID"), Orange);
								PrintI(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSPRofileID);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - ProfileRadius"), Orange);
								PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSProfileRadius);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - MFP"), Orange);
								PrintFColorAdapt(DrawPosX + DataOffset, DrawPosY, Closure.SSSMFPR, Closure.SSSMFPG, Closure.SSSMFPB);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Phase"), Orange);
								PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSPhase);
								NewLine();

								if (Closure.bIsThin)
								{
									PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Thickness"), Orange);
									PrintFAdapt(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSThickness);
									NewLine();
								}
							}
							else if (Closure.SSSType == SSS_TYPE_DIFFUSION)
							{
								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("SSS Substrate Per Pixel Diffusion"), Orange);
								NewLine();

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - MFP"), Orange);
								PrintFColorAdapt(DrawPosX + DataOffset, DrawPosY, Closure.SSSMFPR, Closure.SSSMFPG, Closure.SSSMFPB);
								NewLine();

								if (Closure.bIsThin)
								{
									PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - MFPNorm"), Orange);
									PrintFColorAdapt(DrawPosX + DataOffset, DrawPosY, Closure.SSSRescaledMFPR, Closure.SSSRescaledMFPG, Closure.SSSRescaledMFPB);
									NewLine();

									PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Thickness"), Orange);
									PrintFAdapt(DrawPosX + DataOffset, DrawPosY, Closure.SSSThickness);
									NewLine();
								}

								PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Phase"), Orange);
								PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SSSPhase);
								NewLine();
							}
						}

						if (Data.Closures[i].FuzzAmount > 0.0f)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Fuzz"), Orange);
							NewLine();

							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - FuzzAmount"), Orange);
							PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].FuzzAmount);
							NewLine();

							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - FuzzColor"), Orange);
							PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.FuzzColorR, Closure.FuzzColorG, Closure.FuzzColorB);
							NewLine();

							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - FuzzRoughness"), Orange);
							PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].FuzzRoughness);
							NewLine();
						}

						if (Data.Closures[i].GlintValue < 1.0f)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Glints"), Orange);
							NewLine();

							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Density"), Orange);
							PrintFSmall(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].GlintValue);
							NewLine();
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - UV DDX"), Orange);
							PrintF2DAdapt(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].GlintUVDDXx, Data.Closures[i].GlintUVDDXy);
							NewLine();
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - UV DDY"), Orange);
							PrintF2DAdapt(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].GlintUVDDYx, Data.Closures[i].GlintUVDDYy);
							NewLine();
						}

						if (Data.Closures[i].SpecProfileID >= 0)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Specular Profile"), Orange);
							NewLine();

							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - SpecProfileID"), Orange);
							PrintI(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SpecProfileID);
							NewLine();

							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - SpecProfileMode"), Orange);
							PrintS(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].SpecProfileParameterization == 0 ? TEXT("View/Light angles") : TEXT("Half angles"));
							NewLine();
						}
					}
					else if (Closure.Type == SUBSTRATE_BSDF_TYPE_HAIR)
					{
						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("BaseColor"), Orange);
						PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.DiffuseR, Closure.DiffuseG, Closure.DiffuseB);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Specular"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.F0R);// Aliased variable for Hair BSDF
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Roughness"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.Roughness);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Scattering"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.F90R);// Aliased variable for Hair BSDF
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("BackLit"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.F90G);// Aliased variable for Hair BSDF
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Transmittance"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.F90B);// Aliased variable for Hair BSDF
						NewLine();
					}
					else if (Closure.Type == SUBSTRATE_BSDF_TYPE_EYE)
					{
						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Diffuse"), Orange);
						PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.DiffuseR, Closure.DiffuseG, Closure.DiffuseB);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("F0"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.F0R);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Roughness"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.Roughness);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("IrisMask"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.F90R);// Aliased variable for Eye BSDF
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("IrisDistance"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.F90G);// Aliased variable for Eye BSDF
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("IrisNormal"), Orange);
						PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.SSSMFPR, Closure.SSSMFPG, Closure.SSSMFPB);// Aliased variable for Eye BSDF
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("IrisPlaneNormal"), Orange);
						PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.SSSRescaledMFPR, Closure.SSSRescaledMFPG, Closure.SSSRescaledMFPB);// Aliased variable for Eye BSDF
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("SSSProfileID"), Orange);
						PrintI(DrawPosX + DataOffset, DrawPosY, Closure.SSSPRofileID);
						NewLine();
					}
					else if (Closure.Type == SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER)
					{
						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("BaseColor"), Orange);
						PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.DiffuseR, Closure.DiffuseG, Closure.DiffuseB);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Specular"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.F0R);// Aliased variable for SLW BSDF
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Metallic"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.F0G);// Aliased variable for SLW BSDF
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Roughness"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.Roughness);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("TopMatOpacity"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.SSSOpacity);
						NewLine();
					}
					else if (Closure.Type == SUBSTRATE_BSDF_TYPE_TOON)
					{
						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("ToonProfileID"), Orange);
						PrintI(DrawPosX + DataOffset, DrawPosY, Closure.SSSPRofileID);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Diffuse Color"), Orange);
						PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.DiffuseR, Closure.DiffuseG, Closure.DiffuseB);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Specular Color"), Orange);
						PrintFColorSmall(DrawPosX + DataOffset, DrawPosY, Closure.F0R, Closure.F0G, Closure.F0B);
						NewLine();

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Roughness"), Orange);
						PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.Roughness);
						NewLine();

						if (Closure.bHasAnisotropy)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Anisotropy"), Orange);
							PrintFSmall(DrawPosX + DataOffset, DrawPosY, Closure.Anisotropy);
							NewLine();
						}

						PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Pattern UVs"), Orange);
						PrintF2DAdapt(DrawPosX + DataOffset, DrawPosY, Data.Closures[i].GlintUVDDXx, Data.Closures[i].GlintUVDDXy);
						NewLine();

						if (Closure.SSSType == SSS_TYPE_TRANSMITTANCE_SHADOW)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Transmittance Shadow"), Orange);
							NewLine();
				
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - MFP"), Orange);
							PrintFColorAdapt(DrawPosX + DataOffset, DrawPosY, Closure.SSSMFPR, Closure.SSSMFPG, Closure.SSSMFPB);
							NewLine();
						}
					}

					DispatchBatch();

					NewLine();
					NewLine();
				}

				// Footer
				{
					BeginBatch();
					const float DataOffset = 130.0f;

					PrintS(DrawPosX, DrawPosY, TEXT("Memory Transactions"), FLinearColor(0.2f, 1.0f, 0.2f, 1.0f));
					NewLine();

					if (Data.MemoryDisplayMode > 0)
					{
						if (Data.MemoryDisplayMode == 1)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Header + BSDF"), Orange);
							PrintI(DrawPosX + DataOffset, DrawPosY, Data.MemorySlotA);
							NewLine();
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - TopNormalTex"), Orange);
							PrintI(DrawPosX + DataOffset, DrawPosY, Data.MemorySlotB);
							NewLine();
						}
						else if (Data.MemoryDisplayMode == 2)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Header"), Orange);
							PrintI(DrawPosX + DataOffset, DrawPosY, Data.MemorySlotA);
							NewLine();
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - TopNormalTex"), Orange);
							PrintI(DrawPosX + DataOffset, DrawPosY, Data.MemorySlotB);
							NewLine();
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - BSDF"), Orange);
							PrintI(DrawPosX + DataOffset, DrawPosY, Data.MemorySlotC);
							NewLine();
						}
						else if (Data.MemoryDisplayMode == 3)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Header + Normals"), Orange);
							PrintI(DrawPosX + DataOffset, DrawPosY, Data.MemorySlotA);
							NewLine();
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - BSDFs"), Orange);
							PrintI(DrawPosX + DataOffset, DrawPosY, Data.MemorySlotB);
							NewLine();
						}
						else
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("Unkown Memory Display Mode"), FLinearColor::Red);
						}

						if (Data.MemorySSSData > 0)
						{
							PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - SSS Data"), Orange);
							PrintI(DrawPosX + DataOffset, DrawPosY, Data.MemorySSSData);
							NewLine();
						}
					}

					PrintS(DrawPosX + 0.0f, DrawPosY, TEXT(" - Total"), Orange);
					PrintI(DrawPosX + DataOffset, DrawPosY, Data.MemoryTotal);
					NewLine();
					NewLine();

					DispatchBatch();
				}

				BeginBatch();
				PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("GPUFrameNumber"), FLinearColor::White);
				PrintUI(DrawPosX + 150, DrawPosY, *(uint32*)(&Data.GPUFrameNumber));
				NewLine();
				PrintS(DrawPosX + 0.0f, DrawPosY, TEXT("CPUFrameNumber"), FLinearColor::White);
				PrintUI(DrawPosX + 150, DrawPosY, View.CachedViewUniformShaderParameters->FrameNumber);
				NewLine();
				DispatchBatch();
			});
		////////////////////////////////////////////////////////////////////////////////////////////////////

		AvailableReadback->Unlock();
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Copy the debug data on GPU into a readback buffer for debug display when available later.
	TSharedPtr<FRHIGPUBufferReadback> NewReadBack = MakeShared<FRHIGPUBufferReadback>(TEXT("PixelMaterialDebugDataReadback"));
	AddEnqueueCopyPass(GraphBuilder, NewReadBack.Get(), NewSubstratePixelDebugBuffer.DebugData, SubstrateViewDebugData.PixelMaterialDebugDataSizeBytes);
	SubstrateViewDebugData.PixelMaterialDebugDataReadbackQueries.Enqueue(NewReadBack);
}

static void AddVisualizeMaterialPropertiesPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor, EShaderPlatform Platform)
{
	if (!FMaterialPrintInfoCS::IsSupported(Platform) || !View.ViewState) return;
	FSubstrateViewDebugData& SubstrateViewDebugData = View.ViewState->GetSubstrateViewDebugData();

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);
	ShaderPrint::RequestSpaceForLines(1024);
	ShaderPrint::RequestSpaceForCharacters(1024);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Get a new pixel material buffer and render debug data into it
	FSubstrateViewDebugData::FTransientPixelDebugBuffer NewSubstratePixelDebugBuffer = SubstrateViewDebugData.CreateTransientPixelDebugBuffer(GraphBuilder);
	{
		FMaterialPrintInfoCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMaterialPrintInfoCS::FParameters>();
		PassParameters->bOverrideCursorPosition = WITH_EDITOR ? 0u : 1u;
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, View);
		PassParameters->SceneTexturesStruct = View.GetSceneTextures().UniformBuffer;
		PassParameters->SubstrateDebugDataSizeInUints = NewSubstratePixelDebugBuffer.DebugDataSizeInUints;
		PassParameters->SubstrateDebugDataUAV = NewSubstratePixelDebugBuffer.DebugDataUAV;
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintParameters);

		TShaderMapRef<FMaterialPrintInfoCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("Substrate::VisualizeMaterial(Print)"), ComputeShader, PassParameters, FIntVector(1, 1, 1));
	}

	AddProcessAndPrintSubstrateMaterialPropertiesPasses(GraphBuilder, View, ScreenPassSceneColor.Texture, Platform, NewSubstratePixelDebugBuffer);
}

static void AddVisualizeMaterialCountPasses(FRDGBuilder & GraphBuilder, const FViewInfo & View, FScreenPassTexture & ScreenPassSceneColor, EShaderPlatform Platform, uint32 ViewMode)
{
	if (!FVisualizeMaterialCountPS::IsSupported(Platform)) return;

	ShaderPrint::SetEnabled(true);
	ShaderPrint::RequestSpaceForLines(1024);
	ShaderPrint::RequestSpaceForCharacters(1024);

	FRDGTextureRef SceneColorTexture = ScreenPassSceneColor.Texture;
	FRHIBlendState* PreMultipliedColorTransmittanceBlend = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();

	FVisualizeMaterialCountPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeMaterialCountPS::FParameters>();
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->ViewMode = FMath::Clamp(ViewMode, 2, 3);
	PassParameters->bRealTimeUpdate = View.Family->bRealtimeUpdate ? 1 : 0;
	PassParameters->bOverrideCursorPosition = WITH_EDITOR ? 0u : 1u;
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, View);
	PassParameters->SceneTexturesStruct = View.GetSceneTextures().UniformBuffer;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintParameters);

	FVisualizeMaterialCountPS::FPermutationDomain PermutationVector;
	TShaderMapRef<FVisualizeMaterialCountPS> PixelShader(View.ShaderMap, PermutationVector);

	FPixelShaderUtils::AddFullscreenPass<FVisualizeMaterialCountPS>(GraphBuilder, View.ShaderMap, RDG_EVENT_NAME("Substrate::VisualizeMaterial(Draw)"), PixelShader, PassParameters, ScreenPassSceneColor.ViewRect, PreMultipliedColorTransmittanceBlend);
}

bool IsClassificationAsync();
bool SupportsCMask(const FStaticShaderPlatform InPlatform);
bool UsesSubstrateClosureCountFromMaterialData();
uint32 GetMaterialBufferAllocationMode();
bool Is8bitTileCoordEnabled();

static void AddVisualizeSystemInfoPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor, EShaderPlatform Platform)
{
	if (!FSubstrateSystemInfoCS::IsSupported(Platform) || !View.ViewState)
	{
		return;
	}
	const bool bSubstrateBlendableGBufferEnabled = IsSubstrateBlendableGBufferEnabled(View.GetShaderPlatform());

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);
	ShaderPrint::RequestSpaceForLines(1024);
	ShaderPrint::RequestSpaceForCharacters(1024);

	const FShadingEnergyConservationData ShadingEnergyConservationData = ShadingEnergyConservation::GetData(View);

	FSubstrateSystemInfoCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubstrateSystemInfoCS::FParameters>();
	PassParameters->ClassificationTileDrawIndirectBuffer = GraphBuilder.CreateSRV(View.SubstrateViewData.ClassificationTileDrawIndirectBuffer, PF_R32_UINT);
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, View);
	PassParameters->SceneTexturesStruct = View.GetSceneTextures().UniformBuffer;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Get a new pixel material buffer and render debug data into it
	FSubstrateViewDebugData& SubstrateViewDebugData = View.ViewState->GetSubstrateViewDebugData();
	FSubstrateViewDebugData::FTransientSystemInfoDebugBuffer NewSubstrateSystemInfoDebugBuffer = SubstrateViewDebugData.CreateTransientSystemInfoDebugBuffer(GraphBuilder);
	PassParameters->SubstrateDebugDataSizeInUints = NewSubstrateSystemInfoDebugBuffer.DebugDataSizeInUints;
	PassParameters->SubstrateDebugDataUAV = NewSubstrateSystemInfoDebugBuffer.DebugDataUAV;

	TShaderMapRef<FSubstrateSystemInfoCS> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("Substrate::VisualizeSystemInfo"), ComputeShader, PassParameters, FIntVector(1, 1, 1));

	struct FSystemInformation
	{
		int CurrentBytesPerPixel;
		int ProjectMaxBytesPerPixel;
		int MaterialBufferAllocationInBytes;
		int ViewsMaxBytesPerPixel;
		int CurrentClosuresPerPixel;
		int ViewsMaxClosuresPerPixel;
		int ProjectMaxClosuresPerPixel;
		int MaterialBufferAllocationMode;
		int ShadingQuality;
		int bRoughDiffuse;
		int bEnergyConservation;
		int bEnergyPreservation;
		int bUseClosureCountFromMaterialData;
		int TileClosureCount;
		int bDbufferPass;
		int bRoughRefraction;

		int TileSize;
		int TileCountX;
		int TileCountY;
		int bCMask;
		int bAsync;
		int b8BitsCoord;

		int bAdvancedDebugEnabled;
		int LayerPeelIndex;
	} SystemInformation;
	SystemInformation.CurrentBytesPerPixel = View.SubstrateViewData.SceneData->EffectiveMaxBytesPerPixel;
	SystemInformation.ProjectMaxBytesPerPixel = GetBytePerPixel(View.GetShaderPlatform());
	SystemInformation.MaterialBufferAllocationInBytes = 0;
	if (!bSubstrateBlendableGBufferEnabled)
	{
		const FRDGTextureDesc MaterialBufferDesc = View.SubstrateViewData.SceneData->MaterialTextureArray->Desc;
		SystemInformation.MaterialBufferAllocationInBytes = MaterialBufferDesc.Extent.X * MaterialBufferDesc.Extent.Y * MaterialBufferDesc.ArraySize * sizeof(uint32);
	}
	SystemInformation.ViewsMaxBytesPerPixel = View.SubstrateViewData.SceneData->ViewsMaxBytesPerPixel;
	SystemInformation.CurrentClosuresPerPixel = View.SubstrateViewData.SceneData->EffectiveMaxClosurePerPixel;
	SystemInformation.ViewsMaxClosuresPerPixel = View.SubstrateViewData.SceneData->ViewsMaxClosurePerPixel;
	SystemInformation.ProjectMaxClosuresPerPixel = GetClosurePerPixel(View.GetShaderPlatform(), GetCurrentMaterialQualityLevelChecked());
	SystemInformation.MaterialBufferAllocationMode = GetMaterialBufferAllocationMode();
	SystemInformation.ShadingQuality = Substrate::GetShadingQuality(View.GetShaderPlatform());
	SystemInformation.bRoughDiffuse = View.SubstrateViewData.SceneData->bRoughDiffuse;
	SystemInformation.bEnergyConservation = ShadingEnergyConservationData.bEnergyConservation;
	SystemInformation.bEnergyPreservation = ShadingEnergyConservationData.bEnergyPreservation;
	SystemInformation.bUseClosureCountFromMaterialData = UsesSubstrateClosureCountFromMaterialData() ? 1 : 0;
	SystemInformation.TileClosureCount = GetSubstrateMaxClosureCount(View);
	SystemInformation.bDbufferPass = IsDBufferPassEnabled(View.GetShaderPlatform()) ? 1 : 0;
	SystemInformation.bRoughRefraction = IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()) ? 1 : 0;
	SystemInformation.TileSize = SUBSTRATE_TILE_SIZE;
	SystemInformation.TileCountX = View.SubstrateViewData.TileCount.X;
	SystemInformation.TileCountY = View.SubstrateViewData.TileCount.Y;
	SystemInformation.bCMask = SupportsCMask(View.GetShaderPlatform()) ? 1 : 0;
	SystemInformation.bAsync = IsClassificationAsync() ? 1 : 0;
	SystemInformation.b8BitsCoord = Is8bitTileCoordEnabled() ? 1 : 0;
	SystemInformation.bAdvancedDebugEnabled = IsAdvancedVisualizationEnabled(View.GetShaderPlatform()) ? 1 : 0;
	SystemInformation.LayerPeelIndex = View.SubstrateViewData.SceneData->PeelLayersAboveDepth;

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Check if the latest readback query is ready and display the data on screen.
	TSharedPtr<FRHIGPUBufferReadback> AvailableReadback;
	SubstrateViewDebugData.SystemInfoDebugDataReadbackQueries.Peek(AvailableReadback);
	if (AvailableReadback.IsValid() && AvailableReadback->IsReady())
	{
		SubstrateViewDebugData.SystemInfoDebugDataReadbackQueries.Dequeue(AvailableReadback);

		// Access the data and copy to a frame transient buffer for rendering pass.
		void* SystemInfoDebugData = AvailableReadback->Lock(SubstrateViewDebugData.SystemInfoDebugDataSizeBytes);
		if (SystemInfoDebugData == nullptr)
		{
			return;
		}
		int* SystemInfoDebugDataToPrint = (int*)GraphBuilder.Alloc(SubstrateViewDebugData.SystemInfoDebugDataSizeBytes);
		memcpy(SystemInfoDebugDataToPrint, SystemInfoDebugData, SubstrateViewDebugData.SystemInfoDebugDataSizeBytes);

		////////////////////////////////////////////////////////////////////////////////////////////////////
		AddDrawCanvasPass(GraphBuilder, {}, View, FScreenPassRenderTarget(ScreenPassSceneColor.Texture, View.ViewRect, ERenderTargetLoadAction::ELoad),
			[&View, SystemInfoDebugDataToPrint, SystemInformation, bSubstrateBlendableGBufferEnabled](FCanvas& Canvas)
		{
			Canvas.SetScaledToRenderTarget(true);

			FSubstrateDebugDataSerializer S;
			S.SubstratePixelDebugData = (int*)SystemInfoDebugDataToPrint;
			FSubstrateSystemInfoData Data;
			SerializeSubstrateSystemInfoDebugData(S, Data);

			float DrawPosX = 50.0f;
			float DrawPosY = 50.0f;

			auto PrintI = [&](float X, float Y, int Value, FLinearColor Color = FLinearColor::White)
			{
				FString String = FString::Printf(TEXT("%i"), Value);
				FCanvasTextItem Item = FCanvasTextItem(FVector2D(X, Y), FText::FromString(*String), GetStatsFont(), Color);
				Canvas.DrawItem(Item);
			};
			auto PrintS = [&](float X, float Y, FString Text, FLinearColor Color = FLinearColor::White)
			{
				FCanvasTextItem Item = FCanvasTextItem(FVector2D(X, Y), FText::FromString(*Text), GEngine->GetSmallFont(), Color);
				Canvas.DrawItem(Item);
			};
			auto PrintBool = [&](float X, float Y, int Value)
			{
				FString String = Value > 0.0f ? TEXT("Yes") : TEXT("No");
				FCanvasTextItem Item = FCanvasTextItem(FVector2D(X, Y), FText::FromString(*String), GetStatsFont(), Value > 0.0f ? FLinearColor::Green : FLinearColor::Red);
				Canvas.DrawItem(Item);
			};
			auto NewLine = [&]()
			{
				DrawPosY += 17;
			};

			Canvas.DrawTile(DrawPosX - 10.0f, DrawPosY - 10.0f, 350.0f, 800.0f, 0, 0, 1, 1, FLinearColor(0.0f, 0.0f, 0.0f, 0.3f));

			const FLinearColor White = FLinearColor::White;
			const FLinearColor Yellow = FLinearColor(1, 1, 0, 1);
			const FLinearColor DarkYellow = FLinearColor(0.5, 0.5, 0, 1);
			const FLinearColor Orange = FLinearColor(243.f / 255.f, 156.f / 255.f, 18.f / 255.f, 1.0);
			FString Text;

			PrintS(DrawPosX, DrawPosY, TEXT("General"), Yellow);
			NewLine();
			{
				if (bSubstrateBlendableGBufferEnabled)
				{
					PrintS(DrawPosX, DrawPosY, TEXT("  Using blendable GBuffer with fixed layout."), DarkYellow);
					NewLine();
					PrintS(DrawPosX, DrawPosY, TEXT("  Bytes Per Pixel"), White);
					PrintI(DrawPosX + 150, DrawPosY, 4*4, Orange);
					NewLine();
					PrintS(DrawPosX, DrawPosY, TEXT("  Closures Per Pixel"), White);
					PrintI(DrawPosX + 150, DrawPosY, 1, Orange);
				}
				else
				{
					PrintS(DrawPosX, DrawPosY, TEXT("  Bytes Per Pixel"), DarkYellow);
					NewLine();
					PrintS(DrawPosX + 0, DrawPosY, TEXT("    Max"), White);
					PrintI(DrawPosX + 150, DrawPosY, SystemInformation.CurrentBytesPerPixel, Orange);
					PrintS(DrawPosX + 180, DrawPosY, TEXT("/"), DarkYellow);
					PrintI(DrawPosX + 190, DrawPosY, SystemInformation.ProjectMaxBytesPerPixel, Orange);
					Text = FString::Printf(TEXT("[%i MB]"), SystemInformation.MaterialBufferAllocationInBytes / (1024 * 1024));
					PrintS(DrawPosX + 220, DrawPosY, *Text, White);
					NewLine();
					PrintS(DrawPosX + 0, DrawPosY, TEXT("    Views"), White);
					PrintI(DrawPosX + 150, DrawPosY, SystemInformation.ViewsMaxBytesPerPixel, Orange);
					PrintS(DrawPosX + 180, DrawPosY, TEXT("/ "), DarkYellow);
					PrintI(DrawPosX + 190, DrawPosY, SystemInformation.ProjectMaxBytesPerPixel, Orange);
					NewLine();
					NewLine();

					PrintS(DrawPosX, DrawPosY, TEXT("  Closures Per Pixel"), DarkYellow);
					NewLine();
					PrintS(DrawPosX + 0, DrawPosY, TEXT("    Max"), White);
					PrintI(DrawPosX + 150, DrawPosY, SystemInformation.CurrentClosuresPerPixel, Orange);
					PrintS(DrawPosX + 180, DrawPosY, TEXT("/ "), DarkYellow);
					PrintI(DrawPosX + 190, DrawPosY, SystemInformation.ProjectMaxClosuresPerPixel, Orange);
					NewLine();
					PrintS(DrawPosX + 0, DrawPosY, TEXT("    Views"), White);
					PrintI(DrawPosX + 150, DrawPosY, SystemInformation.ViewsMaxClosuresPerPixel, Orange);
					PrintS(DrawPosX + 180, DrawPosY, TEXT("/ "), DarkYellow);
					PrintI(DrawPosX + 190, DrawPosY, SystemInformation.ProjectMaxClosuresPerPixel, Orange);
					NewLine();
					NewLine();

					PrintS(DrawPosX, DrawPosY, TEXT("  Allocation mode"), White);
					switch (SystemInformation.MaterialBufferAllocationMode)
					{
					case 0: PrintS(DrawPosX + 150, DrawPosY, TEXT("View based"), Orange); break;
					case 1: PrintS(DrawPosX + 150, DrawPosY, TEXT("View based | Growing-only"), Orange); break;
					case 2: PrintS(DrawPosX + 150, DrawPosY, TEXT("Setting based"), Orange); break;
					default: PrintS(DrawPosX + 150, DrawPosY, TEXT("Unkown"), Orange); break;
					}
				}

				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  Shading quality"), White);
				PrintI(DrawPosX + 150, DrawPosY, SystemInformation.ShadingQuality, Orange);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  Rough diffuse"), White);
				PrintBool(DrawPosX + 150, DrawPosY, SystemInformation.bRoughDiffuse);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  Energy conservation"), White);
				PrintBool(DrawPosX + 150, DrawPosY, SystemInformation.bEnergyConservation);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  Energy preservation"), White);
				PrintBool(DrawPosX + 150, DrawPosY, SystemInformation.bEnergyPreservation);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  Use #closures mat."), White);
				PrintBool(DrawPosX + 150, DrawPosY, SystemInformation.bUseClosureCountFromMaterialData);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  Tile closure Count"), White);
				PrintI(DrawPosX + 150, DrawPosY, SystemInformation.TileClosureCount, Orange);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  DBuffser pass"), White);
				PrintBool(DrawPosX + 150, DrawPosY, SystemInformation.bDbufferPass);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  Rough refraction"), White);
				PrintBool(DrawPosX + 150, DrawPosY, SystemInformation.bRoughRefraction);
				NewLine();
			}
			NewLine();

			PrintS(DrawPosX, DrawPosY, TEXT("Classification"), Yellow);
			NewLine();
			{
				PrintS(DrawPosX, DrawPosY, TEXT("  Tile size"), White);
				Text = FString::Printf(TEXT("%ix%i"), SystemInformation.TileSize, SystemInformation.TileSize);
				PrintS(DrawPosX + 150, DrawPosY, *Text, Orange);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  Tile Resolution"), White);
				Text = FString::Printf(TEXT("%ix%i"), SystemInformation.TileCountX, SystemInformation.TileCountY);
				PrintS(DrawPosX + 150, DrawPosY, *Text, Orange);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  CMask Classification"), White);
				PrintBool(DrawPosX + 150, DrawPosY, SystemInformation.bCMask);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  Async Compute"), White);
				PrintBool(DrawPosX + 150, DrawPosY, SystemInformation.bAsync);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  8Bits coord"), White);
				PrintBool(DrawPosX + 150, DrawPosY, SystemInformation.b8BitsCoord);
				NewLine();
			}
			NewLine();

			PrintS(DrawPosX, DrawPosY, TEXT("Material Tiles (GPU)"), Yellow);
			NewLine();
			{
				PrintS(DrawPosX, DrawPosY, TEXT("  # Simple tiles"), White);
				PrintI(DrawPosX + 150, DrawPosY, Data.TileCount[SUBSTRATE_TILE_TYPE_SIMPLE], Orange);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  # Single tiles"), White);
				PrintI(DrawPosX + 150, DrawPosY, Data.TileCount[SUBSTRATE_TILE_TYPE_SINGLE], Orange);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  # Complex tiles"), White);
				PrintI(DrawPosX + 150, DrawPosY, Data.TileCount[SUBSTRATE_TILE_TYPE_COMPLEX], Orange);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  # Complex Special tiles"), White);
				PrintI(DrawPosX + 150, DrawPosY, Data.TileCount[SUBSTRATE_TILE_TYPE_COMPLEX_SPECIAL], Orange);
				NewLine();
				if (SystemInformation.bRoughRefraction)
				{
					PrintS(DrawPosX, DrawPosY, TEXT("  # Rough refract. tiles"), White);
					PrintI(DrawPosX + 150, DrawPosY, Data.TileCount[SUBSTRATE_TILE_TYPE_ROUGH_REFRACT], Orange);
					NewLine();
					PrintS(DrawPosX, DrawPosY, TEXT("  # Rough r. w/o SSS tiles"), White);
					PrintI(DrawPosX + 150, DrawPosY, Data.TileCount[SUBSTRATE_TILE_TYPE_ROUGH_REFRACT_SSS_WITHOUT], Orange);
					NewLine();
				}
				if (SystemInformation.bDbufferPass)
				{
					PrintS(DrawPosX, DrawPosY, TEXT("  # Decal Simple tiles"), White);
					PrintI(DrawPosX + 150, DrawPosY, Data.TileCount[SUBSTRATE_TILE_TYPE_DECAL_SIMPLE], Orange);
					NewLine();
					PrintS(DrawPosX, DrawPosY, TEXT("  # Decal Single tiles"), White);
					PrintI(DrawPosX + 150, DrawPosY, Data.TileCount[SUBSTRATE_TILE_TYPE_DECAL_SINGLE], Orange);
					NewLine();
					PrintS(DrawPosX, DrawPosY, TEXT("  # Decal Complex tiles"), White);
					PrintI(DrawPosX + 150, DrawPosY, Data.TileCount[SUBSTRATE_TILE_TYPE_DECAL_COMPLEX], Orange);
					NewLine();
				}
			}
			NewLine();

			PrintS(DrawPosX, DrawPosY, TEXT("Debug"), Yellow);
			NewLine();
			{
				PrintS(DrawPosX, DrawPosY, TEXT("  # ADv. Debug"), White);
				PrintBool(DrawPosX + 150, DrawPosY, SystemInformation.bAdvancedDebugEnabled);
				NewLine();
				PrintS(DrawPosX, DrawPosY, TEXT("  # Decal Complex tiles"), White);
				PrintI(DrawPosX + 150, DrawPosY, SystemInformation.LayerPeelIndex, Orange);
				NewLine();
			}
			NewLine();

			PrintS(DrawPosX, DrawPosY, TEXT("In View (CPU)"), Yellow);
			NewLine();
			{
				for (uint32 TileTypeIndex = 0; TileTypeIndex <= ESubstrateTileType::EComplexSpecial; ++TileTypeIndex)
				{
					const ESubstrateTileType TileType = ESubstrateTileType(TileTypeIndex);
					const FString TileTypeName = ToString(ESubstrateTileType(TileType));
					const FString FormattedTypeType = FString::Format(TEXT("  {0}"), {*TileTypeName});
					PrintS(DrawPosX, DrawPosY, FormattedTypeType, White);
					PrintBool(DrawPosX + 150, DrawPosY, Substrate::GetSubstrateUsesTileType(View, TileType));
					NewLine();
				}
				PrintS(DrawPosX, DrawPosY, TEXT("  Anisotropy"), White);
				PrintBool(DrawPosX + 150, DrawPosY, Substrate::GetSubstrateUsesAnisotropy(View));
				NewLine();

				PrintS(DrawPosX, DrawPosY, TEXT("  Sub-Surface"), White);
				PrintBool(DrawPosX + 150, DrawPosY, IsSubsurfaceRequiredForView(View));
				NewLine();
			}
			NewLine();
		});
		////////////////////////////////////////////////////////////////////////////////////////////////////

		AvailableReadback->Unlock();
	}


	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Copy the debug data on GPU into a readback buffer for debug display when available later.
	TSharedPtr<FRHIGPUBufferReadback> NewReadBack = MakeShared<FRHIGPUBufferReadback>(TEXT("SystemInfoDebugDataReadback"));
	AddEnqueueCopyPass(GraphBuilder, NewReadBack.Get(), NewSubstrateSystemInfoDebugBuffer.DebugData, SubstrateViewDebugData.SystemInfoDebugDataSizeBytes);
	SubstrateViewDebugData.SystemInfoDebugDataReadbackQueries.Enqueue(NewReadBack);
}

// Draw each material layer independently
static void AddVisualizeAdvancedMaterialPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor, EShaderPlatform Platform)
{
	if (!IsAdvancedVisualizationEnabled(Platform) ||
		!FMaterialDebugSubstrateTreeCS::IsSupported(Platform) || 
		!FMaterialDebugSubstrateTreePS::IsSupported(Platform))
	{
		return;
	}

	ShaderPrint::SetEnabled(true);
	ShaderPrint::RequestSpaceForLines(1024);
	ShaderPrint::RequestSpaceForCharacters(1024);

	FRDGTextureRef SceneColorTexture = ScreenPassSceneColor.Texture;
	FRHIBlendState* PreMultipliedColorTransmittanceBlend = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
	{
		FMaterialDebugSubstrateTreeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMaterialDebugSubstrateTreeCS::FParameters>();
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, View);
		PassParameters->SceneTexturesStruct = View.GetSceneTextures().UniformBuffer;
		PassParameters->bOverrideCursorPosition = WITH_EDITOR ? 0u : 1u;
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintParameters);

		TShaderMapRef<FMaterialDebugSubstrateTreeCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("Substrate::SubstrateAdvancedVisualization(Print)"), ComputeShader, PassParameters, FIntVector(1, 1, 1));
	}

	{
		FMaterialDebugSubstrateTreePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMaterialDebugSubstrateTreePS::FParameters>();
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, View);
		PassParameters->SceneTexturesStruct = View.GetSceneTextures().UniformBuffer;
		PassParameters->bOverrideCursorPosition = WITH_EDITOR ? 0u : 1u;
		PassParameters->ReflectionStruct = CreateReflectionUniformBuffer(GraphBuilder, View);
		PassParameters->ReflectionCapture = View.ReflectionCaptureUniformBuffer;
		PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);

		const float DynamicBentNormalAO = 0.0f;
		FSkyLightSceneProxy* NullSkyLight = nullptr;
		PassParameters->SkyDiffuseLighting = GetSkyDiffuseLightingParameters(NullSkyLight, DynamicBentNormalAO);

		PassParameters->PreIntegratedGF = GSystemTextures.PreintegratedGF->GetRHI();
		PassParameters->PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		FMaterialDebugSubstrateTreePS::FPermutationDomain PermutationVector;
		TShaderMapRef<FMaterialDebugSubstrateTreePS> PixelShader(View.ShaderMap, PermutationVector);

		FPixelShaderUtils::AddFullscreenPass<FMaterialDebugSubstrateTreePS>(GraphBuilder, View.ShaderMap, RDG_EVENT_NAME("Substrate::SubstrateAdvancedVisualization(Draw)"), PixelShader, PassParameters, ScreenPassSceneColor.ViewRect, PreMultipliedColorTransmittanceBlend);
	}
}

bool ShouldRenderSubstrateRoughRefractionRnD();
void SubstrateRoughRefractionRnD(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor);

static FSubstrateViewMode GetSubstrateVisualizeMode(const FViewInfo & View)
{
	FSubstrateViewMode Out = FSubstrateViewMode::None;
	if (IsSubstrateEnabled() && SubstrateDebugVisualizationCanRunOnPlatform(View.GetShaderPlatform()))
	{
		const uint32 ViewMode = FSubstrateVisualizationData::GetViewMode();
		switch (ViewMode)
		{
			case 1: return FSubstrateViewMode::MaterialProperties;
			case 2: return FSubstrateViewMode::MaterialCount;
			case 3: return FSubstrateViewMode::AdvancedMaterialProperties;
			case 4: return FSubstrateViewMode::MaterialClassification;
			case 5: return FSubstrateViewMode::DecalClassification;
			case 6: return FSubstrateViewMode::RoughRefractionClassification;
			case 7: return FSubstrateViewMode::SubstrateInfo;
			case 8: return FSubstrateViewMode::MaterialByteCount;
		}

		const FSubstrateVisualizationData& VisualizationData = GetSubstrateVisualizationData();
		if (View.Family && View.Family->EngineShowFlags.VisualizeSubstrate)
		{
			Out = VisualizationData.GetViewMode(View.CurrentSubstrateVisualizationMode);
		}
	}
	return Out;
}

bool ShouldRenderSubstrateDebugPasses(const FViewInfo& View)
{
	return GetSubstrateVisualizeMode(View) != FSubstrateViewMode::None || ShouldRenderSubstrateRoughRefractionRnD();
}

FScreenPassTexture AddSubstrateDebugPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor)
{
	check(IsSubstrateEnabled());

	const FSubstrateViewMode DebugMode = GetSubstrateVisualizeMode(View);
	if (DebugMode != FSubstrateViewMode::None)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "Substrate::VisualizeMaterial");

		const bool bDebugPass = true;
		if (DebugMode == FSubstrateViewMode::MaterialProperties)
		{
			AddVisualizeMaterialPropertiesPasses(GraphBuilder, View, ScreenPassSceneColor, View.GetShaderPlatform());
		}
		if (DebugMode == FSubstrateViewMode::MaterialCount)
		{
			AddVisualizeMaterialCountPasses(GraphBuilder, View, ScreenPassSceneColor, View.GetShaderPlatform(), 2);
		}
		if (DebugMode == FSubstrateViewMode::MaterialByteCount)
		{
			AddVisualizeMaterialCountPasses(GraphBuilder, View, ScreenPassSceneColor, View.GetShaderPlatform(), 3);
		}
		if (DebugMode == FSubstrateViewMode::AdvancedMaterialProperties)
		{
			AddVisualizeAdvancedMaterialPasses(GraphBuilder, View, ScreenPassSceneColor, View.GetShaderPlatform());
		}
		else if (DebugMode == FSubstrateViewMode::SubstrateInfo)
		{
			AddVisualizeSystemInfoPasses(GraphBuilder, View, ScreenPassSceneColor, View.GetShaderPlatform());
		}
		else if (DebugMode == FSubstrateViewMode::DecalClassification)
		{
			if (IsDBufferPassEnabled(View.GetShaderPlatform()))
			{
				AddSubstrateInternalClassificationTilePass(GraphBuilder, View, nullptr, &ScreenPassSceneColor.Texture, ESubstrateTileType::EDecalSimple, bDebugPass);
				AddSubstrateInternalClassificationTilePass(GraphBuilder, View, nullptr, &ScreenPassSceneColor.Texture, ESubstrateTileType::EDecalSingle, bDebugPass);
				AddSubstrateInternalClassificationTilePass(GraphBuilder, View, nullptr, &ScreenPassSceneColor.Texture, ESubstrateTileType::EDecalComplex, bDebugPass);
			}
		}
		else if (DebugMode == FSubstrateViewMode::RoughRefractionClassification)
		{
			if (IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()))
			{
				AddSubstrateInternalClassificationTilePass(GraphBuilder, View, nullptr, &ScreenPassSceneColor.Texture, ESubstrateTileType::EOpaqueRoughRefraction, bDebugPass);
				AddSubstrateInternalClassificationTilePass(GraphBuilder, View, nullptr, &ScreenPassSceneColor.Texture, ESubstrateTileType::EOpaqueRoughRefractionSSSWithout, bDebugPass);
			}
		}
		else if (DebugMode == FSubstrateViewMode::MaterialClassification)
		{
			if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplexSpecial))
			{
				AddSubstrateInternalClassificationTilePass(GraphBuilder, View, nullptr, &ScreenPassSceneColor.Texture, ESubstrateTileType::EComplexSpecial, bDebugPass);
			}
			if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplex))
			{
				AddSubstrateInternalClassificationTilePass(GraphBuilder, View, nullptr, &ScreenPassSceneColor.Texture, ESubstrateTileType::EComplex, bDebugPass);
			}
			if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESingle))
			{
				AddSubstrateInternalClassificationTilePass(GraphBuilder, View, nullptr, &ScreenPassSceneColor.Texture, ESubstrateTileType::ESingle, bDebugPass);
			}
			if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESimple))
			{
				AddSubstrateInternalClassificationTilePass(GraphBuilder, View, nullptr, &ScreenPassSceneColor.Texture, ESubstrateTileType::ESimple, bDebugPass);
			}
		}
	}

	SubstrateRoughRefractionRnD(GraphBuilder, View, ScreenPassSceneColor);

	return MoveTemp(ScreenPassSceneColor);
}

} // namespace Substrate

FSubstrateViewDebugData::FSubstrateViewDebugData()
{
	//
}

template<typename T>
static T InternalCreateTransientPixelDebugBuffer(FRDGBuilder& GraphBuilder, uint32 DataSizeInBytes, const TCHAR* Name)
{
	T Out;

	const uint32 SizeOfUint = sizeof(uint32);
	const uint32 NumUints = FMath::DivideAndRoundUp(DataSizeInBytes, SizeOfUint);

	FRDGBufferDesc RDGBufferDesc = FRDGBufferDesc::CreateStructuredDesc(SizeOfUint, NumUints);
	RDGBufferDesc.Usage |= BUF_SourceCopy;
	Out.DebugDataSizeInUints = NumUints;
	Out.DebugData = GraphBuilder.CreateBuffer(RDGBufferDesc, Name);
	Out.DebugDataUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Out.DebugData, PF_R32_UINT));
	return Out;
}

FSubstrateViewDebugData::FTransientPixelDebugBuffer FSubstrateViewDebugData::CreateTransientPixelDebugBuffer(FRDGBuilder& GraphBuilder)
{
	FTransientPixelDebugBuffer Out = InternalCreateTransientPixelDebugBuffer<FSubstrateViewDebugData::FTransientPixelDebugBuffer>(GraphBuilder, sizeof(FSubstratePixelDebugData), TEXT("PixelMaterialDebugData"));
	PixelMaterialDebugDataSizeBytes = Out.DebugDataSizeInUints * sizeof(uint32);
	return Out;
}

FSubstrateViewDebugData::FTransientPixelDebugBuffer FSubstrateViewDebugData::CreateDummyPixelDebugBuffer(FRDGBuilder& GraphBuilder)
{
	return InternalCreateTransientPixelDebugBuffer<FSubstrateViewDebugData::FTransientPixelDebugBuffer>(GraphBuilder, sizeof(uint32), TEXT("DummyPixelMaterialDebugData"));
}

FSubstrateViewDebugData::FTransientSystemInfoDebugBuffer FSubstrateViewDebugData::CreateTransientSystemInfoDebugBuffer(FRDGBuilder& GraphBuilder)
{
	FSubstrateViewDebugData::FTransientSystemInfoDebugBuffer Out = InternalCreateTransientPixelDebugBuffer<FSubstrateViewDebugData::FTransientSystemInfoDebugBuffer>(GraphBuilder, SUBSTRATE_TILE_TYPE_COUNT * sizeof(uint32), TEXT("SystemInfoDebugData"));
	SystemInfoDebugDataSizeBytes = Out.DebugDataSizeInUints * sizeof(uint32);
	return Out;
}

void FSubstrateViewDebugData::SafeRelease()
{
	TSharedPtr<FRHIGPUBufferReadback> It;
	while (PixelMaterialDebugDataReadbackQueries.Dequeue(It))
	{
		It.Reset();
	}
	while (SystemInfoDebugDataReadbackQueries.Dequeue(It))
	{
		It.Reset();
	}
}

