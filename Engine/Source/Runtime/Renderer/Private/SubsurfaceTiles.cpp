// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubsurfaceTiles.h"
#include "SceneRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/SubsurfaceProfile.h"

bool FSubsurfaceTilePassVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
}

void FSubsurfaceTilePassVS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("SHADER_TILE_VS"), 1);
	OutEnvironment.SetDefine(TEXT("SUBSURFACE_TILE_SIZE"), FSubsurfaceTiles::TileSize);
}

bool FSubsurfaceTileFallbackScreenPassVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
}

FSubsurfaceTilePassVS::FParameters GetSubsurfaceTileParameters(const FScreenPassTextureViewport& TileViewport, const FSubsurfaceTiles& InTile, FSubsurfaceTiles::ETileType TileType)
{
	FSubsurfaceTilePassVS::FParameters Out;
	Out.TileType = uint32(TileType);
	Out.bRectPrimitive = InTile.bRectPrimitive ? 1 : 0;
	Out.ViewMin = TileViewport.Rect.Min;
	Out.ViewMax = TileViewport.Rect.Max;
	Out.ExtentInverse = FVector2f(1.f / TileViewport.Extent.X, 1.f / TileViewport.Extent.Y);
	Out.TileDataBuffer = InTile.GetTileBufferSRV(TileType);
	Out.TileIndirectBuffer = InTile.TileIndirectDrawBuffer;
	return Out;
}

class FClearUAVBuildIndirectDispatchBufferCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FClearUAVBuildIndirectDispatchBufferCS)
	SHADER_USE_PARAMETER_STRUCT(FClearUAVBuildIndirectDispatchBufferCS, FGlobalShader)

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, ViewportSize)
		SHADER_PARAMETER(uint32, Offset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint32>, ConditionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint32>, RWIndirectDispatchArgsBuffer)
		END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_COMPUTE"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_TILE_SIZE"), FSubsurfaceTiles::TileSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FClearUAVBuildIndirectDispatchBufferCS, "/Engine/Private/PostProcessSubsurfaceTile.usf", "BuildIndirectDispatchArgsCS", SF_Compute);

class FClearUAVCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FClearUAVCS)
	SHADER_USE_PARAMETER_STRUCT(FClearUAVCS, FGlobalShader)

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, TextureExtent)
		SHADER_PARAMETER(FIntPoint, ViewportMin)
		RDG_BUFFER_ACCESS(IndirectDispatchArgsBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>,TextureOutput)
		END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_COMPUTE"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_TILE_SIZE"), FSubsurfaceTiles::TileSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FClearUAVCS, "/Engine/Private/PostProcessSubsurfaceTile.usf", "ClearUAV", SF_Compute);


void AddConditionalClearBlackUAVPass(FRDGBuilder& GraphBuilder, FRDGEventName&& Name, 
	FRDGTextureRef Texture, const FScreenPassTextureViewport& ScreenPassViewport, FRDGBufferRef ConditionBuffer, uint32 Offset)
{

	FRDGBufferRef IndirectDispatchArgsBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1),
		TEXT("IndirectDispatchArgsBuffer"));

	{
		// build the indirect dispatch arguments buffer ( compute the group count on GPU conditionally)
		FClearUAVBuildIndirectDispatchBufferCS::FParameters* PassParameters =
			GraphBuilder.AllocParameters<FClearUAVBuildIndirectDispatchBufferCS::FParameters>();
		PassParameters->ViewportSize = FIntPoint(ScreenPassViewport.Rect.Max.X - ScreenPassViewport.Rect.Min.X + 1,
												ScreenPassViewport.Rect.Max.Y - ScreenPassViewport.Rect.Min.Y + 1);
		PassParameters->Offset = Offset;
		PassParameters->ConditionBuffer = 
			GraphBuilder.CreateSRV(ConditionBuffer, EPixelFormat::PF_R32_UINT);
		PassParameters->RWIndirectDispatchArgsBuffer = 
			GraphBuilder.CreateUAV(IndirectDispatchArgsBuffer, EPixelFormat::PF_R32_UINT);

		TShaderMapRef<FClearUAVBuildIndirectDispatchBufferCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("SSS::ClearUAV(BuildIndirectDispatchBuffer)"),
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1));
	}

	FClearUAVCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FClearUAVCS::FParameters>();
	PassParameters->TextureExtent = Texture->Desc.Extent;
	PassParameters->ViewportMin = ScreenPassViewport.Rect.Min;
	PassParameters->TextureOutput = GraphBuilder.CreateUAV(Texture);
	PassParameters->IndirectDispatchArgsBuffer = IndirectDispatchArgsBuffer;

	TShaderMapRef<FClearUAVCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		Forward<FRDGEventName>(Name),
		ComputeShader,
		PassParameters,
		IndirectDispatchArgsBuffer, 0);
}

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceTilePassVS, "/Engine/Private/PostProcessSubsurfaceTile.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FSubsurfaceTileFallbackScreenPassVS, "/Engine/Private/PostProcessSubsurfaceTile.usf", "SubsurfaceTileFallbackScreenPassVS", SF_Vertex);

const TCHAR* ToString(FSubsurfaceTiles::ETileType Type)
{
	switch (Type)
	{
	case FSubsurfaceTiles::ETileType::All:			return TEXT("SSS(All)");
	case FSubsurfaceTiles::ETileType::AFIS:		return TEXT("SSS(AFIS)");
	case FSubsurfaceTiles::ETileType::SEPARABLE:		return TEXT("SSS(Separable)");
	case FSubsurfaceTiles::ETileType::PASSTHROUGH:      return TEXT("SSS(Passthrough)");
	case FSubsurfaceTiles::ETileType::AllNonPassthrough:      return TEXT("SSS(All,NonPassthrough)");
	default:											return TEXT("Unknown");
	}
}

class FSSSTileCategorisationMarkCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSSTileCategorisationMarkCS);
	SHADER_USE_PARAMETER_STRUCT(FSSSTileCategorisationMarkCS, FGlobalShader)

	class FDimensionHalfRes : SHADER_PERMUTATION_BOOL("SUBSURFACE_HALF_RES");
	using FPermutationDomain = TShaderPermutationDomain<FDimensionHalfRes>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector4f, SubsurfaceParams)
		SHADER_PARAMETER(float, SubsurfaceSubpixelThreshold)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, SubsurfaceInput0)
		SHADER_PARAMETER(FIntPoint, TiledViewRes)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, GroupBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, TileMaskBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, TileMaskPassthroughBufferOut)
	    SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, TileMaskNonPassthroughBufferOut)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("TILE_CATERGORISATION_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_TILE_SIZE"), FSubsurfaceTiles::TileSize);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_KERNEL_SIZE"), SUBSURFACE_KERNEL_SIZE);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSSSTileCategorisationMarkCS, "/Engine/Private/PostProcessSubsurfaceTile.usf", "SSSTileCategorisationMarkCS", SF_Compute);

class FSSSTileClassificationBuildListsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSSTileClassificationBuildListsCS);
	SHADER_USE_PARAMETER_STRUCT(FSSSTileClassificationBuildListsCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FIntPoint, TiledViewRes)
		SHADER_PARAMETER(int, TileType)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileMaskBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWTileTypeCountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWSSSTileListDataBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static int32 GetGroupSize()
	{
		return FSubsurfaceTiles::TileSize;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("TILE_CATERGORISATION_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_TILE_SIZE"), GetGroupSize());
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_KERNEL_SIZE"), SUBSURFACE_KERNEL_SIZE);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSSSTileClassificationBuildListsCS, "/Engine/Private/PostProcessSubsurfaceTile.usf", "SSSTileClassificationBuildListsCS", SF_Compute);

class FSubsurfaceTileBuildIndirectDispatchArgsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSubsurfaceTileBuildIndirectDispatchArgsCS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceTileBuildIndirectDispatchArgsCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, VertexCountPerInstanceIndirect)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDispatchArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDrawArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileTypeCountBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("TILE_CATERGORISATION_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_TILE_SIZE"),  FSubsurfaceTiles::TileSize);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_KERNEL_SIZE"), SUBSURFACE_KERNEL_SIZE);

	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceTileBuildIndirectDispatchArgsCS, "/Engine/Private/PostProcessSubsurfaceTile.usf", "SubsurfaceTileBuildIndirectDispatchArgsCS", SF_Compute);

// Returns the [0, N] clamped value of the 'r.SSS.Scale' CVar.
static float GetSubsurfaceRadiusScaleForTiling()
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.SSS.Scale"));
	check(CVar);

	return FMath::Max(0.0f, CVar->GetValueOnRenderThread());
}

static float GetSubsurfaceSubpixelThreshold()
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.SSS.Subpixel.Threshold"));
	check(CVar);

	return FMath::Max(0.0f, CVar->GetValueOnRenderThread());
}

//@TODO: remove the duplicate from PostProcessSubsurface.cpp
FVector4f GetSubsurfaceParams(const FViewInfo& View)
{
	const float DistanceToProjectionWindow = View.ViewMatrices.GetViewToClip().M[0][0];
	const float SSSScaleZ = DistanceToProjectionWindow * GetSubsurfaceRadiusScaleForTiling();
	const float SSSScaleX = SSSScaleZ / SUBSURFACE_KERNEL_SIZE * 0.5f;

	return FVector4f(SSSScaleX, SSSScaleZ, 0, 0);
}

/**
* Code adapted from ScreenSpaceReflectionTiles to reduce Setup SSS cost.
* Build lists of 8x8 tiles used by SSS pixels
* Mark and build list steps are separated in order to build a more coherent list (z-ordered over a larger region), which is important for the performance of future passes due to neighbor diffusion.
*/
FSubsurfaceTiles ClassifySSSTiles(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FSceneTextures& SceneTextures, 
	const FScreenPassTextureViewportParameters SceneViewportParameters, const FScreenPassTextureViewportParameters SubsurfaceViewportParameters, const bool IsHalfRes)
{
	FSubsurfaceTiles Result;
	
	{
		check(FSubsurfaceTiles::TilePerThread_GroupSize == 64); // If this value change, we need to update the shaders using 
		check(FSubsurfaceTiles::TileSize == 8); // only size supported for now

		FIntPoint SubsurfaceExtent = FIntPoint(SubsurfaceViewportParameters.ViewportSize.X, SubsurfaceViewportParameters.ViewportSize.Y);
		Result.TileDimension = FIntPoint::DivideAndRoundUp(SubsurfaceExtent, FSubsurfaceTiles::TileSize);
		Result.TileCount = Result.TileDimension.X * Result.TileDimension.Y;
		Result.bRectPrimitive = GRHISupportsRectTopology;
		Result.TileTypeCountBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), FSubsurfaceTiles::TileTypeCount), TEXT("Subsurface.TileCountBuffer"));
		Result.TileTypeCountSRV = GraphBuilder.CreateSRV(Result.TileTypeCountBuffer,EPixelFormat::PF_R32_UINT);
		Result.TileIndirectDispatchBuffer = 
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(FSubsurfaceTiles::TileTypeCount), TEXT("Subsurface.TileIndirectDispatchBuffer"));
		Result.TileIndirectDrawBuffer =
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndirectParameters>(FSubsurfaceTiles::TileTypeCount), TEXT("Subsurface.TileIndirectDrawBuffer"));

		Result.TileDataBuffer[ToIndex(FSubsurfaceTiles::ETileType::All)] = 
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Result.TileCount), TEXT("Subsurface.TileDataBuffer(All)"));
		Result.TileDataBuffer[ToIndex(FSubsurfaceTiles::ETileType::AFIS)] = 
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Result.TileCount), TEXT("Subsurface.TileDataBuffer(AFIS)"));
		Result.TileDataBuffer[ToIndex(FSubsurfaceTiles::ETileType::SEPARABLE)] = 
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Result.TileCount), TEXT("Subsurface.TileDataBuffer(SEPARABLE)"));
		Result.TileDataBuffer[ToIndex(FSubsurfaceTiles::ETileType::PASSTHROUGH)] = 
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Result.TileCount), TEXT("Subsurface.TileDataBuffer(PASSTHROUGH)"));
		Result.TileDataBuffer[ToIndex(FSubsurfaceTiles::ETileType::AllNonPassthrough)] = 
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Result.TileCount), TEXT("Subsurface.TileDataBuffer(AllNonPassthrough)"));

		for (uint32 BufferIt = 0; BufferIt < FSubsurfaceTiles::TileTypeCount; ++BufferIt)
		{
			if (Result.TileDataBuffer[BufferIt])
			{
				Result.TileDataSRV[BufferIt] = GraphBuilder.CreateSRV(Result.TileDataBuffer[BufferIt], PF_R32_UINT);
			}
		}

		// We write to FSubsurfaceTiles::All, FSubsurfaceTiles::AllNonPassthrough, and FSubsurfaceTiles::Passthrough
		FRDGBufferRef TileListDataBuffer = Result.TileDataBuffer[ToIndex(FSubsurfaceTiles::ETileType::All)];//GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Result.TileDimension.X * Result.TileDimension.Y), TEXT("Subsurface.TileListDataBuffer"));
		FRDGBufferSRVRef TileListDataBufferSRV = Result.TileDataSRV[ToIndex(FSubsurfaceTiles::ETileType::All)];//GraphBuilder.CreateSRV(TileListDataBuffer, PF_R32_UINT);

		FRDGBufferUAVRef DrawIndirectParametersBufferUAV = GraphBuilder.CreateUAV(Result.TileIndirectDrawBuffer, EPixelFormat::PF_R32_UINT);
		FRDGBufferUAVRef DispatchIndirectParametersBufferUAV = GraphBuilder.CreateUAV(Result.TileIndirectDispatchBuffer, EPixelFormat::PF_R32_UINT);

		// Allocate buffer with 1 bit / tile
		FRDGBufferRef TileMaskBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FMath::DivideAndRoundUp(Result.TileDimension.X * Result.TileDimension.Y, 32)), TEXT("SSR.Classify.TileMaskBuffer"));
		FRDGBufferUAVRef TileMaskBufferUAV = GraphBuilder.CreateUAV(TileMaskBuffer);
		FRDGBufferRef TileMaskPassthroughBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FMath::DivideAndRoundUp(Result.TileDimension.X * Result.TileDimension.Y, 32)), TEXT("SSR.Classify.TileMaskPassthroughBuffer"));
		FRDGBufferUAVRef TileMaskPassthroughBufferUAV = GraphBuilder.CreateUAV(TileMaskPassthroughBuffer);
		FRDGBufferRef TileMaskNonPassthroughBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FMath::DivideAndRoundUp(Result.TileDimension.X * Result.TileDimension.Y, 32)), TEXT("SSR.Classify.TileMaskNonPassthroughBuffer"));
		FRDGBufferUAVRef TileMaskNonPassthroughBufferUAV = GraphBuilder.CreateUAV(TileMaskNonPassthroughBuffer);
		
		AddClearUAVPass(GraphBuilder, TileMaskBufferUAV, 0);
		AddClearUAVPass(GraphBuilder, TileMaskPassthroughBufferUAV, 0);
		AddClearUAVPass(GraphBuilder, TileMaskNonPassthroughBufferUAV, 0);

		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Result.TileTypeCountBuffer,EPixelFormat::PF_R32_UINT), 0);

		// Mark used tiles based on SHADING_MODEL_ID, and whether it is subpixel scattering only.
		{
			typedef FSSSTileCategorisationMarkCS SHADER;
			SHADER::FPermutationDomain PermutationVector;
			PermutationVector.Set<SHADER::FDimensionHalfRes>(IsHalfRes);
			TShaderMapRef<SHADER> ComputeShader(View.ShaderMap, PermutationVector);

			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			PassParameters->SceneTextures = SceneTextures.UniformBuffer;
			PassParameters->View = View.GetShaderParameters();
			PassParameters->SubsurfaceParams = GetSubsurfaceParams(View);
			PassParameters->SubsurfaceSubpixelThreshold = GetSubsurfaceSubpixelThreshold();
			PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
			PassParameters->Output = SubsurfaceViewportParameters;
			PassParameters->SubsurfaceInput0 = SceneViewportParameters;
			PassParameters->TiledViewRes = Result.TileDimension;

			PassParameters->TileMaskBufferOut = TileMaskBufferUAV;
			PassParameters->TileMaskPassthroughBufferOut = TileMaskPassthroughBufferUAV;
			PassParameters->TileMaskNonPassthroughBufferOut = TileMaskNonPassthroughBufferUAV;

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSS::TileCategorisationMarkTiles(All)"),
				ComputeShader,
				PassParameters,
				FIntVector(Result.TileDimension.X, Result.TileDimension.Y, 1)
			);
		}

		// Build compacted and coherent z-order tile lists from bit-marked tiles
		{
			typedef FSSSTileClassificationBuildListsCS SHADER;

			struct FSubsurfaceBuildListsInfo
			{
				FSubsurfaceBuildListsInfo( FSubsurfaceTiles::ETileType inTileType, FRDGBufferRef inInput, FRDGBufferRef inOutput)
					: Name(ToString(inTileType)), TileType(inTileType), Input(inInput), Output(inOutput)
				{}

				const TCHAR* Name;
				FSubsurfaceTiles::ETileType TileType;
				FRDGBufferRef Input;
				FRDGBufferRef Output;
			};

			const int NumOfBuildListsPass = 3;

			const FSubsurfaceBuildListsInfo SubsurfaceBuildListsInfo[NumOfBuildListsPass] =
			{
				{FSubsurfaceTiles::ETileType::All,               TileMaskBuffer,               Result.GetTileBuffer(FSubsurfaceTiles::ETileType::All)},
				{FSubsurfaceTiles::ETileType::PASSTHROUGH,       TileMaskPassthroughBuffer,    Result.GetTileBuffer(FSubsurfaceTiles::ETileType::PASSTHROUGH)},
				{FSubsurfaceTiles::ETileType::AllNonPassthrough, TileMaskNonPassthroughBuffer, Result.GetTileBuffer(FSubsurfaceTiles::ETileType::AllNonPassthrough)}
			};

			for (int PassIndex = 0 ; PassIndex < NumOfBuildListsPass ; ++PassIndex)
			{
				const FSubsurfaceBuildListsInfo PassInfo = SubsurfaceBuildListsInfo[PassIndex];

				TShaderMapRef<SHADER> ComputeShader(View.ShaderMap);

				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();

				PassParameters->View = View.GetShaderParameters();
				PassParameters->TiledViewRes = Result.TileDimension;
				PassParameters->TileType = ToIndex(PassInfo.TileType);
				PassParameters->TileMaskBuffer = GraphBuilder.CreateSRV(PassInfo.Input);
				PassParameters->RWTileTypeCountBuffer = GraphBuilder.CreateUAV(Result.TileTypeCountBuffer, PF_R32_UINT);
				PassParameters->RWSSSTileListDataBuffer = GraphBuilder.CreateUAV(PassInfo.Output, PF_R32_UINT);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("SSS::TileCategorisationBuildList(%s)",
						PassInfo.Name),
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(Result.TileDimension, SHADER::GetGroupSize())
				);
			}

			// Setup the indirect dispatch & draw arguments for All, PASSTHROUGH, and AllNonPassthrough.
			{
				typedef FSubsurfaceTileBuildIndirectDispatchArgsCS ARGSETUPSHADER;
				ARGSETUPSHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<ARGSETUPSHADER::FParameters>();
				PassParameters->VertexCountPerInstanceIndirect = GRHISupportsRectTopology ? 4 : 6;
				PassParameters->RWIndirectDispatchArgsBuffer = DispatchIndirectParametersBufferUAV;
				PassParameters->RWIndirectDrawArgsBuffer = DrawIndirectParametersBufferUAV;
				PassParameters->TileTypeCountBuffer = Result.TileTypeCountSRV;
				TShaderMapRef<ARGSETUPSHADER> ComputeShader(View.ShaderMap);
				FComputeShaderUtils::AddPass(GraphBuilder, FRDGEventName(TEXT("SSS::BuildIndirectArgs(Dispatch & Draw)")), ComputeShader, PassParameters, FIntVector(1, 1, 1));
			}
		}
	}
	return Result;
}
