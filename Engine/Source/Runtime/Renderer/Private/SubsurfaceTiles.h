// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SubsurfaceTiles.h: Screenspace subsurface scattering tile buffer implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "GlobalShader.h"
#include "ShaderParameterStruct.h"
#include "ScreenPass.h"
#include "SceneRendering.h"

// Adapted from FHairStrandsTile
struct FSubsurfaceTiles
{
	// Support two types of acceleration. It is independent of SSS profile
	// AFIS: adaptive filtered importance sampling
	// SEPARABLE: two pass separable filter
	// PASSTHROUGH: directly pass through because of subpixel SSS
	// AllNonPassthrough: all subsurface tiles but without passthrough tiles
	enum class ETileType : uint8 {AFIS, SEPARABLE, PASSTHROUGH, AllNonPassthrough, All, Count};
	static const uint32 TileTypeCount = uint32(ETileType::Count);

	FIntPoint			BufferResolution = FIntPoint(0, 0);
	static const uint32 GroupSize = 64;

	// Define the size of subsurface tile. @TODO: Set to 16 to use LDS.
	static const uint32	TileSize = 8;

	static const uint32	TilePerThread_GroupSize = 64;
	uint32				TileCount = 0;
	FIntPoint			TileDimension = FIntPoint(0, 0);
	bool				bRectPrimitive = false;

	// Buffers per tile types
	FRDGBufferSRVRef	TileDataSRV[TileTypeCount] = { nullptr, nullptr };
	FRDGBufferRef		TileDataBuffer[TileTypeCount] = { nullptr, nullptr };

	FRDGBufferSRVRef	TileTypeCountSRV = nullptr;
	FRDGBufferRef		TileTypeCountBuffer = nullptr;

	FRDGBufferRef		TileIndirectDispatchBuffer = nullptr;
	FRDGBufferRef		TileIndirectDrawBuffer = nullptr;
	FRDGBufferRef		TileIndirectRayDispatchBuffer = nullptr;

	FRDGBufferRef		TilePerThreadIndirectDispatchBuffer = nullptr;

	static FORCEINLINE uint32 GetIndirectDrawArgOffset(ETileType Type)		{ return uint32(Type) * sizeof(FRHIDrawIndirectParameters); }
	static FORCEINLINE uint32 GetIndirectDispatchArgOffset(ETileType Type)	{ return uint32(Type) * sizeof(FRHIDispatchIndirectParameters); }

	bool IsValid() const										{ return TileCount > 0 && TileDataBuffer[uint32(ETileType::All)];}
	FRDGBufferRef GetTileBuffer(ETileType Type) const			{ const uint32 Index = uint32(Type); check(TileDataBuffer[Index] != nullptr); return TileDataBuffer[Index]; }
	FRDGBufferSRVRef GetTileBufferSRV(ETileType Type) const		{ const uint32 Index = uint32(Type); check(TileDataSRV[Index] != nullptr); return TileDataSRV[Index]; }
};

FORCEINLINE uint32 ToIndex(FSubsurfaceTiles::ETileType Type) { return uint32(Type); }
const TCHAR* ToString(FSubsurfaceTiles::ETileType Type);

class FSubsurfaceTilePassVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSubsurfaceTilePassVS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceTilePassVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, ViewMin)
		SHADER_PARAMETER(FIntPoint, ViewMax)
		SHADER_PARAMETER(int32, bRectPrimitive)
		SHADER_PARAMETER(uint32, TileType)
		SHADER_PARAMETER(FVector2f, ExtentInverse)
		SHADER_PARAMETER(uint32, OutputFlag)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, TileDataBuffer)
		RDG_BUFFER_ACCESS(TileIndirectBuffer, ERHIAccess::IndirectArgs)
		END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

class FSubsurfaceTileFallbackScreenPassVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSubsurfaceTileFallbackScreenPassVS);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	FSubsurfaceTileFallbackScreenPassVS() = default;
	FSubsurfaceTileFallbackScreenPassVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};


FSubsurfaceTilePassVS::FParameters GetSubsurfaceTileParameters(const FScreenPassTextureViewport& TileViewport, const FSubsurfaceTiles& InTile, FSubsurfaceTiles::ETileType TileType);

template <typename FSubsurfacePassPS, typename FSubsurfacePassVS>
void AddSubsurfaceTiledScreenPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& Name,
	const FViewInfo& View,
	typename FSubsurfacePassPS::FParameters* PassParameters,
	TShaderMapRef<FSubsurfacePassVS>& VertexShader,
	TShaderMapRef<FSubsurfacePassPS>& PixelShader,
	FRHIBlendState* BlendState,
	FRHIDepthStencilState* DepthStencilState,
	const FScreenPassTextureViewport SceneViewport,
	FSubsurfaceTiles::ETileType TileType,
	const bool bShouldFallbackToFullScreenPass = false)
{
	if (bShouldFallbackToFullScreenPass)
	{
		TShaderMapRef<FSubsurfaceTileFallbackScreenPassVS> FallBackVertexShader(View.ShaderMap);
		AddDrawScreenPass(GraphBuilder,
			Forward<FRDGEventName>(Name),
			View,
			SceneViewport,
			SceneViewport,
			FallBackVertexShader,
			PixelShader,
			BlendState,
			DepthStencilState,
			PassParameters);
	}
	else
	{
		GraphBuilder.AddPass(
			Forward<FRDGEventName>(Name),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, VertexShader, PixelShader, DepthStencilState, BlendState, Viewport = SceneViewport, TileType](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				typename FSubsurfacePassVS::FParameters ParametersVS = PassParameters->TileParameters;

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.BlendState = BlendState;
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = DepthStencilState;

				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PassParameters->TileParameters.bRectPrimitive > 0 ? PT_RectList : PT_TriangleList;
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

				RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, Viewport.Extent.X, Viewport.Extent.Y, 1.0f);
				RHICmdList.SetStreamSource(0, nullptr, 0);
				RHICmdList.DrawPrimitiveIndirect(PassParameters->TileParameters.TileIndirectBuffer->GetRHI(), FSubsurfaceTiles::GetIndirectDrawArgOffset(TileType));
			});
	}
}

/** Clear the UAV texture to black only when ConditionBuffer[Offset] > 0 */
void AddConditionalClearBlackUAVPass(FRDGBuilder& GraphBuilder, FRDGEventName&& PassName, 
	FRDGTextureRef Texture, const FScreenPassTextureViewport& ScreenPassViewport, FRDGBufferRef ConditionBuffer, uint32 Offset);

/**
* Code adapted from ScreenSpaceReflectionTiles to reduce Setup SSS cost.
* Build lists of 8x8 tiles used by SSS pixels
* Mark and build list steps are separated in order to build a more coherent list (z-ordered over a larger region), which is important for the performance of future passes due to neighbor diffusion.
*/
FSubsurfaceTiles ClassifySSSTiles(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FSceneTextures& SceneTextures, 
	const FScreenPassTextureViewportParameters SceneViewportParameters, const FScreenPassTextureViewportParameters SubsurfaceViewportParameters, const bool IsHalfRes);

