// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SystemTextures.cpp: System textures implementation.
=============================================================================*/

#include "SystemTextures.h"
#include "Math/RandomStream.h"
#include "Math/Sobol.h"
#include "Math/Float16.h"
#include "RenderTargetPool.h"
#include "ClearQuad.h"
#include "Math/PackedVector.h"
#include "GlobalRenderResources.h"
#include "Engine/Texture2D.h"
#include "Engine/Engine.h"
#include "Containers/ResourceArray.h"

/*-----------------------------------------------------------------------------
 Export System textures (LTC / EnergyConservation / Hair LUT / ...)
-----------------------------------------------------------------------------*/
#define EXPORT_SYSTEM_TEXTURES 0

#if EXPORT_SYSTEM_TEXTURES
#include "LTC.h"
#include "ImageUtils.h"

static void SaveEXR(const FString& Filename, const FIntPoint& DataSize, const TArray<FFloat16Color>& Data)
{
	const FString RelativeFilepath = FPaths::EngineSavedDir() + Filename + TEXT(".exr");
	const FString AbsoluteFilepath = FPaths::ConvertRelativePathToFull(RelativeFilepath);
	FImageView ImageView(Data.GetData(), DataSize.X, DataSize.Y);
	const bool bSucceed = FImageUtils::SaveImageByExtension(*AbsoluteFilepath, ImageView);
	check(bSucceed)
}

static void ExportSystemTextures()
{
	// LTC Textures	(used by Rect Lights and/or BSDF evaluation)
	// GGX - LTC matrix coefficients (4-coefficients)
	{ 
		const FIntPoint DataSize(GGX_LTC_Size, GGX_LTC_Size);
		TArray<FFloat16Color> RawPixels;
		RawPixels.SetNum(DataSize.X * DataSize.Y);
		for (int32 y = 0; y < GGX_LTC_Size; ++y)
		{
			for (int32 x = 0; x < GGX_LTC_Size; ++x)
			{
				FFloat16Color& Dest = RawPixels[x + y * DataSize.X];
				Dest.R = FFloat16(GGX_LTC_Mat[4 * (x + y * GGX_LTC_Size) + 0]);
				Dest.G = FFloat16(GGX_LTC_Mat[4 * (x + y * GGX_LTC_Size) + 1]);
				Dest.B = FFloat16(GGX_LTC_Mat[4 * (x + y * GGX_LTC_Size) + 2]);
				Dest.A = FFloat16(GGX_LTC_Mat[4 * (x + y * GGX_LTC_Size) + 3]);
			}
		}
		SaveEXR(TEXT("GGX_LTCMat"), DataSize, RawPixels);
	}

	// GGX - Split-Sum Amplitude coefficients (2-components)
	{
		const FIntPoint DataSize(GGX_LTC_Size, GGX_LTC_Size);
		TArray<FFloat16Color> RawPixels;
		RawPixels.SetNum(DataSize.X * DataSize.Y);
		for (int32 y = 0; y < GGX_LTC_Size; ++y)
		{
			for (int32 x = 0; x < GGX_LTC_Size; ++x)
			{
				FFloat16Color& Dest = RawPixels[x + y * DataSize.X];
				Dest.R = FFloat16(GGX_LTC_Amp[4 * (x + y * GGX_LTC_Size) + 0]);
				Dest.G = FFloat16(GGX_LTC_Amp[4 * (x + y * GGX_LTC_Size) + 1]);
				Dest.B = 0;
				Dest.A = 0;
			}
		}
		SaveEXR(TEXT("GGX_LTCAmp"), DataSize, RawPixels);
	}

	// Sheen - Matrix & directional albedo (3-components)
	{
		const FIntPoint DataSize(Sheen_LTC_Size, Sheen_LTC_Size);
		TArray<FFloat16Color> RawPixels;
		RawPixels.SetNum(DataSize.X * DataSize.Y);
		for (int32 y = 0; y < DataSize.Y; ++y)
		{
			for (int32 x = 0; x < DataSize.X; ++x)
			{
				FFloat16Color& Dest = RawPixels[x + y * DataSize.X];
				Dest.R = FFloat16(Sheen_LTC_Volume[x][y][0]);
				Dest.G = FFloat16(Sheen_LTC_Volume[x][y][1]);
				Dest.B = FFloat16(Sheen_LTC_Volume[x][y][2]);
				Dest.A = FFloat16(0);
			}
		}
		SaveEXR(TEXT("Sheen_LTC"), DataSize, RawPixels);
	}
}
#endif // EXPORT_SYSTEM_TEXTURES

/*-----------------------------------------------------------------------------
 SystemTextures
-----------------------------------------------------------------------------*/

RDG_REGISTER_BLACKBOARD_STRUCT(FRDGSystemTextures);

const FRDGSystemTextures& FRDGSystemTextures::Create(FRDGBuilder& GraphBuilder)
{
	const auto Register = [&](const TRefCountPtr<IPooledRenderTarget>& RenderTarget)
	{
		return TryRegisterExternalTexture(GraphBuilder, RenderTarget, ERDGTextureFlags::SkipTracking);
	};

	auto& SystemTextures = GraphBuilder.Blackboard.Create<FRDGSystemTextures>();
	SystemTextures.White = Register(GSystemTextures.WhiteDummy);
	SystemTextures.Black = Register(GSystemTextures.BlackDummy);
	SystemTextures.BlackAlphaOne = Register(GSystemTextures.BlackAlphaOneDummy);
	SystemTextures.BlackArray = Register(GSystemTextures.BlackArrayDummy);
	SystemTextures.MaxFP16Depth = Register(GSystemTextures.MaxFP16Depth);
	SystemTextures.DepthDummy = Register(GSystemTextures.DepthDummy);
	SystemTextures.BlackDepthCube = Register(GSystemTextures.BlackDepthCube);
	SystemTextures.StencilDummy = Register(GSystemTextures.StencilDummy);
	SystemTextures.Green = Register(GSystemTextures.GreenDummy);
	SystemTextures.DefaultNormal8Bit = Register(GSystemTextures.DefaultNormal8Bit);
	SystemTextures.MidGrey = Register(GSystemTextures.MidGreyDummy);
	SystemTextures.VolumetricBlack = Register(GSystemTextures.VolumetricBlackDummy);
	SystemTextures.VolumetricBlackAlphaOne = Register(GSystemTextures.VolumetricBlackAlphaOneDummy);
	SystemTextures.VolumetricBlackUint = Register(GSystemTextures.VolumetricBlackUintDummy);
	SystemTextures.CubeBlack = Register(GSystemTextures.CubeBlackDummy);
	SystemTextures.CubeArrayBlack = Register(GSystemTextures.CubeArrayBlackDummy);
	SystemTextures.StencilDummySRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SystemTextures.StencilDummy));
	SystemTextures.ZeroUIntDummy = Register(GSystemTextures.ZeroUIntDummy);
	
	return SystemTextures;
}

const FRDGSystemTextures& FRDGSystemTextures::Get(FRDGBuilder& GraphBuilder)
{
	const FRDGSystemTextures* SystemTextures = GraphBuilder.Blackboard.Get<FRDGSystemTextures>();
	checkf(SystemTextures, TEXT("FRDGSystemTextures were not initialized. Call FRDGSystemTextures::Create() first."));
	return *SystemTextures;
}

bool FRDGSystemTextures::IsValid(FRDGBuilder& GraphBuilder)
{
	return GraphBuilder.Blackboard.Get<FRDGSystemTextures>() != nullptr;
}

/** The global render targets used for scene rendering. */
TGlobalResource<FSystemTextures> GSystemTextures;

void FSystemTextures::InitializeTextures(FRHICommandListImmediate& RHICmdList, const ERHIFeatureLevel::Type InFeatureLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSystemTextures::InitializeTextures);

	// When we render to system textures it should occur on all GPUs since this only
	// happens once on startup (or when the feature level changes).
	SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::All());

	// if this is the first call initialize everything
	if (FeatureLevelInitializedTo == ERHIFeatureLevel::Num)
	{
		InitializeCommonTextures(RHICmdList);
		InitializeFeatureLevelDependentTextures(RHICmdList, InFeatureLevel);
	}
	// otherwise, if we request a higher feature level, we might need to initialize those textures that depend on the feature level
	else if (InFeatureLevel > FeatureLevelInitializedTo)
	{
		InitializeFeatureLevelDependentTextures(RHICmdList, InFeatureLevel);
	}
	// there's no needed setup for those feature levels lower or identical to the current one

	// Initialized resources depending on GEngine data
	if (!bEngineDependentTexturesInitialized && GEngine)
	{
		InitializeEngineDependentTextures(RHICmdList);
	}
}

const static FLazyName SystemTexturesName(TEXT("FSystemTextures"));

template <typename DataType>
static FTextureRHIRef CreateDummyTexture(FRHICommandListImmediate& RHICmdList, const FRHITextureCreateDesc& InCreateDesc, const DataType& DummyData)
{
	FResourceBulkDataArrayView InitData(&DummyData, sizeof(DataType));

	FRHITextureCreateDesc CreateDesc(InCreateDesc);
	CreateDesc.AddFlags(ETextureCreateFlags::ShaderResource);
	CreateDesc.SetClassName(SystemTexturesName);
	CreateDesc.SetInitActionBulkData(&InitData);

	return RHICmdList.CreateTexture(CreateDesc);
}

static TRefCountPtr<IPooledRenderTarget> CreateTexture(FRHICommandListImmediate& RHICmdList, TObjectPtr<class UTexture2D>& InCPUTexture, EPixelFormat InFormat, const TCHAR* InName)
{
	check(InCPUTexture->Availability == ETextureAvailability::CPU);
	FSharedImageConstRef Data = InCPUTexture->GetCPUCopy();
	check(Data && Data->Format == ERawImageFormat::RGBA16F);
	const TArrayView64<const FFloat16Color> DataView = Data->AsRGBA16F();

	const FIntPoint DataSize(Data->SizeX, Data->SizeY);

	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(InName, DataSize.X, DataSize.Y, InFormat)
		.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::FastVRAM)
		.SetClassName(SystemTexturesName)
		.SetInitActionInitializer();

	FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);

	check(InFormat == PF_FloatRGBA || InFormat == PF_G16R16F);
	const uint32 ComponentCount = InFormat == PF_G16R16F ? 2u: 4u;

	// Write the contents of the texture.
	FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DSubresource(0);

	uint8* DestBuffer = reinterpret_cast<uint8*>(Subresource.Data);
	for (int32 y = 0; y < DataSize.Y; ++y)
	{
		for (int32 x = 0; x < DataSize.X; ++x)
		{
			uint16* Dest = (uint16*)(DestBuffer + x * ComponentCount * sizeof(uint16) + y * Subresource.Stride);

			const FFloat16Color Value = DataView[x + y * DataSize.X];
			Dest[0] = Value.R.Encoded;
			if (ComponentCount > 1) { Dest[1] = Value.G.Encoded; }
			if (ComponentCount > 2) { Dest[2] = Value.B.Encoded; }
			if (ComponentCount > 3) { Dest[3] = Value.A.Encoded; }
		}
	}

	FTextureRHIRef Texture = Initializer.Finalize();

	// Release CPU data which are no longer needed
	#if !WITH_EDITORONLY_DATA
	InCPUTexture->RemoveFromRoot();
	InCPUTexture = nullptr;
	#endif

	return CreateRenderTarget(Texture, Desc.DebugName);
}

static bool IsTextureDataValid(UTexture2D* In)
{
	return In && In->GetPlatformData() && In->GetCPUCopy();
}

void FSystemTextures::InitializeCommonTextures(FRHICommandListImmediate& RHICmdList)
{
	// First initialize textures that are common to all feature levels. This is always done the first time we come into this function, as doesn't care about the
	// requested feature level

	// Create a WhiteDummy texture
	{
		WhiteDummy = CreateRenderTarget(GWhiteTexture->TextureRHI, TEXT("WhiteDummy"));
		WhiteDummySRV = RHICmdList.CreateShaderResourceView((FRHITexture*)WhiteDummy->GetRHI(),FRHIViewDesc::CreateTextureSRV().SetDimensionFromTexture(WhiteDummy->GetRHI()));
	}

	// Create a BlackDummy texture
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("BlackDummy"), 1, 1, PF_B8G8R8A8);

		FTextureRHIRef Texture = CreateDummyTexture<FColor>(RHICmdList, Desc, FColor(0, 0, 0, 0));
		BlackDummy = CreateRenderTarget(Texture, Desc.DebugName);
	}
	
	// Create a texture array that has a single black dummy slice 
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2DArray(TEXT("BlackArrayDummy"), 1, 1, 1, PF_B8G8R8A8);

		FTextureRHIRef Texture = CreateDummyTexture<FColor>(RHICmdList, Desc, FColor(0, 0, 0, 0));
		BlackArrayDummy = CreateRenderTarget(Texture, Desc.DebugName);
	}

	// Create a texture that is a single UInt32 value set to 0
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("ZeroUIntDummy"), 1, 1, PF_R32_UINT);

		FTextureRHIRef Texture = CreateDummyTexture<uint32>(RHICmdList, Desc, 0u);
		ZeroUIntDummy = CreateRenderTarget(Texture, Desc.DebugName);
	}

	// Create a texture array that is a single UInt32 value set to 0
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2DArray(TEXT("ZeroUIntArrayDummy"), 1, 1, 1, PF_R32_UINT);

		FTextureRHIRef Texture = CreateDummyTexture<uint32>(RHICmdList, Desc, 0u);
		ZeroUIntArrayDummy = CreateRenderTarget(Texture, Desc.DebugName);
	}

	// Create a BlackAlphaOneDummy texture
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("BlackAlphaOneDummy"), 1, 1, PF_B8G8R8A8);

		FTextureRHIRef Texture = CreateDummyTexture<FColor>(RHICmdList, Desc, FColor(0, 0, 0, 255));
		BlackAlphaOneDummy = CreateRenderTarget(Texture, Desc.DebugName);
	}

	// Create a GreenDummy texture
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2DArray(TEXT("GreenDummy"), 1, 1, PF_B8G8R8A8)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(SystemTexturesName);

		FTextureRHIRef Texture = CreateDummyTexture<FColor>(RHICmdList, Desc, FColor(0, 255, 0, 255));
		GreenDummy = CreateRenderTarget(Texture, Desc.DebugName);
	}

	// Create a DefaultNormal8Bit texture
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("DefaultNormal8Bit"), 1, 1, PF_B8G8R8A8);

		FTextureRHIRef Texture = CreateDummyTexture<FColor>(RHICmdList, Desc, FColor(128, 128, 128, 255));
		DefaultNormal8Bit = CreateRenderTarget(Texture, Desc.DebugName);
	}

	// Create the PerlinNoiseGradient texture
	{
		const uint32 Extent = 128;

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("PerlinNoiseGradient"), Extent, Extent, PF_B8G8R8A8)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(SystemTexturesName)
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);

		// Write the contents of the texture.
		FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DSubresource(0);

		uint8* DestBuffer = reinterpret_cast<uint8*>(Subresource.Data);

		// seed the pseudo random stream with a good value
		FRandomStream RandomStream(12345);
		// Values represent float3 values in the -1..1 range.
		// The vectors are the edge mid point of a cube from -1 .. 1
		static uint32 gradtable[] =
		{
			0x88ffff, 0xff88ff, 0xffff88,
			0x88ff00, 0xff8800, 0xff0088,
			0x8800ff, 0x0088ff, 0x00ff88,
			0x880000, 0x008800, 0x000088,
		};
		for (int32 y = 0; y < Extent; ++y)
		{
			for (int32 x = 0; x < Extent; ++x)
			{
				uint32* Dest = (uint32*)(DestBuffer + x * sizeof(uint32) + y * Subresource.Stride);

				// pick a random direction (hacky way to overcome the quality issues FRandomStream has)
				*Dest = gradtable[(uint32)(RandomStream.GetFraction() * 11.9999999f)];
			}
		}

		FTextureRHIRef Texture = Initializer.Finalize();
		PerlinNoiseGradient = CreateRenderTarget(Texture, Desc.DebugName);
	}

	if (GPixelFormats[PF_FloatRGBA].Supported)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("MaxFP16Depth"), 1, 1, PF_FloatRGBA);

		FTextureRHIRef Texture = CreateDummyTexture<FFloat16Color>(RHICmdList, Desc, FFloat16Color(FLinearColor(65500.0f, 65500.0f, 65500.0f, 65500.0f)));
		MaxFP16Depth = CreateRenderTarget(Texture, Desc.DebugName);
	}

	{
		DepthDummy = BlackDummy;
		BlackDepthCube = CreateRenderTarget(GBlackTextureDepthCube->TextureRHI, TEXT("BlackDepthCube"));
	}

	// Create a dummy stencil SRV.
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("StencilDummy"), 1, 1, PF_R8G8B8A8_UINT);

		FTextureRHIRef Texture = CreateDummyTexture<FColor>(RHICmdList, Desc, FColor::White);
		StencilDummy = CreateRenderTarget(Texture, Desc.DebugName);
		StencilDummySRV = RHICmdList.CreateShaderResourceView(StencilDummy->GetRHI(), FRHIViewDesc::CreateTextureSRV().SetDimensionFromTexture(Texture));
	}

	if (GPixelFormats[PF_FloatRGBA].Supported)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("MidGreyDummy"), 1, 1, PF_FloatRGBA);

		FTextureRHIRef Texture = CreateDummyTexture<FFloat16Color>(RHICmdList, Desc, FFloat16Color(FLinearColor(0.5f, 0.5f, 0.5f, 0.5f)));
		MidGreyDummy = CreateRenderTarget(Texture, Desc.DebugName);
	}

	// Create a VolumetricBlackDummy texture
	{
		VolumetricBlackDummy = CreateRenderTarget(GBlackVolumeTexture->TextureRHI, TEXT("VolumetricBlackDummy"));
		VolumetricBlackAlphaOneDummy = CreateRenderTarget(GBlackAlpha1VolumeTexture->TextureRHI, TEXT("VolumetricBlackAlphaOneDummy"));
		VolumetricBlackUintDummy = CreateRenderTarget(GBlackUintVolumeTexture->TextureRHI, TEXT("VolumetricBlackUintDummy"));
	}

	// Create Cube BlackDummy textures
	{
		CubeBlackDummy = CreateRenderTarget(GBlackTextureCube->TextureRHI, TEXT("CubeBlackDummy"));
		CubeArrayBlackDummy = CreateRenderTarget(GBlackCubeArrayTexture->TextureRHI, TEXT("CubeArrayBlackDummy"));
	}
}

void FSystemTextures::InitializeFeatureLevelDependentTextures(FRHICommandListImmediate& RHICmdList, const ERHIFeatureLevel::Type InFeatureLevel)
{
	// this function will be called every time the feature level will be updated and some textures require a minimum feature level to exist
	// the below declared variable (CurrentFeatureLevel) will guard against reinitialization of those textures already created in a previous call
	// if FeatureLevelInitializedTo has its default value (ERHIFeatureLevel::Num) it means that setup was never performed and all textures are invalid
	// thus CurrentFeatureLevel will be set to ERHIFeatureLevel::ES2_REMOVED to validate all 'is valid' branching conditions below
    ERHIFeatureLevel::Type CurrentFeatureLevel = FeatureLevelInitializedTo == ERHIFeatureLevel::Num ? ERHIFeatureLevel::ES2_REMOVED : FeatureLevelInitializedTo;

		// Create the SobolSampling texture
	if (CurrentFeatureLevel < ERHIFeatureLevel::ES3_1 && GPixelFormats[PF_R16_UINT].Supported)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("SobolSampling"), 32, 16, PF_R16_UINT)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(SystemTexturesName)
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);

		// Write the contents of the texture.
		FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DSubresource(0);

		uint8* DestBuffer = reinterpret_cast<uint8*>(Subresource.Data);

		for (int y = 0; y < 16; ++y)
		{
			uint16* Dest = (uint16*)(DestBuffer + y * Subresource.Stride);

			// 16x16 block starting at 0,0 = Sobol X,Y from bottom 4 bits of cell X,Y
			for (int x = 0; x < 16; ++x, ++Dest)
			{
				*Dest = FSobol::ComputeGPUSpatialSeed(x, y, /* Index = */ 0);
			}

			// 16x16 block starting at 16,0 = Sobol X,Y from 2nd 4 bits of cell X,Y
			for (int x = 0; x < 16; ++x, ++Dest)
			{
				*Dest = FSobol::ComputeGPUSpatialSeed(x, y, /* Index = */ 1);
			}
		}

		FTextureRHIRef Texture = Initializer.Finalize();
		SobolSampling = CreateRenderTarget(Texture, Desc.DebugName);
	}

	if (CurrentFeatureLevel < ERHIFeatureLevel::SM5 && InFeatureLevel >= ERHIFeatureLevel::SM5)
	{
		FPooledRenderTargetDesc Desc(FPooledRenderTargetDesc::CreateVolumeDesc(1, 1, 1, PF_B8G8R8A8, FClearValueBinding::Transparent, TexCreate_HideInVisualizeTexture, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_NoFastClear, false));
		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, HairLUT0, TEXT("HairLUT0"));

		// Init with dummy textures. The texture will be initialize with real values if needed
		const uint8 BlackBytes[4] = { 0, 0, 0, 0 };
		FUpdateTextureRegion3D Region(0, 0, 0, 0, 0, 0, Desc.Extent.X, Desc.Extent.Y, Desc.Depth);
		RHICmdList.UpdateTexture3D(HairLUT0->GetRHI(), 0, Region, Desc.Extent.X * sizeof(BlackBytes), Desc.Extent.X * Desc.Extent.Y * sizeof(BlackBytes), BlackBytes);

		// UpdateTexture3D before and after state is currently undefined
		RHICmdList.Transition(FRHITransitionInfo(HairLUT0->GetRHI(), ERHIAccess::Unknown, ERHIAccess::SRVMask));
		HairLUT1 = HairLUT0;
		HairLUT2 = HairLUT0;
	}

	// ASCII texture
	{
		EPixelFormat Format = PF_R8;
		const uint32 BytesPerPixel	= 1;
		const uint32 CharacterCount = 96u;
		const uint32 CharacterRes	= 8u;
		const uint32 CharacterStride= CharacterRes * BytesPerPixel;

		FPooledRenderTargetDesc Desc(FPooledRenderTargetDesc::Create2DDesc(FIntPoint(CharacterCount * CharacterRes, CharacterRes), Format, FClearValueBinding::None, TexCreate_None, TexCreate_ShaderResource, false));
		
		const uint32 Characters[] =
		{
			0x00000000,0x00000000,0x0c1e1e0c,0x000c000c,0x00363636,0x00000000,0x367f3636,0x0036367f,
			0x1e033e0c,0x000c1f30,0x18336300,0x0063660c,0x6e1c361c,0x006e333b,0x00030606,0x00000000,
			0x06060c18,0x00180c06,0x18180c06,0x00060c18,0xff3c6600,0x0000663c,0x3f0c0c00,0x00000c0c,
			0x00000000,0x060c0e00,0x3f000000,0x00000000,0x00000000,0x000c0c00,0x0c183060,0x00010306,
			0x3f3b331e,0x001e3337,0x0c0c0f0c,0x003f0c0c,0x1c30331e,0x003f3306,0x1c30331e,0x001e3330,
			0x33363c38,0x0030307f,0x301f033f,0x001e3330,0x1f03061c,0x001e3333,0x1830333f,0x0006060c,
			0x1e33331e,0x001e3333,0x3e33331e,0x000e1830,0x0c0c0000,0x000c0c00,0x0c0c0000,0x060c0e00,
			0x03060c18,0x00180c06,0x003f0000,0x0000003f,0x30180c06,0x00060c18,0x1830331e,0x000c000c,
			0x7b7b633e,0x001e037b,0x33331e0c,0x0033333f,0x3e66663f,0x003f6666,0x0303663c,0x003c6603,
			0x6666363f,0x003f3666,0x1e16467f,0x007f4616,0x1e16467f,0x000f0616,0x0303663c,0x007c6673,
			0x3f333333,0x00333333,0x0c0c0c1e,0x001e0c0c,0x30303078,0x001e3333,0x1e366667,0x00676636,
			0x0606060f,0x007f6646,0x6b7f7763,0x00636363,0x7b6f6763,0x00636373,0x6363361c,0x001c3663,
			0x3e66663f,0x000f0606,0x3333331e,0x00381e3b,0x3e66663f,0x0067361e,0x1c07331e,0x001e3338,
			0x0c0c2d3f,0x001e0c0c,0x33333333,0x003f3333,0x33333333,0x000c1e33,0x6b636363,0x0063777f,
			0x1c366363,0x00636336,0x1e333333,0x001e0c0c,0x0c19337f,0x007f6346,0x0606061e,0x001e0606,
			0x180c0603,0x00406030,0x1818181e,0x001e1818,0x63361c08,0x00000000,0x00000000,0xff000000,
			0x00180c0c,0x00000000,0x301e0000,0x006e333e,0x663e0607,0x003d6666,0x331e0000,0x001e3303,
			0x3e303038,0x006e3333,0x331e0000,0x001e033f,0x0f06361c,0x000f0606,0x336e0000,0x1f303e33,
			0x6e360607,0x00676666,0x0c0e000c,0x001e0c0c,0x181e0018,0x0e1b1818,0x36660607,0x0067361e,
			0x0c0c0c0e,0x001e0c0c,0x7f370000,0x0063636b,0x331f0000,0x00333333,0x331e0000,0x001e3333,
			0x663b0000,0x0f063e66,0x336e0000,0x78303e33,0x361b0000,0x000f0636,0x033e0000,0x001f301e,
			0x0c3e0c08,0x00182c0c,0x33330000,0x006e3333,0x33330000,0x000c1e33,0x63630000,0x00367f6b,
			0x36630000,0x0063361c,0x33330000,0x1f303e33,0x193f0000,0x003f260c,0x070c0c38,0x00380c0c,
			0x00181818,0x00181818,0x380c0c07,0x00070c0c,0x00003b6e,0x00000000,0x00000000,0x00000000
		};

		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, AsciiTexture, TEXT("AsciiTexture"));
		uint32 DestStride;
		uint8* DestBuffer = (uint8*)RHICmdList.LockTexture2D(AsciiTexture->GetRHI(), 0, RLM_WriteOnly, DestStride, false);

		for (int32 c = 0; c < CharacterCount; c++)
		{
			const uint32 Hi = Characters[c * 2];
			const uint32 Lo = Characters[c * 2 + 1];

			for (int32 y = 0; y < CharacterRes; y++)
			for (int32 x = 0; x < CharacterRes; x++)
			{
				uint8* Dest = (uint8*)(DestBuffer + x * BytesPerPixel + (c * CharacterStride) +  y * DestStride);

				const uint32 C = y < CharacterRes/2 ? Hi : Lo;
				const bool bFilled = (C & (1u << ((y * CharacterRes + x) & 31u)));
				Dest[0] = bFilled ? 0xFF : 0x0;
			}
		}
		RHICmdList.UnlockTexture2D(AsciiTexture->GetRHI(), 0, false);
	}

	// The PreintegratedGF maybe used on forward shading including mobile platform, initialize it anyway.
	{
		// for testing, with 128x128 R8G8 we are very close to the reference (if lower res is needed we might have to add an offset to counter the 0.5f texel shift)
		const bool bReference = false;

		EPixelFormat Format = PF_R8G8;
		// for low roughness we would get banding with PF_R8G8 but for low spec it could be used, for now we don't do this optimization
		if (GPixelFormats[PF_G16R16].Supported && UE::PixelFormat::HasCapabilities(PF_G16R16, EPixelFormatCapabilities::TextureFilterable))
		{
			Format = PF_G16R16;
		}

		FIntPoint Extent(128, 32);

		if (bReference)
		{
			Extent.X = 128;
			Extent.Y = 128;
		}

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("PreintegratedGF"), Extent, Format)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(SystemTexturesName)
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);

		// Write the contents of the texture.
		FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DSubresource(0);

		uint8* DestBuffer = reinterpret_cast<uint8*>(Subresource.Data);

		// x is NoV, y is roughness
		for (int32 y = 0; y < Extent.Y; y++)
		{
			float Roughness = (float)(y + 0.5f) / Extent.Y;
			float m = Roughness * Roughness;
			float m2 = m * m;

			for (int32 x = 0; x < Extent.X; x++)
			{
				float NoV = (float)(x + 0.5f) / Extent.X;

				FVector3f V;
				V.X = FMath::Sqrt(1.0f - NoV * NoV);	// sin
				V.Y = 0.0f;
				V.Z = NoV;								// cos

				float A = 0.0f;
				float B = 0.0f;
				float C = 0.0f;

				const uint32 NumSamples = 128;
				for (uint32 i = 0; i < NumSamples; i++)
				{
					float E1 = (float)i / NumSamples;
					float E2 = (double)ReverseBits(i) / (double)0x100000000LL;

					{
						float Phi = 2.0f * PI * E1;
						float CosPhi = FMath::Cos(Phi);
						float SinPhi = FMath::Sin(Phi);
						float CosTheta = FMath::Sqrt((1.0f - E2) / (1.0f + (m2 - 1.0f) * E2));
						float SinTheta = FMath::Sqrt(1.0f - CosTheta * CosTheta);

						FVector3f H(SinTheta * FMath::Cos(Phi), SinTheta * FMath::Sin(Phi), CosTheta);
						FVector3f L = 2.0f * (V | H) * H - V;

						float NoL = FMath::Max(L.Z, 0.0f);
						float NoH = FMath::Max(H.Z, 0.0f);
						float VoH = FMath::Max(V | H, 0.0f);

						if (NoL > 0.0f)
						{
							float Vis_SmithV = NoL * (NoV * (1 - m) + m);
							float Vis_SmithL = NoV * (NoL * (1 - m) + m);
							float Vis = 0.5f / (Vis_SmithV + Vis_SmithL);

							float NoL_Vis_PDF = NoL * Vis * (4.0f * VoH / NoH);
							float Fc = 1.0f - VoH;
							Fc *= FMath::Square(Fc*Fc);
							A += NoL_Vis_PDF * (1.0f - Fc);
							B += NoL_Vis_PDF * Fc;
						}
					}

					{
						float Phi = 2.0f * PI * E1;
						float CosPhi = FMath::Cos(Phi);
						float SinPhi = FMath::Sin(Phi);
						float CosTheta = FMath::Sqrt(E2);
						float SinTheta = FMath::Sqrt(1.0f - CosTheta * CosTheta);

						FVector3f L(SinTheta * FMath::Cos(Phi), SinTheta * FMath::Sin(Phi), CosTheta);
						FVector3f H = (V + L).GetUnsafeNormal();

						float NoL = FMath::Max(L.Z, 0.0f);
						float NoH = FMath::Max(H.Z, 0.0f);
						float VoH = FMath::Max(V | H, 0.0f);

						float FD90 = 0.5f + 2.0f * VoH * VoH * Roughness;
						float FdV = 1.0f + (FD90 - 1.0f) * pow(1.0f - NoV, 5);
						float FdL = 1.0f + (FD90 - 1.0f) * pow(1.0f - NoL, 5);
						C += FdV * FdL;// * ( 1.0f - 0.3333f * Roughness );
					}
				}
				A /= NumSamples;
				B /= NumSamples;
				C /= NumSamples;

				if (Format == PF_A16B16G16R16)
				{
					uint16* Dest = (uint16*)(DestBuffer + x * 8 + y * Subresource.Stride);
					Dest[0] = (int32)(FMath::Clamp(A, 0.0f, 1.0f) * 65535.0f + 0.5f);
					Dest[1] = (int32)(FMath::Clamp(B, 0.0f, 1.0f) * 65535.0f + 0.5f);
					Dest[2] = (int32)(FMath::Clamp(C, 0.0f, 1.0f) * 65535.0f + 0.5f);
				}
				else if (Format == PF_G16R16)
				{
					uint16* Dest = (uint16*)(DestBuffer + x * 4 + y * Subresource.Stride);
					Dest[0] = (int32)(FMath::Clamp(A, 0.0f, 1.0f) * 65535.0f + 0.5f);
					Dest[1] = (int32)(FMath::Clamp(B, 0.0f, 1.0f) * 65535.0f + 0.5f);
				}
				else
				{
					check(Format == PF_R8G8);

					uint8* Dest = (uint8*)(DestBuffer + x * 2 + y * Subresource.Stride);
					Dest[0] = (int32)(FMath::Clamp(A, 0.0f, 1.0f) * 255.f + 0.5f);
					Dest[1] = (int32)(FMath::Clamp(B, 0.0f, 1.0f) * 255.f + 0.5f);
				}
			}
		}

		FTextureRHIRef Texture = Initializer.Finalize();
		PreintegratedGF = CreateRenderTarget(Texture, Desc.DebugName);
	}

	{
	    // Create the PerlinNoise3D texture (similar to http://prettyprocs.wordpress.com/2012/10/20/fast-perlin-noise/)
	    {
		    const uint32 Extent = 16;
    
		    const uint32 Square = Extent * Extent;

			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create3D(TEXT("PerlinNoise3D"), Extent, Extent, Extent, PF_B8G8R8A8)
				.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::NoTiling)
				.SetClassName(SystemTexturesName);

			FTextureRHIRef Texture3D = RHICmdList.CreateTexture(Desc);

		    // Write the contents of the texture.
		    TArray<uint32> DestBuffer;
    
		    DestBuffer.AddZeroed(Extent * Extent * Extent);
		    // seed the pseudo random stream with a good value
		    FRandomStream RandomStream(0x1234);
		    // Values represent float3 values in the -1..1 range.
		    // The vectors are the edge mid point of a cube from -1 .. 1
		    // -1:0 0:7f 1:fe, can be reconstructed with * 512/254 - 1
		    // * 2 - 1 cannot be used because 0 would not be mapped
			    static uint32 gradtable[] =
		    {
			    0x7ffefe, 0xfe7ffe, 0xfefe7f,
			    0x7ffe00, 0xfe7f00, 0xfe007f,
			    0x7f00fe, 0x007ffe, 0x00fe7f,
			    0x7f0000, 0x007f00, 0x00007f,
		    };
		    // set random directions
		    {
				    for (uint32 z = 0; z < Extent - 1; ++z)
			    {
					    for (uint32 y = 0; y < Extent - 1; ++y)
				    {
						    for (uint32 x = 0; x < Extent - 1; ++x)
					    {
						    uint32& Value = DestBuffer[x + y * Extent + z * Square];
    
						    // pick a random direction (hacky way to overcome the quality issues FRandomStream has)
						    Value = gradtable[(uint32)(RandomStream.GetFraction() * 11.9999999f)];
					    }
				    }
			    }
		    }
		    // replicate a border for filtering
		    {
			    uint32 Last = Extent - 1;
    
				    for (uint32 z = 0; z < Extent; ++z)
			    {
					    for (uint32 y = 0; y < Extent; ++y)
				    {
					    DestBuffer[Last + y * Extent + z * Square] = DestBuffer[0 + y * Extent + z * Square];
				    }
			    }
				for (uint32 z = 0; z < Extent; ++z)
				{
					for (uint32 x = 0; x < Extent; ++x)
					{
					    DestBuffer[x + Last * Extent + z * Square] = DestBuffer[x + 0 * Extent + z * Square];
				    }
			    }
				for (uint32 y = 0; y < Extent; ++y)
				{
					for (uint32 x = 0; x < Extent; ++x)
				    {
					    DestBuffer[x + y * Extent + Last * Square] = DestBuffer[x + y * Extent + 0 * Square];
				    }
			    }
		    }
		    // precompute gradients
			{
			    uint32* Dest = DestBuffer.GetData();
    
				for (uint32 z = 0; z < Extent; ++z)
			    {
					for (uint32 y = 0; y < Extent; ++y)
				    {
						for (uint32 x = 0; x < Extent; ++x)
					    {
						    uint32 Value = *Dest;
    
						    // todo: check if rgb order is correct
						    int32 r = Value >> 16;
						    int32 g = (Value >> 8) & 0xff;
						    int32 b = Value & 0xff;
    
						    int nx = (r / 0x7f) - 1;
						    int ny = (g / 0x7f) - 1;
						    int nz = (b / 0x7f) - 1;
    
						    int32 d = nx * x + ny * y + nz * z;
    
						    // compress in 8bit
						    uint32 a = d + 127;
    
						    *Dest++ = Value | (a << 24);
					    }
				    }
			    }
		    }
    
		    FUpdateTextureRegion3D Region(0, 0, 0, 0, 0, 0, Extent, Extent, Extent);
    
		    RHICmdList.UpdateTexture3D(
			    Texture3D,
			    0,
			    Region,
			    Extent * sizeof(uint32),
			    Extent * Extent * sizeof(uint32),
			    (const uint8*)DestBuffer.GetData());

			PerlinNoise3D = CreateRenderTarget(Texture3D, Desc.DebugName);

		} // end Create the PerlinNoise3D texture
	}

	// Create the SSAO randomization texture
	{
		{
			float g_AngleOff1 = 127;
			float g_AngleOff2 = 198;
			float g_AngleOff3 = 23;

			FColor Bases[16];

			for (int32 Pos = 0; Pos < 16; ++Pos)
			{
				// distribute rotations over 4x4 pattern
						//			int32 Reorder[16] = { 0, 8, 2, 10, 12, 6, 14, 4, 3, 11, 1, 9, 15, 5, 13, 7 };
				int32 Reorder[16] = { 0, 11, 7, 3, 10, 4, 15, 12, 6, 8, 1, 14, 13, 2, 9, 5 };
				int32 w = Reorder[Pos];

				// ordered sampling of the rotation basis (*2 is missing as we use mirrored samples)
				float ww = w / 16.0f * PI;

				// randomize base scale
				float lenm = 1.0f - (FMath::Sin(g_AngleOff2 * w * 0.01f) * 0.5f + 0.5f) * g_AngleOff3 * 0.01f;
				float s = FMath::Sin(ww) * lenm;
				float c = FMath::Cos(ww) * lenm;

				Bases[Pos] = FColor(FMath::Quantize8SignedByte(c), FMath::Quantize8SignedByte(s), 0, 0);
			}

			{
				const uint32 Extent = 64;

				const FRHITextureCreateDesc Desc =
					FRHITextureCreateDesc::Create2D(TEXT("SSAORandomization"), Extent, Extent, PF_R8G8)
					.SetFlags(ETextureCreateFlags::ShaderResource)
					.SetClassName(SystemTexturesName)
					.SetInitActionInitializer();

				FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);

				// Write the contents of the texture.
				FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DSubresource(0);
				uint8* DestBuffer = (uint8*)Subresource.Data;

				for (int32 y = 0; y < Extent; ++y)
				{
					for (int32 x = 0; x < Extent; ++x)
					{
						uint8* Dest = (uint8*)(DestBuffer + x * sizeof(uint16) + y * Subresource.Stride);

						uint32 Index = (x % 4) + (y % 4) * 4;

						Dest[0] = Bases[Index].R;
						Dest[1] = Bases[Index].G;
					}
				}

				FTextureRHIRef Texture = Initializer.Finalize();
				SSAORandomization = CreateRenderTarget(Texture, Desc.DebugName);
			}
		}
	}
		
	static const auto MobileGTAOPreIntegratedTextureTypeCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.GTAOPreIntegratedTextureType"));

	if (CurrentFeatureLevel < ERHIFeatureLevel::ES3_1 && MobileGTAOPreIntegratedTextureTypeCVar && MobileGTAOPreIntegratedTextureTypeCVar->GetValueOnAnyThread() > 0)
	{
		uint32 Extent = 16; // should be consistent with LUTSize in PostprocessMobile.usf

		const uint32 Square = Extent * Extent;

		bool bGTAOPreIngegratedUsingVolumeLUT = MobileGTAOPreIntegratedTextureTypeCVar->GetValueOnAnyThread() == 2;

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc(TEXT("GTAOPreIntegrated"), bGTAOPreIngegratedUsingVolumeLUT ? ETextureDimension::Texture3D : ETextureDimension::Texture2D)
			.SetExtent(Extent)
			.SetDepth(bGTAOPreIngegratedUsingVolumeLUT ? Extent : 1)
			.SetFormat(PF_R16F)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetClassName(SystemTexturesName)
			.SetInitAction(bGTAOPreIngegratedUsingVolumeLUT ? ERHITextureInitAction::Default : ERHITextureInitAction::Initializer);

		FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(Desc);

		// Write the contents of the texture.
		TArray<FFloat16> TempBuffer;
		TempBuffer.AddZeroed(Extent * Extent * Extent);

		FFloat16* DestBuffer = nullptr;

		if (bGTAOPreIngegratedUsingVolumeLUT)
		{
			DestBuffer = TempBuffer.GetData();
		}
		else
		{
			FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DSubresource(0);
			DestBuffer = (FFloat16*)Subresource.Data;
		}

		for (uint32 z = 0; z < Extent; ++z)
		{
			for (uint32 y = 0; y < Extent; ++y)
			{
				for (uint32 x = 0; x < Extent; ++x)
				{
					uint32 DestBufferIndex = 0;

					if (bGTAOPreIngegratedUsingVolumeLUT)
					{
						DestBufferIndex = x + y * Extent + z * Square;
					}
					else
					{
						DestBufferIndex = (x + z * Extent) + y * Square;
					}
					FFloat16& Value = DestBuffer[DestBufferIndex];

					float cosAngle1 = ((x + 0.5f) / (Extent) - 0.5f) * 2;
					float cosAngle2 = ((y + 0.5f) / (Extent) - 0.5f) * 2;
					float cosAng = ((z + 0.5f) / (Extent) - 0.5f) * 2;

					float Gamma = FMath::Acos(cosAng) - HALF_PI;
					float CosGamma = FMath::Cos(Gamma);
					float SinGamma = cosAng * -2.0f;

					float Angle1 = FMath::Acos(cosAngle1);
					float Angle2 = FMath::Acos(cosAngle2);
					// clamp to normal hemisphere 
					Angle1 = Gamma + FMath::Max(-Angle1 - Gamma, -(HALF_PI));
					Angle2 = Gamma + FMath::Min(Angle2 - Gamma, (HALF_PI));

					float AO = (0.25f *
						((Angle1 * SinGamma + CosGamma - cos((2.0 * Angle1) - Gamma)) +
						(Angle2 * SinGamma + CosGamma - cos((2.0 * Angle2) - Gamma))));

					Value = AO;
				}
			}
		}

		TRefCountPtr<FRHITexture> Texture = Initializer.Finalize();

		if (bGTAOPreIngegratedUsingVolumeLUT)
		{
			FUpdateTextureRegion3D Region(0, 0, 0, 0, 0, 0, Extent, Extent, Extent);

			RHICmdList.UpdateTexture3D(
				(FRHITexture*)Texture.GetReference(),
				0,
				Region,
				Extent * sizeof(FFloat16),
				Extent * Extent * sizeof(FFloat16),
				(const uint8*)DestBuffer);
		}

		GTAOPreIntegrated = CreateRenderTarget(Texture, Desc.DebugName);
	}

    // Create a texture array that is a single UInt32 value set to 0, this texture is AtomicCompatible on SM6
    {
        FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2DArray(TEXT("ZeroUIntArrayAtomicCompatDummy"), 1, 1, 1, PF_R32_UINT)
			.SetFlags(ETextureCreateFlags::ShaderResource);

		if (InFeatureLevel >= ERHIFeatureLevel::SM6)
		{
			Desc.AddFlags(ETextureCreateFlags::AtomicCompatible);
		}

		FTextureRHIRef Texture = CreateDummyTexture<uint32>(RHICmdList, Desc, 0u);
        ZeroUIntArrayAtomicCompatDummy = CreateRenderTarget(Texture, Desc.DebugName);
    }
    
#if EXPORT_SYSTEM_TEXTURES
	ExportSystemTextures();
#endif

	// Initialize textures only once.
	FeatureLevelInitializedTo = InFeatureLevel;
}

void FSystemTextures::InitializeEngineDependentTextures(FRHICommandListImmediate& RHICmdList)
{	
	if (GEngine)
	{
		// LTC Textures	(used by Rect Lights and/or BSDF evaluation)

		// GGX - LTC matrix coefficients (4-coefficients)
		if (GGXLTCMat == nullptr && IsTextureDataValid(GEngine->GGXLTCMatTexture))
		{
			GGXLTCMat = CreateTexture(RHICmdList, GEngine->GGXLTCMatTexture, PF_FloatRGBA, TEXT("GGX.LTCMat"));
		}
		// GGX - Split-Sum Amplitude coefficients (2-components)
		if (GGXLTCAmp == nullptr && IsTextureDataValid(GEngine->GGXLTCAmpTexture))
		{
			GGXLTCAmp = CreateTexture(RHICmdList, GEngine->GGXLTCAmpTexture, PF_G16R16F, TEXT("GGX.LTCAmp"));
		}
		// Sheen - Matrix & directional albedo (3-components)
		if (SheenLTC == nullptr && IsTextureDataValid(GEngine->SheenLTCTexture))
		{
			SheenLTC  = CreateTexture(RHICmdList, GEngine->SheenLTCTexture,  PF_FloatRGBA, TEXT("Sheen.LTC"));
		}
	}

	// Initialize textures only once
	bEngineDependentTexturesInitialized = GGXLTCMat && GGXLTCAmp && SheenLTC;
}

void FSystemTextures::ReleaseRHI()
{
	WhiteDummySRV.SafeRelease();
	WhiteDummy.SafeRelease();
	BlackDummy.SafeRelease();
	BlackArrayDummy.SafeRelease();
	BlackAlphaOneDummy.SafeRelease();
	BlackArrayDummy.SafeRelease();
	PerlinNoiseGradient.SafeRelease();
	PerlinNoise3D.SafeRelease();
	SobolSampling.SafeRelease();
	SSAORandomization.SafeRelease();
	GTAOPreIntegrated.SafeRelease();
	PreintegratedGF.SafeRelease();
	HairLUT0.SafeRelease();
	HairLUT1.SafeRelease();
	HairLUT2.SafeRelease();
	GGXLTCMat.SafeRelease();
	GGXLTCAmp.SafeRelease();
	SheenLTC.SafeRelease();
	MaxFP16Depth.SafeRelease();
	DepthDummy.SafeRelease();
	GreenDummy.SafeRelease();
	DefaultNormal8Bit.SafeRelease();
	VolumetricBlackDummy.SafeRelease();
	VolumetricBlackAlphaOneDummy.SafeRelease();
	VolumetricBlackUintDummy.SafeRelease();
	CubeBlackDummy.SafeRelease();
	CubeArrayBlackDummy.SafeRelease();
	ZeroUIntDummy.SafeRelease();
    ZeroUIntArrayDummy.SafeRelease();
    ZeroUIntArrayAtomicCompatDummy.SafeRelease();
    MidGreyDummy.SafeRelease();
	StencilDummy.SafeRelease();
	StencilDummySRV.SafeRelease();
	BlackDepthCube.SafeRelease();
	GTAOPreIntegrated.SafeRelease();
	AsciiTexture.SafeRelease();
	SpaceFillingCurve.SafeRelease();

	DefaultTextures.Empty();
	DefaultBuffers.Empty();
	HashDefaultTextures.Clear();
	HashDefaultBuffers.Clear();

	GRenderTargetPool.FreeUnusedResources();

	// Indicate that textures will need to be reinitialized.
	FeatureLevelInitializedTo = ERHIFeatureLevel::Num;
}

FRDGTextureRef FSystemTextures::GetBlackDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(BlackDummy, TEXT("BlackDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetBlackAlphaOneDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(BlackAlphaOneDummy, TEXT("BlackAlphaOneDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetBlackArrayDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(BlackArrayDummy, TEXT("BlackArrayDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetWhiteDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(WhiteDummy, TEXT("WhiteDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetMaxFP16Depth(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(MaxFP16Depth, TEXT("MaxFP16Depth"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetDepthDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(DepthDummy, TEXT("DepthDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetStencilDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(StencilDummy, TEXT("StencilDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetGreenDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(GreenDummy, TEXT("GreenDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetDefaultNormal8Bit(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(DefaultNormal8Bit, TEXT("DefaultNormal8Bit"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetMidGreyDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(MidGreyDummy, TEXT("MidGreyDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetVolumetricBlackDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(VolumetricBlackDummy, TEXT("VolumetricBlackDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetVolumetricBlackUintDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(VolumetricBlackUintDummy, TEXT("VolumetricBlackUintDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetCubeBlackDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(CubeBlackDummy, TEXT("CubeBlackDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetCubeArrayBlackDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(CubeArrayBlackDummy, TEXT("CubeArrayBlackDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetZeroUIntDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(ZeroUIntDummy, TEXT("ZeroUIntDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetZeroUIntArrayDummy(FRDGBuilder& GraphBuilder) const
{
	return GraphBuilder.RegisterExternalTexture(ZeroUIntArrayDummy, TEXT("ZeroUIntArrayDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetZeroUIntArrayAtomicCompatDummy(FRDGBuilder& GraphBuilder) const
{
    return GraphBuilder.RegisterExternalTexture(ZeroUIntArrayAtomicCompatDummy, TEXT("ZeroUIntArrayAtomicCompatDummy"), ERDGTextureFlags::SkipTracking);
}

FRDGTextureRef FSystemTextures::GetSpaceFillingCurveTexture(FRDGBuilder& GraphBuilder)
{
	if (!SpaceFillingCurve.IsValid())
	{
		// Initialize the texture the first time it is used
		constexpr int Res = 256;
		FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create2D(TEXT("SpaceFillingCurve"), Res, Res, PF_R16_UINT)
			.SetFlags(ETextureCreateFlags::ShaderResource);
		TArray<FIntVector> Temp;
		Temp.Reserve(Res * Res);
		for (int Dy = 0; Dy < Res; Dy++)
		{
			for (int Dx = 0; Dx < Res; Dx++)
			{
				unsigned X = Dx, Y = Dy;
				// "A ROUTING SYSTEM BASED ON SPACEFILLING CURVES"
				// https://www2.isye.gatech.edu/~jjb/research/mow/mow.pdf
				// The paper describes a space filling curve from Sierpinski that passes through each pixel in a Res x Res square.
				// This routine only gives results that are in the right _order_, but do not form a strict permutation
				// so we need to store the inverse mapping and explicitly number the points after sorting them.
				const unsigned M = Res - 1;
				unsigned Result = 0;
				if (X > Y)
				{
					Result++;
					X = M - X;
					Y = M - Y;
				}
				for (unsigned L = M; L > 0; L /= 2)
				{
					Result += Result;
					if (X + Y > unsigned(M))
					{
						Result++;
						unsigned OldX = X;
						X = M - Y;
						Y = OldX;
					}
					X += X;
					Y += Y;
					Result += Result;
					if (Y > unsigned(M))
					{
						Result++;
						unsigned OldX = X;
						X = Y - M;
						Y = M - OldX;
					}
				}
				Temp.Add(FIntVector(Dx, Dy, Result));
			}
		}
		Temp.Sort([](FIntVector A, FIntVector B) { return A.Z < B.Z; });
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, SpaceFillingCurve, Desc.DebugName);
		check(SpaceFillingCurve.IsValid());
		uint32 DestStride;
		uint16* DestBuffer = (uint16*) GraphBuilder.RHICmdList.LockTexture2D(SpaceFillingCurve->GetRHI(), 0, RLM_WriteOnly, DestStride, false);
		DestStride /= sizeof(uint16);
		for (int Index = 0; Index < Res * Res; Index++)
		{
			FIntVector P = Temp[Index];
			DestBuffer[P.X + DestStride * P.Y] = Index;
		}
		GraphBuilder.RHICmdList.UnlockTexture2D(SpaceFillingCurve->GetRHI(), 0, false);
	}
	return GraphBuilder.RegisterExternalTexture(GSystemTextures.SpaceFillingCurve, TEXT("SpaceFillingCurve"), ERDGTextureFlags::SkipTracking);
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////
// Default textures

bool operator !=(const FDefaultTextureKey& A, const FDefaultTextureKey& B)
{
	return A.Format != B.Format ||
		A.Dimension != B.Dimension ||
		A.ValueAsUInt[0] != B.ValueAsUInt[0] ||
		A.ValueAsUInt[1] != B.ValueAsUInt[1] ||
		A.ValueAsUInt[2] != B.ValueAsUInt[2] ||
		A.ValueAsUInt[3] != B.ValueAsUInt[3];
}

template<typename T>
static FDefaultTextureKey GetDefaultTextureKey(ETextureDimension Dimension, EPixelFormat Format, const T& In)
{
	FDefaultTextureKey Out;
	const uint32 Size = sizeof(T);
	const uint32* InAsUInt = (const uint32*)&In;
	Out.ValueAsUInt[0] = InAsUInt[0];
	Out.ValueAsUInt[1] = Size > 4  ? InAsUInt[1] : 0u;
	Out.ValueAsUInt[2] = Size > 8  ? InAsUInt[2] : 0u;
	Out.ValueAsUInt[3] = Size > 12 ? InAsUInt[3] : 0u;
	Out.Format = Format;
	Out.Dimension = Dimension;
	return Out;
}

// Convert from X to 4 components data float/uint/int. Supported input are:
// * float
// * int32
// * uint32
// * FVector2D
// * FIntPoint
// * FVector3f
// * FVector4f
// * FUintVector4
// * FClearValueBinding
FIntVector4		ToVector(int32 Value)						{ return FIntVector4(Value, Value, Value, Value); }
FVector4f		ToVector(float Value)						{ return FVector4f(Value, Value, Value, Value); }
FUintVector4	ToVector(uint32 Value)						{ return FUintVector4(Value, Value, Value, Value); }
FVector4f		ToVector(const FVector3f & Value)			{ return FVector4f(Value.X, Value.Y, Value.Z, 0); }
FVector4f		ToVector(const FVector4f & Value)			{ return Value; }
FVector4f		ToVector(const FVector2D& Value)			{ return FVector4f(Value.X, Value.Y, 0, 0); }
FIntVector4		ToVector(const FIntPoint& Value)			{ return FIntVector4(Value.X, Value.Y, 0, 0); }
FUintVector4	ToVector(const FUintVector4& Value)			{ return Value; }
FVector4f		ToVector(const FClearValueBinding & Value)	{ return FVector4f(Value.Value.Color[0], Value.Value.Color[1], Value.Value.Color[2], Value.Value.Color[3]); }

template <typename TInputType>	struct TFormatConversionTraits				{ /*Error*/ };
template <>						struct TFormatConversionTraits<FVector4f>	{ typedef float  Type; };
template <>						struct TFormatConversionTraits<FUintVector4>{ typedef uint32 Type; };
template <>						struct TFormatConversionTraits<FIntVector4> { typedef int32  Type; };

enum class EDefaultInputType
{
	Typed,
	UNorm,
	SNorm,
	UNorm10,
	UNorm11,
	UNorm2,
	UNorm5,
	UNorm1
};

// Convert input type into the final type. This function manages UNorm/SNorm type by assuming if the input if float, its value is normalized in [0..1].
template <typename TInType, typename TOutType, EDefaultInputType InputFormatType>
TOutType ConvertInputFormat(const TInType& In)
{
	return TOutType(In);
}
template<> uint64 ConvertInputFormat<float, uint64, EDefaultInputType::UNorm>(const float& In) { return FMath::Clamp(In,  0.f, 1.f) * float(MAX_uint64); }
template<>  int64 ConvertInputFormat<float,  int64, EDefaultInputType::SNorm>(const float& In) { return FMath::Clamp(In, -1.f, 1.f) * float(MAX_int64); }
template<> uint32 ConvertInputFormat<float, uint32, EDefaultInputType::UNorm>(const float& In) { return FMath::Clamp(In,  0.f, 1.f) * float(MAX_uint32); }
template<>  int32 ConvertInputFormat<float,  int32, EDefaultInputType::SNorm>(const float& In) { return FMath::Clamp(In, -1.f, 1.f) * float(MAX_int32); }
template<> uint16 ConvertInputFormat<float, uint16, EDefaultInputType::UNorm>(const float& In) { return FMath::Clamp(In,  0.f, 1.f) * MAX_uint16; }
template<>  int16 ConvertInputFormat<float,  int16, EDefaultInputType::SNorm>(const float& In) { return FMath::Clamp(In, -1.f, 1.f) * MAX_int16; }
template<>  uint8 ConvertInputFormat<float,  uint8, EDefaultInputType::UNorm>(const float& In) { return FMath::Clamp(In,  0.f, 1.f) * MAX_uint8; }
template<>   int8 ConvertInputFormat<float,   int8, EDefaultInputType::SNorm>(const float& In) { return FMath::Clamp(In, -1.f, 1.f) * MAX_int8; }

template<> uint32 ConvertInputFormat<float, uint32, EDefaultInputType::UNorm10>(const float& In) { return FMath::Clamp(In, 0.f, 1.f) * 1024u; }
template<> uint32 ConvertInputFormat<float, uint32, EDefaultInputType::UNorm11>(const float& In) { return FMath::Clamp(In, 0.f, 1.f) * 2048u; }
template<> uint32 ConvertInputFormat<float, uint32, EDefaultInputType::UNorm2> (const float& In) { return FMath::Clamp(In, 0.f, 1.f) * 3u; }
template<> uint32 ConvertInputFormat<float, uint32, EDefaultInputType::UNorm1> (const float& In) { return uint32(In > 0.5f);}
template<> uint32 ConvertInputFormat<float, uint32, EDefaultInputType::UNorm5>(const float& In) { return FMath::Clamp(In, 0.f, 1.f) * 31u; }
// 4 components conversion with swizzling
template <EDefaultInputType InputFormatType, typename TInType, typename TOutType, uint32 SwizzleX, uint32 SwizzleY, uint32 SwizzleZ, uint32 SwizzleW>
void FormatData(const TInType& In, uint8* Out, uint32& OutByteCount)
{
	TOutType* OutTyped = (TOutType*)Out;
	OutTyped[0] = ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, TOutType, InputFormatType>(In[SwizzleX]);
	OutTyped[1] = ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, TOutType, InputFormatType>(In[SwizzleY]);
	OutTyped[2] = ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, TOutType, InputFormatType>(In[SwizzleZ]);
	OutTyped[3] = ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, TOutType, InputFormatType>(In[SwizzleW]);
	OutByteCount = 4 * sizeof(TOutType);
}

// 3 components conversion with swizzling
template <EDefaultInputType InputFormatType, typename TInType, typename TOutType, uint32 SwizzleX, uint32 SwizzleY, uint32 SwizzleZ>
void FormatData(const TInType& In, uint8* Out, uint32& OutByteCount)
{
	TOutType* OutTyped = (TOutType*)Out;
	OutTyped[0] = ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, TOutType, InputFormatType>(In[SwizzleX]);
	OutTyped[1] = ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, TOutType, InputFormatType>(In[SwizzleY]);
	OutTyped[2] = ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, TOutType, InputFormatType>(In[SwizzleZ]);
	OutByteCount = 3 * sizeof(TOutType);
}

// 2 components conversion with swizzling
template <EDefaultInputType InputFormatType, typename TInType, typename TOutType, uint32 SwizzleX, uint32 SwizzleY>
void FormatData(const TInType& In, uint8* Out, uint32& OutByteCount)
{
	TOutType* OutTyped = (TOutType*)Out;
	OutTyped[0] = ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, TOutType, InputFormatType>(In[SwizzleX]);
	OutTyped[1] = ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, TOutType, InputFormatType>(In[SwizzleY]);
	OutByteCount = 2 * sizeof(TOutType);
}

// 1 component conversion
template <EDefaultInputType InputFormatType, typename TInType, typename TOutType>
void FormatData(const TInType& In, uint8* Out, uint32& OutByteCount)
{
	TOutType* OutTyped = (TOutType*)Out;
	OutTyped[0] = ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, TOutType, InputFormatType>(In[0]);
	OutByteCount = sizeof(TOutType);
}

template <typename TInType>
void FormatData111110(const TInType& In, uint8* Out, uint32& OutByteCount)
{
	uint32* OutTyped = (uint32*)Out;
	*OutTyped = 
		 (2047u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm11>(In[0]))     |
		((2047u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm11>(In[1]))<<11)|
		((1023u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm10>(In[2]))<<22);
	OutByteCount = 4;
}

template <typename TInType>
void FormatData1010102(const TInType& In, uint8* Out, uint32& OutByteCount)
{
	uint32* OutTyped = (uint32*)Out;
	*OutTyped =
		 (1023u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm10>(In[0]))        |
		((1023u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm10>(In[1])) << 10) |
		((1023u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm10>(In[2])) << 20) |
		((   3u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm2> (In[3])) << 30);
	OutByteCount = 4;
}

template<typename TInType>
void FormatData5551(const TInType& In, uint8* Out, uint32& OutByteCount)
{
	uint16* OutTyped = (uint16*)Out;
	*OutTyped =
		(
		((31u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm5>(In[0])) << 1) |
		((31u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm5>(In[1])) << 6) |
		((31u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm5>(In[2])) << 11) |
		((1u & ConvertInputFormat<typename TFormatConversionTraits<TInType>::Type, uint32, EDefaultInputType::UNorm1>(In[3]))));
	OutByteCount = 2;
}

template<typename TInType>
void FormatData9995(const TInType& In, uint8* Out, uint32& OutByteCount)
{
	FFloat3PackedSE PackedFloat(FLinearColor(In[0], In[1], In[2], 1.0f));
	uint32* OutTyped = (uint32*)Out;
	OutByteCount = 4;
	*OutTyped = PackedFloat.EncodedValue;
}

template<typename TInType>
void InitializeData(const TInType& InData, EPixelFormat InFormat, uint8* OutData, uint32& OutByteCount)
{
	// If a new format is added insure that it is either supported here, or at least flagged as not supported
	static_assert(PF_MAX == 94);

	switch (InFormat)
	{
		// 64bits
		case PF_R64_UINT:				{ FormatData<EDefaultInputType::Typed, TInType, uint64>					(InData, OutData, OutByteCount); } break;

		// 32bits
		case PF_R32G32B32A32_UINT:		{ FormatData<EDefaultInputType::Typed, TInType,	uint32,  0, 1, 2, 3>	(InData, OutData, OutByteCount); } break;
		case PF_A32B32G32R32F:			{ FormatData<EDefaultInputType::Typed, TInType,	float,   3, 2, 1, 0>	(InData, OutData, OutByteCount); } break;
		case PF_R32G32B32_UINT:			{ FormatData<EDefaultInputType::Typed, TInType, uint32,  0, 1, 2>		(InData, OutData, OutByteCount); } break;
		case PF_R32G32B32_SINT:			{ FormatData<EDefaultInputType::Typed, TInType, int32,   0, 1, 2>		(InData, OutData, OutByteCount); } break;
		case PF_R32G32B32F:				{ FormatData<EDefaultInputType::Typed, TInType, float,   0, 1, 2>		(InData, OutData, OutByteCount); } break;
		case PF_R32G32_UINT:			{ FormatData<EDefaultInputType::Typed, TInType,	uint32,  0, 1>			(InData, OutData, OutByteCount); } break;
		case PF_G32R32F:				{ FormatData<EDefaultInputType::Typed, TInType,	float,   1, 0>			(InData, OutData, OutByteCount); } break;
		case PF_R32_UINT:				{ FormatData<EDefaultInputType::Typed, TInType,	uint32>					(InData, OutData, OutByteCount); } break;
		case PF_R32_SINT:				{ FormatData<EDefaultInputType::Typed, TInType,	int32>					(InData, OutData, OutByteCount); } break;
		case PF_R32_FLOAT:				{ FormatData<EDefaultInputType::Typed, TInType,	float>					(InData, OutData, OutByteCount); } break;

		// 16bits
		case PF_R16G16B16A16_UINT:		{ FormatData<EDefaultInputType::Typed, TInType,	uint16,   0, 1, 2, 3>	(InData, OutData, OutByteCount); } break;
		case PF_R16G16B16A16_SINT:		{ FormatData<EDefaultInputType::Typed, TInType,	int16,    0, 1, 2, 3>	(InData, OutData, OutByteCount); } break;
		case PF_R16G16B16A16_UNORM:		{ FormatData<EDefaultInputType::UNorm, TInType,	uint16,   0, 1, 2, 3>	(InData, OutData, OutByteCount); } break;
		case PF_R16G16B16A16_SNORM:		{ FormatData<EDefaultInputType::SNorm, TInType,	int16,    0, 1, 2, 3>	(InData, OutData, OutByteCount); } break;
		case PF_A16B16G16R16:			{ FormatData<EDefaultInputType::UNorm, TInType,	uint16,   3, 2, 1, 0>	(InData, OutData, OutByteCount); } break;
		case PF_FloatRGBA:				{ FormatData<EDefaultInputType::Typed, TInType,	FFloat16, 0, 1, 2, 3>	(InData, OutData, OutByteCount); } break;
		case PF_R16G16_UINT:			{ FormatData<EDefaultInputType::Typed, TInType,	uint16,   0, 1>			(InData, OutData, OutByteCount); } break;
		case PF_R16G16_SINT:			{ FormatData<EDefaultInputType::Typed, TInType,	int16,    0, 1>			(InData, OutData, OutByteCount); } break;
		case PF_G16R16:					{ FormatData<EDefaultInputType::UNorm, TInType,	uint16,   1, 0>			(InData, OutData, OutByteCount); } break;
		case PF_G16R16_SNORM:			{ FormatData<EDefaultInputType::SNorm, TInType,	int16,    1, 0>			(InData, OutData, OutByteCount); } break;
		case PF_G16R16F:				{ FormatData<EDefaultInputType::Typed, TInType,	FFloat16, 0, 1>			(InData, OutData, OutByteCount); } break;
		case PF_G16R16F_FILTER:			{ FormatData<EDefaultInputType::Typed, TInType,	FFloat16, 0, 1>			(InData, OutData, OutByteCount); } break;
		case PF_R16F_FILTER:			{ FormatData<EDefaultInputType::Typed, TInType,	FFloat16>				(InData, OutData, OutByteCount); } break;
		case PF_R16F:					{ FormatData<EDefaultInputType::Typed, TInType,	FFloat16>				(InData, OutData, OutByteCount); } break;
		case PF_G16:					{ FormatData<EDefaultInputType::UNorm, TInType,	uint16>					(InData, OutData, OutByteCount); } break;
		case PF_R16_UINT:				{ FormatData<EDefaultInputType::Typed, TInType,	uint16>					(InData, OutData, OutByteCount); } break;
		case PF_R16_SINT:				{ FormatData<EDefaultInputType::Typed, TInType,	int16>					(InData, OutData, OutByteCount); } break;

		// 8bits
		case PF_B8G8R8A8:				{ FormatData<EDefaultInputType::UNorm, TInType,	uint8,    2, 1, 0, 3>	(InData, OutData, OutByteCount); } break;
		case PF_R8G8B8A8:				{ FormatData<EDefaultInputType::UNorm, TInType,	uint8,    0, 1, 2, 3>	(InData, OutData, OutByteCount); } break;
		case PF_A8R8G8B8:				{ FormatData<EDefaultInputType::UNorm, TInType,	uint8,    3, 2, 1, 0>	(InData, OutData, OutByteCount); } break;
		case PF_R8G8B8A8_UINT:			{ FormatData<EDefaultInputType::Typed, TInType,	uint8,    0, 1, 2, 3>	(InData, OutData, OutByteCount); } break;
		case PF_R8G8B8A8_SNORM:			{ FormatData<EDefaultInputType::SNorm, TInType,	int8,     0, 1, 2, 3>	(InData, OutData, OutByteCount); } break;
		case PF_R8G8B8:					{ FormatData<EDefaultInputType::UNorm, TInType,	uint8,    0, 1, 2>		(InData, OutData, OutByteCount); } break;
		case PF_R8G8:					{ FormatData<EDefaultInputType::UNorm, TInType,	uint8,    0, 1>			(InData, OutData, OutByteCount); } break;
		case PF_R8G8_UINT:				{ FormatData<EDefaultInputType::Typed, TInType, uint8,    0, 1>			(InData, OutData, OutByteCount); } break;
		case PF_R8_UINT:				{ FormatData<EDefaultInputType::Typed, TInType,	uint8>					(InData, OutData, OutByteCount); } break;
		case PF_R8_SINT:				{ FormatData<EDefaultInputType::Typed, TInType, int8>					(InData, OutData, OutByteCount); } break;
		case PF_R8:						{ FormatData<EDefaultInputType::UNorm, TInType,	uint8>					(InData, OutData, OutByteCount); } break;
		case PF_G8:						{ FormatData<EDefaultInputType::UNorm, TInType,	uint8>					(InData, OutData, OutByteCount); } break;
		case PF_L8:						{ FormatData<EDefaultInputType::UNorm, TInType,	uint8>					(InData, OutData, OutByteCount); } break;
		case PF_A1:						{ FormatData<EDefaultInputType::UNorm, TInType,	uint8>					(InData, OutData, OutByteCount); } break;
		case PF_A8:						{ FormatData<EDefaultInputType::UNorm, TInType,	uint8>					(InData, OutData, OutByteCount); } break;

		// Depth/Stencil. Since these texture will only be used as SRV, we handle them as regular float/float16.
		case PF_D24:					{ FormatData<EDefaultInputType::Typed, TInType,	float>					(InData, OutData, OutByteCount); } break;
		case PF_DepthStencil:			{ FormatData<EDefaultInputType::Typed, TInType,	float>					(InData, OutData, OutByteCount); } break;
		case PF_ShadowDepth:			{ FormatData<EDefaultInputType::Typed, TInType,	FFloat16>				(InData, OutData, OutByteCount); } break;

		// Custom
		case PF_FloatRGB:				{ FormatData111110<TInType>	(InData, OutData, OutByteCount); } break;
		case PF_A2B10G10R10:			{ FormatData1010102<TInType>(InData, OutData, OutByteCount); } break;
		case PF_FloatR11G11B10:			{ FormatData111110<TInType>	(InData, OutData, OutByteCount); } break;
		case PF_B5G5R5A1_UNORM:         { FormatData5551<TInType>(InData, OutData, OutByteCount); } break;
		case PF_R9G9B9EXP5:				{ FormatData9995<TInType>(InData, OutData, OutByteCount); }	break;
			return;

		// Not supported
		case PF_R5G6B5_UNORM:
		case PF_BC5:
		case PF_V8U8:
		case PF_PVRTC2:
		case PF_PVRTC4:
		case PF_UYVY:
		case PF_DXT1:
		case PF_DXT3:
		case PF_DXT5:
		case PF_BC4:
		case PF_ATC_RGB:
		case PF_ATC_RGBA_E:
		case PF_ATC_RGBA_I:
		case PF_X24_G8:
		case PF_ETC1:
		case PF_ETC2_RGB:
		case PF_ETC2_RGBA:
		case PF_ASTC_4x4:
		case PF_ASTC_6x6:
		case PF_ASTC_8x8:
		case PF_ASTC_10x10:
		case PF_ASTC_12x12:
		case PF_ASTC_4x4_HDR:
		case PF_ASTC_6x6_HDR:
		case PF_ASTC_8x8_HDR:
		case PF_ASTC_10x10_HDR:
		case PF_ASTC_12x12_HDR:
		case PF_ASTC_4x4_NORM_RG:
		case PF_ASTC_6x6_NORM_RG:
		case PF_ASTC_8x8_NORM_RG:
		case PF_ASTC_10x10_NORM_RG:
		case PF_ASTC_12x12_NORM_RG:
		case PF_BC6H:
		case PF_BC7:
		case PF_XGXR8:
		case PF_PLATFORM_HDR_0:
		case PF_PLATFORM_HDR_1:
		case PF_PLATFORM_HDR_2:
		case PF_NV12:
		case PF_ETC2_R11_EAC:
		case PF_ETC2_RG11_EAC:
		case PF_P010:
		case PF_Unknown:
		case PF_MAX:
			OutByteCount = 0;
			return;
	}
}

template <typename DataType>
void SetDefaultTextureData2D(FRHITextureInitializer& Initializer, const FRHITextureCreateDesc& InDesc, const DataType& InData)
{
	FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DSubresource(0);

	uint32 SrcByteCount = 0;
	InitializeData(ToVector(InData), InDesc.Format, (uint8*)Subresource.Data, SrcByteCount);
}

template <typename DataType>
void SetDefaultTextureData2DArray(FRHITextureInitializer& Initializer, const FRHITextureCreateDesc& InDesc, const DataType& InData)
{
	FRHITextureSubresourceInitializer Subresource = Initializer.GetTexture2DArraySubresource(0, 0);

	uint32 SrcByteCount = 0;
	InitializeData(ToVector(InData), InDesc.Format, (uint8*)Subresource.Data, SrcByteCount);
}

template <typename DataType>
void SetDefaultTextureData3D(FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, const FRHITextureCreateDesc& InDesc, const DataType& InData)
{
	uint8 SrcData[16] = { 0,0,0,0, 0,0,0,0, 0,0,0,0, 0,0,0,0, };
	uint32 SrcByteCount = 0;
	InitializeData(ToVector(InData), InDesc.Format, SrcData, SrcByteCount);

	FUpdateTextureRegion3D Region(0, 0, 0, 0, 0, 0, 1, 1, 1);
	RHICmdList.UpdateTexture3D(
		Texture,
		0,
		Region,
		SrcByteCount,
		SrcByteCount,
		SrcData);

	// UpdateTexture3D before and after state is currently undefined
	RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::Unknown, ERHIAccess::SRVMask));
}

template <typename DataType>
void SetDefaultTextureDataCube(FRHITextureInitializer& Initializer, const FRHITextureCreateDesc& InDesc, const DataType& InData)
{
	uint8 SrcData[16] = { 0,0,0,0, 0,0,0,0, 0,0,0,0, 0,0,0,0, };
	uint32 SrcByteCount = 0;
	InitializeData(ToVector(InData), InDesc.Format, SrcData, SrcByteCount);

	for (uint32 FaceIt = 0; FaceIt < 6; ++FaceIt)
	{
		FRHITextureSubresourceInitializer Subresource = Initializer.GetTextureCubeSubresource(FaceIt, 0);
		Subresource.WriteData(SrcData, SrcByteCount);
	}
}

template<typename TClearValue>
FRDGTextureRef GetInternalDefaultTexture(
	FRDGBuilder& GraphBuilder, 
	TArray<FDefaultTexture>& DefaultTextures,
	FHashTable& HashDefaultTextures,
	ETextureDimension Dimension, 
	EPixelFormat Format, 
	TClearValue Value)
{
	// Check this is a valid format
	check(Format != PF_Unknown && Format != PF_MAX && GPixelFormats[Format].BlockSizeX == 1 && GPixelFormats[Format].BlockSizeY == 1 && GPixelFormats[Format].BlockSizeZ == 1);

	// Convert Depth/Stencil format to float/float16 since these texture will only be used as SRV
	if (Format == PF_D24 || Format == PF_DepthStencil)	{ Format = PF_R32_FLOAT; }
	if (Format == PF_ShadowDepth)						{ Format = PF_R32_FLOAT; }

	const FDefaultTextureKey Key = GetDefaultTextureKey(Dimension, Format, Value);
	const uint32 Hash = Murmur32({uint32(Key.Dimension), uint32(Key.Format), Key.ValueAsUInt[0], Key.ValueAsUInt[1], Key.ValueAsUInt[2], Key.ValueAsUInt[3]});

	uint32 Index = HashDefaultTextures.First(Hash);
	while (HashDefaultTextures.IsValid(Index) && DefaultTextures[Index].Key != Key)
	{
		Index = HashDefaultTextures.Next(Index);
		check(DefaultTextures[Index].Hash == Hash); //Sanitycheck
	}

	if (HashDefaultTextures.IsValid(Index) && DefaultTextures[Index].Texture != nullptr)
	{
		return GraphBuilder.RegisterExternalTexture(DefaultTextures[Index].Texture, ERDGTextureFlags::SkipTracking);
	}

	FDefaultTexture Entry;
	Entry.Key = Key;
	Entry.Hash = Hash;
	Entry.Texture = nullptr;
	
	if (Dimension == ETextureDimension::Texture2D)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("DefaultTexture2D"), 1, 1, Format)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = GraphBuilder.RHICmdList.CreateTextureInitializer(Desc);
		SetDefaultTextureData2D(Initializer, Desc, Value);

		FTextureRHIRef Texture = Initializer.Finalize();
		Entry.Texture = CreateRenderTarget(Texture, Desc.DebugName);
	}
	else if (Dimension == ETextureDimension::Texture2DArray)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2DArray(TEXT("DefaultTexture2DArray"), 1, 1, 1, Format)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = GraphBuilder.RHICmdList.CreateTextureInitializer(Desc);
		SetDefaultTextureData2DArray(Initializer, Desc, Value);

		FTextureRHIRef Texture = Initializer.Finalize();
		Entry.Texture = CreateRenderTarget(Texture, Desc.DebugName);
	}
	else if (Dimension == ETextureDimension::Texture3D)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create3D(TEXT("DefaultTexture3D"), 1, 1, 1, Format)
			.SetFlags(ETextureCreateFlags::ShaderResource);

		FTextureRHIRef Texture = GraphBuilder.RHICmdList.CreateTexture(Desc);
		SetDefaultTextureData3D(GraphBuilder.RHICmdList, Texture, Desc, Value);
		Entry.Texture = CreateRenderTarget(Texture, Desc.DebugName);
	}
	else if (Dimension == ETextureDimension::TextureCube)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::CreateCube(TEXT("DefaultTextureCube"), 1, Format)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = GraphBuilder.RHICmdList.CreateTextureInitializer(Desc);
		SetDefaultTextureDataCube(Initializer, Desc, Value);

		FTextureRHIRef Texture = Initializer.Finalize();
		Entry.Texture = CreateRenderTarget(Texture, Desc.DebugName);
	}
	else if (Dimension == ETextureDimension::TextureCubeArray)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::CreateCubeArray(TEXT("DefaultTextureCubeArray"), 1, 1, Format)
			.SetFlags(ETextureCreateFlags::ShaderResource)
			.SetInitActionInitializer();

		FRHITextureInitializer Initializer = GraphBuilder.RHICmdList.CreateTextureInitializer(Desc);
		SetDefaultTextureDataCube(Initializer, Desc, Value);

		FTextureRHIRef Texture = Initializer.Finalize();
		Entry.Texture = CreateRenderTarget(Texture, Desc.DebugName);
	}
	else
	{
		return nullptr;
	}

	Index = DefaultTextures.Add(Entry);
	HashDefaultTextures.Add(Hash, Index);
	return GraphBuilder.RegisterExternalTexture(Entry.Texture, ERDGTextureFlags::SkipTracking);
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////
// Default Buffers

template<typename T>
static FDefaultBufferKey GetDefaultBufferKey(uint32 NumBytePerElement, EDefaultBufferType BufferType, const T* In)
{
	FDefaultBufferKey Out;
	if (In)
	{
		const uint32 ClearValueNumBytes = sizeof(T);
		const uint32* InAsUInt = (const uint32*)In;
		Out.ValueAsUInt[0] = InAsUInt[0];
		Out.ValueAsUInt[1] = ClearValueNumBytes > 4  ? InAsUInt[1] : 0u;
		Out.ValueAsUInt[2] = ClearValueNumBytes > 8  ? InAsUInt[2] : 0u;
		Out.ValueAsUInt[3] = ClearValueNumBytes > 12 ? InAsUInt[3] : 0u;
	}

	Out.NumBytePerElement	= NumBytePerElement;
	Out.BufferType			= BufferType;
	return Out;
}

bool operator !=(const FDefaultBufferKey& A, const FDefaultBufferKey& B)
{
	return A.NumBytePerElement != B.NumBytePerElement ||
		A.BufferType != B.BufferType ||
		A.ValueAsUInt[0] != B.ValueAsUInt[0] ||
		A.ValueAsUInt[1] != B.ValueAsUInt[1] ||
		A.ValueAsUInt[2] != B.ValueAsUInt[2] ||
		A.ValueAsUInt[3] != B.ValueAsUInt[3];
}

template<typename TClearValue>
FRDGBufferRef GetInternalDefaultBuffer(
	FRDGBuilder& GraphBuilder, 
	TArray<FDefaultBuffer>& DefaultBuffers,
	FHashTable& HashDefaultBuffers,
	uint32 NumBytePerElement,
	EDefaultBufferType BufferType,
	const TClearValue* Value)
{
	// Buffer key
	const uint32 NumElements = 1;
	const FDefaultBufferKey Key = GetDefaultBufferKey(NumBytePerElement, BufferType, Value);
	const uint32 Hash = Murmur32({(uint32)BufferType, Key.NumBytePerElement, Key.ValueAsUInt[0], Key.ValueAsUInt[1], Key.ValueAsUInt[2], Key.ValueAsUInt[3] });

	// Find existing buffer ("fast" path)
	uint32 Index = HashDefaultBuffers.First(Hash);
	while (HashDefaultBuffers.IsValid(Index) && DefaultBuffers[Index].Key != Key)
	{
		Index = HashDefaultBuffers.Next(Index);
	}

	if (HashDefaultBuffers.IsValid(Index) && DefaultBuffers[Index].Buffer != nullptr)
	{
		check(DefaultBuffers[Index].Hash == Hash); //Sanitycheck
		return GraphBuilder.RegisterExternalBuffer(DefaultBuffers[Index].Buffer, ERDGBufferFlags::SkipTracking);
	}

	const uint32 BufferSize = NumBytePerElement * NumElements;

	const TCHAR* BufferName = TEXT("");


	FRDGBufferDesc BufferDesc;
	FRHICommandListBase& RHICmdList = GraphBuilder.RHICmdList;

	// Adding new buffer if there is no fit (slow path)
	TRefCountPtr<FRHIBuffer> RHIBuffer;
	switch (BufferType)
	{
	case EDefaultBufferType::VertexBuffer:
	{
		BufferName = TEXT("DefaultBuffer");
		BufferDesc = FRDGBufferDesc::CreateUploadDesc(NumBytePerElement, NumElements);

		const FRHIBufferCreateDesc BufferCreateDesc =
			FRHIBufferCreateDesc::CreateVertex(BufferName, BufferSize)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
			.DetermineInitialState();
		RHIBuffer = RHICmdList.CreateBuffer(BufferCreateDesc);
	}
		break;

	case EDefaultBufferType::StructuredBuffer:
	{
		BufferName = TEXT("DefaultStructuredBuffer");
		BufferDesc = FRDGBufferDesc::CreateStructuredDesc(NumBytePerElement, NumElements);
		// Remove the UAV flag, as default resources are supposed to be read-only.
		EnumRemoveFlags(BufferDesc.Usage, EBufferUsageFlags::UnorderedAccess);
		
		const FRHIBufferCreateDesc BufferCreateDesc =
			FRHIBufferCreateDesc::CreateStructured(BufferName, BufferSize, NumBytePerElement)
			.AddUsage(BufferDesc.Usage)
			.DetermineInitialState();
		RHIBuffer = RHICmdList.CreateBuffer(BufferCreateDesc);
	}
		break;

	case EDefaultBufferType::ByteAddressBuffer:
	{
		BufferName = TEXT("DefaultByteAddressBuffer");
		BufferDesc = FRDGBufferDesc::CreateByteAddressDesc(BufferSize);
		// Same as above.
		EnumRemoveFlags(BufferDesc.Usage, EBufferUsageFlags::UnorderedAccess);

		const FRHIBufferCreateDesc BufferCreateDesc =
			FRHIBufferCreateDesc::CreateStructured(BufferName, BufferSize, NumBytePerElement)
			.AddUsage(BufferDesc.Usage)
			.DetermineInitialState();
		RHIBuffer = RHICmdList.CreateBuffer(BufferCreateDesc);
	}
		break;

	}

	uint8* DestPtr = static_cast<uint8*>(RHICmdList.LockBuffer(RHIBuffer, 0, BufferSize, RLM_WriteOnly));
	if (Value)
	{
		const uint8 *EndPtr = DestPtr + BufferSize; 
		for (uint32 Offset = 0; Offset < (BufferSize / sizeof(TClearValue)); Offset++)
		{
			FMemory::Memcpy(DestPtr, Value, sizeof(TClearValue));
			DestPtr += sizeof(TClearValue);
		}
		if (DestPtr < EndPtr)
		{
			// Zero out the remainder. Byte-splitting the init value is undefined.
			FMemory::Memzero(DestPtr, static_cast<SIZE_T>(EndPtr - DestPtr));
		}
	}
	else
	{
		FMemory::Memzero(DestPtr, BufferSize);
	}
	RHICmdList.UnlockBuffer(RHIBuffer);

	FDefaultBuffer Entry;
	Entry.Key = Key;
	Entry.Hash = Hash;
	Entry.Buffer = new FRDGPooledBuffer(RHICmdList, RHIBuffer, BufferDesc, NumElements, BufferName);

	Index = DefaultBuffers.Add(Entry);
	HashDefaultBuffers.Add(Hash, Index);
	return GraphBuilder.RegisterExternalBuffer(Entry.Buffer, ERDGBufferFlags::SkipTracking);
}

FVector4f GetClearBindingValue(EPixelFormat Format, FClearValueBinding Value)
{
	if (IsDepthOrStencilFormat(Format))
	{
		return FVector4f(Value.Value.DSValue.Depth, Value.Value.DSValue.Depth, Value.Value.DSValue.Depth, Value.Value.DSValue.Depth);
	}
	else
	{
		return FVector4f(Value.Value.Color[0], Value.Value.Color[1], Value.Value.Color[2], Value.Value.Color[3]);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Textures 

FRDGTextureRef FSystemTextures::GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, float Value)												{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, ETextureDimension::Texture2D, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, uint32 Value)												{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, ETextureDimension::Texture2D, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, const FVector3f& Value)										{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, ETextureDimension::Texture2D, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, const FVector4f& Value)										{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, ETextureDimension::Texture2D, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, const FUintVector4& Value)									{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, ETextureDimension::Texture2D, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, const FClearValueBinding& Value)							{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, ETextureDimension::Texture2D, Format, GetClearBindingValue(Format, Value)); }

FRDGTextureRef FSystemTextures::GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, float Value)						{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, Dimension, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, uint32 Value)					{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, Dimension, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FVector2D& Value)			{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, Dimension, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FIntPoint& Value)			{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, Dimension, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FVector3f& Value)			{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, Dimension, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FVector4f& Value)			{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, Dimension, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FUintVector4& Value)		{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, Dimension, Format, Value); }
FRDGTextureRef FSystemTextures::GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FClearValueBinding& Value)	{ return GetInternalDefaultTexture(GraphBuilder, DefaultTextures, HashDefaultTextures, Dimension, Format, GetClearBindingValue(Format, Value)); }

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Buffers

// Default init to 0
FRDGBufferRef FSystemTextures::GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement)										{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::VertexBuffer, (uint32*)nullptr); }
FRDGBufferRef FSystemTextures::GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement)								{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::StructuredBuffer, (uint32*)nullptr); }
FRDGBufferRef FSystemTextures::GetDefaultByteAddressBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement)								{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::ByteAddressBuffer, (uint32*)nullptr); }

// Default value of an element
FRDGBufferRef FSystemTextures::GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, float Value)							{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::VertexBuffer, &Value); }
FRDGBufferRef FSystemTextures::GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, uint32 Value)							{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::VertexBuffer, &Value); }
FRDGBufferRef FSystemTextures::GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FVector3f& Value)				{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::VertexBuffer, &Value); }
FRDGBufferRef FSystemTextures::GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FVector4f& Value)				{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::VertexBuffer, &Value); }
FRDGBufferRef FSystemTextures::GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FUintVector4& Value)				{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::VertexBuffer, &Value); }

FRDGBufferRef FSystemTextures::GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, float Value)					{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::StructuredBuffer, &Value); }
FRDGBufferRef FSystemTextures::GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, uint32 Value)				{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::StructuredBuffer, &Value); }
FRDGBufferRef FSystemTextures::GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FVector3f& Value)		{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::StructuredBuffer, &Value); }
FRDGBufferRef FSystemTextures::GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FVector4f& Value)		{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::StructuredBuffer, &Value); }
FRDGBufferRef FSystemTextures::GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FUintVector4& Value)	{ return GetInternalDefaultBuffer(GraphBuilder, DefaultBuffers, HashDefaultBuffers, NumBytePerElement, EDefaultBufferType::StructuredBuffer, &Value); }

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Default resources

void FSystemTextures::SetDefaultIfNull(FRDGBuilder& GraphBuilder, FRDGTextureRef& Out)
{
	if (!Out)
	{
		Out = GetBlackDummy(GraphBuilder);
	}
}

void FSystemTextures::SetDefaultIfNull(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef& Out)
{
	if (!Out)
	{
		const FRDGTextureRef DummyTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_R32_UINT, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV), TEXT("DummyUAV"));
		Out = GraphBuilder.CreateUAV(DummyTexture);
	}
}

void FSystemTextures::SetDefaultIfNull(FRDGBuilder& GraphBuilder, TRDGBufferAccess<ERHIAccess::IndirectArgs>& Out)
{
	if (!Out)
	{
		Out = GetDefaultBuffer(GraphBuilder, 4u, 0u);
	}
}

void FSystemTextures::SetDefaultIfNull(FRDGBuilder& GraphBuilder, FRDGBufferRef& Out)
{
	if (!Out)
	{
		Out = GetDefaultBuffer(GraphBuilder, 4u, 0u);
	}
}

void FSystemTextures::SetDefaultIfNull(FRDGBuilder& GraphBuilder, FRDGBufferSRVRef& Out)
{
	if (!Out)
	{
		Out = GraphBuilder.CreateSRV(GetDefaultBuffer(GraphBuilder, 4u, 0u), PF_R32_UINT);
	}
}

void FSystemTextures::SetDefaultIfNull(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef& Out)
{
	if (!Out)
	{
		Out = GraphBuilder.CreateUAV(GetDefaultBuffer(GraphBuilder, 4u, 0u), PF_R32_UINT);
	}
}
