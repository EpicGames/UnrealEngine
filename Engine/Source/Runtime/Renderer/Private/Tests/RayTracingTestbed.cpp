// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHIResourceUtils.h"
#include "Misc/AutomationTest.h"
#include "Math/DoubleFloat.h"

#if WITH_DEV_AUTOMATION_TESTS || WITH_EDITOR

class FRayTracingTestbedBase : public FAutomationTestBase
{
public:
	FRayTracingTestbedBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

	virtual bool CanRunInEnvironment(const FString& TestParams, FString* OutReason, bool* OutWarn) const override
	{
		if (!GRHISupportsRayTracing || !GRHISupportsRayTracingShaders)
		{
			if (OutReason)
			{
				*OutReason = TEXT("RHI does not support Ray Tracing and/or Ray Tracing Shaders.");
			}

			if (OutWarn)
			{
				*OutWarn = false;
			}

			return false;
		}

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FRayTracingTestbed, FRayTracingTestbedBase, "System.Renderer.RayTracing.BasicRayTracing", EAutomationTestFlags::EditorContext | EAutomationTestFlags::HighPriority | EAutomationTestFlags::EngineFilter)

#if RHI_RAYTRACING

#include "RayTracingGeometryInstance.h"
#include "RayTracing/RayTracingBasicShaders.h"
#include "RayTracingInstanceBufferUtil.h"
#include "RayTracingDefinitions.h"
#include "RayTracingPayloadType.h"
#include "GlobalShader.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "RenderUtils.h"
#include "RHIUtilities.h"

// HINT: Execute this test via console command in editor: Automation RunTest System.Renderer.RayTracing.BasicRayTracing
bool RunRayTracingTestbed_RenderThread(const FString& Parameters)
{
	check(IsInRenderingThread());

	// The ray tracing testbed currently rquires full ray tracing pipeline support.
	if (!GRHISupportsRayTracing || !GRHISupportsRayTracingShaders)
	{
		//Return true so the test passes in DX11, until the testing framework allows to skip tests depending on defined preconditions
		return true;
	}

	FBufferRHIRef VertexBuffer;
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	{
		const FVector3f PositionData[] =
		{
			FVector3f( 1, -1, 0),
			FVector3f( 1,  1, 0),
			FVector3f(-1, -1, 0),
		};

		VertexBuffer = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("RayTracingTestbedVB"), EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource, MakeConstArrayView(PositionData));
	}

	FBufferRHIRef IndexBuffer;

	{
		const uint16 IndexData[] =
		{
			0, 1, 2
		};

		IndexBuffer = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("RayTracingTestbedIB"), EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource, MakeConstArrayView(IndexData));
	}

	static constexpr uint32 NumRays = 4;

	FBufferRHIRef RayBuffer;
	FShaderResourceViewRHIRef RayBufferView;

	{
		const FBasicRayTracingRay RayData[] =
		{
			FBasicRayTracingRay{ { 0.75f, 0.0f, -1.0f}, 0xFFFFFFFF, {0.0f, 0.0f,  1.0f}, 100000.0f }, // expected to hit
			FBasicRayTracingRay{ { 0.75f, 0.0f, -1.0f}, 0xFFFFFFFF, {0.0f, 0.0f,  1.0f},      0.5f }, // expected to miss (short ray)
			FBasicRayTracingRay{ { 0.75f, 0.0f,  1.0f}, 0xFFFFFFFF, {0.0f, 0.0f, -1.0f}, 100000.0f }, // expected to hit  (should hit back face)
			FBasicRayTracingRay{ {-0.75f, 0.0f, -1.0f}, 0xFFFFFFFF, {0.0f, 0.0f,  1.0f}, 100000.0f }, // expected to miss (doesn't intersect)
		};

		RayBuffer = UE::RHIResourceUtils::CreateBufferFromArray(
			RHICmdList,
			TEXT("RayBuffer"), 
			BUF_Static | BUF_ShaderResource | BUF_StructuredBuffer,
			ERHIAccess::SRVMask,
			MakeConstArrayView(RayData)
		);

		RayBufferView = RHICmdList.CreateShaderResourceView(RayBuffer, 
			FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(sizeof(FBasicRayTracingRay))
			.SetNumElements(NumRays)
		);
	}

	FBufferRHIRef OcclusionResultBuffer;
	FUnorderedAccessViewRHIRef OcclusionResultBufferView;

	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured<uint32>(TEXT("OcclusionResultBuffer"), NumRays)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::UnorderedAccess)
			.SetInitialState(ERHIAccess::UAVMask);

		OcclusionResultBuffer = RHICmdList.CreateBuffer(CreateDesc);
		OcclusionResultBufferView = RHICmdList.CreateUnorderedAccessView(OcclusionResultBuffer, 
			FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(sizeof(uint32))
			.SetNumElements(NumRays)
		);
	}

	FBufferRHIRef IntersectionResultBuffer;
	FUnorderedAccessViewRHIRef IntersectionResultBufferView;

	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured<FBasicRayTracingIntersectionResult>(TEXT("IntersectionResultBuffer"), NumRays)
			.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::UnorderedAccess)
			.SetInitialState(ERHIAccess::UAVMask);

		IntersectionResultBuffer = RHICmdList.CreateBuffer(CreateDesc);
		IntersectionResultBufferView = RHICmdList.CreateUnorderedAccessView(IntersectionResultBuffer, 
			FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(sizeof(FBasicRayTracingIntersectionResult))
			.SetNumElements(NumRays)
		);
	}

	FRayTracingGeometryInitializer GeometryInitializer;
	GeometryInitializer.DebugName = FName("DebugTriangle");
	GeometryInitializer.IndexBuffer = IndexBuffer;
	GeometryInitializer.GeometryType = RTGT_Triangles;
	GeometryInitializer.bFastBuild = false;
	FRayTracingGeometrySegment Segment;
	Segment.VertexBuffer = VertexBuffer;
	Segment.NumPrimitives = 1;
	Segment.MaxVertices = 3;
	GeometryInitializer.Segments.Add(Segment);
	GeometryInitializer.TotalPrimitiveCount = Segment.NumPrimitives;
	FRayTracingGeometryRHIRef Geometry = RHICmdList.CreateRayTracingGeometry(GeometryInitializer);

	static constexpr uint32 NumTransforms = 1;
	static constexpr uint32 NumInstances = 1;

	FRayTracingGeometryInstance Instances[NumInstances] = {};
	Instances[0].GeometryRHI = Geometry;
	Instances[0].NumTransforms = NumTransforms;
	Instances[0].Transforms = MakeArrayView(&FMatrix::Identity, 1);
	Instances[0].InstanceContributionToHitGroupIndex = 0;

	FRayTracingInstanceBufferBuilder RayTracingInstanceBufferBuilder;
	RayTracingInstanceBufferBuilder.Init(Instances, FVector::ZeroVector);

	FRayTracingSceneRHIRef RayTracingSceneRHI;
	{
		FRayTracingSceneInitializer Initializer;
		Initializer.DebugName = FName("FRayTracingScene");
		Initializer.MaxNumInstances = RayTracingInstanceBufferBuilder.GetMaxNumInstances();
		Initializer.BuildFlags = ERayTracingAccelerationStructureFlags::FastTrace;

		RayTracingSceneRHI = RHICreateRayTracingScene(MoveTemp(Initializer));
	}

	const FRayTracingSceneInitializer& SceneInitializer = RayTracingSceneRHI->GetInitializer();

	FRayTracingAccelerationStructureSize SceneSizeInfo = RHICalcRayTracingSceneSize(SceneInitializer);

	const FRHIBufferCreateDesc SceneBufferCreateDesc =
		FRHIBufferCreateDesc::Create(TEXT("RayTracingTestBedSceneBuffer"), uint32(SceneSizeInfo.ResultSize), 0, EBufferUsageFlags::AccelerationStructure)
		.SetInitialState(ERHIAccess::BVHWrite);
	FBufferRHIRef SceneBuffer = RHICmdList.CreateBuffer(SceneBufferCreateDesc);

	const FRHIBufferCreateDesc ScratchBufferCreateDesc =
		FRHIBufferCreateDesc::Create(TEXT("RayTracingTestBedScratchBuffer"), uint32(SceneSizeInfo.BuildScratchSize), GRHIRayTracingScratchBufferAlignment, EBufferUsageFlags::UnorderedAccess)
		.SetInitialState(ERHIAccess::UAVCompute);
	FBufferRHIRef ScratchBuffer = RHICmdList.CreateBuffer(ScratchBufferCreateDesc);

	FRWBufferStructured InstanceBuffer;
	InstanceBuffer.Initialize(RHICmdList, TEXT("RayTracingTestBedInstanceBuffer"), GRHIRayTracingInstanceDescriptorSize, SceneInitializer.MaxNumInstances);
	
	FRWBufferStructured HitGroupContributionsBuffer;
	
	if(GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer)
	{
		HitGroupContributionsBuffer.Initialize(RHICmdList, TEXT("RayTracingTestBedInstanceBuffer"), 4, SceneInitializer.MaxNumInstances);
	}

	RayTracingInstanceBufferBuilder.FillRayTracingInstanceUploadBuffer(RHICmdList);
	RayTracingInstanceBufferBuilder.FillAccelerationStructureAddressesBuffer(RHICmdList);

	RayTracingInstanceBufferBuilder.BuildRayTracingInstanceBuffer(
		RHICmdList,
		nullptr,
		nullptr,
		nullptr,
		InstanceBuffer.UAV,
		GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer ? HitGroupContributionsBuffer.UAV : nullptr,
		SceneInitializer.MaxNumInstances,
		nullptr,
		/*bCompactOutput*/ false,
		nullptr,
		0,
		nullptr);

	RHICmdList.BindAccelerationStructureMemory(RayTracingSceneRHI, SceneBuffer, 0);

	RHICmdList.BuildAccelerationStructure(Geometry);

	// #yuriy_todo: explicit transitions and state validation for BLAS
	// RHICmdList.Transition(FRHITransitionInfo(Geometry.GetReference(), ERHIAccess::BVHWrite, ERHIAccess::BVHRead));

	FRayTracingSceneBuildParams BuildParams;
	BuildParams.Scene = RayTracingSceneRHI;
	BuildParams.ScratchBuffer = ScratchBuffer;
	BuildParams.ScratchBufferOffset = 0;
	BuildParams.InstanceBuffer = InstanceBuffer.Buffer;
	BuildParams.InstanceBufferOffset = 0;
	
	if(GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer)
	{
		BuildParams.HitGroupContributionsBuffer = HitGroupContributionsBuffer.Buffer;
		BuildParams.HitGroupContributionsBufferOffset = 0;
	}
	
	BuildParams.ReferencedGeometries = RayTracingInstanceBufferBuilder.GetReferencedGeometries();
	BuildParams.NumInstances = RayTracingInstanceBufferBuilder.GetMaxNumInstances();

	RHICmdList.Transition(FRHITransitionInfo(InstanceBuffer.Buffer, ERHIAccess::UAVMask, ERHIAccess::SRVCompute));

	RHICmdList.BuildAccelerationStructure(BuildParams);

	RHICmdList.Transition(FRHITransitionInfo(RayTracingSceneRHI.GetReference(), ERHIAccess::BVHWrite, ERHIAccess::BVHRead));

	FShaderResourceViewInitializer RayTracingSceneViewInitializer(SceneBuffer, RayTracingSceneRHI, 0);
	FShaderResourceViewRHIRef RayTracingSceneView = RHICmdList.CreateShaderResourceView(RayTracingSceneViewInitializer);

	DispatchBasicOcclusionRays(RHICmdList, RayTracingSceneView, Geometry, RayBufferView, OcclusionResultBufferView, NumRays);
	DispatchBasicIntersectionRays(RHICmdList, RayTracingSceneView, Geometry, RayBufferView, IntersectionResultBufferView, NumRays);

	const bool bValidateResults = true;
	bool bOcclusionTestOK = false;
	bool bIntersectionTestOK = false;

	if (bValidateResults)
	{
		RHICmdList.SubmitAndBlockUntilGPUIdle();

		// Read back and validate occlusion trace results

		{
			auto MappedResults = (const uint32*)RHICmdList.LockBuffer(OcclusionResultBuffer, 0, sizeof(uint32)*NumRays, RLM_ReadOnly);

			check(MappedResults);

			check(MappedResults[0] != 0); // expect hit
			check(MappedResults[1] == 0); // expect miss
			check(MappedResults[2] != 0); // expect hit
			check(MappedResults[3] == 0); // expect miss

			RHICmdList.UnlockBuffer(OcclusionResultBuffer);

			bOcclusionTestOK = (MappedResults[0] != 0) && (MappedResults[1] == 0) && (MappedResults[2] != 0) && (MappedResults[3] == 0);
		}

		// Read back and validate intersection trace results

		{
			auto MappedResults = (const FBasicRayTracingIntersectionResult*)RHICmdList.LockBuffer(IntersectionResultBuffer, 0, sizeof(FBasicRayTracingIntersectionResult)*NumRays, RLM_ReadOnly);

			check(MappedResults);

			// expect hit primitive 0, instance 0, barycentrics {0.5, 0.125}
			check(MappedResults[0].HitT >= 0);
			check(MappedResults[0].PrimitiveIndex == 0);
			check(MappedResults[0].InstanceIndex == 0);
			check(FMath::IsNearlyEqual(MappedResults[0].Barycentrics[0], 0.5f));
			check(FMath::IsNearlyEqual(MappedResults[0].Barycentrics[1], 0.125f));

			check(MappedResults[1].HitT < 0); // expect miss
			check(MappedResults[2].HitT >= 0); // expect hit back face
			check(MappedResults[3].HitT < 0); // expect miss

			RHICmdList.UnlockBuffer(IntersectionResultBuffer);

			bIntersectionTestOK = (MappedResults[0].HitT >= 0) && (MappedResults[1].HitT < 0) && (MappedResults[2].HitT >= 0) && (MappedResults[3].HitT < 0);
		}
	}

	return (bOcclusionTestOK && bIntersectionTestOK);
}
 
// Dummy shader to test shader compilation and reflection.
class FTestRaygenShader : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FTestRaygenShader, Global);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform) && !IsMetalPlatform(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Minimal;
	}

	FTestRaygenShader() {}
	//virtual ~FTestRaygenShader() {}

	/** Initialization constructor. */
	FTestRaygenShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		TLAS.Bind(Initializer.ParameterMap, TEXT("TLAS"));
		Rays.Bind(Initializer.ParameterMap, TEXT("Rays"));
		Output.Bind(Initializer.ParameterMap, TEXT("Output"));
	}

	/*bool Serialize(FArchive& Ar)
	{
		bool bShaderHasOutdatedParameters = FGlobalShader::Serialize(Ar);
		Ar << TLAS;
		Ar << Rays;
		Ar << Output;
		return bShaderHasOutdatedParameters;
	}*/

	LAYOUT_FIELD(FShaderResourceParameter, TLAS)   // SRV RaytracingAccelerationStructure
	LAYOUT_FIELD(FShaderResourceParameter, Rays)   // SRV StructuredBuffer<FBasicRayData>
	LAYOUT_FIELD(FShaderResourceParameter, Output) // UAV RWStructuredBuffer<uint>
};

IMPLEMENT_RT_PAYLOAD_TYPE(ERayTracingPayloadType::Minimal, 4);
IMPLEMENT_GLOBAL_SHADER(FTestRaygenShader, "/Engine/Private/RayTracing/RayTracingTest.usf", "TestMainRGS", SF_RayGen);


bool FRayTracingTestbed::RunTest(const FString& Parameters)
{
	bool bTestPassed = false;
	FlushRenderingCommands();

	ENQUEUE_RENDER_COMMAND(FRayTracingTestbed)(
		[&](FRHICommandListImmediate& RHICmdList)
	{
		bTestPassed = RunRayTracingTestbed_RenderThread(Parameters);
	}
	);  

	FlushRenderingCommands();

	return bTestPassed;
}

#else // RHI_RAYTRACING

bool FRayTracingTestbed::RunTest(const FString& Parameters)
{
	// Nothing to do when ray tracing is disabled
	return true;
}

#endif // RHI_RAYTRACING

#endif //WITH_DEV_AUTOMATION_TESTS
