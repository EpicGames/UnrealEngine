// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "VT/VirtualTextureShared.h"
#include "VirtualTextureEnum.h"
#include "VirtualTexturing.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS || WITH_EDITOR

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FVirtualTextureTestbed, FAutomationTestBase, "System.Renderer.VirtualTexture", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter)

// TODO [jonathan.bard] : when it'll be possible, remove this and change FVirtualTextureTestbed into a Low-Level Test (LLT) : 
#if WITH_LOW_LEVEL_TESTS
#define LOCAL_CHECK(test) CHECK(test)
#define LOCAL_REQUIRE_CHECK_SLOW(test) REQUIRE_CHECK_SLOW(test)
#else // WITH_LOW_LEVEL_TESTS
#define LOCAL_CHECK(test) check(test)
#define LOCAL_REQUIRE_CHECK_SLOW(test) {}
#endif // !WITH_LOW_LEVEL_TESTS

void PerformVTTilePriorityAndIndexTests()
{
	{
		FVTRequestPriority PriorityKey(/*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Highest, EVTInvalidatePriority::High, /*InPagePriority = */0x1234);
		PriorityKey.PackedValue |= 1ull << 63;
		LOCAL_REQUIRE_CHECK_SLOW(FVTRequestPriorityAndIndex A(/*InIndex = */42, PriorityKey));
	}

	{
		FVTRequestPriority PriorityKey(/*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Highest, EVTInvalidatePriority::High, /*InPagePriority = */0x1234);
		FVTRequestPriorityAndIndex A(/*InIndex = */(uint16)~0, PriorityKey);
		LOCAL_REQUIRE_CHECK_SLOW(FVTRequestPriorityAndIndex B(/*InIndex = */(uint16)~0 + 1, PriorityKey));
	}

	{
		FVTRequestPriorityAndIndex A(/*InIndex = */42, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Highest, EVTInvalidatePriority::High, /*InPagePriority = */0x1234);
		FVTRequestPriority APriority = A.GetPriorityKey();
		FVTRequestPriorityAndIndex B(/*InIndex = */42, /*bInLocked = */true, /*bInStreaming = */true, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0x3456);
		LOCAL_CHECK(A.Index == B.Index);
	}

	{
		FVTLocalTilePriorityAndIndex A(/*InIndex = */0, EVTProducerPriority::Highest, EVTInvalidatePriority::High, /*InMipLevel = */5);
		FVTLocalTilePriorityAndIndex B(/*InIndex = */0, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InMipLevel = */0);
		LOCAL_CHECK(A.Index == B.Index);
	}

	{
		LOCAL_CHECK(FVTRequestPriorityAndIndex().SortablePackedValue == 0);
		LOCAL_CHECK(
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0) <
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::BelowNormal, EVTInvalidatePriority::Normal, /*InPagePriority = */0));
		LOCAL_CHECK(
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */true, /*bInStreaming = */false, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0) <
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0));
		LOCAL_CHECK(
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Normal, EVTInvalidatePriority::High, /*InPagePriority = */0) <
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0));
		LOCAL_CHECK(
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */true, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0) <
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0));
		LOCAL_CHECK(
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */1) <
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0));
		LOCAL_CHECK(
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */true, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0) <
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0xffffffff));
		LOCAL_CHECK(
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */true, /*bInStreaming = */false, EVTProducerPriority::Highest, EVTInvalidatePriority::High, /*InPagePriority = */0x1234) <
			FVTRequestPriorityAndIndex(/*InIndex = */0, /*bInLocked = */false, /*bInStreaming = */false, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InPagePriority = */0x3456));

		LOCAL_CHECK(FVTLocalTilePriorityAndIndex().SortablePackedValue == 0);
		LOCAL_CHECK(
			FVTLocalTilePriorityAndIndex(/*InIndex = */0, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InMipLevel = */0) <
			FVTLocalTilePriorityAndIndex(/*InIndex = */0, EVTProducerPriority::BelowNormal, EVTInvalidatePriority::Normal, /*InMipLevel = */0));
		LOCAL_CHECK(
			FVTLocalTilePriorityAndIndex(/*InIndex = */0, EVTProducerPriority::Normal, EVTInvalidatePriority::High, /*InMipLevel = */0) <
			FVTLocalTilePriorityAndIndex(/*InIndex = */0, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InMipLevel = */0));
		LOCAL_CHECK(
			FVTLocalTilePriorityAndIndex(/*InIndex = */0, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InMipLevel = */5) <
			FVTLocalTilePriorityAndIndex(/*InIndex = */0, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal, /*InMipLevel = */0));
	}
}
#undef LOCAL_CHECK
#undef LOCAL_REQUIRE_CHECK_SLOW

bool FVirtualTextureTestbed::RunTest(const FString& Parameters)
{
	PerformVTTilePriorityAndIndexTests();

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
