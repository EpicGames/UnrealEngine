// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"

enum class ETranslucencyView
{
	None       = 0,
	UnderWater = 1 << 0,
	AboveWater = 1 << 1,
	RayTracing = 1 << 2
};
ENUM_CLASS_FLAGS(ETranslucencyView);