// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TranslucentLighting.cpp: Translucent lighting implementation.
=============================================================================*/

#include "TranslucentLighting.h"
#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "EngineDefines.h"
#include "RHI.h"
#include "RenderResource.h"
#include "HitProxies.h"
#include "FinalPostProcessSettings.h"
#include "ShaderParameters.h"
#include "RendererInterface.h"
#include "PrimitiveViewRelevance.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Components/LightComponent.h"
#include "Materials/Material.h"
#include "PostProcess/SceneRenderTargets.h"
#include "LightSceneInfo.h"
#include "GlobalShader.h"
#include "MaterialShaderType.h"
#include "MaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "MeshMaterialShader.h"
#include "ShadowRendering.h"
#include "SceneRendering.h"
#include "DeferredShadingRenderer.h"
#include "TranslucentRendering.h"
#include "ClearQuad.h"
#include "ScenePrivate.h"
#include "OneColorShader.h"
#include "LightGrid.h"
#include "LightFunctionRendering.h"
#include "LightRendering.h"
#include "ScreenRendering.h"
#include "AmbientCubemapParameters.h"
#include "VolumeRendering.h"
#include "VolumeLighting.h"
#include "PipelineStateCache.h"
#include "VisualizeTexture.h"
#include "MeshPassProcessor.inl"
#include "SkyAtmosphereRendering.h"
#include "VolumetricCloudRendering.h"
#include "TranslucentLightingViewState.h"
#include "RenderCore.h"
#include "StaticMeshBatch.h"
#include "LightFunctionAtlas.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "DeferredShadingRenderer.h"
#include "BasePassRendering.h"
#include "LightFunctionAtlas.h"
#include "BlueNoise.h"
#include "PSOPrecacheSettings.h"
#include "SimpleMeshDrawCommandPass.h"
#include "SceneViewState.h"
#include "TextureFallbacksRDG.h"

using namespace LightFunctionAtlas;

class FMaterial;

/** Whether to allow rendering translucency shadow depths. */
bool GUseTranslucencyShadowDepths = true;

DECLARE_GPU_STAT_NAMED(TranslucentLighting, TEXT("Translucent Lighting"));
 
int32 GUseTranslucencyLightingVolumes = 1;
FAutoConsoleVariableRef CVarTranslucencyLightingVolumes(
	TEXT("r.TranslucencyLightingVolume"),
	GUseTranslucencyLightingVolumes,
	TEXT("Whether to allow updating the translucency lighting volumes.\n")
	TEXT("0:off, otherwise on, default is 1"),
	ECVF_RenderThreadSafe | ECVF_Scalability
	);

float GTranslucencyLightingVolumeMinFOV = 45;
static FAutoConsoleVariableRef CVarTranslucencyLightingVolumeMinFOV(
	TEXT("r.TranslucencyLightingVolume.MinFOV"),
	GTranslucencyLightingVolumeMinFOV,
	TEXT("Minimum FOV for translucency lighting volume. Prevents popping in lighting when zooming in."),
	ECVF_RenderThreadSafe | ECVF_Scalability
	);

float GTranslucencyLightingVolumeFOVSnapFactor = 10;
static FAutoConsoleVariableRef CVarTranslucencyLightingVolumeFOVSnapFactor(
	TEXT("r.TranslucencyLightingVolume.FOVSnapFactor"),
	GTranslucencyLightingVolumeFOVSnapFactor,
	TEXT("FOV will be snapped to a factor of this before computing volume bounds."),
	ECVF_RenderThreadSafe | ECVF_Scalability
	);

int32 GTranslucencyLightingVolumeBlur = 1;
FAutoConsoleVariableRef CVarTranslucencyLightingVolumeBlur(
	TEXT("r.TranslucencyLightingVolume.Blur"),
	GTranslucencyLightingVolumeBlur,
	TEXT("Whether to blur the translucency lighting volumes.\n")
	TEXT("0:off, otherwise on, default is 1"),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GTranslucencyLightingVolumeDim = 64;
FAutoConsoleVariableRef CVarTranslucencyLightingVolumeDim(
	TEXT("r.TranslucencyLightingVolume.Dim"),
	GTranslucencyLightingVolumeDim,
	TEXT("Dimensions of the volume textures used for translucency lighting.  Larger textures result in higher resolution but lower performance."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<float> CVarTranslucencyLightingVolumeInnerDistance(
	TEXT("r.TranslucencyLightingVolume.InnerDistance"),
	1500.0f,
	TEXT("Distance from the camera that the first volume cascade should end"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarTranslucencyLightingVolumeOuterDistance(
	TEXT("r.TranslucencyLightingVolume.OuterDistance"),
	5000.0f,
	TEXT("Distance from the camera that the second volume cascade should end"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarTranslucencyLightingVolumePositionOffsetRadius(
	TEXT("r.TranslucencyLightingVolume.PositionOffsetRadius"),
	0.0f,
	TEXT("Radius of per-pixel offset applied to position when sampling translucency lighting volume."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarTranslucencyLightingVolumeTemporal(
	TEXT("r.TranslucencyLightingVolume.Temporal"),
	false,
	TEXT("Whether to use temporal accumulation instead of spatial filter when updating the translucency lighting volume."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarTranslucencyLightingVolumeHistoryWeight(
	TEXT("r.TranslucencyLightingVolume.Temporal.HistoryWeight"),
	0.9f,
	TEXT("How much the history value should be weighted each frame.  This is a tradeoff between visible jittering and responsiveness."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarTranslucencyLightingVolumeSpecularDirectionHistoryWeight(
	TEXT("r.TranslucencyLightingVolume.Temporal.SpecularDirectionHistoryWeight"),
	0.9f,
	TEXT("History weight for the direction used to approximate specular using the translucent lighting volume."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarTranslucencyLightingVolumeMarkVoxelsSupported(
	TEXT("r.TranslucencyLightingVolume.MarkVoxels.Supported"),
	true,
	TEXT("Whether marking used voxels is supported by the project. Avoids compiling some shaders when disabled.\n")
	TEXT("This setting can't be changed at runtime since it affects cooking."),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<bool> CVarTranslucencyLightingVolumeMarkVoxels(
	TEXT("r.TranslucencyLightingVolume.MarkVoxels"),
	false,
	TEXT("Whether to mark which volume voxels are sampled during rendering and only update those that are."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarTranslucencyLightingVolumeBatch(
	TEXT("r.TranslucencyLightingVolume.Batch"),
	1,
	TEXT("When enabled, batches supported lights into a single draw call for efficiency."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<bool> CVarTranslucencyLightingVolumeUseTypedUAVLoad(
	TEXT("r.TranslucencyLightingVolume.UseTypedUAVLoad"),
	true,
	TEXT("Whether to use Typed UAV loads when updating the translucency lighting volume.\n")
	TEXT("Recommended to be enabled since it reduces transient memory usage."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarTranslucencyLightingVolumeAccurateRectLights(
	TEXT("r.TranslucencyLightingVolume.AccurateRectLights"),
	1,
	TEXT("When disabled rect lights are approximated as spot lights in the translucency volume.\n")
	TEXT("Only accurate rect lights are included in batching, so it is recommended to enable this when batching is used."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarTranslucencyLightingVolumeInjectDirectionalLightCSM(
	TEXT("r.TranslucencyLightingVolume.InjectDirectionalLightCSM"),
	1,
	TEXT("Enable sampling of the directional light CSM.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

// Adaptation to camera angle is now disabled by default to avoid change on lighting when zoomin in or out.
// The volume remains around the camera anyway, so the camera angle should not matter when the setup mostly depends on start and end distance for each cascade.
int32 GTranslucencyLightingVolumeAdaptToPerspective = 0;
static FAutoConsoleVariableRef CVarTranslucencyLightingVolumeAdaptToPerspective(
	TEXT("r.TranslucencyLightingVolume.AdaptToPerspective"),
	GTranslucencyLightingVolumeAdaptToPerspective,
	TEXT("The translucent volume will adapt to the camera perspective when zooming. This can can result in pops for extreme zoom-in so it can be disabled if needed."),
	ECVF_RenderThreadSafe
);

int32 GTranslucencyLightingVolumeVisualize = 0;
static FAutoConsoleVariableRef CVarTranslucencyLightingVolumeVisualize(
	TEXT("r.TranslucencyLightingVolume.Visualize"),
	GTranslucencyLightingVolumeVisualize,
	TEXT("Whether to visualize the translucency lighting volume."),
	ECVF_RenderThreadSafe
);

float GTranslucencyLightingVolumeVisualizeRadiusScale = 2.0f;
FAutoConsoleVariableRef CVarTranslucencyLightingVolumeVisualizeRadiusScale(
	TEXT("r.TranslucencyLightingVolume.Visualize.RadiusScale"),
	GTranslucencyLightingVolumeVisualizeRadiusScale,
	TEXT("Scales the size of the spheres used to visualize the translucency lighting volume."),
	ECVF_RenderThreadSafe
);

int32 GTranslucencyLightingVolumeVisualizeCascadeIndex = -1;
FAutoConsoleVariableRef CVarTranslucencyLightingVolumeVisualizeCascadeIndex(
	TEXT("r.TranslucencyLightingVolume.VisualizeCascadeIndex"),
	GTranslucencyLightingVolumeVisualizeCascadeIndex,
	TEXT("Selects which translucency lighting volume cascade should be visualized. -1 visualizes all cascades at once."),
	ECVF_RenderThreadSafe
);

int32 GTranslucencyLightingVolumeDebug = 0;
static FAutoConsoleVariableRef CVarTranslucencyLightingVolumeDebug(
	TEXT("r.TranslucencyLightingVolume.Debug"),
	GTranslucencyLightingVolumeDebug,
	TEXT("Debug information for the translucency lighting volume."),
	ECVF_RenderThreadSafe
);

int32 GTranslucencyLightingVolumeDebugFreezePosition = 0;
static FAutoConsoleVariableRef CVarTranslucencyLightingVolumeDebugFreezePosition(
	TEXT("r.TranslucencyLightingVolume.Debug.FreezePosition"),
	GTranslucencyLightingVolumeDebugFreezePosition,
	TEXT("Whether to freeze the position of the translucency lighting volume."),
	ECVF_RenderThreadSafe
);

static const TCHAR* TranslucentLightingMaterialPSOCollectorName = TEXT("TranslucentLightingMaterialPSOCollector");

float GetTranslucencyLightingVolumePositionOffsetRadius()
{
	return FMath::Max(0.0f, CVarTranslucencyLightingVolumePositionOffsetRadius.GetValueOnAnyThread());
}

/** Function returning current translucency lighting volume dimensions. */
int32 GetTranslucencyLightingVolumeDim()
{
	return FMath::Clamp(GTranslucencyLightingVolumeDim, 4, 2048);
}

float GetTranslucencyLightingVolumeInnerDistance()
{
	return FMath::Max(0.0f, CVarTranslucencyLightingVolumeInnerDistance.GetValueOnAnyThread());
}

float GetTranslucencyLightingVolumeOuterDistance()
{
	return FMath::Max(0.0f, CVarTranslucencyLightingVolumeOuterDistance.GetValueOnAnyThread());
}

void FViewInfo::CalcTranslucencyLightingVolumeBounds(FBox* InOutCascadeBoundsArray, int32 NumCascades) const
{
	if (GTranslucencyLightingVolumeDebugFreezePosition != 0 && ViewState)
	{
		FTranslucencyLightingViewState& TranslucencyLightingViewState = ViewState->TranslucencyLighting;

		for (int32 CascadeIndex = 0; CascadeIndex < NumCascades; CascadeIndex++)
		{
			InOutCascadeBoundsArray[CascadeIndex] = FBox(TranslucencyLightingViewState.HistoryVolumeMin[CascadeIndex], TranslucencyLightingViewState.HistoryVolumeMin[CascadeIndex] + TranslucencyLightingViewState.HistoryVolumeSize[CascadeIndex]);
		}

		return;
	}

	for (int32 CascadeIndex = 0; CascadeIndex < NumCascades; CascadeIndex++)
	{
		const double InnerDistance = GetTranslucencyLightingVolumeInnerDistance();
		const double OuterDistance = GetTranslucencyLightingVolumeOuterDistance();

		const double FrustumStartDistance = CascadeIndex == 0 ? 0 : InnerDistance;
		const double FrustumEndDistance = CascadeIndex == 0 ? InnerDistance : OuterDistance;

		double FieldOfView = DOUBLE_PI / 4.0;
		double AspectRatio = 1.0;

		const FViewMatrices& LocalShadowViewMatrices = GetShadowViewMatrices();

		if (IsPerspectiveProjection() && GTranslucencyLightingVolumeAdaptToPerspective > 0)
		{
			// Derive FOV and aspect ratio from the perspective projection matrix
			FieldOfView = FMath::Atan(1.0 / LocalShadowViewMatrices.GetViewToClip().M[0][0]);
			// Clamp to prevent shimmering when zooming in
			FieldOfView = FMath::Max(FieldOfView, GTranslucencyLightingVolumeMinFOV * DOUBLE_PI / 180.0);
			const double RoundFactorRadians = GTranslucencyLightingVolumeFOVSnapFactor * DOUBLE_PI / 180.0;
			// Round up to a fixed factor
			// This causes the volume lighting to make discreet jumps as the FOV animates, instead of slowly crawling over a long period
			FieldOfView = FieldOfView + RoundFactorRadians - FMath::Fmod(FieldOfView, RoundFactorRadians);
			AspectRatio = LocalShadowViewMatrices.GetViewToClip().M[1][1] / LocalShadowViewMatrices.GetViewToClip().M[0][0];
		}

		// Tan of field of view can explode when FieldOfView is close to 180 degree, when using the adaptation to Fov above.
		// Close to this edge case, the volume fitted on the frustum vertices can explode to infinity.
		// In order to fix that, we clamp the value to the length of the diagonal of a cube of size OuterDistance.
		const double MaxTanFieldOfViewLength = FMath::Sqrt(FMath::Max(1.0f, OuterDistance * OuterDistance + OuterDistance * OuterDistance + OuterDistance * OuterDistance));
		const double TanFieldOfView = FMath::Tan(FieldOfView);

		const double StartHorizontalLength = FMath::Min(MaxTanFieldOfViewLength, FrustumStartDistance * TanFieldOfView);
		const FVector StartCameraRightOffset = LocalShadowViewMatrices.GetWorldToView().GetColumn(0) * StartHorizontalLength;
		const double StartVerticalLength = StartHorizontalLength / AspectRatio;
		const FVector StartCameraUpOffset = LocalShadowViewMatrices.GetWorldToView().GetColumn(1) * StartVerticalLength;

		const double EndHorizontalLength = FMath::Min(MaxTanFieldOfViewLength, FrustumEndDistance * TanFieldOfView);
		const FVector EndCameraRightOffset = LocalShadowViewMatrices.GetWorldToView().GetColumn(0) * EndHorizontalLength;
		const double EndVerticalLength = EndHorizontalLength / AspectRatio;
		const FVector EndCameraUpOffset = LocalShadowViewMatrices.GetWorldToView().GetColumn(1) * EndVerticalLength;

		FVector SplitVertices[8];
		const FVector ShadowViewOrigin = LocalShadowViewMatrices.GetViewOrigin();

		SplitVertices[0] = ShadowViewOrigin + GetViewDirection() * FrustumStartDistance + StartCameraRightOffset + StartCameraUpOffset;
		SplitVertices[1] = ShadowViewOrigin + GetViewDirection() * FrustumStartDistance + StartCameraRightOffset - StartCameraUpOffset;
		SplitVertices[2] = ShadowViewOrigin + GetViewDirection() * FrustumStartDistance - StartCameraRightOffset + StartCameraUpOffset;
		SplitVertices[3] = ShadowViewOrigin + GetViewDirection() * FrustumStartDistance - StartCameraRightOffset - StartCameraUpOffset;

		SplitVertices[4] = ShadowViewOrigin + GetViewDirection() * FrustumEndDistance + EndCameraRightOffset + EndCameraUpOffset;
		SplitVertices[5] = ShadowViewOrigin + GetViewDirection() * FrustumEndDistance + EndCameraRightOffset - EndCameraUpOffset;
		SplitVertices[6] = ShadowViewOrigin + GetViewDirection() * FrustumEndDistance - EndCameraRightOffset + EndCameraUpOffset;
		SplitVertices[7] = ShadowViewOrigin + GetViewDirection() * FrustumEndDistance - EndCameraRightOffset - EndCameraUpOffset;

		FVector Center(0,0,0);
		// Weight the far vertices more so that the bounding sphere will be further from the camera
		// This minimizes wasted shadowmap space behind the viewer
		const double FarVertexWeightScale = 10.0;
		for (int32 VertexIndex = 0; VertexIndex < 8; VertexIndex++)
		{
			const double Weight = VertexIndex > 3 ? 1 / (4.0 + 4.0 / FarVertexWeightScale) : 1 / (4.0 + 4.0 * FarVertexWeightScale);
			Center += SplitVertices[VertexIndex] * Weight;
		}

		double RadiusSquared = 0;
		for (int32 VertexIndex = 0; VertexIndex < 8; VertexIndex++)
		{
			RadiusSquared = FMath::Max(RadiusSquared, (Center - SplitVertices[VertexIndex]).SizeSquared());
		}

		if (RadiusSquared > 0) // Avoid issues with bad cvar usage, e.g. r.TranslucencyLightingVolume.InnerDistance.
		{
			FSphere SphereBounds(Center, FMath::Sqrt(RadiusSquared));

			// Snap the center to a multiple of the volume dimension for stability
			const int32 TranslucencyLightingVolumeDim = GetTranslucencyLightingVolumeDim();
			SphereBounds.Center.X = SphereBounds.Center.X - FMath::Fmod(SphereBounds.Center.X, SphereBounds.W * 2 / TranslucencyLightingVolumeDim);
			SphereBounds.Center.Y = SphereBounds.Center.Y - FMath::Fmod(SphereBounds.Center.Y, SphereBounds.W * 2 / TranslucencyLightingVolumeDim);
			SphereBounds.Center.Z = SphereBounds.Center.Z - FMath::Fmod(SphereBounds.Center.Z, SphereBounds.W * 2 / TranslucencyLightingVolumeDim);

			InOutCascadeBoundsArray[CascadeIndex] = FBox(SphereBounds);
		}
		else
		{
			InOutCascadeBoundsArray[CascadeIndex] = FBox(Center, Center);
		}
	}
}

void ReleaseTranslucencyLightingVolumeHistory(TArrayView<FViewInfo> Views)
{
	for (FViewInfo& View : Views)
	{
		if (View.ViewState && !View.bStatePrevViewInfoIsReadOnly)
		{
			FTranslucencyLightingViewState& TranslucencyLightingViewState = View.ViewState->TranslucencyLighting;

			for (int32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
			{
				TranslucencyLightingViewState.HistoryAmbient[VolumeCascadeIndex] = nullptr;
				TranslucencyLightingViewState.HistoryDirectional[VolumeCascadeIndex] = nullptr;
				TranslucencyLightingViewState.HistoryMark[VolumeCascadeIndex] = nullptr;
				TranslucencyLightingViewState.HistorySpecularDirection[VolumeCascadeIndex] = nullptr;
			}
		}
	}
}

class FTranslucencyDepthShaderElementData : public FMeshMaterialShaderElementData
{
public:
	float TranslucentShadowStartOffset;
};

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FTranslucencyDepthPassUniformParameters,)
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER(FMatrix44f, ProjectionMatrix)
	SHADER_PARAMETER(float, bClampToNearPlane)
	SHADER_PARAMETER(float, InvMaxSubjectDepth)
	SHADER_PARAMETER_STRUCT(FTranslucentSelfShadowUniformParameters, TranslucentSelfShadow)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FTranslucencyDepthPassUniformParameters, "TranslucentDepthPass", SceneTextures);

void SetupTranslucencyDepthPassUniformBuffer(
	const FProjectedShadowInfo* ShadowInfo,
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FTranslucencyDepthPassUniformParameters& TranslucencyDepthPassParameters)
{
	// Note - scene depth can be bound by the material for use in depth fades
	// This is incorrect when rendering a shadowmap as it's not from the camera's POV
	// Set the scene depth texture to something safe when rendering shadow depths
	SetupSceneTextureUniformParameters(GraphBuilder, View.GetSceneTexturesChecked(), View.FeatureLevel, ESceneTextureSetupMode::None, TranslucencyDepthPassParameters.SceneTextures);

	TranslucencyDepthPassParameters.ProjectionMatrix = FTranslationMatrix44f(FVector3f(ShadowInfo->PreShadowTranslation - View.ViewMatrices.GetPreViewTranslation())) * ShadowInfo->TranslatedWorldToClipInnerMatrix;

	// Only clamp vertices to the near plane when rendering whole scene directional light shadow depths or preshadows from directional lights
	const bool bClampToNearPlaneValue = ShadowInfo->IsWholeSceneDirectionalShadow() || (ShadowInfo->bPreShadow && ShadowInfo->bDirectionalLight);
	TranslucencyDepthPassParameters.bClampToNearPlane = bClampToNearPlaneValue ? 1.0f : 0.0f;

	TranslucencyDepthPassParameters.InvMaxSubjectDepth = ShadowInfo->InvMaxSubjectDepth;

	SetupTranslucentSelfShadowUniformParameters(ShadowInfo, TranslucencyDepthPassParameters.TranslucentSelfShadow);
}

/**
* Vertex shader used to render shadow maps for translucency.
*/
class FTranslucencyShadowDepthVS : public FMeshMaterialShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FTranslucencyShadowDepthVS, NonVirtual);
public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return AllowTranslucencyPerObjectShadows(Parameters.Platform) && IsTranslucentBlendMode(Parameters.MaterialParameters);
	}

	FTranslucencyShadowDepthVS() {}
	FTranslucencyShadowDepthVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer) :
		FMeshMaterialShader(Initializer)
	{}
};

enum ETranslucencyShadowDepthShaderMode
{
	TranslucencyShadowDepth_PerspectiveCorrect,
	TranslucencyShadowDepth_Standard,
};

template <ETranslucencyShadowDepthShaderMode ShaderMode>
class TTranslucencyShadowDepthVS : public FTranslucencyShadowDepthVS
{
	DECLARE_SHADER_TYPE(TTranslucencyShadowDepthVS, MeshMaterial);
public:

	TTranslucencyShadowDepthVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FTranslucencyShadowDepthVS(Initializer)
	{}

	TTranslucencyShadowDepthVS() {}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FTranslucencyShadowDepthVS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUPPORT_SHADOW_DEPTH_SPOT"), (uint32)(ShaderMode == TranslucencyShadowDepth_PerspectiveCorrect ? 1 : 0));
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TTranslucencyShadowDepthVS<TranslucencyShadowDepth_PerspectiveCorrect>,TEXT("/Engine/Private/TranslucentShadowDepthShaders.usf"),TEXT("MainVS"),SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TTranslucencyShadowDepthVS<TranslucencyShadowDepth_Standard>,TEXT("/Engine/Private/TranslucentShadowDepthShaders.usf"),TEXT("MainVS"),SF_Vertex);

/**
 * Pixel shader used for accumulating translucency layer densities
 */
class FTranslucencyShadowDepthPS : public FMeshMaterialShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FTranslucencyShadowDepthPS, NonVirtual);
public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return AllowTranslucencyPerObjectShadows(Parameters.Platform) && IsTranslucentBlendMode(Parameters.MaterialParameters);
	}

	FTranslucencyShadowDepthPS() = default;
	FTranslucencyShadowDepthPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FMeshMaterialShader(Initializer)
	{
		TranslucentShadowStartOffset.Bind(Initializer.ParameterMap, TEXT("TranslucentShadowStartOffset"));
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FTranslucencyDepthShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		ShaderBindings.Add(TranslucentShadowStartOffset, ShaderElementData.TranslucentShadowStartOffset);
	}

private:
	LAYOUT_FIELD(FShaderParameter, TranslucentShadowStartOffset);
};

template <ETranslucencyShadowDepthShaderMode ShaderMode>
class TTranslucencyShadowDepthPS : public FTranslucencyShadowDepthPS
{
public:
	DECLARE_SHADER_TYPE(TTranslucencyShadowDepthPS, MeshMaterial);

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FTranslucencyShadowDepthPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUPPORT_SHADOW_DEPTH_SPOT"), (uint32)(ShaderMode == TranslucencyShadowDepth_PerspectiveCorrect ? 1 : 0));
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_INLINE_SHADING"), 1);
	}

	TTranslucencyShadowDepthPS() = default;
	TTranslucencyShadowDepthPS(const ShaderMetaType::CompiledShaderInitializerType & Initializer) :
		FTranslucencyShadowDepthPS(Initializer)
	{}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TTranslucencyShadowDepthPS<TranslucencyShadowDepth_PerspectiveCorrect>,TEXT("/Engine/Private/TranslucentShadowDepthShaders.usf"),TEXT("MainOpacityPS"),SF_Pixel);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TTranslucencyShadowDepthPS<TranslucencyShadowDepth_Standard>,TEXT("/Engine/Private/TranslucentShadowDepthShaders.usf"),TEXT("MainOpacityPS"),SF_Pixel);

class FTranslucencyDepthPassMeshProcessor : public FMeshPassProcessor
{
public:
	FTranslucencyDepthPassMeshProcessor(const FScene* Scene,
		const FSceneView* InViewIfDynamicMeshCommand,
		const FMeshPassProcessorRenderState& InPassDrawRenderState,
		const FProjectedShadowInfo* InShadowInfo,
		FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch, 
		uint64 BatchElementMask, 
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, 
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	template<ETranslucencyShadowDepthShaderMode ShaderMode>
	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		float MaterialTranslucentShadowStartOffset,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	FMeshPassProcessorRenderState PassDrawRenderState;
	const FProjectedShadowInfo* ShadowInfo;
	const bool bDirectionalLight;
};

FTranslucencyDepthPassMeshProcessor::FTranslucencyDepthPassMeshProcessor(const FScene* Scene,
	const FSceneView* InViewIfDynamicMeshCommand,
	const FMeshPassProcessorRenderState& InPassDrawRenderState,
	const FProjectedShadowInfo* InShadowInfo,
	FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::Num, Scene, Scene->GetFeatureLevel(), InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InPassDrawRenderState)
	, ShadowInfo(InShadowInfo)
	, bDirectionalLight(InShadowInfo->bDirectionalLight)
{
}

bool FTranslucencyDepthPassMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch, 
	uint64 BatchElementMask, 
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, 
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	// Determine the mesh's material and blend mode.
	const float MaterialTranslucentShadowStartOffset = Material.GetTranslucentShadowStartOffset();
	const bool MaterialCastDynamicShadowAsMasked = Material.GetCastDynamicShadowAsMasked();
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
	const bool bIsTranslucent = IsTranslucentBlendMode(Material);

	// Only render translucent meshes into the Fourier opacity maps
	if (bIsTranslucent && ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain()) && !MaterialCastDynamicShadowAsMasked)
	{
		if (bDirectionalLight)
		{
			return Process<TranslucencyShadowDepth_Standard>(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, MaterialRenderProxy, Material, MaterialTranslucentShadowStartOffset, MeshFillMode, MeshCullMode);
		}
		else
		{
			return Process<TranslucencyShadowDepth_PerspectiveCorrect>(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, MaterialRenderProxy, Material, MaterialTranslucentShadowStartOffset, MeshFillMode, MeshCullMode);
		}
	}

	return true;
}

template<ETranslucencyShadowDepthShaderMode ShaderMode>
bool FTranslucencyDepthPassMeshProcessor::Process(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	int32 StaticMeshId,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	float MaterialTranslucentShadowStartOffset,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		TTranslucencyShadowDepthVS<ShaderMode>,
		TTranslucencyShadowDepthPS<ShaderMode>> PassShaders;

	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<TTranslucencyShadowDepthVS<ShaderMode>>();
	ShaderTypes.AddShaderType<TTranslucencyShadowDepthPS<ShaderMode>>();

	FVertexFactoryType* VertexFactoryType = VertexFactory->GetType();

	FMaterialShaders Shaders;
	if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(PassShaders.VertexShader);
	Shaders.TryGetPixelShader(PassShaders.PixelShader);

	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);

	FTranslucencyDepthShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const float LocalToWorldScale = ShadowInfo->GetParentSceneInfo()->Proxy->GetLocalToWorld().GetScaleVector().GetMax();
	const float TranslucentShadowStartOffsetValue = MaterialTranslucentShadowStartOffset * LocalToWorldScale;
	ShaderElementData.TranslucentShadowStartOffset = TranslucentShadowStartOffsetValue / (ShadowInfo->MaxSubjectZ - ShadowInfo->MinSubjectZ);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		DrawRenderState,
		PassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

void FTranslucencyDepthPassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (MeshBatch.CastShadow)
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material && Material->GetRenderingThreadShaderMap())
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FTranslucencyDepthPassParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FTranslucencyDepthPassUniformParameters, PassUniformBuffer)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FProjectedShadowInfo::RenderTranslucencyDepths(FRDGBuilder& GraphBuilder, FSceneRenderer* SceneRenderer, const FRenderTargetBindingSlots& InRenderTargets, FInstanceCullingManager& InstanceCullingManager)
{
	check(IsInRenderingThread());
	checkSlow(!bWholeSceneShadow);
	SCOPE_CYCLE_COUNTER(STAT_RenderPerObjectShadowDepthsTime);

	BeginRenderView(GraphBuilder, SceneRenderer->Scene);

	auto* TranslucencyDepthPassParameters = GraphBuilder.AllocParameters<FTranslucencyDepthPassUniformParameters>();
	SetupTranslucencyDepthPassUniformBuffer(this, GraphBuilder, *ShadowDepthView, *TranslucencyDepthPassParameters);
	TRDGUniformBufferRef<FTranslucencyDepthPassUniformParameters> PassUniformBuffer = GraphBuilder.CreateUniformBuffer(TranslucencyDepthPassParameters);

	auto* PassParameters = GraphBuilder.AllocParameters<FTranslucencyDepthPassParameters>();
	PassParameters->View = ShadowDepthView->ViewUniformBuffer;
	PassParameters->PassUniformBuffer = PassUniformBuffer;
	PassParameters->RenderTargets = InRenderTargets;

	FSimpleMeshDrawCommandPass* SimpleMeshDrawCommandPass = GraphBuilder.AllocObject<FSimpleMeshDrawCommandPass>(*ShadowDepthView, &InstanceCullingManager);

	FMeshPassProcessorRenderState DrawRenderState;
	DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
	DrawRenderState.SetBlendState(TStaticBlendState<
		CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
		CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI());


	FTranslucencyDepthPassMeshProcessor TranslucencyDepthPassMeshProcessor(
		SceneRenderer->Scene,
		ShadowDepthView,
		DrawRenderState,
		this,
		SimpleMeshDrawCommandPass->GetDynamicPassMeshDrawListContext());

	for (int32 MeshBatchIndex = 0; MeshBatchIndex < DynamicSubjectTranslucentMeshElements.Num(); MeshBatchIndex++)
	{
		const FMeshBatchAndRelevance& MeshAndRelevance = DynamicSubjectTranslucentMeshElements[MeshBatchIndex];
		const uint64 BatchElementMask = ~0ull;
		TranslucencyDepthPassMeshProcessor.AddMeshBatch(*MeshAndRelevance.Mesh, BatchElementMask, MeshAndRelevance.PrimitiveSceneProxy);
	}

	for (int32 PrimitiveIndex = 0; PrimitiveIndex < SubjectTranslucentPrimitives.Num(); PrimitiveIndex++)
	{
		const FPrimitiveSceneInfo* PrimitiveSceneInfo = SubjectTranslucentPrimitives[PrimitiveIndex];
		int32 PrimitiveId = PrimitiveSceneInfo->GetIndex();
		FPrimitiveViewRelevance ViewRelevance = ShadowDepthView->PrimitiveViewRelevanceMap[PrimitiveId];

		if (!ViewRelevance.bInitializedThisFrame)
		{
			// Compute the subject primitive's view relevance since it wasn't cached
			ViewRelevance = PrimitiveSceneInfo->Proxy->GetViewRelevance(ShadowDepthView);
		}

		if (ViewRelevance.bDrawRelevance && ViewRelevance.bStaticRelevance)
		{
			int8 MinLOD, MaxLOD;
			PrimitiveSceneInfo->GetStaticMeshesLODRange(MinLOD, MaxLOD);
			// For any primitive, we only render LOD0 meshes since we do not have FSceneView available to use ComputeLODForMeshes.
			for (int32 MeshIndex = 0; MeshIndex < PrimitiveSceneInfo->StaticMeshes.Num(); MeshIndex++)
			{
				const FStaticMeshBatch& StaticMeshBatch = PrimitiveSceneInfo->StaticMeshes[MeshIndex];
				if (StaticMeshBatch.LODIndex != MinLOD)
				{
					continue;
				}
				const uint64 DefaultBatchElementMask = ~0ul;
				TranslucencyDepthPassMeshProcessor.AddMeshBatch(StaticMeshBatch, DefaultBatchElementMask, StaticMeshBatch.PrimitiveSceneInfo->Proxy, StaticMeshBatch.Id);
			}
		}
	}

	SimpleMeshDrawCommandPass->BuildRenderingCommands(GraphBuilder, *ShadowDepthView, SceneRenderer->Scene->GPUScene, PassParameters->InstanceCullingDrawParams);


	FString EventName;
#if WANTS_DRAW_MESH_EVENTS
	if (GetEmitDrawEvents())
	{
		GetShadowTypeNameForDrawEvent(EventName);
	}
#endif

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ShadowProjectionTranslucent %s", *EventName),
		PassParameters,
		ERDGPassFlags::Raster,
		[this, SimpleMeshDrawCommandPass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FMeshPassProcessorRenderState DrawRenderState;

		// Clear the shadow and its border
		RHICmdList.SetViewport(
			X,
			Y,
			0.0f,
			(X + BorderSize * 2 + ResolutionX),
			(Y + BorderSize * 2 + ResolutionY),
			1.0f
		);

		FLinearColor ClearColors[2] = { FLinearColor(0,0,0,0), FLinearColor(0,0,0,0) };
		DrawClearQuadMRT(RHICmdList, true, UE_ARRAY_COUNT(ClearColors), ClearColors, false, 1.0f, false, 0);

		// Set the viewport for the shadow.
		RHICmdList.SetViewport(
			(X + BorderSize),
			(Y + BorderSize),
			0.0f,
			(X + BorderSize + ResolutionX),
			(Y + BorderSize + ResolutionY),
			1.0f
		);
		SimpleMeshDrawCommandPass->SubmitDraw(RHICmdList, PassParameters->InstanceCullingDrawParams);
	});
}

class FDilateMarkedVoxelsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDilateMarkedVoxelsCS);
	SHADER_USE_PARAMETER_STRUCT(FDilateMarkedVoxelsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, RWVolumeMarkTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, VolumeMarkTexture)

		SHADER_PARAMETER(FIntVector, VolumeSize)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(4, 4, 4);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		const FIntVector GroupSize = GetGroupSize();

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), GroupSize.Z);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDilateMarkedVoxelsCS, "/Engine/Private/TranslucentLightingShaders.usf", "DilateMarkedVoxelsCS", SF_Compute);

class FGatherMarkedVoxelsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGatherMarkedVoxelsCS);
	SHADER_USE_PARAMETER_STRUCT(FGatherMarkedVoxelsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, RWVoxelAllocator)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, RWVoxelData)

		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, VolumeMarkTexture)

		SHADER_PARAMETER(FIntVector, VolumeSize)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(4, 4, 4);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		const FIntVector GroupSize = GetGroupSize();

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), GroupSize.Z);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGatherMarkedVoxelsCS, "/Engine/Private/TranslucentLightingShaders.usf", "GatherMarkedVoxelsCS", SF_Compute);

class FInitIndirectArgsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FInitIndirectArgsCS);
	SHADER_USE_PARAMETER_STRUCT(FInitIndirectArgsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, RWIndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, VoxelAllocator)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static uint32 GetGroupSize()
	{
		return 64;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FInitIndirectArgsCS, "/Engine/Private/TranslucentLightingShaders.usf", "InitIndirectArgsCS", SF_Compute);

/** Compute shader used to filter a single volume lighting cascade. */
class FFilterTranslucentVolumeCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FFilterTranslucentVolumeCS);
	SHADER_USE_PARAMETER_STRUCT(FFilterTranslucentVolumeCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, RWTranslucencyLightingVolumeAmbient)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, RWTranslucencyLightingVolumeDirectional)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, RWTranslucencySpecularDirection)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeAmbient)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeDirectional)
		SHADER_PARAMETER_SAMPLER(SamplerState, TranslucencyLightingVolumeAmbientSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, TranslucencyLightingVolumeDirectionalSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeMarkTexture)

		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, HistoryAmbient)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, HistoryDirectional)
		SHADER_PARAMETER_SAMPLER(SamplerState, HistoryAmbientSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, HistoryDirectionalSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, HistoryMark)

		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, HistorySpecularDirection)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, MegaLightsAmbient)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, MegaLightsDirectional)

		SHADER_PARAMETER(FIntVector, VolumeSize)
		SHADER_PARAMETER(float, TexelSize)
		SHADER_PARAMETER(uint32, VolumeCascadeIndex)

		SHADER_PARAMETER(FVector4f, PrevTranslucencyLightingVolumeMin)
		SHADER_PARAMETER(FVector4f, PrevTranslucencyLightingVolumeInvSize)

		SHADER_PARAMETER(FVector3f, HistoryTextureBilinearUVMin)
		SHADER_PARAMETER(FVector3f, HistoryTextureBilinearUVMax)

		SHADER_PARAMETER(float, HistoryWeight)

		SHADER_PARAMETER(uint32, OutputSpecularDirection)
		SHADER_PARAMETER(float, SpecularDirectionHistoryWeight)

		SHADER_PARAMETER(uint32, HasSpecularDirectionHistory)
		SHADER_PARAMETER(uint32, HasMarkHistory)
	END_SHADER_PARAMETER_STRUCT()

	class FUseTemporalFilter : SHADER_PERMUTATION_BOOL("USE_TEMPORAL_FILTER");
	class FUseSpatialFilter : SHADER_PERMUTATION_BOOL("USE_SPATIAL_FILTER");
	class FUseMarkTexture : SHADER_PERMUTATION_BOOL("USE_MARK_TEXTURE");
	class FInjectMegaLights : SHADER_PERMUTATION_BOOL("INJECT_MEGALIGHTS");
	using FPermutationDomain = TShaderPermutationDomain<FUseTemporalFilter, FUseSpatialFilter, FUseMarkTexture, FInjectMegaLights>;

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FUseSpatialFilter>())
		{
			PermutationVector.Set<FUseTemporalFilter>(false);
			PermutationVector.Set<FInjectMegaLights>(false);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(4, 4, 4);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		FIntVector GroupSize = GetGroupSize();

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), GroupSize.Z);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFilterTranslucentVolumeCS, "/Engine/Private/TranslucentLightingShaders.usf", "FilterTranslucentVolumeCS", SF_Compute);

/** Shader that adds direct lighting contribution from the given light to the current volume lighting cascade. */
class FTranslucentLightingInjectPS : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FTranslucentLightingInjectPS, Material);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrint)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FDeferredLightUniformStruct, DeferredLight)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParameters, VolumeShadowingParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightCloudTransmittanceParameters, LightCloudTransmittanceParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FAdaptiveVolumetricShadowMapUniformBufferParameters, AVSM)
		SHADER_PARAMETER(FMatrix44f, LightFunctionTranslatedWorldToLight)
		SHADER_PARAMETER(FVector4f, LightFunctionParameters)
		SHADER_PARAMETER(FVector3f, CameraRelativeLightPosition)
		SHADER_PARAMETER(float, SpotlightMask)
		SHADER_PARAMETER(uint32, VolumeCascadeIndex)
		SHADER_PARAMETER(int32, VirtualShadowMapId)
		SHADER_PARAMETER(uint32, AtmospherePerPixelTransmittanceEnabled)
		SHADER_PARAMETER(uint32, VolumetricCloudShadowEnabled)
	END_SHADER_PARAMETER_STRUCT()

	class FRadialAttenuation	: SHADER_PERMUTATION_BOOL("RADIAL_ATTENUATION");
	class FDynamicallyShadowed	: SHADER_PERMUTATION_BOOL("DYNAMICALLY_SHADOWED");
	class FLightFunction		: SHADER_PERMUTATION_BOOL("APPLY_LIGHT_FUNCTION");
	class FVirtualShadowMap		: SHADER_PERMUTATION_BOOL("VIRTUAL_SHADOW_MAP");
	class FAdaptiveVolumetricShadowMap : SHADER_PERMUTATION_BOOL("ADAPTIVE_VOLUMETRIC_SHADOW_MAP");

	using FPermutationDomain = TShaderPermutationDomain<
		FRadialAttenuation,
		FDynamicallyShadowed,
		FLightFunction,
		FVirtualShadowMap,
		FAdaptiveVolumetricShadowMap >;

public:

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment )
	{
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INJECTION_PIXEL_SHADER"), 1);
	}

	/**
	  * Makes sure only shaders for materials that are explicitly flagged
	  * as 'UsedAsLightFunction' in the Material Editor gets compiled into
	  * the shader cache.
	  */
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (!DoesPlatformSupportVirtualShadowMaps(Parameters.Platform) && PermutationVector.Get<FVirtualShadowMap>() != 0)
		{
			return false;
		}

		if (!DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform) && PermutationVector.Get<FAdaptiveVolumetricShadowMap>() != 0)
		{
			return false;
		}

		return (Parameters.MaterialParameters.MaterialDomain == MD_LightFunction || Parameters.MaterialParameters.bIsSpecialEngineMaterial) &&
			(IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			(RHISupportsGeometryShaders(Parameters.Platform) || RHISupportsVertexShaderLayer(Parameters.Platform)));
	}

	FTranslucentLightingInjectPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters( 
			this, 
			Initializer.PermutationId, 
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(), 
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false); 
	}

	FTranslucentLightingInjectPS() {}

	void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FViewInfo& View, 
		const FMaterialRenderProxy* MaterialProxy)
	{
		const FMaterial& Material = MaterialProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialProxy);
		FMaterialShader::SetParameters(BatchedParameters, MaterialProxy, Material, View);
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(,FTranslucentLightingInjectPS, TEXT("/Engine/Private/TranslucentLightInjectionShaders.usf"), TEXT("InjectMainPS"), SF_Pixel);

/** Shader that adds direct lighting contribution from multiple lights. */
class FTranslucentLightingInjectBatchCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTranslucentLightingInjectBatchCS);
	SHADER_USE_PARAMETER_STRUCT(FTranslucentLightingInjectBatchCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLightFunctionAtlasGlobalParameters, LightFunctionAtlas)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, BatchedLocalLights)
		SHADER_PARAMETER(uint32, MaxBatchedLocalLights)
		SHADER_PARAMETER(uint32, VolumeCascadeIndex)
		SHADER_PARAMETER(FIntVector, VolumeSize)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, RWTranslucencyLightingVolumeAmbient)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, RWTranslucencyLightingVolumeDirectional)

		// used when UAV typed loads are not supported
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeAmbient)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeDirectional)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, VoxelAllocator)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, VoxelData)

		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

	class FVirtualShadowMap : SHADER_PERMUTATION_BOOL("VIRTUAL_SHADOW_MAP");
	class FUseLightFunctionAtlas : SHADER_PERMUTATION_BOOL("USE_LIGHT_FUNCTION_ATLAS");
	class FIndirectVoxelDispatch : SHADER_PERMUTATION_BOOL("INDIRECT_VOXEL_DISPATCH");
	class FUseUAVTypedLoad : SHADER_PERMUTATION_BOOL("USE_UAV_TYPED_LOAD");
	using FPermutationDomain = TShaderPermutationDomain<FVirtualShadowMap, FUseLightFunctionAtlas, FIndirectVoxelDispatch, FUseUAVTypedLoad>;

public:

	static FIntVector GetGroupSize()
	{
		return FIntVector(4, 4, 4);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);

		const FIntVector GroupSize = GetGroupSize();
		
		const FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (PermutationVector.Get<FIndirectVoxelDispatch>())
		{
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X * GroupSize.Y * GroupSize.Z);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), 1);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), 1);
		}
		else
		{
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), GroupSize.Z);
		}

		// This shader must support typed UAV load and we are testing if it is supported at runtime
		// using UE::PixelFormat::HasCapabilities(..., EPixelFormatCapabilities::TypedUAVLoad)
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (!DoesPlatformSupportVirtualShadowMaps(Parameters.Platform) && PermutationVector.Get<FVirtualShadowMap>() != 0)
		{
			return false;
		}

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FTranslucentLightingInjectBatchCS, "/Engine/Private/TranslucentLightInjectionShaders.usf", "InjectBatchMainCS", SF_Compute);



class FClearTranslucentLightingVolumeCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FClearTranslucentLightingVolumeCS);
	SHADER_USE_PARAMETER_STRUCT(FClearTranslucentLightingVolumeCS, FGlobalShader)

	static const int32 CLEAR_BLOCK_SIZE = 4;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWAmbient0)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWDirectional0)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWAmbient1)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWDirectional1)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("CLEAR_BLOCK_SIZE"), CLEAR_BLOCK_SIZE);
	}
};

IMPLEMENT_GLOBAL_SHADER(FClearTranslucentLightingVolumeCS, "/Engine/Private/TranslucentLightInjectionShaders.usf", "ClearTranslucentLightingVolumeCS", SF_Compute);

static bool IsTranslucencyLightingForwardUsingVolume(const FViewInfo& View)
{
	return MegaLights::IsEnabled(*View.Family) && MegaLights::UseTranslucencyVolume();
}

void FTranslucencyLightingVolumeTextures::GetTextureFormatAndCreationFlags(EPixelFormat& OutPixelFormat, ETextureCreateFlags& OutCreationFlags)
{
	OutPixelFormat = PF_FloatRGBA;

	// TODO: We can skip the and TLV allocations when rendering in forward shading mode
	OutCreationFlags = TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_ReduceMemoryWithTilingMode | TexCreate_3DTiling | TexCreate_UAV;
}

void FTranslucencyLightingVolumeTextures::GetSpecularDirectionTextureFormatAndCreationFlags(EPixelFormat& OutPixelFormat, ETextureCreateFlags& OutCreationFlags)
{
	OutPixelFormat = PF_A2B10G10R10;

	OutCreationFlags = TexCreate_ShaderResource | TexCreate_ReduceMemoryWithTilingMode | TexCreate_3DTiling | TexCreate_UAV;
}

int32 FTranslucencyLightingVolumeTextures::GetIndex(const FViewInfo& View, int32 CascadeIndex) const
{
	// if we only have one view or one stereo pair we can just use primary index
	if (Directional.Num() == TVC_MAX)
	{
		return (View.PrimaryViewIndex * TVC_MAX) + CascadeIndex;
	}
	else
	{
		// support uncommon but possible (in theory) situations, like a stereo pair and also multiple views
		return (ViewsToTexturePairs[View.PrimaryViewIndex] * TVC_MAX) + CascadeIndex;
	}
}

void FTranslucencyLightingVolumeTextures::Init(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, ERDGPassFlags PassFlags)
{
	// Skip init/clear if disabled
	// GetTranslucencyLightingVolumeParameters will return black system textures
	if (!GUseTranslucencyLightingVolumes)
	{
		return;
	}

	check(PassFlags == ERDGPassFlags::Compute || PassFlags == ERDGPassFlags::AsyncCompute);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, TranslucentLighting, "InitTranslucencyLightingVolumeTextures");

	VolumeDim = GetTranslucencyLightingVolumeDim();
	const FIntVector TranslucencyLightingVolumeDim(VolumeDim);

	EPixelFormat TranslucencyPixelFormat;
	ETextureCreateFlags TranslucencyTargetFlags;
	GetTextureFormatAndCreationFlags(TranslucencyPixelFormat, TranslucencyTargetFlags);

	EPixelFormat SpecularDirectionPixelFormat;
	ETextureCreateFlags SpecularDirectionTargetFlags;
	GetSpecularDirectionTextureFormatAndCreationFlags(SpecularDirectionPixelFormat, SpecularDirectionTargetFlags);

	// calculate the number of textures needed given that for each stereo pair the primary view's textures will be shared between the "eyes"
	const int32 ViewCount = Views.Num();
	uint32 NumViewsWithTextures = 0;
	ViewsToTexturePairs.SetNumZeroed(Views.Num());
	for (int32 ViewIndex = 0, NumViews = Views.Num(); ViewIndex < NumViews; ++ViewIndex)
	{
		ViewsToTexturePairs[ViewIndex] = NumViewsWithTextures;
		NumViewsWithTextures += (ViewIndex == Views[ViewIndex].PrimaryViewIndex) ? 1 : 0;	// this will add 0 for those views who aren't primary
	}
	check(NumViewsWithTextures > 0);
	{
		Ambient.SetNum(NumViewsWithTextures * TVC_MAX);
		Directional.SetNum(NumViewsWithTextures * TVC_MAX);
		SpecularDirection.SetNum(NumViewsWithTextures * TVC_MAX);

		for (int32 ViewIndex = 0; ViewIndex < ViewCount; ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			for (int32 CascadeIndex = 0; CascadeIndex < TVC_MAX; ++CascadeIndex)
			{
				const uint32 TextureIndex = FTranslucencyLightingVolumeTextures::GetIndex(View, CascadeIndex);
				check(TextureIndex <= NumViewsWithTextures * TVC_MAX);

				FRDGTextureRef AmbientTexture = GraphBuilder.CreateTexture(
					FRDGTextureDesc::Create3D(
						TranslucencyLightingVolumeDim,
						TranslucencyPixelFormat,
						FClearValueBinding::Transparent,
						TranslucencyTargetFlags),
					TEXT("TranslucentVolumeAmbient"));

				FRDGTextureRef DirectionalTexture = GraphBuilder.CreateTexture(
					FRDGTextureDesc::Create3D(
						TranslucencyLightingVolumeDim,
						TranslucencyPixelFormat,
						FClearValueBinding::Transparent,
						TranslucencyTargetFlags),
					TEXT("TranslucentVolumeDirectional"));

				Ambient[TextureIndex] = AmbientTexture;
				Directional[TextureIndex] = DirectionalTexture;

				const bool bTranslucencyLightingForwardUsingVolume = IsTranslucencyLightingForwardUsingVolume(View);

				if (bTranslucencyLightingForwardUsingVolume)
				{
					SpecularDirection[TextureIndex] = GraphBuilder.CreateTexture(
						FRDGTextureDesc::Create3D(
							TranslucencyLightingVolumeDim,
							SpecularDirectionPixelFormat,
							FClearValueBinding::Transparent,
							SpecularDirectionTargetFlags),
						TEXT("TranslucentVolumeSpecularDirection"));
				}
			}
		}
	}

	const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(TranslucencyLightingVolumeDim, FClearTranslucentLightingVolumeCS::CLEAR_BLOCK_SIZE);

	TShaderMapRef<FClearTranslucentLightingVolumeCS> ComputeShader(Views[0].ShaderMap);

	for (uint32 TexturePairIndex = 0; TexturePairIndex < NumViewsWithTextures; ++TexturePairIndex)
	{
		auto* PassParameters = GraphBuilder.AllocParameters<FClearTranslucentLightingVolumeCS::FParameters>();
		PassParameters->RWAmbient0 = GraphBuilder.CreateUAV(Ambient[TexturePairIndex * TVC_MAX]);
		PassParameters->RWAmbient1 = GraphBuilder.CreateUAV(Ambient[TexturePairIndex * TVC_MAX + 1]);
		PassParameters->RWDirectional0 = GraphBuilder.CreateUAV(Directional[TexturePairIndex * TVC_MAX]);
		PassParameters->RWDirectional1 = GraphBuilder.CreateUAV(Directional[TexturePairIndex * TVC_MAX + 1]);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ClearTranslucencyLightingVolumeCompute %d", VolumeDim),
			PassFlags,
			ComputeShader,
			PassParameters,
			GroupCount);
	}
}

FTranslucencyLightingVolumeParameters GetTranslucencyLightingVolumeParameters(FRDGBuilder& GraphBuilder, const FTranslucencyLightingVolumeTextures& Textures, const FViewInfo& View)
{
	FTranslucencyLightingVolumeParameters Parameters;
	if (Textures.IsValid())
	{
		const uint32 InnerIndex = Textures.GetIndex(View, TVC_Inner);
		const uint32 OuterIndex = Textures.GetIndex(View, TVC_Outer);

		Parameters.TranslucencyLightingVolumeAmbientInner = Textures.Ambient[InnerIndex];
		Parameters.TranslucencyLightingVolumeAmbientOuter = Textures.Ambient[OuterIndex];
		Parameters.TranslucencyLightingVolumeDirectionalInner = Textures.Directional[InnerIndex];
		Parameters.TranslucencyLightingVolumeDirectionalOuter = Textures.Directional[OuterIndex];
		Parameters.TranslucencySpecularDirectionInner = OrBlack3DIfNull(GraphBuilder, Textures.SpecularDirection[InnerIndex]);
		Parameters.TranslucencySpecularDirectionOuter = OrBlack3DIfNull(GraphBuilder, Textures.SpecularDirection[OuterIndex]);
	}
	else
	{
		const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
		Parameters.TranslucencyLightingVolumeAmbientInner = SystemTextures.VolumetricBlack;
		Parameters.TranslucencyLightingVolumeAmbientOuter = SystemTextures.VolumetricBlack;
		Parameters.TranslucencyLightingVolumeDirectionalInner = SystemTextures.VolumetricBlack;
		Parameters.TranslucencyLightingVolumeDirectionalOuter = SystemTextures.VolumetricBlack;
		Parameters.TranslucencySpecularDirectionInner = SystemTextures.VolumetricBlack;
		Parameters.TranslucencySpecularDirectionOuter = SystemTextures.VolumetricBlack;
	}

	Parameters.TranslucencyLightingRandomPositionOffsetRadius = GetTranslucencyLightingVolumePositionOffsetRadius();
	Parameters.TranslucencyLightingForwardUsingVolume = IsTranslucencyLightingForwardUsingVolume(View);

	return Parameters;
}

bool IsTranslucencyLightingVolumeUsingVoxelMarkingSupported()
{
	return CVarTranslucencyLightingVolumeMarkVoxelsSupported.GetValueOnAnyThread();
}

bool IsTranslucencyLightingVolumeUsingVoxelMarking()
{
	return GUseTranslucencyLightingVolumes && IsTranslucencyLightingVolumeUsingVoxelMarkingSupported() && CVarTranslucencyLightingVolumeMarkVoxels.GetValueOnAnyThread();
}

bool IsTranslucencyLightingVolumeUsingBlueNoise()
{
	return GUseTranslucencyLightingVolumes && GetTranslucencyLightingVolumePositionOffsetRadius();
}

class FInjectAmbientCubemapPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FInjectAmbientCubemapPS);
	SHADER_USE_PARAMETER_STRUCT(FInjectAmbientCubemapPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAmbientCubemapParameters, AmbientCubemap)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FInjectAmbientCubemapPS, "/Engine/Private/TranslucentLightingShaders.usf", "InjectAmbientCubemapMainPS", SF_Pixel);

void InjectTranslucencyLightingVolumeAmbientCubemap(
	FRDGBuilder& GraphBuilder,
	const TArrayView<const FViewInfo> Views,
	const FTranslucencyLightingVolumeTextures& Textures)
{
	if (!GUseTranslucencyLightingVolumes || Views.Num() == 0 || !RHISupportsVolumeTextureRendering(Views[0].GetShaderPlatform()))
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, TranslucentLighting, "InjectAmbientCubemapTranslucentVolumeLighting");

	const int32 TranslucencyLightingVolumeDim = Textures.VolumeDim;
	const FVolumeBounds VolumeBounds(TranslucencyLightingVolumeDim);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		for (int32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; ++VolumeCascadeIndex)
		{
			FRDGTextureRef VolumeAmbientTexture = Textures.GetAmbientTexture(View, VolumeCascadeIndex);

			for (const FFinalPostProcessSettings::FCubemapEntry& CubemapEntry : View.FinalPostProcessSettings.ContributingCubemaps)
			{
				auto* PassParameters = GraphBuilder.AllocParameters<FInjectAmbientCubemapPS::FParameters>();
				SetupAmbientCubemapParameters(CubemapEntry, &PassParameters->AmbientCubemap);
				PassParameters->RenderTargets[0] = FRenderTargetBinding(VolumeAmbientTexture, ERenderTargetLoadAction::ELoad);
				PassParameters->View = View.ViewUniformBuffer;

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("Cascade %d", VolumeCascadeIndex),
					PassParameters,
					ERDGPassFlags::Raster,
					[&View, PassParameters, VolumeBounds, TranslucencyLightingVolumeDim](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					TShaderMapRef<FWriteToSliceVS> VertexShader(View.ShaderMap);
					TOptionalShaderMapRef<FWriteToSliceGS> GeometryShader(View.ShaderMap);
					TShaderMapRef<FInjectAmbientCubemapPS> PixelShader(View.ShaderMap);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GScreenVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.SetGeometryShader(GeometryShader.GetGeometryShader());
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParametersLegacyVS(RHICmdList, VertexShader, VolumeBounds, FIntVector(TranslucencyLightingVolumeDim));
					if (GeometryShader.IsValid())
					{
						SetShaderParametersLegacyGS(RHICmdList, GeometryShader, VolumeBounds.MinZ);
					}
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

					RasterizeToVolumeTexture(RHICmdList, VolumeBounds);
				});
			}
		}
	}
}

/** Calculates volume texture bounds for the given light in the given translucency lighting volume cascade. */
FVolumeBounds CalculateLightVolumeBounds(const FSphere& LightBounds, const FViewInfo& View, uint32 VolumeCascadeIndex, bool bDirectionalLight)
{
	const int32 TranslucencyLightingVolumeDim = GetTranslucencyLightingVolumeDim();

	FVolumeBounds VolumeBounds;

	if (bDirectionalLight)
	{
		VolumeBounds = FVolumeBounds(TranslucencyLightingVolumeDim);
	}
	else
	{
		// Determine extents in the volume texture
		const FVector MinPosition = (LightBounds.Center - LightBounds.W - View.TranslucencyLightingVolumeMin[VolumeCascadeIndex]) / View.TranslucencyVolumeVoxelSize[VolumeCascadeIndex];
		const FVector MaxPosition = (LightBounds.Center + LightBounds.W - View.TranslucencyLightingVolumeMin[VolumeCascadeIndex]) / View.TranslucencyVolumeVoxelSize[VolumeCascadeIndex];

		VolumeBounds.MinX = FMath::Max(FMath::TruncToInt32(MinPosition.X), 0);
		VolumeBounds.MinY = FMath::Max(FMath::TruncToInt32(MinPosition.Y), 0);
		VolumeBounds.MinZ = FMath::Max(FMath::TruncToInt32(MinPosition.Z), 0);

		VolumeBounds.MaxX = FMath::Min(FMath::TruncToInt32(MaxPosition.X) + 1, TranslucencyLightingVolumeDim);
		VolumeBounds.MaxY = FMath::Min(FMath::TruncToInt32(MaxPosition.Y) + 1, TranslucencyLightingVolumeDim);
		VolumeBounds.MaxZ = FMath::Min(FMath::TruncToInt32(MaxPosition.Z) + 1, TranslucencyLightingVolumeDim);
	}

	return VolumeBounds;
}

FTranslucentLightInjectionCollector::FTranslucentLightInjectionCollector(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	bool bAreLightsInLightGrid)
	// NOTE: This data is directly referenced inside the render pass lamba, so must be allocated in the graph
	: InjectionDataPerView(*GraphBuilder.AllocObject<TArray<FPerViewData, SceneRenderingAllocator>>())
{
	InjectionDataPerView.SetNum(Views.Num());

	// Static conditions for supporting batching
	bCollectorSupportsBatching =
		CVarTranslucencyLightingVolumeBatch.GetValueOnRenderThread() != 0 &&
		bAreLightsInLightGrid;
}

/**
 * Adds a light to LightInjectionData if it should be injected into the translucent volume, and caches relevant information in a FTranslucentLightInjectionData.
 * @param InProjectedShadowInfo is 0 for unshadowed lights
 */
void FTranslucentLightInjectionCollector::AddLightForInjection(
	const FViewInfo& View,
	const uint32 ViewIndex,
	FLightGridSceneExtension::FViewData* LightGridViewData,
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FLightSceneInfo& LightSceneInfo,
	const FProjectedShadowInfo* InProjectedShadowInfo)
{
	if (LightSceneInfo.Proxy->AffectsTranslucentLighting())
	{
		const uint8 LightType = LightSceneInfo.Proxy->GetLightType();

		FVolumeBounds VolumeBounds[TVC_MAX];
		bool bAnyBoundsValid = false;
		for (uint32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; ++VolumeCascadeIndex)
		{
			VolumeBounds[VolumeCascadeIndex] = CalculateLightVolumeBounds(LightSceneInfo.Proxy->GetBoundingSphere(), View, VolumeCascadeIndex, LightType == LightType_Directional);
			bAnyBoundsValid = bAnyBoundsValid || VolumeBounds[VolumeCascadeIndex].IsValid();
		}
		if (!bAnyBoundsValid)
		{
			return;
		}

		const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo.Id];
		const ERHIFeatureLevel::Type FeatureLevel = View.FeatureLevel;

		bool bLightFunctionUsesAtlas = false;
		bool bApplyLightFunction = false;
		const FMaterialRenderProxy* LightFunctionMaterialRenderProxy = LightSceneInfo.Proxy->GetLightFunctionMaterial();
		if (LightFunctionMaterialRenderProxy && View.Family->EngineShowFlags.LightFunctions)
		{
			const FMaterial& LightFunctionMaterial	= LightFunctionMaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);

			bApplyLightFunction						= LightFunctionMaterial.IsLightFunction();
			bLightFunctionUsesAtlas					= bApplyLightFunction && LightFunctionMaterial.MaterialIsLightFunctionAtlasCompatible_RenderThread();
		}

		const int32 VirtualShadowMapId = VisibleLightInfo.GetVirtualShadowMapId(&View);
		
		const bool bUseAdaptiveVolumetricShadowMap =
			LightSceneInfo.Proxy->CastsVolumetricShadow() &&
			ShouldRenderHeterogeneousVolumesForView(View) &&
			ShouldHeterogeneousVolumesCastShadows();

		const bool bStaticShadowing =
			LightSceneInfo.Proxy->GetStaticShadowDepthMap() &&
			LightSceneInfo.Proxy->GetStaticShadowDepthMap()->Data;

		// We only support the "accurate" rect light model through this path since the LightGrid stores them that way
		// Thus if the spot light approximation is used we have to send them through the unbatched path.
		const bool bSupportRectLights = CVarTranslucencyLightingVolumeAccurateRectLights.GetValueOnRenderThread() != 0;

		// Lights without certain features can be batched into a single draw (loop in shader) which is more efficient
		bool bSupportsBatching =
			bCollectorSupportsBatching &&
			LightType != LightType_Directional &&
			(bSupportRectLights || LightType != LightType_Rect) &&
			!bStaticShadowing &&
			InProjectedShadowInfo == nullptr &&
			(!bApplyLightFunction || bLightFunctionUsesAtlas) &&
			!bUseAdaptiveVolumetricShadowMap;

		// If it would otherwise be supported, see if we can find the local light index in the light grid
		if (bSupportsBatching && View.ForwardLightingResources.ForwardLightUniformBuffer && LightGridViewData)
		{
			int32 ForwardLightIndex = INDEX_NONE;
			
			if (LightGridViewData->LightSceneIdToForwardLightIndex.IsValidIndex(LightSceneInfo.Id.GetIndex()))
			{
				ForwardLightIndex = LightGridViewData->LightSceneIdToForwardLightIndex[LightSceneInfo.Id.GetIndex()];
			}

			if (ForwardLightIndex != INDEX_NONE)
			{
				FPerViewData& InjectionData = InjectionDataPerView[ViewIndex];
				// There shouldn't be any duplication here
				++InjectionData.BatchedLocalLightCount;

				for (uint32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; ++VolumeCascadeIndex)
				{
					if (VolumeBounds[VolumeCascadeIndex].IsValid())
					{
						TBitArray<SceneRenderingAllocator>& Bits = InjectionData.BatchedLocalLights[VolumeCascadeIndex];
						Bits.PadToNum(ForwardLightIndex + 1, false);
						Bits[ForwardLightIndex] = true;
					}
				}

				if (VirtualShadowMapId != INDEX_NONE)
				{
					// Note if there are any batched lights with VSM for permutation selection later
					InjectionData.bAnyBatchedLightsWithVirtualShadowMaps = true;
				}
			}
			else
			{
				bSupportsBatching = false;
			}
		}
		
		if (!bSupportsBatching)
		{
			// Skip rendering if the DefaultLightFunctionMaterial isn't compiled yet
			const FMaterialRenderProxy* MaterialProxy = bApplyLightFunction ?
				LightSceneInfo.Proxy->GetLightFunctionMaterial() :
				UMaterial::GetDefaultMaterial(MD_LightFunction)->GetRenderProxy();

			if (MaterialProxy->GetIncompleteMaterialWithFallback(FeatureLevel).IsLightFunction())
			{
				FTranslucentLightInjectionCollector::FInjectionData Data;
				Data.LightSceneInfo = &LightSceneInfo;
				Data.ProjectedShadowInfo = InProjectedShadowInfo;
				Data.bApplyLightFunction = bApplyLightFunction;
				Data.LightFunctionMaterialProxy = MaterialProxy;
				Data.VirtualShadowMapId = VirtualShadowMapId;
				Data.bUseAdaptiveVolumetricShadowMap = bUseAdaptiveVolumetricShadowMap;
				for (uint32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; ++VolumeCascadeIndex)
				{
					Data.VolumeBounds[VolumeCascadeIndex] = VolumeBounds[VolumeCascadeIndex];
				}
				InjectionDataPerView[ViewIndex].Unbatched.Add(Data);
			}
		}
	}
}

static FRDGTextureRef GetSkyTransmittanceLutTexture(FRDGBuilder& GraphBuilder, const FScene* Scene, const FViewInfo& View)
{
	FRDGTextureRef TransmittanceLutTexture = nullptr;
	if (ShouldRenderSkyAtmosphere(Scene, View.Family->EngineShowFlags))
	{
		if (const FSkyAtmosphereRenderSceneInfo* SkyInfo = Scene->GetSkyAtmosphereSceneInfo())
		{
			TransmittanceLutTexture = SkyInfo->GetTransmittanceLutTexture(GraphBuilder);
		}
	}
	return TransmittanceLutTexture;
}

static bool SetupPSOStateForVolumeInjection(
	TShaderMapRef<FWriteToSliceVS> VertexShader,
	TOptionalShaderMapRef<FWriteToSliceGS> GeometryShader,
	FRHIPixelShader* PixelShaderRHI,
	bool bDirectionalLight,
	FGraphicsPipelineStateInitializer &OutGraphicsPSOInit,
	bool bRequired = true)
{
	OutGraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	OutGraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	OutGraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

	// Accumulate the contribution of multiple lights
	if (bDirectionalLight)
	{		
		// Directional lights write their shadowing into alpha of the ambient texture
		OutGraphicsPSOInit.BlendState = TStaticBlendState<
			CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
		CW_RGB, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
	}
	else
	{
		OutGraphicsPSOInit.BlendState = TStaticBlendState<
			CW_RGB, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_One,
		CW_RGB, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_One>::GetRHI();
	}

	// Early out on missing vertex and pixel shaders. The geometry shader is optional, so we do not early out for that. 
	FRHIVertexShader* VertexShaderRHI = VertexShader.GetVertexShader(bRequired);
	if (!VertexShaderRHI || !PixelShaderRHI)
	{
		return false;
	}
	OutGraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GScreenVertexDeclaration.VertexDeclarationRHI;
	OutGraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShaderRHI;
	OutGraphicsPSOInit.BoundShaderState.SetGeometryShader(GeometryShader.GetGeometryShader());
	OutGraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShaderRHI;
	return true;
}

static void SetPSOStateForVolumeInjection(
	TShaderMapRef<FWriteToSliceVS> VertexShader,
	TOptionalShaderMapRef<FWriteToSliceGS> GeometryShader,
	FRHIPixelShader* PixelShaderRHI,
	FVolumeBounds VolumeBounds,
	bool bDirectionalLight,
	FRHICommandList& RHICmdList,
	FGraphicsPipelineStateInitializer &OutGraphicsPSOInit)
{
	RHICmdList.ApplyCachedRenderTargets(OutGraphicsPSOInit);

	// Shaders are required for regular rendering, so no need to check the return value here
	SetupPSOStateForVolumeInjection(VertexShader, GeometryShader, PixelShaderRHI, bDirectionalLight, OutGraphicsPSOInit);

	SetGraphicsPipelineState(RHICmdList, OutGraphicsPSOInit, 0);

	const int32 TranslucencyLightingVolumeDim = GetTranslucencyLightingVolumeDim();

	SetShaderParametersLegacyVS(RHICmdList, VertexShader, VolumeBounds, FIntVector(TranslucencyLightingVolumeDim));
	if (GeometryShader.IsValid())
	{
		SetShaderParametersLegacyGS(RHICmdList, GeometryShader, VolumeBounds.MinZ);
	}
}


static void InjectTranslucencyLightingVolumeBatch(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const uint32 ViewIndex,
	uint32 MaxBatchedLocalLights,
	FRDGBufferRef BatchedLocalLightsRDG,
	bool bSupportVirtualShadowMaps,
	uint32 VolumeCascadeIndex,
	FRDGTextureRef& VolumeAmbientTexture,
	FRDGTextureRef& VolumeDirectionalTexture,
	const FSceneRenderer& Renderer)
{
	check(MaxBatchedLocalLights > 0);
	check(BatchedLocalLightsRDG);

	const FIntVector VolumeSize = FIntVector(GetTranslucencyLightingVolumeDim());

	const FViewInfo::FTranslucencyVolumeMarkData& VolumeMarkData = View.TranslucencyVolumeMarkData[VolumeCascadeIndex];
	const bool bUseVolumeMarkTexture = IsTranslucencyLightingVolumeUsingVoxelMarking() && VolumeMarkData.MarkTexture != nullptr;

	EPixelFormat TranslucencyPixelFormat;
	ETextureCreateFlags TranslucencyTargetFlags;
	FTranslucencyLightingVolumeTextures::GetTextureFormatAndCreationFlags(TranslucencyPixelFormat, TranslucencyTargetFlags);

	const bool bUseLightFunctionAtlas = LightFunctionAtlas::IsEnabled(View, ELightFunctionAtlasSystem::DeferredLighting);
	const bool bUseUAVTypedLoad = CVarTranslucencyLightingVolumeUseTypedUAVLoad.GetValueOnRenderThread() && UE::PixelFormat::HasCapabilities(TranslucencyPixelFormat, EPixelFormatCapabilities::TypedUAVLoad);

	auto* PassParameters = GraphBuilder.AllocParameters<FTranslucentLightingInjectBatchCS::FParameters>();
	if (bSupportVirtualShadowMaps)
	{
		PassParameters->VirtualShadowMapSamplingParameters = Renderer.VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
	}
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
	PassParameters->BatchedLocalLights = GraphBuilder.CreateSRV(BatchedLocalLightsRDG);
	PassParameters->LightFunctionAtlas = LightFunctionAtlas::BindGlobalParameters(GraphBuilder, View);;
	PassParameters->MaxBatchedLocalLights = MaxBatchedLocalLights;
	PassParameters->VolumeCascadeIndex = VolumeCascadeIndex;
	PassParameters->VolumeSize = VolumeSize;

	if (bUseUAVTypedLoad)
	{
		PassParameters->RWTranslucencyLightingVolumeAmbient = GraphBuilder.CreateUAV(VolumeAmbientTexture);
		PassParameters->RWTranslucencyLightingVolumeDirectional = GraphBuilder.CreateUAV(VolumeDirectionalTexture);
	}
	else
	{
		// need to output to auxiliary textures to be able to read from existing volumes

		FRDGTextureRef OutputVolumeAmbientTexture = GraphBuilder.CreateTexture(VolumeAmbientTexture->Desc, VolumeAmbientTexture->Name);
		FRDGTextureRef OutputVolumeDirectionalTexture = GraphBuilder.CreateTexture(VolumeDirectionalTexture->Desc, VolumeDirectionalTexture->Name);

		PassParameters->RWTranslucencyLightingVolumeAmbient = GraphBuilder.CreateUAV(OutputVolumeAmbientTexture);
		PassParameters->RWTranslucencyLightingVolumeDirectional = GraphBuilder.CreateUAV(OutputVolumeDirectionalTexture);

		PassParameters->TranslucencyLightingVolumeAmbient = VolumeAmbientTexture;
		PassParameters->TranslucencyLightingVolumeDirectional = VolumeDirectionalTexture;

		VolumeAmbientTexture = OutputVolumeAmbientTexture;
		VolumeDirectionalTexture = OutputVolumeDirectionalTexture;
	}

	FTranslucentLightingInjectBatchCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FTranslucentLightingInjectBatchCS::FVirtualShadowMap>(bSupportVirtualShadowMaps);
	PermutationVector.Set<FTranslucentLightingInjectBatchCS::FUseLightFunctionAtlas>(bUseLightFunctionAtlas);
	PermutationVector.Set<FTranslucentLightingInjectBatchCS::FIndirectVoxelDispatch>(bUseVolumeMarkTexture);
	PermutationVector.Set<FTranslucentLightingInjectBatchCS::FUseUAVTypedLoad>(bUseUAVTypedLoad);

	auto ComputeShader = View.ShaderMap->GetShader<FTranslucentLightingInjectBatchCS>(PermutationVector);

	if (bUseVolumeMarkTexture)
	{
		PassParameters->VoxelAllocator = GraphBuilder.CreateSRV(VolumeMarkData.VoxelAllocator);
		PassParameters->VoxelData = GraphBuilder.CreateSRV(VolumeMarkData.VoxelData);
		PassParameters->IndirectArgs = VolumeMarkData.VoxelIndirectArgs;

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("InjectTranslucencyLightingVolumeBatch(VolumeCascade=%d,Max=%d%s)",
				VolumeCascadeIndex, MaxBatchedLocalLights,
				bSupportVirtualShadowMaps ? TEXT(",VirtualShadowMap") : TEXT("")),
			ComputeShader,
			PassParameters,
			VolumeMarkData.VoxelIndirectArgs,
			0);
	}
	else
	{
		FIntVector NumGroups = FComputeShaderUtils::GetGroupCount(VolumeSize, FTranslucentLightingInjectBatchCS::GetGroupSize());

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("InjectTranslucencyLightingVolumeBatch(VolumeCascade=%d,Max=%d%s)",
				VolumeCascadeIndex, MaxBatchedLocalLights,
				bSupportVirtualShadowMaps ? TEXT(",VirtualShadowMap") : TEXT("")),
			ComputeShader,
			PassParameters,
			NumGroups);
	}
}


BEGIN_SHADER_PARAMETER_STRUCT(FInjectTranslucentLightArrayParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FTranslucentLightingInjectPS::FParameters, PS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVolumetricCloudShadowAOParameters, CloudShadowAO)
	RDG_TEXTURE_ACCESS(TransmittanceLutTexture, ERHIAccess::SRVGraphics)
	RDG_TEXTURE_ACCESS(ShadowDepthTexture, ERHIAccess::SRVGraphics)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

/** Injects all the lights in LightInjectionData into the translucency lighting volume textures. */
void InjectTranslucencyLightingVolume(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const uint32 ViewIndex,
	const FScene* Scene,
	const FSceneRenderer& Renderer,
	const FTranslucentLightInjectionCollector& Collector,
	FTranslucencyLightingVolumeTextures& Textures)
{
	if (!GUseTranslucencyLightingVolumes || !RHISupportsVolumeTextureRendering(View.GetShaderPlatform()))
	{
		return;
	}

	const FTranslucentLightInjectionCollector::FPerViewData& LightInjectionData = Collector.InjectionDataPerView[ViewIndex];

	INC_DWORD_STAT_BY(STAT_NumLightsInjectedIntoTranslucencyBatched, LightInjectionData.BatchedLocalLightCount);
	INC_DWORD_STAT_BY(STAT_NumLightsInjectedIntoTranslucency, LightInjectionData.Unbatched.Num());

	const FVolumetricCloudShadowAOParameters CloudShadowAOParameters = GetCloudShadowAOParameters(GraphBuilder, View, Scene->GetVolumetricCloudSceneInfo());
	const bool bUseLightFunctionAtlas = View.LightFunctionAtlasViewData.UsesLightFunctionAtlas(LightFunctionAtlas::ELightFunctionAtlasSystem::DeferredLighting);

	FRDGTextureRef TransmittanceLutTexture = GetSkyTransmittanceLutTexture(GraphBuilder, Scene, View);

	// When accurate rect lights is disabled we approximate rect lights as spotlights
	uint32 DeferredLightParameterFlags = CVarTranslucencyLightingVolumeAccurateRectLights.GetValueOnRenderThread() != 0 ?
		0U : ELightShaderParameterFlags::RectAsSpotLight;

	// Inject into each volume cascade. Operate on one cascade at a time to reduce render target switches.
	for (uint32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
	{
		// for stereo case, using PrimaryViewIndex essentially shares the lighting volume textures
		const uint32 TextureIndex = Textures.GetIndex(View, VolumeCascadeIndex);
		FRDGTextureRef VolumeAmbientTexture = Textures.Ambient[TextureIndex];
		FRDGTextureRef VolumeDirectionalTexture = Textures.Directional[TextureIndex];

		// Batched lights
		{
			TBitArray<SceneRenderingAllocator>& BatchedLocalLights = Collector.InjectionDataPerView[ViewIndex].BatchedLocalLights[VolumeCascadeIndex];
			if (BatchedLocalLights.Num() > 0)
			{
				const uint32 NumUint32Elements = FMath::DivideAndRoundUp(BatchedLocalLights.Num(), 32);
				const uint32 InitialDataSize = NumUint32Elements * sizeof(uint32);
				BatchedLocalLights.PadToNum(NumUint32Elements * 32, false);

				FRDGBufferRef BatchedLocalLightsRDG = CreateStructuredBuffer(
					GraphBuilder,
					TEXT("TranslucencyLightingVolume.BatchedLocalLights"),
					sizeof(uint32),
					NumUint32Elements,
					BatchedLocalLights.GetData(),
					InitialDataSize
				);

				InjectTranslucencyLightingVolumeBatch(GraphBuilder, View, ViewIndex,
					BatchedLocalLights.Num(), BatchedLocalLightsRDG,
					LightInjectionData.bAnyBatchedLightsWithVirtualShadowMaps,
					VolumeCascadeIndex, VolumeAmbientTexture, VolumeDirectionalTexture,
					Renderer);
			}
		}

		// Unbatched lights
		for (int32 LightIndex = 0; LightIndex < LightInjectionData.Unbatched.Num(); LightIndex++)
		{
			const FTranslucentLightInjectionCollector::FInjectionData& InjectionData = LightInjectionData.Unbatched[LightIndex];
			const FLightSceneInfo* const LightSceneInfo = InjectionData.LightSceneInfo;
			const FVisibleLightInfo& VisibleLightInfo = Renderer.VisibleLightInfos[LightSceneInfo->Id];
			const bool bInverseSquared = LightSceneInfo->Proxy->IsInverseSquared();
			const bool bDirectionalLight = LightSceneInfo->Proxy->GetLightType() == LightType_Directional;

			const FVolumeBounds VolumeBounds = InjectionData.VolumeBounds[VolumeCascadeIndex];
			if (VolumeBounds.IsValid())
			{
				TShaderMapRef<FWriteToSliceVS> VertexShader(View.ShaderMap);
				TOptionalShaderMapRef<FWriteToSliceGS> GeometryShader(View.ShaderMap);

				FRDGTextureRef ShadowDepthTexture = nullptr;

				if (InjectionData.ProjectedShadowInfo)
				{
					ShadowDepthTexture = TryRegisterExternalTexture(GraphBuilder, InjectionData.ProjectedShadowInfo->RenderTargets.DepthTarget);
				}

				auto* PassParameters = GraphBuilder.AllocParameters< FInjectTranslucentLightArrayParameters >();
				PassParameters->TransmittanceLutTexture = TransmittanceLutTexture;
				PassParameters->ShadowDepthTexture = ShadowDepthTexture;
				PassParameters->CloudShadowAO = CloudShadowAOParameters;
				PassParameters->PS.VirtualShadowMapSamplingParameters = Renderer.VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
				PassParameters->RenderTargets[0] = FRenderTargetBinding(VolumeAmbientTexture, ERenderTargetLoadAction::ELoad);
				PassParameters->RenderTargets[1] = FRenderTargetBinding(VolumeDirectionalTexture, ERenderTargetLoadAction::ELoad);

				PassParameters->PS.ViewUniformBuffer = View.ViewUniformBuffer;

				FDeferredLightUniformStruct* DeferredLightStruct = GraphBuilder.AllocParameters<FDeferredLightUniformStruct>();
				*DeferredLightStruct = GetDeferredLightParameters(View, *LightSceneInfo, bUseLightFunctionAtlas, DeferredLightParameterFlags);
				PassParameters->PS.DeferredLight = GraphBuilder.CreateUniformBuffer(DeferredLightStruct);

				GetVolumeShadowingShaderParameters(GraphBuilder, View, LightSceneInfo, InjectionData.ProjectedShadowInfo, PassParameters->PS.VolumeShadowingParameters);

				PassParameters->PS.VirtualShadowMapId = InjectionData.VirtualShadowMapId;
				PassParameters->PS.LightFunctionParameters = FLightFunctionSharedParameters::GetLightFunctionSharedParameters(LightSceneInfo, 1.0f);
				PassParameters->PS.VolumeCascadeIndex = VolumeCascadeIndex;
				PassParameters->PS.AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);

				bool bIsSpotlight = LightSceneInfo->Proxy->GetLightType() == LightType_Spot;
				PassParameters->PS.SpotlightMask = bIsSpotlight ? 1.0f : 0.0f; //@todo - needs to be a permutation to reduce shadow filtering work

				{
					const FVector Scale = LightSceneInfo->Proxy->GetLightFunctionScale();
					// Switch x and z so that z of the user specified scale affects the distance along the light direction
					const FVector InverseScale = FVector(1.f / Scale.Z, 1.f / Scale.Y, 1.f / Scale.X);
					const FMatrix WorldToLight = LightSceneInfo->Proxy->GetWorldToLight() * FScaleMatrix(InverseScale);
					const FMatrix TranslatedWorldToWorld = FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation());

					PassParameters->PS.LightFunctionTranslatedWorldToLight = FMatrix44f(TranslatedWorldToWorld * WorldToLight);
				}

				const bool bCloudShadowEnabled = SetupLightCloudTransmittanceParameters(GraphBuilder, Scene, View, LightSceneInfo, PassParameters->PS.LightCloudTransmittanceParameters);
				PassParameters->PS.VolumetricCloudShadowEnabled = bCloudShadowEnabled ? 1 : 0;

				PassParameters->PS.AtmospherePerPixelTransmittanceEnabled = IsLightAtmospherePerPixelTransmittanceEnabled(Scene, View, LightSceneInfo);

				PassParameters->PS.CameraRelativeLightPosition = GetCamRelativeLightPosition(View.ViewMatrices, *LightSceneInfo);

			#if 0 // Enable this to be able to debug using DEBUG_ONE_VOXEL
				ShaderPrint::SetEnabled(true);
				ShaderPrint::RequestSpaceForLines(64u);
				ShaderPrint::RequestSpaceForCharacters(64u);
				ShaderPrint::RequestSpaceForTriangles(64u);
				ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->PS.ShaderPrint);
			#endif

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("InjectTranslucencyLightingVolume(VolumeCascade=%d%s%s%s)",
						VolumeCascadeIndex,
						InjectionData.VirtualShadowMapId != INDEX_NONE ? TEXT(",VirtualShadowMap") : TEXT(""),
						InjectionData.ProjectedShadowInfo != nullptr ? TEXT(",ShadowMap") : TEXT(""),
						InjectionData.bApplyLightFunction ? TEXT(",LightFunction") : TEXT("")),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, VertexShader, GeometryShader, &View, &Renderer, &InjectionData, LightSceneInfo, bDirectionalLight, VolumeBounds, VolumeCascadeIndex](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					FGraphicsPipelineStateInitializer GraphicsPSOInit;

					const FMaterialRenderProxy* MaterialProxy = InjectionData.LightFunctionMaterialProxy;
					const FMaterial& Material = MaterialProxy->GetMaterialWithFallback( View.GetFeatureLevel(), MaterialProxy );
					const FMaterialShaderMap* MaterialShaderMap = Material.GetRenderingThreadShaderMap();

					const bool bUseVSM = InjectionData.VirtualShadowMapId != INDEX_NONE;
					bool bDynamicShadow = InjectionData.ProjectedShadowInfo != nullptr;

					if (CVarTranslucencyLightingVolumeInjectDirectionalLightCSM.GetValueOnRenderThread() <= 0 && bDirectionalLight)
					{
						bDynamicShadow = false;
					}

					FTranslucentLightingInjectPS::FPermutationDomain PermutationVector;
					PermutationVector.Set< FTranslucentLightingInjectPS::FRadialAttenuation >(!bDirectionalLight);
					PermutationVector.Set< FTranslucentLightingInjectPS::FDynamicallyShadowed >(bDynamicShadow);
					PermutationVector.Set< FTranslucentLightingInjectPS::FLightFunction >(InjectionData.bApplyLightFunction);
					PermutationVector.Set< FTranslucentLightingInjectPS::FVirtualShadowMap >(bUseVSM);
					PermutationVector.Set< FTranslucentLightingInjectPS::FAdaptiveVolumetricShadowMap >(InjectionData.bUseAdaptiveVolumetricShadowMap);

					auto PixelShader = MaterialShaderMap->GetShader< FTranslucentLightingInjectPS >( PermutationVector );

					SetPSOStateForVolumeInjection(VertexShader, GeometryShader, PixelShader.GetPixelShader(),
						VolumeBounds, bDirectionalLight, RHICmdList, GraphicsPSOInit);

#if PSO_PRECACHING_VALIDATE
					if (PSOCollectorStats::IsFullPrecachingValidationEnabled())
					{
						static const int32 MaterialPSOCollectorIndex = FPSOCollectorCreateManager::GetIndex(GetFeatureLevelShadingPath(GMaxRHIFeatureLevel), TranslucentLightingMaterialPSOCollectorName);
						PSOCollectorStats::CheckFullPipelineStateInCache(GraphicsPSOInit, EPSOPrecacheResult::Unknown, MaterialProxy, nullptr, nullptr, MaterialPSOCollectorIndex);
					}
#endif // PSO_PRECACHING_VALIDATE

					SetShaderParametersLegacyPS(RHICmdList, PixelShader, View, InjectionData.LightFunctionMaterialProxy);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

					RasterizeToVolumeTexture(RHICmdList, VolumeBounds);
				});
			}
		}

		Textures.Ambient[TextureIndex] = VolumeAmbientTexture;
		Textures.Directional[TextureIndex] = VolumeDirectionalTexture;
	}

	GraphBuilder.FlushSetupQueue();
}

class FSimpleLightTranslucentLightingInjectPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSimpleLightTranslucentLightingInjectPS);
	SHADER_USE_PARAMETER_STRUCT(FSimpleLightTranslucentLightingInjectPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER(FVector4f, SimpleLightPositionAndRadius)
		SHADER_PARAMETER(FVector4f, SimpleLightColorAndExponent)
		SHADER_PARAMETER(uint32, VolumeCascadeIndex)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && (RHISupportsGeometryShaders(Parameters.Platform) || RHISupportsVertexShaderLayer(Parameters.Platform));
	}
};

IMPLEMENT_GLOBAL_SHADER(FSimpleLightTranslucentLightingInjectPS, "/Engine/Private/TranslucentLightInjectionShaders.usf", "SimpleLightInjectMainPS", SF_Pixel);

void InjectSimpleTranslucencyLightingVolumeArray(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const uint32 ViewIndex,
	const uint32 ViewCount,
	const FTranslucencyLightingVolumeTextures& Textures,
	const FSimpleLightArray& SimpleLights)
{
	int32 NumLightsToInject = 0;

	for (int32 LightIndex = 0; LightIndex < SimpleLights.InstanceData.Num(); LightIndex++)
	{
		if (SimpleLights.InstanceData[LightIndex].bAffectTranslucency)
		{
			NumLightsToInject++;
		}
	}

	if (NumLightsToInject > 0)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "InjectSimpleTranslucentLightArray");

		INC_DWORD_STAT_BY(STAT_NumLightsInjectedIntoTranslucency, NumLightsToInject);

		const int32 TranslucencyLightingVolumeDim = GetTranslucencyLightingVolumeDim();

		const float Exposure = View.GetLastEyeAdaptationExposure();

		// Inject into each volume cascade
		// Operate on one cascade at a time to reduce render target switches
		for (int32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
		{
			const uint32 TextureIndex = Textures.GetIndex(View, VolumeCascadeIndex);

			RDG_EVENT_SCOPE(GraphBuilder, "Cascade%d", VolumeCascadeIndex);
			FRDGTextureRef VolumeAmbientTexture = Textures.Ambient[TextureIndex];
			FRDGTextureRef VolumeDirectionalTexture = Textures.Directional[TextureIndex];

			for (int32 LightIndex = 0; LightIndex < SimpleLights.InstanceData.Num(); LightIndex++)
			{
				const FSimpleLightEntry& SimpleLight = SimpleLights.InstanceData[LightIndex];
				const FSimpleLightPerViewEntry& SimpleLightPerViewData = SimpleLights.GetViewDependentData(LightIndex, ViewIndex, ViewCount);

				if (SimpleLight.bAffectTranslucency)
				{
					const FSphere LightBounds(SimpleLightPerViewData.Position, SimpleLight.Radius);
					const FVolumeBounds VolumeBounds = CalculateLightVolumeBounds(LightBounds, View, VolumeCascadeIndex, false);

					if (VolumeBounds.IsValid())
					{
						const FVector3f TranslatedLightPosition = FVector3f(SimpleLightPerViewData.Position + View.ViewMatrices.GetPreViewTranslation());

						auto* PassParameters = GraphBuilder.AllocParameters<FSimpleLightTranslucentLightingInjectPS::FParameters>();
						PassParameters->View = View.ViewUniformBuffer;
						PassParameters->VolumeCascadeIndex = VolumeCascadeIndex;
						PassParameters->SimpleLightPositionAndRadius = FVector4f(TranslatedLightPosition, SimpleLight.Radius);
						PassParameters->SimpleLightColorAndExponent = FVector4f((FVector3f)SimpleLight.Color * FLightRenderParameters::GetLightExposureScale(Exposure, SimpleLight.InverseExposureBlend), SimpleLight.Exponent);
						PassParameters->RenderTargets[0] = FRenderTargetBinding(VolumeAmbientTexture, ERenderTargetLoadAction::ELoad);
						PassParameters->RenderTargets[1] = FRenderTargetBinding(VolumeDirectionalTexture, ERenderTargetLoadAction::ELoad);

						TShaderMapRef<FWriteToSliceVS> VertexShader(View.ShaderMap);
						TOptionalShaderMapRef<FWriteToSliceGS> GeometryShader(View.ShaderMap);
						TShaderMapRef<FSimpleLightTranslucentLightingInjectPS> PixelShader(View.ShaderMap);

						GraphBuilder.AddPass(
							{},
							PassParameters,
							ERDGPassFlags::Raster,
							[VertexShader, GeometryShader, PixelShader, PassParameters, VolumeBounds, TranslucencyLightingVolumeDim](FRDGAsyncTask, FRHICommandList& RHICmdList)
						{
							FGraphicsPipelineStateInitializer GraphicsPSOInit;
							
							SetPSOStateForVolumeInjection(VertexShader, GeometryShader, PixelShader.GetPixelShader(),
								VolumeBounds, false, RHICmdList, GraphicsPSOInit);

							SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
							RasterizeToVolumeTexture(RHICmdList, VolumeBounds);
						});
					}
				}
			}
		}
	}
}

class FFrustumCullVoxelsCS : public FGlobalShader
{
public:
	static const int32 NumCameraFrustumPlanes = 5;

	DECLARE_GLOBAL_SHADER(FFrustumCullVoxelsCS)
	SHADER_USE_PARAMETER_STRUCT(FFrustumCullVoxelsCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, RWVoxelAllocator0)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, RWVoxelAllocator1)
		SHADER_PARAMETER(uint32, NumVoxels)
		SHADER_PARAMETER_ARRAY(FVector4f, CameraFrustumPlanes, [NumCameraFrustumPlanes])
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static int32 GetGroupSize()
	{
		return 4;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}
};

IMPLEMENT_GLOBAL_SHADER(FFrustumCullVoxelsCS, "/Engine/Private/TranslucentLightingShaders.usf", "FrustumCullVoxelsCS", SF_Compute);

class FDebugTranslucencyLightingVolumeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDebugTranslucencyLightingVolumeCS)
	SHADER_USE_PARAMETER_STRUCT(FDebugTranslucencyLightingVolumeCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, VolumeMarkTexture0)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, VolumeMarkTexture1)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, VoxelAllocator0)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, VoxelAllocator1)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, FrustumCullVoxelAllocator0)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, FrustumCullVoxelAllocator1)

		SHADER_PARAMETER(uint32, NumVoxels)

		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, SpecularDirectionTexture0)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, SpecularDirectionTexture1)
		SHADER_PARAMETER(uint32, DrawSpecularDirection)
		SHADER_PARAMETER(uint32, VolumeCascadeIndex)
		SHADER_PARAMETER(float, ProbeRadiusScale)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static int32 GetGroupSize()
	{
		return 4;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}
};

IMPLEMENT_GLOBAL_SHADER(FDebugTranslucencyLightingVolumeCS, "/Engine/Private/TranslucentLightingShaders.usf", "DebugTranslucencyLightingVolumeCS", SF_Compute);

static void DrawDebugTranslucencyLightingVolume(
	FRDGBuilder& GraphBuilder,
	const TArrayView<const FViewInfo> Views,
	FTranslucencyLightingVolumeTextures& Textures)
{
	// Draw debug the translucency volume lighting buffer
	if (GTranslucencyLightingVolumeDebug <= 0)
	{
		return;
	}

	const FIntVector TranslucencyLightingVolumeDim(GetTranslucencyLightingVolumeDim());
	const uint32 NumVoxels = TranslucencyLightingVolumeDim.X * TranslucencyLightingVolumeDim.Y * TranslucencyLightingVolumeDim.Z;

	FRDGTextureRef DummyTexture = GSystemTextures.GetVolumetricBlackUintDummy(GraphBuilder);
	FRDGBufferRef DummyAllocatorBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32), IsTranslucencyLightingVolumeUsingVoxelMarking() ? 0 : NumVoxels);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE(GraphBuilder, "View%d", ViewIndex);

		ShaderPrint::SetEnabled(true);
		ShaderPrint::RequestSpaceForLines(12 * int32(NumVoxels * 0.1f));
		ShaderPrint::RequestSpaceForCharacters(128u);

		const FViewInfo::FTranslucencyVolumeMarkData& VolumeMarkData0 = View.TranslucencyVolumeMarkData[0];
		const FViewInfo::FTranslucencyVolumeMarkData& VolumeMarkData1 = View.TranslucencyVolumeMarkData[1];

		FRDGBufferRef FrustumCullVoxelAllocator0 = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("TranslucencyLightingVolume.FrustumCullVoxelAllocator"));
		FRDGBufferRef FrustumCullVoxelAllocator1 = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("TranslucencyLightingVolume.FrustumCullVoxelAllocator"));

		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(FrustumCullVoxelAllocator0), 0);
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(FrustumCullVoxelAllocator1), 0);

		{
			FMatrix TranslatedViewProjMatrix = View.ViewMatrices.GetTranslatedWorldToClip();

			FPlane FrustumPlanes[FFrustumCullVoxelsCS::NumCameraFrustumPlanes];
			TranslatedViewProjMatrix.GetFrustumNearPlane(FrustumPlanes[0]);
			TranslatedViewProjMatrix.GetFrustumLeftPlane(FrustumPlanes[1]);
			TranslatedViewProjMatrix.GetFrustumRightPlane(FrustumPlanes[2]);
			TranslatedViewProjMatrix.GetFrustumTopPlane(FrustumPlanes[3]);
			TranslatedViewProjMatrix.GetFrustumBottomPlane(FrustumPlanes[4]);
			static_assert(FFrustumCullVoxelsCS::NumCameraFrustumPlanes == 5, "Update code above for the new number of frustum planes");

			FFrustumCullVoxelsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFrustumCullVoxelsCS::FParameters>();
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->RWVoxelAllocator0 = GraphBuilder.CreateUAV(FrustumCullVoxelAllocator0);
			PassParameters->RWVoxelAllocator1 = GraphBuilder.CreateUAV(FrustumCullVoxelAllocator1);
			PassParameters->NumVoxels = NumVoxels;

			for (uint32 PlaneIndex = 0; PlaneIndex < FFrustumCullVoxelsCS::NumCameraFrustumPlanes; ++PlaneIndex)
			{
				PassParameters->CameraFrustumPlanes[PlaneIndex] = FVector4f(FrustumPlanes[PlaneIndex].X, FrustumPlanes[PlaneIndex].Y, FrustumPlanes[PlaneIndex].Z, FrustumPlanes[PlaneIndex].W);
			}

			FFrustumCullVoxelsCS::FPermutationDomain PermutationVector;
			auto ComputeShader = View.ShaderMap->GetShader<FFrustumCullVoxelsCS>(PermutationVector);
			const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(TranslucencyLightingVolumeDim, FFrustumCullVoxelsCS::GetGroupSize());

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("FrustumCullVoxelsCS"),
				ComputeShader,
				PassParameters,
				GroupCount);
		}

		{
			FDebugTranslucencyLightingVolumeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDebugTranslucencyLightingVolumeCS::FParameters>();
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->VolumeMarkTexture0 = VolumeMarkData0.MarkTexture ? VolumeMarkData0.MarkTexture : DummyTexture;
			PassParameters->VolumeMarkTexture1 = VolumeMarkData1.MarkTexture ? VolumeMarkData1.MarkTexture : DummyTexture;
			PassParameters->VoxelAllocator0 = GraphBuilder.CreateSRV(VolumeMarkData0.VoxelAllocator ? VolumeMarkData0.VoxelAllocator : DummyAllocatorBuffer);
			PassParameters->VoxelAllocator1 = GraphBuilder.CreateSRV(VolumeMarkData1.VoxelAllocator ? VolumeMarkData1.VoxelAllocator : DummyAllocatorBuffer);
			PassParameters->FrustumCullVoxelAllocator0 = GraphBuilder.CreateSRV(FrustumCullVoxelAllocator0);
			PassParameters->FrustumCullVoxelAllocator1 = GraphBuilder.CreateSRV(FrustumCullVoxelAllocator1);
			PassParameters->NumVoxels = NumVoxels;

			FRDGTextureRef SpecularDirectionTexture0 = Textures.GetSpecularDirectionTexture(View, 0);
			FRDGTextureRef SpecularDirectionTexture1 = Textures.GetSpecularDirectionTexture(View, 1);

			PassParameters->SpecularDirectionTexture0 = OrBlack3DIfNull(GraphBuilder, SpecularDirectionTexture0);
			PassParameters->SpecularDirectionTexture1 = OrBlack3DIfNull(GraphBuilder, SpecularDirectionTexture1);
			PassParameters->DrawSpecularDirection = SpecularDirectionTexture0 != nullptr && SpecularDirectionTexture1 != nullptr;
			PassParameters->ProbeRadiusScale = GTranslucencyLightingVolumeVisualizeRadiusScale;
			PassParameters->VolumeCascadeIndex = FMath::Clamp(GTranslucencyLightingVolumeVisualizeCascadeIndex, -1, TVC_MAX - 1);

			ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintUniformBuffer);

			FDebugTranslucencyLightingVolumeCS::FPermutationDomain PermutationVector;
			auto ComputeShader = View.ShaderMap->GetShader<FDebugTranslucencyLightingVolumeCS>(PermutationVector);
			const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(TranslucencyLightingVolumeDim, FDebugTranslucencyLightingVolumeCS::GetGroupSize());

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DebugTranslucencyLightingVolumeCS"),
				ComputeShader,
				PassParameters,
				GroupCount);
		}
	}
}

static bool IsMegaLightsTranslucencyVolumeEnabled(const FSceneViewFamily& ViewFamily, const FSortedLightSetSceneInfo& SortedLightSet)
{
	return MegaLights::IsEnabled(ViewFamily)
		&& MegaLights::UseTranslucencyVolume()
		&& SortedLightSet.MegaLightsLightStart < SortedLightSet.SortedLights.Num();
}

static bool SetReprojectionParameters(
	const FViewInfo& View,
	int32 VolumeCascadeIndex,
	FFilterTranslucentVolumeCS::FParameters& OutParameters)
{
	const int32 TranslucencyLightingVolumeDim = GetTranslucencyLightingVolumeDim();

	OutParameters.HistoryTextureBilinearUVMin = FVector3f(0.5f / TranslucencyLightingVolumeDim);
	OutParameters.HistoryTextureBilinearUVMax = FVector3f((TranslucencyLightingVolumeDim - 0.5f) / TranslucencyLightingVolumeDim);

	OutParameters.PrevTranslucencyLightingVolumeMin = FVector4f::Zero();
	OutParameters.PrevTranslucencyLightingVolumeInvSize = FVector4f::One();

	if (View.ViewState && !View.bCameraCut && !View.bPrevTransformsReset)
	{
		FTranslucencyLightingViewState& TranslucencyLightingViewState = View.ViewState->TranslucencyLighting;

		const FVector VolumeWorldMin = TranslucencyLightingViewState.HistoryVolumeMin[VolumeCascadeIndex];
		const float VolumeVoxelSize = TranslucencyLightingViewState.HistoryVoxelSize[VolumeCascadeIndex];
		const FVector3f HistoryVolumeSize = FVector3f(TranslucencyLightingViewState.HistoryVolumeSize[VolumeCascadeIndex]);

		const FVector3f VolumeTranslatedWorldMin = FVector3f(VolumeWorldMin + View.ViewMatrices.GetPreViewTranslation());

		OutParameters.PrevTranslucencyLightingVolumeMin = FVector4f(VolumeTranslatedWorldMin, 1.0f / TranslucencyLightingVolumeDim);
		OutParameters.PrevTranslucencyLightingVolumeInvSize = FVector4f(FVector3f(1.0f) / HistoryVolumeSize, VolumeVoxelSize);

		return true;
	}

	return false;
}

void FilterTranslucencyLightingVolume(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> Views,
	FTranslucencyLightingVolumeTextures& Textures,
	const FSortedLightSetSceneInfo& SortedLightSet)
{
	// The TLV is filtered using 2 passes, temporal and spatial filter passes.
	// The temporal pass is also responsible for combining the MegaLights TLV in which case it runs even if the temporal filter is disabled.
	// The approximate temporal direction volume has it's own temporal filter which runs inline in whichever pass runs last.

	const bool bMegaLightsTranslucencyVolume = Views.Num() > 0 && IsMegaLightsTranslucencyVolumeEnabled(*Views[0].Family, SortedLightSet);

	const bool bSpatialFilter = bMegaLightsTranslucencyVolume
		? MegaLights::IsTranslucencyVolumeSpatialFilterEnabled()
		: GTranslucencyLightingVolumeBlur != 0;

	const bool bTemporalFilter = bMegaLightsTranslucencyVolume
		? MegaLights::IsTranslucencyVolumeTemporalFilterEnabled()
		: CVarTranslucencyLightingVolumeTemporal.GetValueOnRenderThread();

	// Temporal pass is also responsible for injecting the MegaLights TLV so it runs even the temporal filter is disabled
	const bool bRunTemporalPass = bTemporalFilter || bMegaLightsTranslucencyVolume;

	if (!GUseTranslucencyLightingVolumes || Views.Num() == 0 || !RHISupportsVolumeTextureRendering(Views[0].GetShaderPlatform()) || (!bSpatialFilter && !bRunTemporalPass))
	{
		// discard history since we are not updating it this frame
		ReleaseTranslucencyLightingVolumeHistory(Views);

		// if filter pass is disabled, then set SpecularDirection to null to avoid RDG validation issues due to the texture not being written to.
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];

			for (int32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; ++VolumeCascadeIndex)
			{
				const uint32 TextureIndex = Textures.GetIndex(View, VolumeCascadeIndex);

				Textures.SpecularDirection[TextureIndex] = nullptr;
			}
		}

		return;
	}

	const float HistoryWeight = FMath::Clamp(CVarTranslucencyLightingVolumeHistoryWeight.GetValueOnRenderThread(), 0.0f, 1.0f);
	const float SpecularDirectionHistoryWeight = FMath::Clamp(CVarTranslucencyLightingVolumeSpecularDirectionHistoryWeight.GetValueOnRenderThread(), 0.0f, 1.0f);

	FRHISamplerState* SamplerStateRHI = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const int32 TranslucencyLightingVolumeDim = GetTranslucencyLightingVolumeDim();
	const FIntVector VolumeSize = FIntVector(TranslucencyLightingVolumeDim);

	EPixelFormat SpecularDirectionPixelFormat;
	ETextureCreateFlags SpecularDirectionTargetFlags;
	FTranslucencyLightingVolumeTextures::GetSpecularDirectionTextureFormatAndCreationFlags(SpecularDirectionPixelFormat, SpecularDirectionTargetFlags);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, TranslucentLighting, "FilterTranslucentVolume %dx%dx%d Cascades:%d", TranslucencyLightingVolumeDim, TranslucencyLightingVolumeDim, TranslucencyLightingVolumeDim, TVC_MAX);

	if (bRunTemporalPass)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View %d", ViewIndex);

			{
				FRDGTextureAccessArray BatchBarrierTextures;

				for (int32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; ++VolumeCascadeIndex)
				{
					const uint32 TextureIndex = Textures.GetIndex(View, VolumeCascadeIndex);

					FRDGTextureRef InputVolumeAmbientTexture = Textures.Ambient[TextureIndex];
					FRDGTextureRef InputVolumeDirectionalTexture = Textures.Directional[TextureIndex];
					if (InputVolumeAmbientTexture && InputVolumeDirectionalTexture)
					{
						BatchBarrierTextures.Emplace(InputVolumeAmbientTexture, ERHIAccess::SRVCompute);
						BatchBarrierTextures.Emplace(InputVolumeDirectionalTexture, ERHIAccess::SRVCompute);
					}

					FRDGTextureRef MegaLightsAmbient = View.GetMegaLightsVolume().TranslucencyAmbient[VolumeCascadeIndex];
					FRDGTextureRef MegaLightsDirectional = View.GetMegaLightsVolume().TranslucencyDirectional[VolumeCascadeIndex];
					if (MegaLightsAmbient && MegaLightsDirectional)
					{
						BatchBarrierTextures.Emplace(MegaLightsAmbient, ERHIAccess::SRVCompute);
						BatchBarrierTextures.Emplace(MegaLightsDirectional, ERHIAccess::SRVCompute);
					}
				}

				AddBarrierPass(GraphBuilder, ERDGPassFlags::Compute | ERDGPassFlags::NeverCull, BatchBarrierTextures);
			}

			for (int32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
			{
				const uint32 TextureIndex = Textures.GetIndex(View, VolumeCascadeIndex);

				const FMegaLightsVolume& MegaLightsVolume = View.GetMegaLightsVolume();
				FRDGTextureRef MegaLightsAmbient = MegaLightsVolume.TranslucencyAmbient[VolumeCascadeIndex];
				FRDGTextureRef MegaLightsDirectional = MegaLightsVolume.TranslucencyDirectional[VolumeCascadeIndex];
				FRDGTextureRef TranslucencyLightingVolumeMarkTexture = View.TranslucencyVolumeMarkData[VolumeCascadeIndex].MarkTexture;
				const bool bUseVolumeMarkTexture = IsTranslucencyLightingVolumeUsingVoxelMarking() && TranslucencyLightingVolumeMarkTexture != nullptr;

				const bool bOutputSpecularDirection = !bSpatialFilter && Textures.SpecularDirection[TextureIndex] != nullptr; // output specular direction during temporal pass if spatial filter is disabled;

				auto* PassParameters = GraphBuilder.AllocParameters<FFilterTranslucentVolumeCS::FParameters>();

				const bool bHasValidReprojection = SetReprojectionParameters(View, VolumeCascadeIndex, *PassParameters);

				bool bHasValidHistory = false;

				FRDGTextureRef HistoryAmbient = nullptr;
				FRDGTextureRef HistoryDirectional = nullptr;
				FRDGTextureRef HistoryMark = nullptr;
				FRDGTextureRef HistorySpecularDirection = nullptr;

				if (bTemporalFilter && bHasValidReprojection)
				{
					FTranslucencyLightingViewState& TranslucencyLightingViewState = View.ViewState->TranslucencyLighting;

					if (TranslucencyLightingViewState.HistoryAmbient[VolumeCascadeIndex] && TranslucencyLightingViewState.HistoryDirectional[VolumeCascadeIndex])
					{
						HistoryAmbient = GraphBuilder.RegisterExternalTexture(TranslucencyLightingViewState.HistoryAmbient[VolumeCascadeIndex]);
						HistoryDirectional = GraphBuilder.RegisterExternalTexture(TranslucencyLightingViewState.HistoryDirectional[VolumeCascadeIndex]);
					}

					if (TranslucencyLightingViewState.HistoryMark[VolumeCascadeIndex])
					{
						HistoryMark = GraphBuilder.RegisterExternalTexture(TranslucencyLightingViewState.HistoryMark[VolumeCascadeIndex]);
					}

					if (TranslucencyLightingViewState.HistorySpecularDirection[VolumeCascadeIndex])
					{
						HistorySpecularDirection = GraphBuilder.RegisterExternalTexture(TranslucencyLightingViewState.HistorySpecularDirection[VolumeCascadeIndex]);
					}

					bHasValidHistory = HistoryAmbient && HistoryDirectional;
				}

				if (bHasValidHistory || bMegaLightsTranslucencyVolume)
				{
					FRDGTextureRef InputVolumeAmbientTexture = Textures.Ambient[TextureIndex];
					FRDGTextureRef InputVolumeDirectionalTexture = Textures.Directional[TextureIndex];

					FRDGTextureRef OutputVolumeAmbientTexture = GraphBuilder.CreateTexture(InputVolumeAmbientTexture->Desc, InputVolumeAmbientTexture->Name);
					FRDGTextureRef OutputVolumeDirectionalTexture = GraphBuilder.CreateTexture(InputVolumeDirectionalTexture->Desc, InputVolumeDirectionalTexture->Name);

					Textures.Ambient[TextureIndex] = OutputVolumeAmbientTexture;
					Textures.Directional[TextureIndex] = OutputVolumeDirectionalTexture;

					FRDGTextureRef OutputSpecularDirectionTexture = Textures.SpecularDirection[TextureIndex];
					if (OutputSpecularDirectionTexture == nullptr)
					{
						OutputSpecularDirectionTexture = GraphBuilder.CreateTexture(
							FRDGTextureDesc::Create3D(FIntVector(1, 1, 1), SpecularDirectionPixelFormat, FClearValueBinding::Transparent, SpecularDirectionTargetFlags),
							TEXT("TranslucentVolumeSpecularDirection.Placeholder"));
					}

					PassParameters->View = View.ViewUniformBuffer;
					PassParameters->VolumeSize = VolumeSize;
					PassParameters->TexelSize = 1.0f / TranslucencyLightingVolumeDim;
					PassParameters->VolumeCascadeIndex = VolumeCascadeIndex;
					PassParameters->TranslucencyLightingVolumeAmbient = InputVolumeAmbientTexture;
					PassParameters->TranslucencyLightingVolumeDirectional = InputVolumeDirectionalTexture;
					PassParameters->TranslucencyLightingVolumeAmbientSampler = SamplerStateRHI;
					PassParameters->TranslucencyLightingVolumeDirectionalSampler = SamplerStateRHI;
					PassParameters->TranslucencyLightingVolumeMarkTexture = TranslucencyLightingVolumeMarkTexture;
					PassParameters->RWTranslucencyLightingVolumeAmbient = GraphBuilder.CreateUAV(OutputVolumeAmbientTexture);
					PassParameters->RWTranslucencyLightingVolumeDirectional = GraphBuilder.CreateUAV(OutputVolumeDirectionalTexture);
					PassParameters->RWTranslucencySpecularDirection = GraphBuilder.CreateUAV(OutputSpecularDirectionTexture);

					PassParameters->HistoryAmbient = HistoryAmbient;
					PassParameters->HistoryDirectional = HistoryDirectional;
					PassParameters->HistoryAmbientSampler = SamplerStateRHI;
					PassParameters->HistoryDirectionalSampler = SamplerStateRHI;

					PassParameters->HistorySpecularDirection = OrBlack3DIfNull(GraphBuilder, HistorySpecularDirection);
					PassParameters->HistoryMark = OrBlack3DIfNull(GraphBuilder, HistoryMark);

					PassParameters->HasSpecularDirectionHistory = HistorySpecularDirection != nullptr ? 1 : 0;
					PassParameters->HasMarkHistory = HistoryMark != nullptr ? 1 : 0;

					PassParameters->MegaLightsAmbient = MegaLightsAmbient ? GraphBuilder.CreateSRV(MegaLightsAmbient) : nullptr;
					PassParameters->MegaLightsDirectional = MegaLightsDirectional ? GraphBuilder.CreateSRV(MegaLightsDirectional) : nullptr;

					PassParameters->HistoryWeight = HistoryWeight;
					PassParameters->SpecularDirectionHistoryWeight = SpecularDirectionHistoryWeight;

					PassParameters->OutputSpecularDirection = bOutputSpecularDirection ? 1 : 0;

					FFilterTranslucentVolumeCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FFilterTranslucentVolumeCS::FUseTemporalFilter>(bHasValidHistory && bTemporalFilter);
					PermutationVector.Set<FFilterTranslucentVolumeCS::FUseSpatialFilter>(false);
					PermutationVector.Set<FFilterTranslucentVolumeCS::FUseMarkTexture>(bUseVolumeMarkTexture);
					PermutationVector.Set<FFilterTranslucentVolumeCS::FInjectMegaLights>(bMegaLightsTranslucencyVolume);
					PermutationVector = FFilterTranslucentVolumeCS::RemapPermutation(PermutationVector);

					auto ComputeShader = View.ShaderMap->GetShader<FFilterTranslucentVolumeCS>(PermutationVector);

					FIntVector NumGroups = FComputeShaderUtils::GetGroupCount(VolumeSize, FFilterTranslucentVolumeCS::GetGroupSize());

					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("Temporal Filter Cascade:%d MegaLights:%d", VolumeCascadeIndex, bMegaLightsTranslucencyVolume),
						ComputeShader,
						PassParameters,
						NumGroups);
				}

				if (View.ViewState && !View.bStatePrevViewInfoIsReadOnly && bTemporalFilter)
				{
					FTranslucencyLightingViewState& TranslucencyLightingViewState = View.ViewState->TranslucencyLighting;

					GraphBuilder.QueueTextureExtraction(Textures.Ambient[TextureIndex], &TranslucencyLightingViewState.HistoryAmbient[VolumeCascadeIndex]);
					GraphBuilder.QueueTextureExtraction(Textures.Directional[TextureIndex], &TranslucencyLightingViewState.HistoryDirectional[VolumeCascadeIndex]);
				}
			}
		}
	}

	if (!bTemporalFilter)
	{
		// release history resources if temporal is not enabled
		ReleaseTranslucencyLightingVolumeHistory(Views);
	}

	if (bSpatialFilter)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View %d", ViewIndex);

			{
				FRDGTextureAccessArray BatchBarrierTextures;

				for (int32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; ++VolumeCascadeIndex)
				{
					const uint32 TextureIndex = Textures.GetIndex(View, VolumeCascadeIndex);
					FRDGTextureRef InputVolumeAmbientTexture = Textures.Ambient[TextureIndex];
					FRDGTextureRef InputVolumeDirectionalTexture = Textures.Directional[TextureIndex];

					BatchBarrierTextures.Emplace(InputVolumeAmbientTexture, ERHIAccess::SRVCompute);
					BatchBarrierTextures.Emplace(InputVolumeDirectionalTexture, ERHIAccess::SRVCompute);
				}

				AddBarrierPass(GraphBuilder, ERDGPassFlags::Compute, BatchBarrierTextures);
			}

			for (int32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
			{
				const uint32 TextureIndex = Textures.GetIndex(View, VolumeCascadeIndex);

				FRDGTextureRef InputVolumeAmbientTexture = Textures.Ambient[TextureIndex];
				FRDGTextureRef InputVolumeDirectionalTexture = Textures.Directional[TextureIndex];

				FRDGTextureRef OutputVolumeAmbientTexture = GraphBuilder.CreateTexture(InputVolumeAmbientTexture->Desc, InputVolumeAmbientTexture->Name);
				FRDGTextureRef OutputVolumeDirectionalTexture = GraphBuilder.CreateTexture(InputVolumeDirectionalTexture->Desc, InputVolumeDirectionalTexture->Name);

				Textures.Ambient[TextureIndex] = OutputVolumeAmbientTexture;
				Textures.Directional[TextureIndex] = OutputVolumeDirectionalTexture;

				FRDGTextureRef OutputSpecularDirectionTexture = Textures.SpecularDirection[TextureIndex];
				if (OutputSpecularDirectionTexture == nullptr)
				{
					OutputSpecularDirectionTexture = GraphBuilder.CreateTexture(
						FRDGTextureDesc::Create3D(FIntVector(1, 1, 1), SpecularDirectionPixelFormat, FClearValueBinding::Transparent, SpecularDirectionTargetFlags),
						TEXT("TranslucentVolumeSpecularDirection.Placeholder"));
				}

				FRDGTextureRef TranslucencyLightingVolumeMarkTexture = View.TranslucencyVolumeMarkData[VolumeCascadeIndex].MarkTexture;
				const bool bUseVolumeMarkTexture = IsTranslucencyLightingVolumeUsingVoxelMarking() && TranslucencyLightingVolumeMarkTexture != nullptr;

				auto* PassParameters = GraphBuilder.AllocParameters<FFilterTranslucentVolumeCS::FParameters>();
				PassParameters->View = View.ViewUniformBuffer;
				PassParameters->VolumeSize = VolumeSize;
				PassParameters->TexelSize = 1.0f / TranslucencyLightingVolumeDim;
				PassParameters->VolumeCascadeIndex = VolumeCascadeIndex;
				PassParameters->TranslucencyLightingVolumeAmbient = InputVolumeAmbientTexture;
				PassParameters->TranslucencyLightingVolumeDirectional = InputVolumeDirectionalTexture;
				PassParameters->TranslucencyLightingVolumeAmbientSampler = SamplerStateRHI;
				PassParameters->TranslucencyLightingVolumeDirectionalSampler = SamplerStateRHI;
				PassParameters->TranslucencyLightingVolumeMarkTexture = TranslucencyLightingVolumeMarkTexture;
				PassParameters->RWTranslucencyLightingVolumeAmbient = GraphBuilder.CreateUAV(OutputVolumeAmbientTexture);
				PassParameters->RWTranslucencyLightingVolumeDirectional = GraphBuilder.CreateUAV(OutputVolumeDirectionalTexture);
				PassParameters->RWTranslucencySpecularDirection = GraphBuilder.CreateUAV(OutputSpecularDirectionTexture);

				// Set reprojection data for specular direction temporal filtering
				const bool bHasValidReprojection = SetReprojectionParameters(View, VolumeCascadeIndex, *PassParameters);

				FRDGTextureRef HistoryMark = nullptr;
				FRDGTextureRef HistorySpecularDirection = nullptr;

				if (bTemporalFilter && bHasValidReprojection)
				{
					FTranslucencyLightingViewState& TranslucencyLightingViewState = View.ViewState->TranslucencyLighting;

					if (TranslucencyLightingViewState.HistoryMark[VolumeCascadeIndex])
					{
						HistoryMark = GraphBuilder.RegisterExternalTexture(TranslucencyLightingViewState.HistoryMark[VolumeCascadeIndex]);
					}

					if (TranslucencyLightingViewState.HistorySpecularDirection[VolumeCascadeIndex])
					{
						HistorySpecularDirection = GraphBuilder.RegisterExternalTexture(TranslucencyLightingViewState.HistorySpecularDirection[VolumeCascadeIndex]);
					}
				}

				PassParameters->HistorySpecularDirection = OrBlack3DIfNull(GraphBuilder, HistorySpecularDirection);
				PassParameters->HistoryMark = OrBlack3DIfNull(GraphBuilder, HistoryMark);

				PassParameters->HasSpecularDirectionHistory = HistorySpecularDirection != nullptr ? 1 : 0;
				PassParameters->HasMarkHistory = HistoryMark != nullptr ? 1 : 0;

				PassParameters->HistoryWeight = HistoryWeight;
				PassParameters->SpecularDirectionHistoryWeight = SpecularDirectionHistoryWeight;

				PassParameters->OutputSpecularDirection = Textures.SpecularDirection[TextureIndex] != nullptr ? 1 : 0;

				FFilterTranslucentVolumeCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FFilterTranslucentVolumeCS::FUseTemporalFilter>(false);
				PermutationVector.Set<FFilterTranslucentVolumeCS::FUseSpatialFilter>(true);
				PermutationVector.Set<FFilterTranslucentVolumeCS::FUseMarkTexture>(bUseVolumeMarkTexture);
				PermutationVector.Set<FFilterTranslucentVolumeCS::FInjectMegaLights>(false);
				PermutationVector = FFilterTranslucentVolumeCS::RemapPermutation(PermutationVector);

				auto ComputeShader = View.ShaderMap->GetShader<FFilterTranslucentVolumeCS>(PermutationVector);

				FIntVector NumGroups = FComputeShaderUtils::GetGroupCount(VolumeSize, FFilterTranslucentVolumeCS::GetGroupSize());

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("Spatial Filter Cascade:%d", VolumeCascadeIndex),
					ComputeShader,
					PassParameters,
					NumGroups);
			}
		}
	}

	if (bTemporalFilter)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];

			if (View.ViewState && !View.bStatePrevViewInfoIsReadOnly)
			{
				for (int32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
				{
					const uint32 TextureIndex = Textures.GetIndex(View, VolumeCascadeIndex);

					FTranslucencyLightingViewState& TranslucencyLightingViewState = View.ViewState->TranslucencyLighting;

					if (Textures.SpecularDirection[TextureIndex])
					{
						GraphBuilder.QueueTextureExtraction(Textures.SpecularDirection[TextureIndex], &TranslucencyLightingViewState.HistorySpecularDirection[VolumeCascadeIndex]);
					}
					else
					{
						TranslucencyLightingViewState.HistorySpecularDirection[VolumeCascadeIndex] = nullptr;
					}

					if (View.TranslucencyVolumeMarkData[VolumeCascadeIndex].MarkTexture)
					{
						GraphBuilder.QueueTextureExtraction(View.TranslucencyVolumeMarkData[VolumeCascadeIndex].MarkTexture, &TranslucencyLightingViewState.HistoryMark[VolumeCascadeIndex]);
					}
					else
					{
						TranslucencyLightingViewState.HistoryMark[VolumeCascadeIndex] = nullptr;
					}
				}
			}
		}
	}

	// update HistoryVolumeMin/Size regardless of temporal filter being enabled or not so r.TranslucencyLightingVolume.Debug.FreezePosition always works
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (View.ViewState && !View.bStatePrevViewInfoIsReadOnly)
		{
			for (int32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
			{
				FTranslucencyLightingViewState& TranslucencyLightingViewState = View.ViewState->TranslucencyLighting;
				TranslucencyLightingViewState.HistoryVolumeMin[VolumeCascadeIndex] = View.TranslucencyLightingVolumeMin[VolumeCascadeIndex];
				TranslucencyLightingViewState.HistoryVoxelSize[VolumeCascadeIndex] = View.TranslucencyVolumeVoxelSize[VolumeCascadeIndex];
				TranslucencyLightingViewState.HistoryVolumeSize[VolumeCascadeIndex] = View.TranslucencyLightingVolumeSize[VolumeCascadeIndex];
			}
		}
	}
}

static void SplitSimpleLightsByView(TConstArrayView<FViewInfo> Views, const FSortedLightSetSceneInfo& SortedLightSet, TArrayView<FSimpleLightArray> OutSimpleLightsByView, bool bMegaLightsTranslucencyVolume)
{
	check(OutSimpleLightsByView.Num() == Views.Num());

	const FSimpleLightArray& SimpleLights = SortedLightSet.SimpleLights;
	const TConstArrayView<FSortedLightSceneInfo> SortedLights = SortedLightSet.SortedLights;

	for (int32 SortedIndex = 0; SortedIndex < SortedLightSet.SimpleLightsEnd; ++SortedIndex)
	{
		const FSortedLightSceneInfo& SortedLightInfo = SortedLights[SortedIndex];
		const int32 SimpleLightsIndex = SortedLightInfo.SimpleLightIndex;

		check(!SortedLightInfo.SortKey.Fields.bIsNotSimpleLight && !SortedLightInfo.SortKey.Fields.bHandledByMegaLights && !SortedLightInfo.LightSceneInfo);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const FSimpleLightEntry& InstanceEntry = SimpleLights.InstanceData[SimpleLightsIndex];
			const FSimpleLightPerViewEntry& PerViewEntry = SimpleLights.GetViewDependentData(SimpleLightsIndex, ViewIndex, Views.Num());

			OutSimpleLightsByView[ViewIndex].InstanceData.Add(InstanceEntry);
			OutSimpleLightsByView[ViewIndex].PerViewData.Add(PerViewEntry);
		}
	}

	// Inject via the regular path if MegaLights is not enabled for TLV
	if (!bMegaLightsTranslucencyVolume)
	{
		for (int32 SortedIndex = SortedLightSet.MegaLightsLightStart; SortedIndex < SortedLightSet.MegaLightsSimpleLightsEnd; ++SortedIndex)
		{
			const FSortedLightSceneInfo& SortedLightInfo = SortedLights[SortedIndex];
			const int32 SimpleLightsIndex = SortedLightInfo.SimpleLightIndex;

			if (!SortedLightInfo.SortKey.Fields.bIsNotSimpleLight)
			{
				check(SortedLightInfo.SortKey.Fields.bHandledByMegaLights && !SortedLightInfo.LightSceneInfo);

				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
				{
					const FSimpleLightEntry& InstanceEntry = SimpleLights.InstanceData[SimpleLightsIndex];
					const FSimpleLightPerViewEntry& PerViewEntry = SimpleLights.GetViewDependentData(SimpleLightsIndex, ViewIndex, Views.Num());

					OutSimpleLightsByView[ViewIndex].InstanceData.Add(InstanceEntry);
					OutSimpleLightsByView[ViewIndex].PerViewData.Add(PerViewEntry);
				}
			}
		}
	}
}

void FDeferredShadingSceneRenderer::GatherTranslucencyVolumeMarkedVoxels(FRDGBuilder& GraphBuilder, const FSortedLightSetSceneInfo& SortedLightSet)
{
	if (!IsTranslucencyLightingVolumeUsingVoxelMarking() || !ViewFamily.EngineShowFlags.DirectLighting || !RHISupportsVolumeTextureRendering(ViewFamily.GetShaderPlatform()))
	{
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "TranslucencyLightingVolumeGatherMarkedVoxels");

	const FIntVector VolumeSize = FIntVector(GetTranslucencyLightingVolumeDim());

	const bool bMegaLightsTranslucencyVolume = Views.Num() > 0 && IsMegaLightsTranslucencyVolumeEnabled(*Views[0].Family, SortedLightSet);

	const bool bSpatialFilter = bMegaLightsTranslucencyVolume
		? MegaLights::IsTranslucencyVolumeSpatialFilterEnabled()
		: GTranslucencyLightingVolumeBlur != 0;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];

		struct FGatherMarkDataResources
		{
			FRDGTextureRef DilatedMarkTexture = nullptr;
			FRDGTextureUAVRef DilatedMarkTextureUAV = nullptr;
		};

		TStaticArray<FGatherMarkDataResources, TVC_MAX> GatherMarkDataResources;

		// Allocate and clear UAVs
		for (uint32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
		{
			FViewInfo::FTranslucencyVolumeMarkData& VolumeMarkData = View.TranslucencyVolumeMarkData[VolumeCascadeIndex];

			const bool bUseVolumeMarkTexture = VolumeMarkData.MarkTexture != nullptr;

			if (VolumeMarkData.MarkTexture == nullptr)
			{
				continue;
			}

			if (bSpatialFilter)
			{
				GatherMarkDataResources[VolumeCascadeIndex].DilatedMarkTexture = GraphBuilder.CreateTexture(View.TranslucencyVolumeMarkData[VolumeCascadeIndex].MarkTexture->Desc, TEXT("TranslucencyLightVolume.DilatedMarkTexture"));

				GatherMarkDataResources[VolumeCascadeIndex].DilatedMarkTextureUAV = GraphBuilder.CreateUAV(GatherMarkDataResources[VolumeCascadeIndex].DilatedMarkTexture);

				AddClearUAVPass(GraphBuilder, GatherMarkDataResources[VolumeCascadeIndex].DilatedMarkTextureUAV, 0u);
			}

			VolumeMarkData.VoxelAllocator = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("TranslucencyLightingVolume.VoxelAllocator"));
			VolumeMarkData.VoxelData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), VolumeSize.X * VolumeSize.Y * VolumeSize.Z), TEXT("TranslucencyLightingVolume.VoxelData"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(VolumeMarkData.VoxelAllocator), 0);

			VolumeMarkData.VoxelIndirectArgs = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("TranslucencyLightingVolume.VoxelIndirectArgs"));
		}

		// Dilate marked voxels when using spatial blur
		if (bSpatialFilter)
		{
			for (uint32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
			{
				FViewInfo::FTranslucencyVolumeMarkData& VolumeMarkData = View.TranslucencyVolumeMarkData[VolumeCascadeIndex];

				const bool bUseVolumeMarkTexture = VolumeMarkData.MarkTexture != nullptr;

				if (VolumeMarkData.MarkTexture == nullptr)
				{
					continue;
				}

				{
					auto* PassParameters = GraphBuilder.AllocParameters<FDilateMarkedVoxelsCS::FParameters>();
					PassParameters->RWVolumeMarkTexture = GatherMarkDataResources[VolumeCascadeIndex].DilatedMarkTextureUAV;
					PassParameters->VolumeMarkTexture = View.TranslucencyVolumeMarkData[VolumeCascadeIndex].MarkTexture;
					PassParameters->VolumeSize = VolumeSize;

					FDilateMarkedVoxelsCS::FPermutationDomain PermutationVector;

					auto ComputeShader = View.ShaderMap->GetShader<FDilateMarkedVoxelsCS>(PermutationVector);

					FIntVector NumGroups = FComputeShaderUtils::GetGroupCount(VolumeSize, FDilateMarkedVoxelsCS::GetGroupSize());

					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("DilateMarkedVoxels(VolumeCascade=%d)"),
						ComputeShader,
						PassParameters,
						NumGroups);

					View.TranslucencyVolumeMarkData[VolumeCascadeIndex].MarkTexture = GatherMarkDataResources[VolumeCascadeIndex].DilatedMarkTexture;
				}
			}
		}

		// Gather marked voxels
		for (uint32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
		{
			FViewInfo::FTranslucencyVolumeMarkData& VolumeMarkData = View.TranslucencyVolumeMarkData[VolumeCascadeIndex];

			const bool bUseVolumeMarkTexture = VolumeMarkData.MarkTexture != nullptr;

			if (VolumeMarkData.MarkTexture == nullptr)
			{
				continue;
			}

			{
				auto* PassParameters = GraphBuilder.AllocParameters<FGatherMarkedVoxelsCS::FParameters>();
				PassParameters->RWVoxelAllocator = GraphBuilder.CreateUAV(VolumeMarkData.VoxelAllocator);
				PassParameters->RWVoxelData = GraphBuilder.CreateUAV(VolumeMarkData.VoxelData);
				PassParameters->VolumeMarkTexture = View.TranslucencyVolumeMarkData[VolumeCascadeIndex].MarkTexture;
				PassParameters->VolumeSize = VolumeSize;

				FGatherMarkedVoxelsCS::FPermutationDomain PermutationVector;

				auto ComputeShader = View.ShaderMap->GetShader<FGatherMarkedVoxelsCS>(PermutationVector);

				FIntVector NumGroups = FComputeShaderUtils::GetGroupCount(VolumeSize, FGatherMarkedVoxelsCS::GetGroupSize());

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("GatherMarkedVoxels(VolumeCascade=%d)"),
					ComputeShader,
					PassParameters,
					NumGroups);
			}
		}

		// Setup indirect args
		for (uint32 VolumeCascadeIndex = 0; VolumeCascadeIndex < TVC_MAX; VolumeCascadeIndex++)
		{
			FViewInfo::FTranslucencyVolumeMarkData& VolumeMarkData = View.TranslucencyVolumeMarkData[VolumeCascadeIndex];

			const bool bUseVolumeMarkTexture = VolumeMarkData.MarkTexture != nullptr;

			if (VolumeMarkData.MarkTexture == nullptr)
			{
				continue;
			}
			
			{
				auto* PassParameters = GraphBuilder.AllocParameters<FInitIndirectArgsCS::FParameters>();
				PassParameters->RWIndirectArgs = GraphBuilder.CreateUAV(VolumeMarkData.VoxelIndirectArgs);
				PassParameters->VoxelAllocator = GraphBuilder.CreateSRV(VolumeMarkData.VoxelAllocator);

				FInitIndirectArgsCS::FPermutationDomain PermutationVector;

				auto ComputeShader = View.ShaderMap->GetShader<FInitIndirectArgsCS>(PermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("InitIndirectArgs(VolumeCascade=%d)"),
					ComputeShader,
					PassParameters,
					FIntVector(1, 1, 1));
			}
		}
	}
}

void FDeferredShadingSceneRenderer::RenderTranslucencyLightingVolume(FRDGBuilder& GraphBuilder, FTranslucencyLightingVolumeTextures& Textures, const FSortedLightSetSceneInfo& SortedLightSet)
{
	if (!GUseTranslucencyLightingVolumes || !ViewFamily.EngineShowFlags.DirectLighting || !RHISupportsVolumeTextureRendering(ViewFamily.GetShaderPlatform()))
	{
		ReleaseTranslucencyLightingVolumeHistory(Views);
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "TranslucencyLightingVolume");
	SCOPE_CYCLE_COUNTER(STAT_TranslucentInjectTime);

	const TArray<FSortedLightSceneInfo, SceneRenderingAllocator>& SortedLights = SortedLightSet.SortedLights;
	const FSimpleLightArray& SimpleLights = SortedLightSet.SimpleLights;
	const int32 SimpleLightsEnd = SortedLightSet.SimpleLightsEnd;

	const bool bMegaLightsTranslucencyVolume = MegaLights::IsEnabled(ViewFamily) && MegaLights::UseTranslucencyVolume();

	if (SimpleLights.InstanceData.Num() > 0)
	{
		auto& SimpleLightsByView = *GraphBuilder.AllocObject<TArray<FSimpleLightArray, SceneRenderingAllocator>>();
		SimpleLightsByView.SetNum(Views.Num());

		SplitSimpleLightsByView(Views, SortedLightSet, SimpleLightsByView, bMegaLightsTranslucencyVolume);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			FSimpleLightArray& SimpleLightArray = SimpleLightsByView[ViewIndex];

			if (SimpleLightArray.InstanceData.Num() > 0)
			{
				const FViewInfo& View = Views[ViewIndex];
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
				RDG_EVENT_SCOPE(GraphBuilder, "InjectSimpleLightsTranslucentLighting");
				InjectSimpleTranslucencyLightingVolumeArray(GraphBuilder, View, ViewIndex, Views.Num(), Textures, SimpleLightArray);
			}
		}
	}

	// Shadowed and light function lights
	{
		FTranslucentLightInjectionCollector Collector(GraphBuilder, Views, bAreLightsInLightGrid);

		// Collect all the light injection data
		for (int32 LightIndex = SimpleLightsEnd; LightIndex < SortedLights.Num(); LightIndex++)
		{
			const FSortedLightSceneInfo& SortedLightInfo = SortedLights[LightIndex];

			if ((bMegaLightsTranslucencyVolume && SortedLightInfo.SortKey.Fields.bHandledByMegaLights)
				// This is a simple light handled by MegaLights. If MegaLights is enabled for TLV, injection is handled later by
				// InjectTranslucencyLightingVolumeMegaLights(). Otherwise, it will be injected above along with regular simple lights.
				|| !SortedLightInfo.SortKey.Fields.bIsNotSimpleLight)
			{
				continue;
			}

			const FLightSceneInfo& LightSceneInfo = *SortedLightInfo.LightSceneInfo;
			const FLightSceneProxy& LightSceneProxy = *LightSceneInfo.Proxy;

			const bool bDrawShadows = SortedLightInfo.SortKey.Fields.bShadowed;
			const ELightOcclusionType OcclusionType = GetLightOcclusionType(LightSceneProxy, ViewFamily);
			const bool bSupportShadowMaps = bDrawShadows && OcclusionType == ELightOcclusionType::Shadowmap;

			// Collect all the light injection data
			CollectLightForTranslucencyLightingVolumeInjection(&LightSceneInfo, bSupportShadowMaps, Collector);
		}

		// Run light injection for each view
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE(GraphBuilder, "InjectTranslucencyLightingVolume(View=%d)", ViewIndex);
			InjectTranslucencyLightingVolume(GraphBuilder, View, ViewIndex, Scene, *this, Collector, Textures);
		}
	}

	InjectTranslucencyLightingVolumeAmbientCubemap(GraphBuilder, Views, Textures);
	FilterTranslucencyLightingVolume(GraphBuilder, Views, Textures, SortedLightSet);
	DrawDebugTranslucencyLightingVolume(GraphBuilder, Views, Textures);
}

BEGIN_SHADER_PARAMETER_STRUCT(FVisualizeTranslucencyVolumeCommonParameters, )
	SHADER_PARAMETER(float, ProbeRadiusScale)
	SHADER_PARAMETER(uint32, VolumeCascadeIndex)
	SHADER_PARAMETER(uint32, VolumeSize)
END_SHADER_PARAMETER_STRUCT()

class FVisualizeTranslucencyVolumeVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeTranslucencyVolumeVS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeTranslucencyVolumeVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVisualizeTranslucencyVolumeCommonParameters, VisualizeCommonParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeTranslucencyVolumeVS, "/Engine/Private/TranslucencyVolumeVisualize.usf", "VisualizeTranslucencyVolumeVS", SF_Vertex);

class FVisualizeTranslucencyVolumePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeTranslucencyVolumePS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeTranslucencyVolumePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FTranslucencyLightingVolumeParameters, TranslucencyLightingVolume)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVisualizeTranslucencyVolumeCommonParameters, VisualizeCommonParameters)

		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeAmbient)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeDirectional)
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeTranslucencyVolumePS, "/Engine/Private/TranslucencyVolumeVisualize.usf", "VisualizeTranslucencyVolumePS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FVisualizeTranslucencyVolumeParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FVisualizeTranslucencyVolumeVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVisualizeTranslucencyVolumePS::FParameters, PS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FDeferredShadingSceneRenderer::RenderTranslucencyVolumeVisualization(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FTranslucencyLightingVolumeTextures& TranslucencyLightingVolumeTextures)
{
	if (!GUseTranslucencyLightingVolumes
		|| !GTranslucencyLightingVolumeVisualize
		|| !ViewFamily.EngineShowFlags.DirectLighting
		|| !RHISupportsVolumeTextureRendering(ViewFamily.GetShaderPlatform())
		|| !TranslucencyLightingVolumeTextures.IsValid())
	{
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeTranslucencyVolume");

	const int32 VolumeDim = GetTranslucencyLightingVolumeDim();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		FRDGTextureRef SceneColor = SceneTextures.Color.Resolve;
		FRDGTextureRef SceneDepth = SceneTextures.Depth.Resolve;

		const int32 VisualizationCascadeIndex = FMath::Clamp(GTranslucencyLightingVolumeVisualizeCascadeIndex, -1, TVC_MAX - 1);
		for (int32 CascadeIndex = 0; CascadeIndex < TVC_MAX; ++CascadeIndex)
		{
			if (VisualizationCascadeIndex != -1 && VisualizationCascadeIndex != CascadeIndex)
			{
				continue;
			}

			FVisualizeTranslucencyVolumeCommonParameters VisualizeCommonParameters;
			VisualizeCommonParameters.ProbeRadiusScale = GTranslucencyLightingVolumeVisualizeRadiusScale;
			VisualizeCommonParameters.VolumeCascadeIndex = CascadeIndex;
			VisualizeCommonParameters.VolumeSize = VolumeDim;

			FVisualizeTranslucencyVolumeParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeTranslucencyVolumeParameters>();
			PassParameters->VS.VisualizeCommonParameters = VisualizeCommonParameters;
			PassParameters->PS.VisualizeCommonParameters = VisualizeCommonParameters;
			PassParameters->VS.View = GetShaderBinding(View.ViewUniformBuffer);
			PassParameters->PS.View = GetShaderBinding(View.ViewUniformBuffer);
			PassParameters->PS.TranslucencyLightingVolume = GetTranslucencyLightingVolumeParameters(GraphBuilder, TranslucencyLightingVolumeTextures, View);
			PassParameters->PS.TranslucencyLightingVolumeAmbient = TranslucencyLightingVolumeTextures.GetAmbientTexture(View, CascadeIndex);
			PassParameters->PS.TranslucencyLightingVolumeDirectional = TranslucencyLightingVolumeTextures.GetDirectionalTexture(View, CascadeIndex);

			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
				SceneDepth,
				ERenderTargetLoadAction::ENoAction,
				ERenderTargetLoadAction::ELoad,
				FExclusiveDepthStencil::DepthWrite_StencilWrite);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColor, ERenderTargetLoadAction::ELoad);

			const int32 NumInstancesPerCascade = VolumeDim * VolumeDim * VolumeDim;

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("Visualize Translucency Volume Cascade:%d", CascadeIndex),
				PassParameters,
				ERDGPassFlags::Raster,
				[PassParameters, &View, NumInstancesPerCascade](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					auto VertexShader = View.ShaderMap->GetShader<FVisualizeTranslucencyVolumeVS>();
					auto PixelShader = View.ShaderMap->GetShader<FVisualizeTranslucencyVolumePS>();

					RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_DepthNear>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

					RHICmdList.SetStreamSource(0, NULL, 0);
					RHICmdList.DrawIndexedPrimitive(GCubeIndexBuffer.IndexBufferRHI, 0, 0, 8, 0, 2 * 6, NumInstancesPerCascade);
				});
		}
	}
}


class FTranslucentLightingMaterialPSOCollector : public IPSOCollector
{
public:
	FTranslucentLightingMaterialPSOCollector(ERHIFeatureLevel::Type InFeatureLevel) : 
		IPSOCollector(FPSOCollectorCreateManager::GetIndex(GetFeatureLevelShadingPath(InFeatureLevel), TranslucentLightingMaterialPSOCollectorName)),
	FeatureLevel(InFeatureLevel)
	{
	}

	virtual void CollectPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams,
		TArray<FPSOPrecacheData>& PSOInitializers
	) override final;

private:

	ERHIFeatureLevel::Type FeatureLevel;
};

void FTranslucentLightingMaterialPSOCollector::CollectPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FPSOPrecacheParams& PreCacheParams,
	TArray<FPSOPrecacheData>& PSOInitializers
)
{
	if (!Material.IsLightFunction() || !UPSOPrecacheSettingsManager::GetSettings().bTranslucencyLightingVolume)
	{
		return;
	}

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);
	TShaderMapRef<FWriteToSliceVS> VertexShader(GlobalShaderMap);
	TOptionalShaderMapRef<FWriteToSliceGS> GeometryShader(GlobalShaderMap);
	const FMaterialShaderMap* MaterialShaderMap = Material.GetGameThreadShaderMap();
	
	EPixelFormat TranslucencyPixelFormat;
	ETextureCreateFlags TranslucencyTargetFlags;
	FTranslucencyLightingVolumeTextures::GetTextureFormatAndCreationFlags(TranslucencyPixelFormat, TranslucencyTargetFlags);

	auto AddPSOInitializer = [&](bool bDirectionalLight, bool bDynamicShadow, bool bApplyLightFunction, bool bUseVSM, bool bUseAdaptiveVolumetricShadowMap)
		{
			FTranslucentLightingInjectPS::FPermutationDomain PermutationVector;
			PermutationVector.Set< FTranslucentLightingInjectPS::FRadialAttenuation >(!bDirectionalLight);
			PermutationVector.Set< FTranslucentLightingInjectPS::FDynamicallyShadowed >(bDynamicShadow);
			PermutationVector.Set< FTranslucentLightingInjectPS::FLightFunction >(bApplyLightFunction);
			PermutationVector.Set< FTranslucentLightingInjectPS::FVirtualShadowMap >(bUseVSM);
			PermutationVector.Set< FTranslucentLightingInjectPS::FAdaptiveVolumetricShadowMap >(bUseAdaptiveVolumetricShadowMap);

			auto PixelShader = MaterialShaderMap->GetShader< FTranslucentLightingInjectPS >(PermutationVector);
			
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			if (!SetupPSOStateForVolumeInjection(VertexShader, GeometryShader, PixelShader.GetPixelShader(false), bDirectionalLight, GraphicsPSOInit, false))
			{
				return;
			}
			
			// What render target formats to support?
			FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
			RenderTargetsInfo.NumSamples = 1;
			AddRenderTargetInfo(TranslucencyPixelFormat, TranslucencyTargetFlags, RenderTargetsInfo);
			AddRenderTargetInfo(TranslucencyPixelFormat, TranslucencyTargetFlags, RenderTargetsInfo);

			GraphicsPSOInit.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(GraphicsPSOInit);
			ApplyTargetsInfo(GraphicsPSOInit, RenderTargetsInfo);

			FPSOPrecacheData PSOPrecacheData;
			PSOPrecacheData.bRequired = true;
			PSOPrecacheData.Type = FPSOPrecacheData::EType::Graphics;
			PSOPrecacheData.GraphicsPSOInitializer = GraphicsPSOInit;
#if PSO_PRECACHING_VALIDATE
			PSOPrecacheData.ResourceName = Material.GetAssetName();
			PSOPrecacheData.PSOCollectorIndex = PSOCollectorIndex;
			PSOPrecacheData.VertexFactoryType = nullptr;
#endif // PSO_PRECACHING_VALIDATE

			PSOInitializers.Add(MoveTemp(PSOPrecacheData));
		};
	
	// Generate PSOs for all possible permutations - we don't have that many light function materials
	for (int32 DirectionalLightIndex = 0; DirectionalLightIndex < 2; ++DirectionalLightIndex)
	{
		bool bDirectionalLight = DirectionalLightIndex > 0;
		for (int32 DynamicShadowIndex = 0; DynamicShadowIndex < 2; ++DynamicShadowIndex)
		{
			bool bDynamicShadow = DynamicShadowIndex > 0;
			for (int32 ApplyLightFunctionIndex = 0; ApplyLightFunctionIndex < 2; ++ApplyLightFunctionIndex)
			{
				bool bApplyLightFunction = ApplyLightFunctionIndex > 0;
				for (int32 UseVSMIndex = 0; UseVSMIndex < 2; ++UseVSMIndex)
				{
					bool bUseVSM = UseVSMIndex > 0;
					for (int32 UseAdaptiveVolumetricShadowMapIndex = 0; UseAdaptiveVolumetricShadowMapIndex < 2; ++UseAdaptiveVolumetricShadowMapIndex)
					{
						bool bUseAdaptiveVolumetricShadowMap = UseAdaptiveVolumetricShadowMapIndex > 0;
						AddPSOInitializer(bDirectionalLight, bDynamicShadow, bApplyLightFunction, bUseVSM, bUseAdaptiveVolumetricShadowMap);
					}
				}
			}
		}
	}
}

IPSOCollector* CreateTranslucentLightingMaterialPSOCollector(ERHIFeatureLevel::Type FeatureLevel)
{
	return new FTranslucentLightingMaterialPSOCollector(FeatureLevel);
}
FRegisterPSOCollectorCreateFunction RegisterTranslucentLightingMaterialPSOCollector(&CreateTranslucentLightingMaterialPSOCollector, EShadingPath::Deferred, TranslucentLightingMaterialPSOCollectorName);

FAutoConsoleVariableDeprecated CVarTranslucentLightingVolumes(TEXT("r.TranslucentLightingVolume"), TEXT("r.TranslucencyLightingVolume"), TEXT("5.7"));
FAutoConsoleVariableDeprecated CVarTranslucentVolumeMinFOV(TEXT("r.TranslucentVolumeMinFOV"), TEXT("r.TranslucencyLightingVolume.MinFOV"), TEXT("5.7"));
FAutoConsoleVariableDeprecated CVarTranslucentVolumeFOVSnapFactor(TEXT("r.TranslucentVolumeFOVSnapFactor"), TEXT("r.TranslucencyLightingVolume.FOVSnapFactor"), TEXT("5.7"));
FAutoConsoleVariableDeprecated CVarTranslucentVolumeBlur(TEXT("r.TranslucencyVolumeBlur"), TEXT("r.TranslucencyLightingVolume.Blur"), TEXT("5.7"));
FAutoConsoleVariableDeprecated CVarTranslucentVolumeDim(TEXT("r.TranslucencyLightingVolumeDim"), TEXT("r.TranslucencyLightingVolume.Dim"), TEXT("5.7"));
FAutoConsoleVariableDeprecated CVarTranslucentVolumeInnerDistance(TEXT("r.TranslucencyLightingVolumeInnerDistance"), TEXT("r.TranslucencyLightingVolume.InnerDistance"), TEXT("5.7"));
FAutoConsoleVariableDeprecated CVarTranslucentVolumeOuterDistance(TEXT("r.TranslucencyLightingVolumeOuterDistance"), TEXT("r.TranslucencyLightingVolume.OuterDistance"), TEXT("5.7"));
