// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneRendering.h"
#include "RenderGraph.h"
#include "VolumeRendering.h"
#include "LightGrid.h"

struct FSortedLightSceneInfo;
struct FCloudShadowAOData;

class FLightSceneInfo;
class FProjectedShadowInfo;

struct FTranslucencyLightingVolumeTextures
{
	/** Retrieve the statically defined pixel format and creation flags used for the translucency light volume textures */
	static void GetTextureFormatAndCreationFlags(EPixelFormat& OutPixelFormat, ETextureCreateFlags& OutCreationFlags);
	static void GetSpecularDirectionTextureFormatAndCreationFlags(EPixelFormat& OutPixelFormat, ETextureCreateFlags& OutCreationFlags);

	/** Maps view to a texture index, accounting for sharing between some views */
	int32 GetIndex(const FViewInfo& View, int32 CascadeIndex) const;

	/** Initializes the translucent volume textures and clears them using a compute pass. PassFlags should be ERDGPassFlags::Compute or ERDGPassFlags::AsyncCompute. */
	void Init(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, ERDGPassFlags PassFlags);

	bool IsValid() const
	{
		check(!VolumeDim || (Ambient.Num() == Directional.Num() && Ambient.Num() > 0));
		return VolumeDim != 0;
	}

	FRDGTextureRef GetAmbientTexture(const FViewInfo& View, int32 CascadeIndex) const
	{
		return Ambient[GetIndex(View, CascadeIndex)];
	}

	FRDGTextureRef GetDirectionalTexture(const FViewInfo& View, int32 CascadeIndex) const
	{
		return Directional[GetIndex(View, CascadeIndex)];
	}

	FRDGTextureRef GetSpecularDirectionTexture(const FViewInfo& View, int32 CascadeIndex) const
	{
		return SpecularDirection[GetIndex(View, CascadeIndex)];
	}

	TArray<FRDGTextureRef, TInlineAllocator<TVC_MAX>> Ambient;
	TArray<FRDGTextureRef, TInlineAllocator<TVC_MAX>> Directional;
	TArray<FRDGTextureRef, TInlineAllocator<TVC_MAX>> SpecularDirection;
	int32 VolumeDim = 0;
	
	/** Mapping between the view index and texture pair - needed because stereo views shares textures. */
	TArray<int32, TInlineAllocator<2>> ViewsToTexturePairs;
};

BEGIN_SHADER_PARAMETER_STRUCT(FTranslucencyLightingVolumeParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeAmbientInner)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeAmbientOuter)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeDirectionalInner)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencyLightingVolumeDirectionalOuter)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencySpecularDirectionInner)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, TranslucencySpecularDirectionOuter)
	SHADER_PARAMETER(float, TranslucencyLightingRandomPositionOffsetRadius)
	SHADER_PARAMETER(float, TranslucencyLightingForwardUsingVolume)
END_SHADER_PARAMETER_STRUCT()

/** Utility for batching together multiple lighting injections */
struct FTranslucentLightInjectionCollector
{
public:
	FTranslucentLightInjectionCollector(
		FRDGBuilder& GraphBuilder,
		TArrayView<const FViewInfo> Views,
		bool bAreLightsInLightGrid);

	/** 
	* Information about a light to be injected.
	* Cached in this struct to avoid recomputing multiple times (multiple cascades).
	*/
	struct FInjectionData
	{
		// must not be 0
		const FLightSceneInfo* LightSceneInfo;
		// can be 0
		const FProjectedShadowInfo* ProjectedShadowInfo;
		//
		bool bApplyLightFunction;
		// must not be 0
		const FMaterialRenderProxy* LightFunctionMaterialProxy;
		// can be INDEX_NONE
		int32 VirtualShadowMapId;
		//
		bool bUseAdaptiveVolumetricShadowMap;
		FVolumeBounds VolumeBounds[TVC_MAX];
	};
	
	typedef TArray<FInjectionData, SceneRenderingAllocator> FInjectionDataArray;

	struct FPerViewData
	{
		// Array of lights that will be injected individually
		FInjectionDataArray Unbatched;

		// Lights that will be injected collectively
		// For these we store a bitmask of light indices in the light grid
		TBitArray<SceneRenderingAllocator> BatchedLocalLights[TVC_MAX];
		uint32 BatchedLocalLightCount = 0;			// Maintain a count for stats
		bool bAnyBatchedLightsWithVirtualShadowMaps = false;
	};

	TArray<FPerViewData, SceneRenderingAllocator>& InjectionDataPerView;

	void AddLightForInjection(
		const FViewInfo& View,
		const uint32 ViewIndex,
		FLightGridSceneExtension::FViewData* LightGridViewData,
		const FVisibleLightInfoArray& VisibleLightInfos,
		const FLightSceneInfo& LightSceneInfo,
		const FProjectedShadowInfo* InProjectedShadowInfo = nullptr);

private:
	bool bCollectorSupportsBatching = false;
};

/** Initializes translucency volume lighting shader parameters from an optional textures struct. If null or uninitialized, fallback textures are used. */
FTranslucencyLightingVolumeParameters GetTranslucencyLightingVolumeParameters(FRDGBuilder& GraphBuilder, const FTranslucencyLightingVolumeTextures& Textures, const FViewInfo& View);

bool IsTranslucencyLightingVolumeUsingVoxelMarkingSupported();
bool IsTranslucencyLightingVolumeUsingVoxelMarking();

bool IsTranslucencyLightingVolumeUsingBlueNoise();
