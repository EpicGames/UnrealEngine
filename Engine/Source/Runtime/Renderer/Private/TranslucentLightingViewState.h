// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneView.h"
#include "Math/Vector4.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"

class FTranslucencyLightingViewState
{
public:
	TStaticArray<TRefCountPtr<IPooledRenderTarget>, TVC_MAX> HistoryAmbient;
	TStaticArray<TRefCountPtr<IPooledRenderTarget>, TVC_MAX> HistoryDirectional;

	TStaticArray<TRefCountPtr<IPooledRenderTarget>, TVC_MAX> HistoryMark;

	TStaticArray<TRefCountPtr<IPooledRenderTarget>, TVC_MAX> HistorySpecularDirection;

	TStaticArray<FVector, TVC_MAX> HistoryVolumeMin;
	TStaticArray<float, TVC_MAX> HistoryVoxelSize;
	TStaticArray<FVector, TVC_MAX> HistoryVolumeSize;

	void SafeRelease()
	{
		for (int32 Index = 0; Index < TVC_MAX; Index++)
		{
			HistoryAmbient[Index].SafeRelease();
			HistoryDirectional[Index].SafeRelease();

			HistoryMark[Index].SafeRelease();

			HistorySpecularDirection[Index].SafeRelease();
		}
	}

	uint64 GetGPUSizeBytes(bool bLogSizes) const;
};