// Copyright Epic Games, Inc. All Rights Reserved.

#include "TranslucentRendering.h"
#include "CoreMinimal.h"
#include "BasePassRendering.h"
#include "BatchedElements.h"
#include "BlueNoise.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DeferredShadingRenderer.h"
#include "DeferredShadingRendererTypes.h"
#include "DynamicRenderScaling.h"
#include "DynamicResolutionState.h"
#include "Engine/EngineTypes.h"
#include "FrontLayerTranslucency.h"
#include "GlobalShader.h"
#include "HairStrands/HairStrandsComposition.h"
#include "HairStrands/HairStrandsUtils.h"
#include "HAL/IConsoleManager.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "HZB.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "Logging/LogMacros.h"
#include "Lumen/LumenTranslucencyVolumeLighting.h"
#include "Lumen/RayTracedTranslucency.h"
#include "MeshBatch.h"
#include "MeshDrawCommands.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "Misc/LargeWorldCoordinates.h"
#include "MultiGPU.h"
#include "Nanite/NaniteTranslucency.h"
#include "OIT/OIT.h"
#include "OIT/OITParameters.h"
#include "PixelShaderUtils.h"
#include "PostProcess/LensDistortion.h"
#include "PostProcess/TemporalAA.h"
#include "RayTracing/RaytracingOptions.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/NaniteStreamingManager.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "ScreenRendering.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "TranslucencyPass.h"
#include "TranslucentLighting.h"
#include "TranslucentPassResource.h"
#include "VariableRateShadingImageManager.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VolumetricCloudRendering.h"
#include "VolumetricRenderTarget.h"
#include "VolumetricRenderTargetViewStateData.h"
#include "SceneViewState.h"

DECLARE_CYCLE_STAT(TEXT("TranslucencyTimestampQueryFence Wait"), STAT_TranslucencyTimestampQueryFence_Wait, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("TranslucencyTimestampQuery Wait"), STAT_TranslucencyTimestampQuery_Wait, STATGROUP_SceneRendering);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Translucency GPU Time (MS)"), STAT_TranslucencyGPU, STATGROUP_SceneRendering);
DEFINE_GPU_DRAWCALL_STAT(Translucency);

// Forward declarations
bool ShouldRenderVolumetricCloud(const FScene* Scene, const FEngineShowFlags& EngineShowFlags);
bool IsVSMTranslucentHighQualityEnabled();

static TAutoConsoleVariable<float> CVarSeparateTranslucencyScreenPercentage(
	TEXT("r.SeparateTranslucencyScreenPercentage"),
	100.0f,
	TEXT("Render separate translucency at this percentage of the full resolution.\n")
	TEXT("in percent, >0 and <=100, larger numbers are possible (supersampling).")
	TEXT("<0 is treated like 100."),
	ECVF_Scalability | ECVF_Default);

static TAutoConsoleVariable<int32> CVarTranslucencyScreenPercentageBasis(
	TEXT("r.Translucency.ScreenPercentage.Basis"), 0,
	TEXT("Basis of the translucency's screen percentage (Experimental).\n")
	TEXT(" 0: Uses the primary view's resolution (notably scaling with r.ScreenPercentage and r.DynamicRes.*)\n")
	TEXT(" 1: Uses the secondary view's resolution (temporal upscale's output resolution)"),
	ECVF_Scalability | ECVF_Default);

static TAutoConsoleVariable<float> CVarTranslucencyMinScreenPercentage(
	TEXT("r.Translucency.DynamicRes.MinScreenPercentage"),
	DynamicRenderScaling::FractionToPercentage(DynamicRenderScaling::FHeuristicSettings::kDefaultMinResolutionFraction),
	TEXT("Minimal screen percentage for translucency."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<float> CVarTranslucencyMaxScreenPercentage(
	TEXT("r.Translucency.DynamicRes.MaxScreenPercentage"),
	DynamicRenderScaling::FractionToPercentage(DynamicRenderScaling::FHeuristicSettings::kDefaultMaxResolutionFraction),
	TEXT("Maximal screen percentage for translucency."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<float> CVarTranslucencyTimeBudget(
	TEXT("r.Translucency.DynamicRes.TimeBudget"),
	DynamicRenderScaling::FHeuristicSettings::kBudgetMsDisabled,
	TEXT("Frame's time budget for translucency rendering in milliseconds."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<float> CVarTranslucencyTargetedHeadRoomPercentage(
	TEXT("r.Translucency.DynamicRes.TargetedHeadRoomPercentage"),
	DynamicRenderScaling::FractionToPercentage(DynamicRenderScaling::FHeuristicSettings::kDefaultTargetedHeadRoom),
	TEXT("Targeted GPU headroom for translucency (in percent from r.DynamicRes.DynamicRes.TimeBudget)."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<float> CVarTranslucencyChangeThreshold(
	TEXT("r.Translucency.DynamicRes.ChangePercentageThreshold"),
	DynamicRenderScaling::FractionToPercentage(DynamicRenderScaling::FHeuristicSettings::kDefaultChangeThreshold),
	TEXT("Minimal increase percentage threshold to alow when changing resolution of translucency."),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<int32> CVarTranslucencyUpperBoundQuantization(
	TEXT("r.Translucency.DynamicRes.UpperBoundQuantization"),
	DynamicRenderScaling::FHeuristicSettings::kDefaultUpperBoundQuantization,
	TEXT("Quantization step count to use for upper bound screen percentage.\n")
	TEXT("If non-zero, rendertargets will be resized based on the dynamic resolution fraction, saving GPU time during clears and resolves.\n")
	TEXT("Only recommended for use with the transient allocator (on supported platforms) with a large transient texture cache (e.g RHI.TransientAllocator.TextureCacheSize=512)"),
	ECVF_RenderThreadSafe | ECVF_Default);

static int32 GTranslucencyHoldoutComposition = static_cast<int32>(ETranslucentHoldoutCompositionType::BeforeDOF);
static FAutoConsoleVariableRef CVarTranslucencyHoldoutLocation(
	TEXT("r.Translucency.Holdout.Location"),
	GTranslucencyHoldoutComposition,
	TEXT("Control the location of translucent holdout composition into scene color alpha.\n")
	TEXT(" 0: BeforeDOF (default)\n")
	TEXT(" 1: AfterDOF\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarVolumetricCloudSoftBlendingDistanceOnTranslucent(
	TEXT("r.VolumetricCloud.SoftBlendingDistanceOnTranslucent"), 0.5,
	TEXT("The soft blending in distance in kilometer used to soft blend in cloud over translucent from the evaluated start depth."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

int32 GSeparateTranslucencyUpsampleMode = 1;
static FAutoConsoleVariableRef CVarSeparateTranslucencyUpsampleMode(
	TEXT("r.SeparateTranslucencyUpsampleMode"),
	GSeparateTranslucencyUpsampleMode,
	TEXT("Upsample method to use on separate translucency.  These are only used when r.SeparateTranslucencyScreenPercentage is less than 100.\n")
	TEXT("0: bilinear 1: Nearest-Depth Neighbor (only when r.SeparateTranslucencyScreenPercentage is 50)"),
	ECVF_Scalability | ECVF_Default);

static TAutoConsoleVariable<int32> CVarParallelTranslucency(
	TEXT("r.ParallelTranslucency"),
	1,
	TEXT("Toggles parallel translucency rendering. Parallel rendering must be enabled for this to have an effect."),
	ECVF_RenderThreadSafe);

DynamicRenderScaling::FHeuristicSettings GetDynamicTranslucencyResolutionSettings()
{
	DynamicRenderScaling::FHeuristicSettings BucketSetting;
	BucketSetting.Model = DynamicRenderScaling::EHeuristicModel::Quadratic;
	BucketSetting.bModelScalesWithPrimaryScreenPercentage = CVarTranslucencyScreenPercentageBasis.GetValueOnAnyThread() != 1;
	BucketSetting.MinResolutionFraction = DynamicRenderScaling::GetPercentageCVarToFraction(CVarTranslucencyMinScreenPercentage);
	BucketSetting.MaxResolutionFraction = DynamicRenderScaling::GetPercentageCVarToFraction(CVarTranslucencyMaxScreenPercentage);
	BucketSetting.BudgetMs              = CVarTranslucencyTimeBudget.GetValueOnAnyThread();
	BucketSetting.ChangeThreshold       = DynamicRenderScaling::GetPercentageCVarToFraction(CVarTranslucencyChangeThreshold);
	BucketSetting.TargetedHeadRoom      = DynamicRenderScaling::GetPercentageCVarToFraction(CVarTranslucencyTargetedHeadRoomPercentage);
	BucketSetting.UpperBoundQuantization = CVarTranslucencyUpperBoundQuantization.GetValueOnAnyThread();
	return BucketSetting;
}

DynamicRenderScaling::FBudget GDynamicTranslucencyResolution(TEXT("DynamicTranslucencyResolution"), &GetDynamicTranslucencyResolutionSettings);


static const TCHAR* kTranslucencyPassName[] = {
	TEXT("BeforeDistortion"),
	TEXT("BeforeDistortionModulate"),
	TEXT("AfterDOF"),
	TEXT("AfterDOFModulate"),
	TEXT("AfterMotionBlur"),
	TEXT("Holdout"),
	TEXT("All"),
};
static_assert(UE_ARRAY_COUNT(kTranslucencyPassName) == int32(ETranslucencyPass::TPT_MAX), "Fix me");

static const TCHAR* kTranslucencyColorTextureName[] = {
	TEXT("Translucency.BeforeDistortion.Color"),
	TEXT("Translucency.BeforeDistortion.Modulate"),
	TEXT("Translucency.AfterDOF.Color"),
	TEXT("Translucency.AfterDOF.Modulate"),
	TEXT("Translucency.AfterMotionBlur.Color"),
	TEXT("Translucency.Holdout.Visibility"),
	TEXT("Translucency.All.Color"),
};
static_assert(UE_ARRAY_COUNT(kTranslucencyColorTextureName) == int32(ETranslucencyPass::TPT_MAX), "Fix me");

static const TCHAR* kTranslucencyColorTextureMultisampledName[] = {
	TEXT("Translucency.BeforeDistortion.ColorMS"),
	TEXT("Translucency.BeforeDistortion.ModulateMS"),
	TEXT("Translucency.AfterDOF.ColorMS"),
	TEXT("Translucency.AfterDOF.ModulateMS"),
	TEXT("Translucency.AfterMotionBlur.ColorMS"),
	TEXT("Translucency.Holdout.VisibilityMS"),
	TEXT("Translucency.All.ColorMS"),
};
static_assert(UE_ARRAY_COUNT(kTranslucencyColorTextureMultisampledName) == UE_ARRAY_COUNT(kTranslucencyColorTextureName), "Fix me");

const TCHAR* TranslucencyPassToString(ETranslucencyPass::Type TranslucencyPass)
{
	return kTranslucencyPassName[TranslucencyPass];
}

EMeshPass::Type TranslucencyPassToMeshPass(ETranslucencyPass::Type TranslucencyPass)
{
	EMeshPass::Type TranslucencyMeshPass = EMeshPass::Num;

	switch (TranslucencyPass)
	{
	case ETranslucencyPass::TPT_TranslucencyStandard:			TranslucencyMeshPass = EMeshPass::TranslucencyStandard; break;
	case ETranslucencyPass::TPT_TranslucencyStandardModulate:	TranslucencyMeshPass = EMeshPass::TranslucencyStandardModulate; break;
	case ETranslucencyPass::TPT_TranslucencyAfterDOF:			TranslucencyMeshPass = EMeshPass::TranslucencyAfterDOF; break;
	case ETranslucencyPass::TPT_TranslucencyAfterDOFModulate:	TranslucencyMeshPass = EMeshPass::TranslucencyAfterDOFModulate; break;
	case ETranslucencyPass::TPT_TranslucencyAfterMotionBlur:	TranslucencyMeshPass = EMeshPass::TranslucencyAfterMotionBlur; break;
	case ETranslucencyPass::TPT_TranslucencyHoldout:			TranslucencyMeshPass = EMeshPass::TranslucencyHoldout; break;
	case ETranslucencyPass::TPT_AllTranslucency:				TranslucencyMeshPass = EMeshPass::TranslucencyAll; break;
	}

	check(TranslucencyMeshPass != EMeshPass::Num);

	return TranslucencyMeshPass;
}

ETranslucencyView GetTranslucencyView(const FViewInfo& View)
{
	if (ShouldRenderRayTracingTranslucency(View) || RayTracedTranslucency::IsEnabled(View))
	{
		return ETranslucencyView::RayTracing;
	}

	return View.IsUnderwater() ? ETranslucencyView::UnderWater : ETranslucencyView::AboveWater;
}

ETranslucencyView GetTranslucencyViews(TArrayView<const FViewInfo> Views)
{
	ETranslucencyView TranslucencyViews = ETranslucencyView::None;
	for (const FViewInfo& View : Views)
	{
		TranslucencyViews |= GetTranslucencyView(View);
	}
	return TranslucencyViews;
}

/** Mostly used to know if debug rendering should be drawn in this pass */
static bool IsMainTranslucencyPass(ETranslucencyPass::Type TranslucencyPass)
{
	return TranslucencyPass == ETranslucencyPass::TPT_AllTranslucency || TranslucencyPass == ETranslucencyPass::TPT_TranslucencyStandard;
}

static bool IsParallelTranslucencyEnabled()
{
	return GRHICommandList.UseParallelAlgorithms() && CVarParallelTranslucency.GetValueOnRenderThread();
}

bool IsSeparateTranslucencyEnabled(ETranslucencyPass::Type TranslucencyPass, float DownsampleScale)
{
	// Currently AfterDOF is rendered earlier in the frame and must be rendered in a separate texture.
	if (   TranslucencyPass == ETranslucencyPass::TPT_TranslucencyAfterDOF
		|| TranslucencyPass == ETranslucencyPass::TPT_TranslucencyAfterDOFModulate
		|| TranslucencyPass == ETranslucencyPass::TPT_TranslucencyStandardModulate
		|| TranslucencyPass == ETranslucencyPass::TPT_TranslucencyAfterMotionBlur
		)
	{
		return true;
	}

	// Otherwise it only gets rendered in the separate buffer if it is downsampled.
	if (DownsampleScale < 1.0f)
	{
		return true;
	}

	return false;
}

static int GetSSRQuality()
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SSR.Quality"));
	int SSRQuality = CVar ? (CVar->GetInt()) : 0;
	return SSRQuality;
}

static bool ShouldRenderTranslucencyScreenSpaceReflections(const FViewInfo& View)
{
	// The screenspace reflection of translucency is not controlled by the postprocessing setting
	// or the raytracing overlay setting. It needs to be turned on/off dynamically to support
	// diffuse only
	if (!View.Family->EngineShowFlags.ScreenSpaceReflections)
	{
		return false;
	}

	int SSRQuality = GetSSRQuality();

	if (SSRQuality <= 0)
	{
		return false;
	}

	return true;
}

FSeparateTranslucencyDimensions UpdateSeparateTranslucencyDimensions(const FSceneRenderer& SceneRenderer)
{
	float TranslucencyResolutionFraction = FMath::Clamp(CVarSeparateTranslucencyScreenPercentage.GetValueOnRenderThread() / 100.0f, 0.0f, 1.0f);
	float MaxTranslucencyResolutionFraction	= TranslucencyResolutionFraction;

	if (GDynamicTranslucencyResolution.GetSettings().IsEnabled())
	{
		TranslucencyResolutionFraction = SceneRenderer.DynamicResolutionFractions[GDynamicTranslucencyResolution];
		MaxTranslucencyResolutionFraction = SceneRenderer.DynamicResolutionUpperBounds[GDynamicTranslucencyResolution];
	}

	if (CVarTranslucencyScreenPercentageBasis.GetValueOnRenderThread() == 1)
	{
		TranslucencyResolutionFraction /= SceneRenderer.DynamicResolutionFractions[GDynamicPrimaryResolutionFraction];
		MaxTranslucencyResolutionFraction /= SceneRenderer.DynamicResolutionUpperBounds[GDynamicPrimaryResolutionFraction];
	}

	FSeparateTranslucencyDimensions Dimensions;
	// TODO: this should be MaxTranslucencyResolutionFraction instead of TranslucencyResolutionFraction to keep the size of render target stable, but the SvPositionToBuffer() is broken in material.
	Dimensions.Extent = GetScaledExtent(SceneRenderer.ViewFamily.SceneTexturesConfig.Extent, TranslucencyResolutionFraction);
	Dimensions.NumSamples = SceneRenderer.ViewFamily.SceneTexturesConfig.NumSamples;
	Dimensions.Scale = TranslucencyResolutionFraction;
	return Dimensions;
}

FTranslucencyPassResourcesMap::FTranslucencyPassResourcesMap(int32 NumViews)
{
	Array.SetNum(NumViews);

	for (int32 ViewIndex = 0; ViewIndex < NumViews; ViewIndex++)
	{
		for (int32 i = 0; i < int32(ETranslucencyPass::TPT_MAX); i++)
		{
			Array[ViewIndex][i].Pass = ETranslucencyPass::Type(i);
		}
	}
}
/** Pixel shader used to copy scene color into another texture so that materials can read from scene color with a node. */
class FCopySceneColorPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopySceneColorPS);
	SHADER_USE_PARAMETER_STRUCT(FCopySceneColorPS, FGlobalShader);

	class FEncodeSceneColorCopy : SHADER_PERMUTATION_BOOL("SCENE_COLOR_COPY_ENCODED");
	using FPermutationDomain = TShaderPermutationDomain<FEncodeSceneColorCopy>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorTextureSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCopySceneColorPS, "/Engine/Private/TranslucentLightingShaders.usf", "CopySceneColorMain", SF_Pixel);

FRDGTextureRef AddCopySceneColorPass(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureMSAA SceneColor, bool WithAlpha, bool bSkipIfUnderwater, bool bEncodeSceneColorCopy)
{
	FRDGTextureRef SceneColorCopyTexture = nullptr;
	ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ENoAction;

	RDG_EVENT_SCOPE(GraphBuilder, "CopySceneColor");

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (View.IsUnderwater() && bSkipIfUnderwater)
		{
			continue;
		}

		bool bNeedsResolve = false;
		for (int32 TranslucencyPass = 0; TranslucencyPass < ETranslucencyPass::TPT_MAX; ++TranslucencyPass)
		{
			if (View.TranslucentPrimCount.UseSceneColorCopy((ETranslucencyPass::Type)TranslucencyPass))
			{
				bNeedsResolve = true;
				break;
			}
		}
		bNeedsResolve = bNeedsResolve || (IsMobilePlatform(View.GetShaderPlatform()) && View.bHasSingleLayerWaterMaterial);

		if (bNeedsResolve)
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

			AddResolveSceneColorPass(GraphBuilder, View, SceneColor);

			const FIntPoint SceneColorExtent = SceneColor.Target->Desc.Extent;

			if (!SceneColorCopyTexture)
			{
				EPixelFormat CopyFormat = (bEncodeSceneColorCopy || WithAlpha) ? PF_B8G8R8A8 : PF_FloatR11G11B10;
				SceneColorCopyTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(SceneColorExtent, CopyFormat, FClearValueBinding::White, TexCreate_ShaderResource | TexCreate_RenderTargetable), TEXT("SceneColorCopy"));
			}

			const FScreenPassTextureViewport Viewport(SceneColorCopyTexture, View.ViewRect);

			TShaderMapRef<FScreenVS> VertexShader(View.ShaderMap);

			FCopySceneColorPS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCopySceneColorPS::FEncodeSceneColorCopy>(bEncodeSceneColorCopy);
			TShaderMapRef<FCopySceneColorPS> PixelShader(View.ShaderMap, PermutationVector);

			auto* PassParameters = GraphBuilder.AllocParameters<FCopySceneColorPS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->SceneColorTexture = SceneColor.Resolve;
			PassParameters->SceneColorTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorCopyTexture, LoadAction);

			if (!View.Family->bMultiGPUForkAndJoin)
			{
				LoadAction = ERenderTargetLoadAction::ELoad;
			}

			FRHIBlendState* BlendState = nullptr;

			if (WithAlpha)
			{
				BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
			}
			else
			{
				// The original behavior sets alpha to zero.
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_Zero>::GetRHI();
			}

			AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("TranslucencyCopySceneColor"), View, Viewport, Viewport, VertexShader, PixelShader, PassParameters);
		}
	}

	return SceneColorCopyTexture;
}

void AddTranslucentHoldoutToSceneColorAlphaPass(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FTranslucencyPassResources& TranslucencyPassResources)
{
	const ETranslucencyPass::Type PassType = TranslucencyPassResources.Pass;
	check(PassType == ETranslucencyPass::TPT_TranslucencyHoldout);

	FTranslucencyComposition TranslucencyComposition;
	TranslucencyComposition.Operation = FTranslucencyComposition::EOperation::ComposeToSceneColorAlpha;
	TranslucencyComposition.SceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, FScreenPassTexture(SceneColorTexture, View.ViewRect));
	TranslucencyComposition.SceneDepth = FScreenPassTexture(SceneDepthTexture, View.ViewRect);
	TranslucencyComposition.OutputViewport = FScreenPassTextureViewport(SceneDepthTexture, View.ViewRect);

	FScreenPassTexture UpscaledTranslucency = TranslucencyComposition.AddPass(
		GraphBuilder, View, TranslucencyPassResources);

	ensure(View.ViewRect == UpscaledTranslucency.ViewRect);
	ensure(UpscaledTranslucency.Texture == SceneColorTexture);

	//Invalidate.
	TranslucencyPassResources = FTranslucencyPassResources();
	TranslucencyPassResources.Pass = PassType;
}

static void AddCopySceneColorAlphaPass(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureRef SourceTexture, FRDGTextureRef TargetTexture, bool bSkipIfUnderwater)
{
	ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ENoAction;

	RDG_EVENT_SCOPE(GraphBuilder, "CopySceneColorAlpha");

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (View.IsUnderwater() && bSkipIfUnderwater)
		{
			continue;
		}

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		const FIntPoint SceneColorExtent = SourceTexture->Desc.Extent;

		const FScreenPassTextureViewport Viewport(TargetTexture, View.ViewRect);

		TShaderMapRef<FScreenVS> VertexShader(View.ShaderMap);

		FCopySceneColorPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FCopySceneColorPS::FEncodeSceneColorCopy>(true);
		TShaderMapRef<FCopySceneColorPS> PixelShader(View.ShaderMap, PermutationVector);

		auto* PassParameters = GraphBuilder.AllocParameters<FCopySceneColorPS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneColorTexture = SourceTexture;
		PassParameters->SceneColorTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(TargetTexture, LoadAction);

		if (!View.Family->bMultiGPUForkAndJoin)
		{
			LoadAction = ERenderTargetLoadAction::ELoad;
		}

		FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_One, BO_Add, BF_One, BF_Zero>::GetRHI();
		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("TranslucencyCopySceneColorAlpha"), View, Viewport, Viewport, VertexShader, PixelShader, BlendState, PassParameters);
	}
}

class FCopyBackgroundVisibilityPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyBackgroundVisibilityPS);
	SHADER_USE_PARAMETER_STRUCT(FCopyBackgroundVisibilityPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, TranslucentHoldoutPointTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TranslucentHoldoutPointSampler)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	enum class EVisibilityCopyType :int32
	{
		FromSceneColor, // Copy alpha (background visibility) of scene color to any of the rgb
		ToSceneColor, // Copy background visibility from any of the rgb to the alpha of an image for scene color composition.
		MAX
	};

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	class FDimVisibilityCopyType :SHADER_PERMUTATION_ENUM_CLASS("VISIBILITY_COPY_TYPE", EVisibilityCopyType);
	using FPermutationDomain = TShaderPermutationDomain<FDimVisibilityCopyType>;
};

class FComposeSeparateTranslucencyPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComposeSeparateTranslucencyPS);
	SHADER_USE_PARAMETER_STRUCT(FComposeSeparateTranslucencyPS, FGlobalShader);

	class FNearestDepthNeighborUpsampling : SHADER_PERMUTATION_BOOL("PERMUTATION_NEARESTDEPTHNEIGHBOR");
	using FPermutationDomain = TShaderPermutationDomain<FNearestDepthNeighborUpsampling>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FScreenTransform, ScreenPosToSceneColorUV)
		SHADER_PARAMETER(FScreenTransform, ScreenPosToSeparateTranslucencyUV)
		SHADER_PARAMETER(FScreenTransform, SeparateTranslucencyUVToViewportUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToSeparateTranslucencyUV)
		SHADER_PARAMETER(FVector2f, SeparateTranslucencyUVMin)
		SHADER_PARAMETER(FVector2f, SeparateTranslucencyUVMax)
		SHADER_PARAMETER(FVector2f, SeparateTranslucencyExtentInverse)
		SHADER_PARAMETER(int32, bLensDistortion)
		SHADER_PARAMETER(int32, bPassthroughAlpha)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  SceneColorSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SeparateTranslucencyPointTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  SeparateTranslucencyPointSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SeparateModulationPointTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  SeparateModulationPointSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SeparateTranslucencyBilinearTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  SeparateTranslucencyBilinearSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SeparateModulationBilinearTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  SeparateModulationBilinearSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, LowResDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LowResDepthSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FullResDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, FullResDepthSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, UndistortingDisplacementSampler)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

class FTranslucencyUpsampleResponsiveAAPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTranslucencyUpsampleResponsiveAAPS);
	SHADER_USE_PARAMETER_STRUCT(FTranslucencyUpsampleResponsiveAAPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, StencilPixelPosMin)
		SHADER_PARAMETER(FIntPoint, StencilPixelPosMax)
		SHADER_PARAMETER(FScreenTransform, SvPositionToStencilPixelCoord)
		SHADER_PARAMETER(int32, StencilMask)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, StencilTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

class FComposeHoldoutWithCoveragePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComposeHoldoutWithCoveragePS);
	SHADER_USE_PARAMETER_STRUCT(FComposeHoldoutWithCoveragePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FScreenTransform, ScreenPosToSeparateTranslucencyUV)
		SHADER_PARAMETER(FVector2f, SeparateTranslucencyUVMin)
		SHADER_PARAMETER(FVector2f, SeparateTranslucencyUVMax)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HoldoutTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  HoldoutSampler)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCopyBackgroundVisibilityPS, "/Engine/Private/ComposeSeparateTranslucency.usf", "CopyBackgroundVisibilityPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FComposeSeparateTranslucencyPS, "/Engine/Private/ComposeSeparateTranslucency.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FComposeHoldoutWithCoveragePS, "/Engine/Private/ComposeSeparateTranslucency.usf", "ComposeHoldoutWithCoveragePS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FTranslucencyUpsampleResponsiveAAPS, "/Engine/Private/TranslucencyUpsampling.usf", "UpsampleResponsiveAAPS", SF_Pixel);

void AddTranslucentHoldoutWithCoverageToSceneColorAlphaPass(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FTranslucencyPassResources& TranslucencyPassResources)
{
	const ETranslucencyPass::Type PassType = TranslucencyPassResources.Pass;
	check(PassType == ETranslucencyPass::TPT_TranslucencyHoldout);
	check(TranslucencyPassResources.ColorTexture.IsValid());

	ensure(SceneColorTexture->Desc.Flags & TexCreate_RenderTargetable);

	const FScreenPassTextureViewport SceneColorViewport(SceneColorTexture, View.ViewRect);
	const FScreenPassTextureViewport HoldoutViewport(TranslucencyPassResources.ColorTexture.Resolve, TranslucencyPassResources.ViewRect);
	const FVector2f HoldoutExtentInv = FVector2f(1.0f, 1.0f) / FVector2f(HoldoutViewport.Extent);

	FComposeHoldoutWithCoveragePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComposeHoldoutWithCoveragePS::FParameters>();
	const FScreenTransform SvPositionToViewportUV = FScreenTransform::SvPositionToViewportUV(SceneColorViewport.Rect);
	PassParameters->ScreenPosToSeparateTranslucencyUV = SvPositionToViewportUV * FScreenTransform::ChangeTextureBasisFromTo(
		HoldoutViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
	PassParameters->SeparateTranslucencyUVMin = (FVector2f(HoldoutViewport.Rect.Min) + FVector2f(0.5f, 0.5f)) * HoldoutExtentInv;
	PassParameters->SeparateTranslucencyUVMax = (FVector2f(HoldoutViewport.Rect.Max) - FVector2f(0.5f, 0.5f)) * HoldoutExtentInv;

	PassParameters->HoldoutTexture = TranslucencyPassResources.ColorTexture.Resolve;
	PassParameters->HoldoutSampler = TStaticSamplerState<SF_Point>::GetRHI();
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);

	// CW_ALPHA writes only the alpha channel; scene-color RGB is untouched.
	FRHIBlendState* CoverageBlendState = TStaticBlendState<CW_ALPHA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();

	RDG_EVENT_SCOPE_STAT(GraphBuilder, Translucency, "ComposeHoldoutWithCoverage");
	DynamicRenderScaling::FRDGScope DynamicTranslucencyResolutionScope(GraphBuilder, GDynamicTranslucencyResolution);

	TShaderMapRef<FComposeHoldoutWithCoveragePS> PixelShader(View.ShaderMap);
	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		View.ShaderMap,
		RDG_EVENT_NAME(
			"ComposeHoldoutWithCoverage(%s) %dx%d",
			kTranslucencyPassName[FMath::Clamp(int32(PassType), 0, int32(ETranslucencyPass::TPT_MAX) - 1)],
			HoldoutViewport.Rect.Width(), HoldoutViewport.Rect.Height()),
		PixelShader,
		PassParameters,
		SceneColorViewport.Rect,
		CoverageBlendState);

	//Invalidate.
	TranslucencyPassResources = FTranslucencyPassResources();
	TranslucencyPassResources.Pass = PassType;
}

static void AddCopyBackgroundVisibilityPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef CopySource,
	FRDGTextureRef CopyDestination,
	FIntRect CopyRect,
	FCopyBackgroundVisibilityPS::EVisibilityCopyType VisibilityCopyType)
{
	typedef FCopyBackgroundVisibilityPS SHADER;
	SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
	{
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->TranslucentHoldoutPointTexture = CopySource;
		PassParameters->TranslucentHoldoutPointSampler = TStaticSamplerState<SF_Point>::GetRHI();

		PassParameters->RenderTargets[0] = FRenderTargetBinding(CopyDestination, ERenderTargetLoadAction::ENoAction);
	}

	SHADER::FPermutationDomain PixelShaderPermutationVector;
	PixelShaderPermutationVector.Set<SHADER::FDimVisibilityCopyType>(VisibilityCopyType);
	TShaderMapRef<SHADER> PixelShader(View.ShaderMap, PixelShaderPermutationVector);

	FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		View.ShaderMap,
		RDG_EVENT_NAME(
			"%s(%s) %dx%d",
			TEXT("CopyBackgroundVisibility"),
			TEXT("Holdout"),
			CopyRect.Width(), CopyRect.Height()),
		PixelShader,
		PassParameters,
		CopyRect,
		BlendState);
}

FScreenPassTexture FTranslucencyComposition::AddPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FTranslucencyPassResources& TranslucencyTextures) const
{
	// if nothing is rendered into the separate translucency, then just return the existing Scenecolor
	ensure(TranslucencyTextures.IsValid());
	if (!TranslucencyTextures.IsValid())
	{
		return FScreenPassTexture(SceneColor);
	}

	ensure(TranslucencyTextures.Pass != ETranslucencyPass::TPT_MAX);

	FRDGTextureRef SeparateModulationTexture = TranslucencyTextures.GetColorModulateForRead(GraphBuilder);
	FRDGTextureRef SeparateTranslucencyTexture = TranslucencyTextures.GetColorForRead(GraphBuilder);

	FScreenPassTextureViewport SceneColorViewport(FIntPoint(1, 1), FIntRect(0, 0, 1, 1));
	if (SceneColor.IsValid())
	{
		SceneColorViewport = FScreenPassTextureViewport(SceneColor);
	}

	FScreenPassTextureViewport TranslucencyViewport(FIntPoint(1, 1), FIntRect(0, 0, 1, 1));
	if (TranslucencyTextures.ColorTexture.IsValid())
	{
		TranslucencyViewport = FScreenPassTextureViewport(TranslucencyTextures.ColorTexture.Resolve, TranslucencyTextures.ViewRect);
	}
	else if (TranslucencyTextures.ColorModulateTexture.IsValid())
	{
		TranslucencyViewport = FScreenPassTextureViewport(TranslucencyTextures.ColorModulateTexture.Resolve, TranslucencyTextures.ViewRect);
	}

	bool bPostMotionBlur = TranslucencyTextures.Pass == ETranslucencyPass::TPT_TranslucencyAfterMotionBlur;
	if (bPostMotionBlur)
	{
		check(!bApplyModulateOnly);
	}
	else if (bApplyModulateOnly)
	{
		if (!TranslucencyTextures.ColorModulateTexture.IsValid())
		{
			return FScreenPassTexture(SceneColor);
		}

		SeparateTranslucencyTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackAlphaOneDummy);
	}

	bool bPassthroughAlpha = IsPrimitiveAlphaHoldoutEnabled(View);

	const TCHAR* OpName = nullptr;
	FRHIBlendState* BlendState = nullptr;
	FRDGTextureRef NewSceneColor = nullptr;
	if (Operation == EOperation::UpscaleOnly)
	{
		check(!SceneColor.IsValid());

		//ensure(!TranslucencyTextures.ColorModulateTexture.IsValid());
		// It is valid to have a valid TranslucencyTextures.ColorModulateTexture in "UpscaleTranslucency". Since it can will be upscaled later in the ComposeTranslucencyToNewSceneColor pass (apply onto scene color with added translucency color on top).

		OpName = TEXT("UpscaleTranslucency");

		FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
			OutputViewport.Extent,
			PF_FloatRGBA,
			FClearValueBinding::Black,
			TexCreate_RenderTargetable | TexCreate_ShaderResource);

		NewSceneColor = GraphBuilder.CreateTexture(
			OutputDesc,
			bPostMotionBlur ? TEXT("PostMotionBlurTranslucency.SceneColor") : TEXT("PostDOFTranslucency.SceneColor"));
	}
	else if (Operation == EOperation::ComposeToExistingSceneColor)
	{
		check(SceneColor.IsValid());
		ensure(!TranslucencyTextures.ColorModulateTexture.IsValid());

		OpName = TEXT("ComposeTranslucencyToExistingColor");
		BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha>::GetRHI();

		ensure(SceneColor.TextureSRV->Desc.Texture->Desc.Flags & TexCreate_RenderTargetable);
		NewSceneColor = SceneColor.TextureSRV->Desc.Texture;
	}
	else if (Operation == EOperation::ComposeToNewSceneColor)
	{
		check(SceneColor.IsValid());

		OpName = TEXT("ComposeTranslucencyToNewSceneColor");

		FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
			OutputViewport.Extent,
			OutputPixelFormat != PF_Unknown ? OutputPixelFormat : SceneColor.TextureSRV->Desc.Texture->Desc.Format,
			FClearValueBinding::Black,
			TexCreate_RenderTargetable | TexCreate_ShaderResource);

		NewSceneColor = GraphBuilder.CreateTexture(
			OutputDesc,
			bPostMotionBlur ? TEXT("PostMotionBlurTranslucency.SceneColor") : TEXT("PostDOFTranslucency.SceneColor"));
	}
	else if (Operation == EOperation::ComposeToSceneColorAlpha)
	{
		check(SceneColor.IsValid());

		// Now we copy any of RGB channel(background visibility) to alpha channel so we can compose against the holdout value of the background.
		FRDGTextureRef ResolvedTranslucentHoldout = GraphBuilder.CreateTexture(SeparateTranslucencyTexture->Desc,TEXT("Translucency.Holdout.Resolved"));
		AddCopyBackgroundVisibilityPass(GraphBuilder, View, SeparateTranslucencyTexture, ResolvedTranslucentHoldout,
			TranslucencyTextures.ViewRect, FCopyBackgroundVisibilityPS::EVisibilityCopyType::ToSceneColor);
		SeparateTranslucencyTexture = ResolvedTranslucentHoldout;

		OpName = TEXT("ComposeToSceneColorAlpha");
		// Keep the color of the target, but override the alpha channel
		// Req: Alpha stores the background visibility of the holdout.
		// If the background is opaque, it is 1, otherwise 0, use Max operator to clamp.
		BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_One, BO_Add, BF_One, BF_Zero>::GetRHI();
		ensure(SceneColor.TextureSRV->Desc.Texture->Desc.Flags & TexCreate_RenderTargetable);
		NewSceneColor = SceneColor.TextureSRV->Desc.Texture;
		bPassthroughAlpha = false;
	}
	else
	{
		unimplemented();
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, Translucency, "%s", OpName);
	DynamicRenderScaling::FRDGScope DynamicTranslucencyResolutionScope(GraphBuilder, GDynamicTranslucencyResolution);

	const FVector2f SeparateTranslucencyExtentInv = FVector2f(1.0f, 1.0f) / FVector2f(TranslucencyViewport.Extent);

	const bool bScaleSeparateTranslucency = OutputViewport.Rect.Size() != TranslucencyTextures.ViewRect.Size();
	const float DownsampleScale = float(TranslucencyTextures.ViewRect.Width()) / float(OutputViewport.Rect.Width());
	const bool DepthUpscampling = (
		bScaleSeparateTranslucency &&
		TranslucencyTextures.DepthTexture.IsValid() &&
		SceneDepth.IsValid() && 
		FMath::IsNearlyEqual(DownsampleScale, 0.5f) &&
		GSeparateTranslucencyUpsampleMode > 0);

	FScreenTransform SvPositionToViewportUV = FScreenTransform::SvPositionToViewportUV(OutputViewport.Rect);

	bool bLensDistortionEnabled = LensDistortionLUT && LensDistortionLUT->IsEnabled();

	FComposeSeparateTranslucencyPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComposeSeparateTranslucencyPS::FParameters>();
	PassParameters->ScreenPosToSceneColorUV = SvPositionToViewportUV * FScreenTransform::ChangeTextureBasisFromTo(
		SceneColorViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
	PassParameters->ScreenPosToSeparateTranslucencyUV = SvPositionToViewportUV * FScreenTransform::ChangeTextureBasisFromTo(
		TranslucencyViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
	PassParameters->SeparateTranslucencyUVToViewportUV = FScreenTransform::ChangeTextureBasisFromTo(
		TranslucencyViewport, FScreenTransform::ETextureBasis::TextureUV, FScreenTransform::ETextureBasis::ViewportUV);
	PassParameters->ViewportUVToSeparateTranslucencyUV = FScreenTransform::ChangeTextureBasisFromTo(
		TranslucencyViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);

	PassParameters->SeparateTranslucencyUVMin = (FVector2f(TranslucencyViewport.Rect.Min) + FVector2f(0.5f, 0.5f)) * SeparateTranslucencyExtentInv;
	PassParameters->SeparateTranslucencyUVMax = (FVector2f(TranslucencyViewport.Rect.Max) - FVector2f(0.5f, 0.5f)) * SeparateTranslucencyExtentInv;
	PassParameters->SeparateTranslucencyExtentInverse = SeparateTranslucencyExtentInv;
	
	PassParameters->SceneColorTexture = Operation == EOperation::ComposeToNewSceneColor
		? SceneColor.TextureSRV
		: GraphBuilder.CreateSRV(FRDGTextureSRVDesc(GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackAlphaOneDummy)));
	PassParameters->SceneColorSampler = TStaticSamplerState<SF_Point>::GetRHI();

	PassParameters->SeparateTranslucencyPointTexture = SeparateTranslucencyTexture;
	PassParameters->SeparateTranslucencyPointSampler = TStaticSamplerState<SF_Point>::GetRHI();
	
	PassParameters->SeparateModulationPointTexture = SeparateModulationTexture;
	PassParameters->SeparateModulationPointSampler = TStaticSamplerState<SF_Point>::GetRHI();

	PassParameters->SeparateTranslucencyBilinearTexture = SeparateTranslucencyTexture;
	PassParameters->SeparateTranslucencyBilinearSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	PassParameters->SeparateModulationBilinearTexture = SeparateModulationTexture;
	PassParameters->SeparateModulationBilinearSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	PassParameters->UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
	PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->bLensDistortion = bLensDistortionEnabled;
	PassParameters->bPassthroughAlpha = bPassthroughAlpha;

	if (bLensDistortionEnabled)
	{
		PassParameters->UndistortingDisplacementTexture = LensDistortionLUT->UndistortingDisplacementTexture;
	}

	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;

	if (Operation == EOperation::ComposeToExistingSceneColor || Operation == EOperation::ComposeToSceneColorAlpha)
	{
		PassParameters->RenderTargets[0] = FRenderTargetBinding(NewSceneColor, ERenderTargetLoadAction::ELoad);
	}
	else
	{
		PassParameters->RenderTargets[0] = FRenderTargetBinding(NewSceneColor, ERenderTargetLoadAction::ENoAction);
	}

	if (DepthUpscampling)
	{
		PassParameters->LowResDepthTexture = TranslucencyTextures.GetDepthForRead(GraphBuilder);
		PassParameters->LowResDepthSampler = TStaticSamplerState<SF_Point>::GetRHI();
		PassParameters->FullResDepthTexture = SceneDepth.Texture;
		PassParameters->FullResDepthSampler = TStaticSamplerState<SF_Point>::GetRHI();
	}

	FComposeSeparateTranslucencyPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FComposeSeparateTranslucencyPS::FNearestDepthNeighborUpsampling>(DepthUpscampling);

	TShaderMapRef<FComposeSeparateTranslucencyPS> PixelShader(View.ShaderMap, PermutationVector);
	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		View.ShaderMap,
		RDG_EVENT_NAME(
			"%s(%s%s%s) %dx%d -> %dx%d",
			OpName,
			kTranslucencyPassName[FMath::Clamp(int32(TranslucencyTextures.Pass), 0, int32(ETranslucencyPass::TPT_MAX) - 1)],
			bApplyModulateOnly ? TEXT(" ModulateOnly") : TEXT(""),
			DepthUpscampling ? TEXT(" DepthUpsampling") : TEXT(""),
			TranslucencyTextures.ViewRect.Width(), TranslucencyTextures.ViewRect.Height(),
			OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
		PixelShader,
		PassParameters,
		OutputViewport.Rect,
		BlendState);

	return FScreenPassTexture(NewSceneColor, OutputViewport.Rect);
}

static void AddUpsampleResponsiveAAPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FScreenPassTexture DownsampledTranslucencyDepth,
	FRDGTextureRef OutputDepthTexture)
{
	FTranslucencyUpsampleResponsiveAAPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FTranslucencyUpsampleResponsiveAAPS::FParameters>();
	PassParameters->StencilPixelPosMin = DownsampledTranslucencyDepth.ViewRect.Min;
	PassParameters->StencilPixelPosMax = DownsampledTranslucencyDepth.ViewRect.Max - 1;
	PassParameters->SvPositionToStencilPixelCoord = (FScreenTransform::Identity - View.ViewRect.Min) * (FVector2f(DownsampledTranslucencyDepth.ViewRect.Size()) / FVector2f(View.ViewRect.Size())) + DownsampledTranslucencyDepth.ViewRect.Min;
	PassParameters->StencilMask = STENCIL_TEMPORAL_RESPONSIVE_AA_MASK;
	PassParameters->StencilTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(DownsampledTranslucencyDepth.Texture, PF_X24_G8));
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(OutputDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthNop_StencilWrite);

	TShaderMapRef<FScreenVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FTranslucencyUpsampleResponsiveAAPS> PixelShader(View.ShaderMap);

	FRHIDepthStencilState* DepthStencilState = TStaticDepthStencilState<false, CF_Always,
		true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		0x00, STENCIL_TEMPORAL_RESPONSIVE_AA_MASK>::GetRHI();
	FRHIBlendState* BlendState = TStaticBlendState<CW_NONE>::GetRHI();

	const FScreenPassPipelineState PipelineState(VertexShader, PixelShader, BlendState, DepthStencilState, /* StencilRef = */ STENCIL_TEMPORAL_RESPONSIVE_AA_MASK);

	ClearUnusedGraphResources(PixelShader, PassParameters);
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("UpsampleResponsiveAA %dx%d -> %dx%d",
			DownsampledTranslucencyDepth.ViewRect.Width(), DownsampledTranslucencyDepth.ViewRect.Height(),
			View.ViewRect.Width(), View.ViewRect.Height()),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, PipelineState, PixelShader, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FScreenPassTextureViewport OutputViewport(PassParameters->RenderTargets.DepthStencil.GetTexture()->Desc.Extent, View.ViewRect);
		DrawScreenPass(RHICmdList, View, OutputViewport, OutputViewport, PipelineState, EScreenPassDrawFlags::None, [&](FRHICommandList&)
		{
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
		});
	});
}

bool FSceneRenderer::ShouldRenderTranslucency() const
{
	return  ViewFamily.EngineShowFlags.Translucency
		&& !ViewFamily.EngineShowFlags.VisualizeLightCulling
		&& !ViewFamily.UseDebugViewPS();
}

bool FSceneRenderer::ShouldRenderTranslucency(ETranslucencyPass::Type TranslucencyPass, TArrayView<FViewInfo> InViews)
{
	extern int32 GLightShaftRenderAfterDOF;

	// Change this condition to control where simple elements should be rendered.
	if (IsMainTranslucencyPass(TranslucencyPass))
	{
		for (const FViewInfo& View : InViews)
		{
			if (View.bHasTranslucentViewMeshElements || View.SimpleElementCollector.BatchedElements.HasPrimsToDraw())
			{
				return true;
			}
		}
	}

	// If lightshafts are rendered in low res, we must reset the offscreen buffer in case is was also used in TPT_TranslucencyStandard.
	if (GLightShaftRenderAfterDOF && TranslucencyPass == ETranslucencyPass::TPT_TranslucencyAfterDOF)
	{
		return true;
	}

	for (const FViewInfo& View : InViews)
	{
		if (View.TranslucentPrimCount.Num(TranslucencyPass) > 0)
		{
			return true;
		}
	}

	return false;
}

FScreenPassTextureViewport FSeparateTranslucencyDimensions::GetInstancedStereoViewport(const FViewInfo& View) const
{
	FIntRect ViewRect = View.ViewRectWithSecondaryViews;
	ViewRect = GetScaledRect(ViewRect, Scale);
	return FScreenPassTextureViewport(Extent, ViewRect);
}

void SetupPostMotionBlurTranslucencyViewParameters(const FViewInfo& View, FViewUniformShaderParameters& Parameters)
{
	// post-motionblur pass without down-sampling requires no Temporal AA jitter
	FBox VolumeBounds[TVC_MAX];
	FViewMatrices ModifiedViewMatrices = View.ViewMatrices;
	ModifiedViewMatrices.HackRemoveTemporalAAProjectionJitter();

	FViewMatrices ModifiedPrevViewViewMatrices = View.PrevViewInfo.ViewMatrices;
	ModifiedPrevViewViewMatrices.HackRemoveTemporalAAProjectionJitter();

	Parameters = *View.CachedViewUniformShaderParameters;
	View.SetupUniformBufferParameters(ModifiedViewMatrices, ModifiedPrevViewViewMatrices, VolumeBounds, TVC_MAX, Parameters);
}

const FRDGTextureDesc GetPostDOFTranslucentTextureDesc(
	ETranslucencyPass::Type TranslucencyPass,
	const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions,
	bool bIsModulate,
	EShaderPlatform ShaderPlatform)
{
	const bool bNeedUAV = SeparateTranslucencyDimensions.NumSamples == 1 && OIT::IsSortedPixelsEnabled(ShaderPlatform);
	
	FClearValueBinding ClearValueBinding = bIsModulate ? FClearValueBinding::White : FClearValueBinding::Black;
	if (TranslucencyPass == ETranslucencyPass::TPT_TranslucencyHoldout)
	{
		ClearValueBinding = FClearValueBinding::Black;
	}

	return FRDGTextureDesc::Create2D(
		SeparateTranslucencyDimensions.Extent,
		bIsModulate ? PF_FloatR11G11B10 : PF_FloatRGBA,
		bIsModulate ? FClearValueBinding::White : FClearValueBinding::Black,
		TexCreate_RenderTargetable | TexCreate_ShaderResource | (bNeedUAV ? TexCreate_UAV : TexCreate_None),
		1,
		SeparateTranslucencyDimensions.NumSamples);
}

FRDGTextureMSAA CreatePostDOFTranslucentTexture(
	FRDGBuilder& GraphBuilder,
	ETranslucencyPass::Type TranslucencyPass,
	const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions,
	bool bIsModulate,
	EShaderPlatform ShaderPlatform)
{
	const FRDGTextureDesc Desc = GetPostDOFTranslucentTextureDesc(TranslucencyPass, SeparateTranslucencyDimensions, bIsModulate, ShaderPlatform);
	return CreateTextureMSAA(
		GraphBuilder, Desc,
		kTranslucencyColorTextureMultisampledName[int32(TranslucencyPass)],
		kTranslucencyColorTextureName[int32(TranslucencyPass)],
		bIsModulate ? GFastVRamConfig.SeparateTranslucencyModulate : GFastVRamConfig.SeparateTranslucency);
}

ETranslucentHoldoutCompositionType GetTranslucentHoldoutComposition()
{
	switch (GTranslucencyHoldoutComposition)
	{
	case 0: return ETranslucentHoldoutCompositionType::BeforeDOF;
	case 1: return ETranslucentHoldoutCompositionType::AfterDOF;
	default: return ETranslucentHoldoutCompositionType::BeforeDOF;
	}
}

void SetupDownsampledTranslucencyViewParameters(
	const FViewInfo& View,
	FIntPoint TextureExtent,
	FIntRect ViewRect,
	ETranslucencyPass::Type TranslucencyPass,
	FViewUniformShaderParameters& DownsampledTranslucencyViewParameters)
{
	DownsampledTranslucencyViewParameters = *View.CachedViewUniformShaderParameters;

	FViewMatrices ViewMatrices = View.ViewMatrices;
	FViewMatrices PrevViewMatrices = View.PrevViewInfo.ViewMatrices;
	if (TranslucencyPass == ETranslucencyPass::TPT_TranslucencyAfterMotionBlur)
	{
		// Remove jitter from this pass
		ViewMatrices.HackRemoveTemporalAAProjectionJitter();
		PrevViewMatrices.HackRemoveTemporalAAProjectionJitter();

		FBox VolumeBounds[TVC_MAX];
		View.SetupUniformBufferParameters(ViewMatrices, PrevViewMatrices, VolumeBounds, TVC_MAX, DownsampledTranslucencyViewParameters);
	}

	// Update the parts of DownsampledTranslucencyParameters which are dependent on the buffer size and view rect
	View.SetupViewRectUniformBufferParameters(
		DownsampledTranslucencyViewParameters,
		TextureExtent,
		ViewRect,
		ViewMatrices,
		PrevViewMatrices);

	// instead of using the expected ratio, use the actual dimensions to avoid rounding errors
	float ActualDownsampleX = float(ViewRect.Width()) / float(View.ViewRect.Width());
	float ActualDownsampleY = float(ViewRect.Height()) / float(View.ViewRect.Height());
	DownsampledTranslucencyViewParameters.LightProbeSizeRatioAndInvSizeRatio = FVector4f(ActualDownsampleX, ActualDownsampleY, 1.0f / ActualDownsampleX, 1.0f / ActualDownsampleY);

	DownsampledTranslucencyViewParameters.BufferToSceneTextureScale = FVector2f(1.0f / ActualDownsampleX, 1.0f / ActualDownsampleY);
}

TRDGUniformBufferRef<FTranslucentBasePassUniformParameters> CreateTranslucentBasePassUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const int32 ViewIndex,
	const FTranslucencyLightingVolumeTextures& TranslucencyLightingVolumeTextures,
	FRDGTextureRef SceneColorCopyTexture,
	const ESceneTextureSetupMode SceneTextureSetupMode,
	bool bLumenGIEnabled,
	const FOITData& OITData,
	const TArrayView<Nanite::FRasterResults>& NaniteRasterResults,
	ETranslucencyPass::Type TranslucencyPass)
{
	FTranslucentBasePassUniformParameters& BasePassParameters = *GraphBuilder.AllocParameters<FTranslucentBasePassUniformParameters>();

	const auto GetRDG = [&](const TRefCountPtr<IPooledRenderTarget>& PooledRenderTarget, ERDGTextureFlags Flags = ERDGTextureFlags::None)
	{
		return GraphBuilder.RegisterExternalTexture(PooledRenderTarget, Flags);
	};

	SetupSharedBasePassParameters(GraphBuilder, View, ViewIndex, BasePassParameters.Shared);
	SetupSceneTextureUniformParameters(GraphBuilder, View.GetSceneTexturesChecked(), View.FeatureLevel, SceneTextureSetupMode, BasePassParameters.SceneTextures);
	Substrate::BindSubstrateForwardPasslUniformParameters(GraphBuilder, View, BasePassParameters.Substrate);

	const FLightSceneProxy* SelectedForwardDirectionalLightProxy = View.ForwardLightingResources.SelectedForwardDirectionalLightProxy;
	SetupLightCloudTransmittanceParameters(GraphBuilder, Scene, View, SelectedForwardDirectionalLightProxy ? SelectedForwardDirectionalLightProxy->GetLightSceneInfo() : nullptr, BasePassParameters.ForwardDirLightCloudShadow);

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	// Material SSR
	{
		float PrevSceneColorPreExposureInvValue = 1.0f / View.PreExposure;

		if (IsHZBValid(View, EHZBType::FurthestHZB))
		{
			BasePassParameters.HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);
			FRDGTextureSRVRef PrevSceneColorTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(SystemTextures.Black));
			FIntRect PrevSceneColorViewRect = FIntRect(0, 0, 1, 1);

			if (View.PrevViewInfo.CustomSSRInput.IsValid())
			{
				PrevSceneColorTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(GetRDG(View.PrevViewInfo.CustomSSRInput.RT[0])));
				PrevSceneColorViewRect = View.PrevViewInfo.CustomSSRInput.ViewportRect;
				PrevSceneColorPreExposureInvValue = 1.0f / View.PrevViewInfo.SceneColorPreExposure;
			}
			else if (View.PrevViewInfo.TemporalAAHistory.IsValid())
			{
				FRDGTextureRef TemporalAAHistoryTexture = GetRDG(View.PrevViewInfo.TemporalAAHistory.RT[0]);
				PrevSceneColorTexture = GraphBuilder.CreateSRV(TemporalAAHistoryTexture->Desc.IsTextureArray()
					? FRDGTextureSRVDesc::CreateForSlice(TemporalAAHistoryTexture, View.PrevViewInfo.TemporalAAHistory.OutputSliceIndex)
					: FRDGTextureSRVDesc(TemporalAAHistoryTexture));
				PrevSceneColorViewRect = View.PrevViewInfo.TemporalAAHistory.ViewportRect;
				PrevSceneColorPreExposureInvValue = 1.0f / View.PrevViewInfo.SceneColorPreExposure;
			}
			else if (View.PrevViewInfo.ScreenSpaceRayTracingInput.IsValid())
			{
				PrevSceneColorTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(GetRDG(View.PrevViewInfo.ScreenSpaceRayTracingInput)));
				PrevSceneColorViewRect = View.PrevViewInfo.ViewRect;
				PrevSceneColorPreExposureInvValue = 1.0f / View.PrevViewInfo.SceneColorPreExposure;
			}

			BasePassParameters.PrevSceneColor = PrevSceneColorTexture;
			BasePassParameters.PrevSceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

			FScreenPassTextureViewportParameters PrevSceneColorParameters = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(PrevSceneColorTexture->Desc.Texture, PrevSceneColorViewRect));
			BasePassParameters.PrevSceneColorBilinearUVMin = PrevSceneColorParameters.UVViewportBilinearMin;
			BasePassParameters.PrevSceneColorBilinearUVMax = PrevSceneColorParameters.UVViewportBilinearMax;
		}
		else
		{
			BasePassParameters.HZBParameters = GetDummyHZBParameters(GraphBuilder);
			BasePassParameters.PrevSceneColor = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(SystemTextures.Black));
			BasePassParameters.PrevSceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			BasePassParameters.PrevSceneColorBilinearUVMin = FVector2f(0.0f, 0.0f);
			BasePassParameters.PrevSceneColorBilinearUVMax = FVector2f(1.0f, 1.0f);
		}

		BasePassParameters.SoftBlendingDistanceKm = FMath::Max(0.0001f, CVarVolumetricCloudSoftBlendingDistanceOnTranslucent.GetValueOnRenderThread());
		BasePassParameters.ApplyVolumetricCloudOnTransparent = 0.0f;
		BasePassParameters.VolumetricCloudColor = nullptr;
		BasePassParameters.VolumetricCloudDepth = nullptr;
		BasePassParameters.VolumetricCloudColorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		BasePassParameters.VolumetricCloudDepthSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		const bool bVRTValid = View.ViewState != nullptr && View.ViewState->VolumetricCloudRenderTarget.IsValid();
		if (bVRTValid && IsVolumetricRenderTargetEnabled() && ShouldRenderVolumetricCloud(Scene, View.Family->EngineShowFlags))
		{
			int32 VRTMode = View.ViewState->VolumetricCloudRenderTarget.GetMode();
			if (VRTMode == 1 || VRTMode == 3)
			{
				FRDGTextureRef VolumetricReconstructRT = View.ViewState->VolumetricCloudRenderTarget.GetOrCreateVolumetricTracingRT(GraphBuilder);
				if (VolumetricReconstructRT)
				{
					BasePassParameters.VolumetricCloudColor = VolumetricReconstructRT;
					BasePassParameters.VolumetricCloudDepth = View.ViewState->VolumetricCloudRenderTarget.GetOrCreateVolumetricTracingRTDepth(GraphBuilder);
					BasePassParameters.ApplyVolumetricCloudOnTransparent = 1.0f;
					BasePassParameters.VolumetricCloudColorUVScale = View.ViewState->VolumetricCloudRenderTarget.GetVolumetricTracingUVScale();
					BasePassParameters.VolumetricCloudColorUVMax = View.ViewState->VolumetricCloudRenderTarget.GetVolumetricTracingUVMax();
				}
			}
			else
			{
				const bool bShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent = ShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent(View);

				TRefCountPtr<IPooledRenderTarget> VolumetricReconstructRT = nullptr;
				if (bShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent)
				{
					VolumetricReconstructRT = View.ViewState->VolumetricCloudRenderTarget.GetVolumetricReconstructRTForCloudOnTranslucent();
				}
				else
				{
					VolumetricReconstructRT = View.ViewState->VolumetricCloudRenderTarget.GetDstVolumetricReconstructRT();
				}

				if (VolumetricReconstructRT.IsValid())
				{
					TRefCountPtr<IPooledRenderTarget> VolumetricReconstructRTDepth = View.ViewState->VolumetricCloudRenderTarget.GetDstVolumetricReconstructRTDepth();
					BasePassParameters.VolumetricCloudColor = GraphBuilder.RegisterExternalTexture(VolumetricReconstructRT);
					BasePassParameters.VolumetricCloudDepth = GraphBuilder.RegisterExternalTexture(VolumetricReconstructRTDepth);
					BasePassParameters.ApplyVolumetricCloudOnTransparent = 1.0f;
					BasePassParameters.VolumetricCloudColorUVScale = View.ViewState->VolumetricCloudRenderTarget.GetDstVolumetricReconstructUVScale();
					BasePassParameters.VolumetricCloudColorUVMax = View.ViewState->VolumetricCloudRenderTarget.GetDstVolumetricReconstructUVMax();
				}
			}
		}
		if (BasePassParameters.VolumetricCloudColor == nullptr)
		{
			BasePassParameters.VolumetricCloudColor = GSystemTextures.GetBlackAlphaOneDummy(GraphBuilder);
			BasePassParameters.VolumetricCloudDepth = GSystemTextures.GetBlackDummy(GraphBuilder);
		}

		FIntPoint ViewportOffset = View.ViewRect.Min;
		FIntPoint ViewportExtent = View.ViewRect.Size();

		// Scene render targets might not exist yet; avoids NaNs.
		FIntPoint EffectiveBufferSize = View.GetSceneTexturesConfig().Extent;
		EffectiveBufferSize.X = FMath::Max(EffectiveBufferSize.X, 1);
		EffectiveBufferSize.Y = FMath::Max(EffectiveBufferSize.Y, 1);

		if (View.PrevViewInfo.CustomSSRInput.IsValid())
		{
			ViewportOffset = View.PrevViewInfo.CustomSSRInput.ViewportRect.Min;
			ViewportExtent = View.PrevViewInfo.CustomSSRInput.ViewportRect.Size();
			EffectiveBufferSize = View.PrevViewInfo.CustomSSRInput.RT[0]->GetDesc().Extent;
		}
		else if (View.PrevViewInfo.TemporalAAHistory.IsValid())
		{
			ViewportOffset = View.PrevViewInfo.TemporalAAHistory.ViewportRect.Min;
			ViewportExtent = View.PrevViewInfo.TemporalAAHistory.ViewportRect.Size();
			EffectiveBufferSize = View.PrevViewInfo.TemporalAAHistory.RT[0]->GetDesc().Extent;
		}
		else if (View.PrevViewInfo.ScreenSpaceRayTracingInput.IsValid())
		{
			ViewportOffset = View.PrevViewInfo.ViewRect.Min;
			ViewportExtent = View.PrevViewInfo.ViewRect.Size();
			EffectiveBufferSize = View.PrevViewInfo.ScreenSpaceRayTracingInput->GetDesc().Extent;
		}

		FVector2f InvBufferSize(1.0f / float(EffectiveBufferSize.X), 1.0f / float(EffectiveBufferSize.Y));

		FVector4f ScreenPosToPixelValue(
			ViewportExtent.X * 0.5f * InvBufferSize.X,
			-ViewportExtent.Y * 0.5f * InvBufferSize.Y,
			(ViewportExtent.X * 0.5f + ViewportOffset.X) * InvBufferSize.X,
			(ViewportExtent.Y * 0.5f + ViewportOffset.Y) * InvBufferSize.Y);

		BasePassParameters.PrevScreenPositionScaleBias = ScreenPosToPixelValue;
		BasePassParameters.PrevSceneColorPreExposureInv = PrevSceneColorPreExposureInvValue;
		BasePassParameters.SSRQuality = ShouldRenderTranslucencyScreenSpaceReflections(View) ? GetSSRQuality() : 0;
	}

	// Translucency Lighting Volume
	BasePassParameters.TranslucencyLightingVolume = GetTranslucencyLightingVolumeParameters(GraphBuilder, TranslucencyLightingVolumeTextures, View);
	BasePassParameters.FrontLayerTranslucency = GetFrontLayerTranslucencyParameters(GraphBuilder, View.FrontLayerTranslucency);
	BasePassParameters.LumenTranslucencyLighting = GetLumenTranslucencyLightingParameters(GraphBuilder, View.GetLumenTranslucencyGIVolume());

	const bool bLumenGIHandlingSkylight = bLumenGIEnabled
		&& BasePassParameters.LumenTranslucencyLighting.TranslucencyGIGridSize.Z > 0;

	// Dynamically pick if we should render skylight in translucent base pass if skylights are enabled.
	const bool bShouldRenderSkylightInBasePass = Scene ? Scene->ShouldRenderSkylightInBasePass(true) : true;
	BasePassParameters.Shared.UseBasePassSkylight = bLumenGIHandlingSkylight ? 0 : bShouldRenderSkylightInBasePass;

	BasePassParameters.SceneColorCopyTexture = SystemTextures.Black;
	BasePassParameters.SceneColorCopySampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	if (SceneColorCopyTexture)
	{
		BasePassParameters.SceneColorCopyTexture = SceneColorCopyTexture;
	}

	BasePassParameters.EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));
	BasePassParameters.PreIntegratedGFTexture = GSystemTextures.PreintegratedGF->GetRHI();
	BasePassParameters.PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	OIT::SetOITParameters(GraphBuilder, View, BasePassParameters.OIT, OITData);

	// Only use blue noise resources if VSM quality is set to high
	if (IsVSMTranslucentHighQualityEnabled() || IsTranslucencyLightingVolumeUsingBlueNoise())
	{
		BasePassParameters.BlueNoise = GetBlueNoiseParameters();
	}
	else
	{
		BasePassParameters.BlueNoise = GetBlueNoiseDummyParameters();
	}

	BasePassParameters.AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricCameraMapParameters(GraphBuilder, View.ViewState);

	// Translucency pass for holdout
	BasePassParameters.TranslucencyPass = TranslucencyPass == ETranslucencyPass::TPT_TranslucencyHoldout ? 1 : 0;

	const Nanite::FRasterResults* NaniteRaster = NaniteRasterResults.IsValidIndex(ViewIndex) ? &NaniteRasterResults[ViewIndex] : nullptr;
	Nanite::SetTranslucencyParameters(GraphBuilder, Scene, View, NaniteRaster, BasePassParameters.NaniteTranslucency);

	return GraphBuilder.CreateUniformBuffer(&BasePassParameters);
}

TRDGUniformBufferRef<FTranslucentBasePassUniformParameters> CreateTranslucentBasePassUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const int32 ViewIndex,
	const FTranslucencyLightingVolumeTextures& TranslucencyLightingVolumeTextures,
	FRDGTextureRef SceneColorCopyTexture,
	const ESceneTextureSetupMode SceneTextureSetupMode,
	bool bLumenGIEnabled,
	const TArrayView<Nanite::FRasterResults>& NaniteRasterResults,
	ETranslucencyPass::Type TranslucencyPass)
{
	FOITData OITData = OIT::CreateOITData(GraphBuilder, View, OITPass_None);
	return CreateTranslucentBasePassUniformBuffer(
		GraphBuilder,
		Scene,
		View,
		ViewIndex,
		TranslucencyLightingVolumeTextures,
		SceneColorCopyTexture,
		SceneTextureSetupMode,
		bLumenGIEnabled,
		OITData,
		NaniteRasterResults,
		TranslucencyPass);
}

static FViewShaderParameters GetSeparateTranslucencyViewParameters(const FViewInfo& View, FIntPoint TextureExtent, float ViewportScale, ETranslucencyPass::Type TranslucencyPass)
{
	FViewShaderParameters ViewParameters;
	const bool bIsPostMotionBlur = (TranslucencyPass == ETranslucencyPass::TPT_TranslucencyAfterMotionBlur);

	if (ViewportScale == 1.0f && !bIsPostMotionBlur)
	{
		// We can use the existing view uniform buffers if no downsampling is required and is not in the post-motionblur pass
		ViewParameters = View.GetShaderParameters();
	}	
	else if (ViewportScale == 1.0f && bIsPostMotionBlur)
	{
		// Full-scale post-motionblur pass
		FViewUniformShaderParameters ViewUniformParameters;
		SetupPostMotionBlurTranslucencyViewParameters(View, ViewUniformParameters);

		ViewParameters.View = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(ViewUniformParameters, UniformBuffer_SingleFrame);

		if (View.bShouldBindInstancedViewUB)
		{
			FInstancedViewUniformShaderParameters LocalInstancedViewUniformShaderParameters;
			InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, ViewUniformParameters, 0);

			if (const FViewInfo* InstancedView = View.GetInstancedView())
			{
				SetupPostMotionBlurTranslucencyViewParameters(*InstancedView, ViewUniformParameters);

				InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, ViewUniformParameters, 1);
			}

			ViewParameters.InstancedView = TUniformBufferRef<FInstancedViewUniformShaderParameters>::CreateUniformBufferImmediate(
				reinterpret_cast<const FInstancedViewUniformShaderParameters&>(LocalInstancedViewUniformShaderParameters),
				UniformBuffer_SingleFrame);
		}
	}
	else
	{
		// Downsampled post-DOF or post-motionblur pass
		FViewUniformShaderParameters DownsampledTranslucencyViewParameters;
		SetupDownsampledTranslucencyViewParameters(
			View,
			TextureExtent,
			GetScaledRect(View.ViewRect, ViewportScale),
			TranslucencyPass,
			DownsampledTranslucencyViewParameters);

		ViewParameters.View = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(DownsampledTranslucencyViewParameters, UniformBuffer_SingleFrame);

		if (View.bShouldBindInstancedViewUB)
		{
			FInstancedViewUniformShaderParameters LocalInstancedViewUniformShaderParameters;
			InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, DownsampledTranslucencyViewParameters, 0);

			if (const FViewInfo* InstancedView = View.GetInstancedView())
			{
				SetupDownsampledTranslucencyViewParameters(
					*InstancedView,
					TextureExtent,
					GetScaledRect(InstancedView->ViewRect, ViewportScale),
					TranslucencyPass,
					DownsampledTranslucencyViewParameters);

				InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, DownsampledTranslucencyViewParameters, 1);
			}

			ViewParameters.InstancedView = TUniformBufferRef<FInstancedViewUniformShaderParameters>::CreateUniformBufferImmediate(
				reinterpret_cast<const FInstancedViewUniformShaderParameters&>(LocalInstancedViewUniformShaderParameters ),
				UniformBuffer_SingleFrame);
		}
	}

	return ViewParameters;
}

FViewShaderParameters GetSeparateTranslucencyViewParameters(const FViewInfo& View, const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions, ETranslucencyPass::Type TranslucencyPass)
{
	return GetSeparateTranslucencyViewParameters(View, SeparateTranslucencyDimensions.Extent, SeparateTranslucencyDimensions.Scale, TranslucencyPass);
}

static void RenderTranslucencyViewInner(
	FRDGBuilder& GraphBuilder,
	const FSceneRenderer& SceneRenderer,
	FViewInfo& View, int32 ViewIndex,
	FScreenPassTextureViewport Viewport,
	float ViewportScale,
	FRDGTextureMSAA SceneColorTexture,
	ERenderTargetLoadAction SceneColorLoadAction,
	FRDGTextureRef SceneDepthTexture,
	TRDGUniformBufferRef<FTranslucentBasePassUniformParameters> BasePassParameters,
	ETranslucencyPass::Type TranslucencyPass,
	bool bResolveColorTexture,
	bool bRenderInParallel,
	FInstanceCullingManager& InstanceCullingManager,
	const TArrayView<Nanite::FRasterResults>& NaniteRasterResults)
{
	if (!View.ShouldRenderView())
	{
		return;
	}

	if (SceneColorLoadAction == ERenderTargetLoadAction::EClear)
	{
		AddClearRenderTargetPass(GraphBuilder, SceneColorTexture.Target);
	}

	View.BeginRenderView();

	FTranslucentBasePassParameters* PassParameters = GraphBuilder.AllocParameters<FTranslucentBasePassParameters>();
	PassParameters->View = GetSeparateTranslucencyViewParameters(View, Viewport.Extent, ViewportScale, TranslucencyPass);
	PassParameters->ReflectionCapture = View.ReflectionCaptureUniformBuffer;
	PassParameters->BasePass = BasePassParameters;

	const Nanite::FRasterResults* NaniteRaster = NaniteRasterResults.IsValidIndex(ViewIndex) ? &NaniteRasterResults[ViewIndex] : nullptr;
	if (NaniteRaster)
	{
		FNaniteRasterUniformParameters* RasterParameters = GraphBuilder.AllocParameters<FNaniteRasterUniformParameters>();
		RasterParameters->PageConstants = NaniteRaster->PageConstants;
		RasterParameters->MaxNodes = NaniteRaster->MaxNodes;
		RasterParameters->MaxVisibleClusters = NaniteRaster->MaxVisibleClusters;
		RasterParameters->MaxCandidatePatches = NaniteRaster->MaxCandidatePatches;
		RasterParameters->MaxPatchesPerGroup = NaniteRaster->MaxPatchesPerGroup;
		RasterParameters->MeshPass = NaniteRaster->MeshPass;
		RasterParameters->InvDiceRate = NaniteRaster->InvDiceRate;
		RasterParameters->RenderFlags = NaniteRaster->RenderFlags;
		RasterParameters->DebugFlags = NaniteRaster->DebugFlags;
		PassParameters->NaniteRaster = GraphBuilder.CreateUniformBuffer(RasterParameters);

		FNaniteShadingUniformParameters* ShadingParameters = GraphBuilder.AllocParameters<FNaniteShadingUniformParameters>();
		ShadingParameters->ClusterPageData = Nanite::GStreamingManager.GetClusterPageDataSRV(GraphBuilder);
		ShadingParameters->HierarchyBuffer = Nanite::GStreamingManager.GetHierarchySRV(GraphBuilder);
		ShadingParameters->VisibleClustersSWHW = GraphBuilder.CreateSRV(NaniteRaster->VisibleClustersSWHW);
		ShadingParameters->AssemblyTransforms = GraphBuilder.CreateSRV(NaniteRaster->AssemblyTransforms);
		ShadingParameters->InViews = GraphBuilder.CreateSRV(NaniteRaster->ViewsBuffer);

		// Dummy parameters
		const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
		ShadingParameters->VisBuffer64 = SystemTextures.Black;
		ShadingParameters->DbgBuffer64 = SystemTextures.Black;
		ShadingParameters->DbgBuffer32 = SystemTextures.Black;
		ShadingParameters->ShadingMask = SystemTextures.Black;
		ShadingParameters->MultiViewEnabled = 0;
		ShadingParameters->MultiViewIndices = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer<uint32>(GraphBuilder));
		ShadingParameters->MultiViewRectScaleOffsets = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer<FVector4>(GraphBuilder));
		ShadingParameters->ThreadGroupData = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer<uint32>(GraphBuilder));
		ShadingParameters->ShadingBinData = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer<uint32>(GraphBuilder), PF_R32_UINT);

		PassParameters->NaniteShading = GraphBuilder.CreateUniformBuffer(ShadingParameters);

		PassParameters->NaniteIndirectArgs = NaniteRaster->RasterBinArgs;
	}
	else
	{
		PassParameters->NaniteRaster = nullptr; // GW-TODO: CreateDebugNaniteRasterUniformBuffer?
		PassParameters->NaniteShading = nullptr; // GW-TODO: CreateDebugNaniteShadingUniformBuffer?
		PassParameters->NaniteIndirectArgs = nullptr;
	}

	// Custom render passes run early in the frame before VSMs have been processed.  The VSM uniforms will be dummy at that point in the frame, so it doesn't matter which
	// ViewIndex in the VirtualShadowMapArray is referenced, but it needs to be an index that's not out of range of the array.  To handle that case, pass in an index of
	// zero, so it arbitrarily uses the first element.
	PassParameters->VirtualShadowMapSamplingParameters = SceneRenderer.VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, View.CustomRenderPass == nullptr ? ViewIndex : 0);

	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture.Target, ERenderTargetLoadAction::ELoad);
	if (TranslucencyPass != ETranslucencyPass::TPT_TranslucencyAfterMotionBlur)
	{
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);
	}

	// Only apply VRS if the translucency render target is at the same scale as the internal resolution
	const bool bUseVRS = (ViewportScale == 1.0f);
	if (bUseVRS)
	{
		PassParameters->RenderTargets.ShadingRateTexture = GVRSImageManager.GetVariableRateShadingImage(GraphBuilder, View, FVariableRateShadingImageManager::EVRSPassType::TranslucencyAll);
	}
	PassParameters->RenderTargets.ResolveRect = FResolveRect(Viewport.Rect);

	const EMeshPass::Type MeshPass = TranslucencyPassToMeshPass(TranslucencyPass);

	if (auto* Pass = View.ParallelMeshDrawCommandPasses[MeshPass])
	{
		Pass->BuildRenderingCommands(GraphBuilder, SceneRenderer.Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

		if (!View.Family->UseDebugViewPS())
		{
			if (bRenderInParallel)
			{
				GraphBuilder.AddDispatchPass(
					RDG_EVENT_NAME("Translucency(%s Parallel) %dx%d",
						TranslucencyPassToString(TranslucencyPass),
						int32(View.ViewRect.Width() * ViewportScale),
						int32(View.ViewRect.Height() * ViewportScale)),
					PassParameters,
					ERDGPassFlags::Raster,
					[Pass, PassParameters, ViewportScale](FRDGDispatchPassBuilder& DispatchPassBuilder)
				{
					Pass->Dispatch(DispatchPassBuilder, &PassParameters->InstanceCullingDrawParams, ViewportScale);
				});
			}
			else
			{
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("Translucency(%s) %dx%d",
						TranslucencyPassToString(TranslucencyPass),
						int32(View.ViewRect.Width() * ViewportScale),
						int32(View.ViewRect.Height() * ViewportScale)),
					PassParameters,
					ERDGPassFlags::Raster,
					[&View, Pass, PassParameters, ViewportScale](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					FSceneRenderer::SetStereoViewport(RHICmdList, View, ViewportScale);
					Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
				});
			}
		}
	}
	else
	{
		InstanceCullingManager.SetDummyCullingParams(GraphBuilder, PassParameters->InstanceCullingDrawParams);
	}

	if (NaniteRaster)
	{
		Nanite::RenderTranslucency(
			GraphBuilder,
			SceneRenderer,
			View,
			ViewIndex,
			Viewport,
			ViewportScale,
			SceneColorTexture,
			SceneColorLoadAction,
			SceneDepthTexture,
			PassParameters,
			BasePassParameters,
			TranslucencyPass,
			NaniteRaster->RasterBinArgs,
			NaniteRaster->TranslucencyContext
		);
	}

	if (IsMainTranslucencyPass(TranslucencyPass) && (View.SimpleElementCollector.HasAnyPrimitives() || View.bHasTranslucentViewMeshElements))
	{
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("TranslucencyElements(%s) %dx%d",
				TranslucencyPassToString(TranslucencyPass),
				int32(View.ViewRect.Width() * ViewportScale),
				int32(View.ViewRect.Height() * ViewportScale)),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, TranslucencyPass, ViewportScale] (FRHICommandList& RHICmdList)
		{
			FSceneRenderer::SetStereoViewport(RHICmdList, View, ViewportScale);

			FMeshPassProcessorRenderState DrawRenderState;
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

			View.SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::Translucent, SDPG_World);
			View.SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::Translucent, SDPG_Foreground);

			// editor and debug rendering
			if (View.bHasTranslucentViewMeshElements)
			{
				{
					QUICK_SCOPE_CYCLE_COUNTER(RenderTranslucencyParallel_SDPG_World);

					DrawDynamicMeshPass(View, RHICmdList,
						[&View, &DrawRenderState, TranslucencyPass](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
					{
						FBasePassMeshProcessor PassMeshProcessor(
							EMeshPass::Num,
							View.Family->Scene->GetRenderScene(),
							View.GetFeatureLevel(),
							&View,
							DrawRenderState,
							DynamicMeshPassContext,
							FBasePassMeshProcessor::EFlags::CanUseDepthStencil,
							TranslucencyPass);

						const uint64 DefaultBatchElementMask = ~0ull;

						for (int32 MeshIndex = 0; MeshIndex < View.ViewMeshElements.Num(); MeshIndex++)
						{
							const FMeshBatch& MeshBatch = View.ViewMeshElements[MeshIndex];
							PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
						}
					});
				}

				if (!View.Family->EngineShowFlags.CompositeEditorPrimitives)
				{
					QUICK_SCOPE_CYCLE_COUNTER(RenderTranslucencyParallel_SDPG_Foreground);

					DrawDynamicMeshPass(View, RHICmdList,
						[&View, &DrawRenderState, TranslucencyPass](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
					{
						FBasePassMeshProcessor PassMeshProcessor(
							EMeshPass::Num,
							View.Family->Scene->GetRenderScene(),
							View.GetFeatureLevel(),
							&View,
							DrawRenderState,
							DynamicMeshPassContext,
							FBasePassMeshProcessor::EFlags::CanUseDepthStencil,
							TranslucencyPass);

						const uint64 DefaultBatchElementMask = ~0ull;

						for (int32 MeshIndex = 0; MeshIndex < View.TopViewMeshElements.Num(); MeshIndex++)
						{
							const FMeshBatch& MeshBatch = View.TopViewMeshElements[MeshIndex];
							PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
						}
					});
				}
			}
		});
	}

	if (bResolveColorTexture)
	{
		AddResolveSceneColorPass(GraphBuilder, View, SceneColorTexture);
	}
}

void FDeferredShadingSceneRenderer::RenderTranslucencyInner(
	FDeferredShadingSceneRenderer& Renderer,
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FTranslucencyLightingVolumeTextures& TranslucentLightingVolumeTextures,
	FTranslucencyPassResourcesMap* OutTranslucencyResourceMap,
	FRDGTextureMSAA SharedDepthTexture,
	TArray<FViewInfo>& InViews,
	ETranslucencyView ViewsToRender,
	const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions,
	FRDGTextureRef SceneColorCopyTexture,
	ETranslucencyPass::Type TranslucencyPass,
	FInstanceCullingManager& InstanceCullingManager,
	bool bStandardTranslucentCanRenderSeparate,
	const TArrayView<Nanite::FRasterResults>& NaniteRasterResults)
{
	if (!ShouldRenderTranslucency(TranslucencyPass, InViews))
	{
		return;
	}

	const FScene* Scene = Renderer.Scene;
	const FSceneViewFamily& ViewFamily = *InViews[0].Family;

	const bool bIsModulate = TranslucencyPass == ETranslucencyPass::TPT_TranslucencyAfterDOFModulate || TranslucencyPass == ETranslucencyPass::TPT_TranslucencyStandardModulate;
	const bool bDepthTest = TranslucencyPass != ETranslucencyPass::TPT_TranslucencyAfterMotionBlur;
	const bool bRenderInParallel = IsParallelTranslucencyEnabled();
	const bool bIsScalingTranslucency = SeparateTranslucencyDimensions.Scale < 1.0f;
	const bool bIsStandardSeparatedTranslucency = bStandardTranslucentCanRenderSeparate && TranslucencyPass == ETranslucencyPass::TPT_TranslucencyStandard && ViewFamily.AllowStandardTranslucencySeparated();
	const bool bRenderInSeparateTranslucency = (IsSeparateTranslucencyEnabled(TranslucencyPass, SeparateTranslucencyDimensions.Scale) && !(TranslucencyPass == ETranslucencyPass::TPT_TranslucencyStandard || TranslucencyPass == ETranslucencyPass::TPT_TranslucencyStandardModulate)) || bIsStandardSeparatedTranslucency;
	
	// Holdout rendering 
	const bool bRenderTranslucencyHold = TranslucencyPass == ETranslucencyPass::TPT_TranslucencyHoldout;

	// Can't reference scene color in scene textures. Scene color copy is used instead.
	ESceneTextureSetupMode SceneTextureSetupMode = ESceneTextureSetupMode::All;
	EnumRemoveFlags(SceneTextureSetupMode, ESceneTextureSetupMode::SceneColor);

	// Create resources shared by each view (each view data is tiled into each of the render target resources)
	FRDGTextureMSAA SharedColorTexture = CreatePostDOFTranslucentTexture(GraphBuilder, TranslucencyPass, SeparateTranslucencyDimensions, bIsModulate, Scene->GetShaderPlatform());

	for (int32 ViewIndex = 0, NumProcessedViews = 0; ViewIndex < InViews.Num(); ++ViewIndex)
	{
		FViewInfo& View = InViews[ViewIndex];
		const ETranslucencyView TranslucencyView = GetTranslucencyView(View);

		if (!EnumHasAnyFlags(TranslucencyView, ViewsToRender))
		{
			continue;
		}

		// We run separate and composited translucent only when the view is NOT under water.
		// When under water, we render each translucency pass in forward on the water buffer itself.
		const bool bViewIsUnderWater = EnumHasAnyFlags(TranslucencyView, ETranslucencyView::UnderWater);
		if (bRenderInSeparateTranslucency && !bViewIsUnderWater)
		{

			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1, "View%d", ViewIndex);

			FIntRect ScaledViewRect = GetScaledRect(View.ViewRect, SeparateTranslucencyDimensions.Scale);

			const FScreenPassTextureViewport SeparateTranslucencyViewport = SeparateTranslucencyDimensions.GetInstancedStereoViewport(View);
			const bool bCompositeBackToSceneColor = (IsMainTranslucencyPass(TranslucencyPass) && !bIsStandardSeparatedTranslucency);
			const bool bLumenGIEnabled = Renderer.GetViewPipelineState(View).DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen;

			/** Separate translucency color is either composited immediately or later during post processing. If done immediately, it's because the view doesn't support
			 *  compositing (e.g. we're rendering an underwater view) or because we're downsampling the main translucency pass. In this case, we use a local set of
			 *  textures instead of the external ones passed in.
			 */
			FRDGTextureMSAA SeparateTranslucencyColorTexture = SharedColorTexture;

			// NOTE: No depth test on post-motionblur translucency
			FRDGTextureMSAA SeparateTranslucencyDepthTexture;
			if (bDepthTest)
			{
				SeparateTranslucencyDepthTexture = SharedDepthTexture;
			}

			const ERenderTargetLoadAction SeparateTranslucencyColorLoadAction = NumProcessedViews == 0 || View.Family->bMultiGPUForkAndJoin
				? ERenderTargetLoadAction::EClear
				: ERenderTargetLoadAction::ELoad;

			FOITData OITData = OIT::CreateOITData(GraphBuilder, View, OITPass_SeparateTranslucency);

			const bool bRenderTranslucencyViewInner = OIT::IsSortedPixelsEnabled(View) ? !bIsModulate : true;
			if (bRenderTranslucencyViewInner)
			{
				RenderTranslucencyViewInner(
					GraphBuilder,
					Renderer,
					View, ViewIndex,
					SeparateTranslucencyViewport,
					SeparateTranslucencyDimensions.Scale,
					SeparateTranslucencyColorTexture,
					SeparateTranslucencyColorLoadAction,
					SeparateTranslucencyDepthTexture.Target,
					CreateTranslucentBasePassUniformBuffer(GraphBuilder, Scene, View, ViewIndex, TranslucentLightingVolumeTextures, SceneColorCopyTexture, SceneTextureSetupMode, bLumenGIEnabled, OITData, NaniteRasterResults, TranslucencyPass),
					TranslucencyPass,
					!bCompositeBackToSceneColor,
					bRenderInParallel,
					InstanceCullingManager,
					NaniteRasterResults);

				{
					FTranslucencyPassResources& TranslucencyPassResources = OutTranslucencyResourceMap->Get(ViewIndex, TranslucencyPass);
					TranslucencyPassResources.ViewRect = ScaledViewRect;
					TranslucencyPassResources.ColorTexture = SharedColorTexture;
					TranslucencyPassResources.DepthTexture = SharedDepthTexture;
				}
			}

			if (OIT::IsSortedPixelsEnabled(View) && !!(OITData.PassType & OITPass_SeparateTranslucency))
			{
				// * If this pass it not modulate (i.e., TranslucencyPass != ETranslucencyPass::TPT_TranslucencyXXXModulate), then: 
				//   * Either the result is directly composed to the the color target. 
				//   * Or the luminance is composed to the color target, and the transmittance composed into a new allocated 'modulate' target
				// * If this pass is modulate (i.e., TranslucencyPass == ETranslucencyPass::TPT_TranslucencyXXXModulate), then we only redirect SharedColorTexture to the modulate target, allocated during the previous pass
				if (!bIsModulate)
				{
					if (bCompositeBackToSceneColor || TranslucencyPass == ETranslucencyPass::TPT_TranslucencyAfterMotionBlur)
					{
						// Compose the final result (Luminance + Transmittance) to the color target
						OIT::AddOITComposePass(GraphBuilder, View, OITData, SeparateTranslucencyColorTexture.Target);
					}
					else
					{
						ETranslucencyPass::Type ModulateTranslucencyPass = ETranslucencyPass::TPT_MAX;
						if (TranslucencyPass == ETranslucencyPass::TPT_TranslucencyStandard)
						{
							ModulateTranslucencyPass = ETranslucencyPass::TPT_TranslucencyStandardModulate;
						}
						else if (TranslucencyPass == ETranslucencyPass::TPT_TranslucencyAfterDOF)
						{
							ModulateTranslucencyPass = ETranslucencyPass::TPT_TranslucencyAfterDOFModulate;
						}

						// * Compose the luminance to the color target
						// * Allocate a new target for the transmittance
						// * Compose the transmittance to the transmittance target
						if (TranslucencyPass != ETranslucencyPass::TPT_MAX)
						{
							FRDGTextureMSAA ModulateTexture = CreatePostDOFTranslucentTexture(GraphBuilder, ModulateTranslucencyPass, SeparateTranslucencyDimensions, true /*bIsModulate*/, Scene->GetShaderPlatform());
							OIT::AddOITComposePass(GraphBuilder, View, OITData, SeparateTranslucencyColorTexture.Target, ModulateTexture.Target);

							FTranslucencyPassResources& ModulateTranslucencyPassResources = OutTranslucencyResourceMap->Get(ViewIndex, ModulateTranslucencyPass);
							ModulateTranslucencyPassResources.ViewRect = ScaledViewRect;
							ModulateTranslucencyPassResources.ColorTexture = nullptr;
							ModulateTranslucencyPassResources.ColorModulateTexture = ModulateTexture;
							ModulateTranslucencyPassResources.DepthTexture = SharedDepthTexture;
						}
					}
				}
				else
				{
					// Retrieve the modulate texture, which was allocated and fill-in during the previous (non-modulate) transparentcy pass.
					check(!bCompositeBackToSceneColor);
					FTranslucencyPassResources& ModulateTranslucencyPassResources = OutTranslucencyResourceMap->Get(ViewIndex, TranslucencyPass);
					SharedColorTexture = ModulateTranslucencyPassResources.ColorModulateTexture;
					ModulateTranslucencyPassResources.ViewRect = ScaledViewRect;
					ModulateTranslucencyPassResources.DepthTexture = SharedDepthTexture;
				}
			}

			if (bCompositeBackToSceneColor)
			{
				FRDGTextureRef SeparateTranslucencyDepthResolve = nullptr;
				FRDGTextureRef SceneDepthResolve = nullptr;
				if (TranslucencyPass != ETranslucencyPass::TPT_TranslucencyAfterMotionBlur)
				{
					::AddResolveSceneDepthPass(GraphBuilder, View, SeparateTranslucencyDepthTexture);

					SeparateTranslucencyDepthResolve = SeparateTranslucencyDepthTexture.Resolve;
					SceneDepthResolve = SceneTextures.Depth.Resolve;
				}

				FTranslucencyPassResources& TranslucencyPassResources = OutTranslucencyResourceMap->Get(ViewIndex, TranslucencyPass);

				FTranslucencyComposition TranslucencyComposition;
				TranslucencyComposition.Operation = FTranslucencyComposition::EOperation::ComposeToExistingSceneColor;
				TranslucencyComposition.SceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, FScreenPassTexture(SceneTextures.Color.Target, View.ViewRect));
				TranslucencyComposition.SceneDepth = FScreenPassTexture(SceneTextures.Depth.Resolve, View.ViewRect);
				TranslucencyComposition.OutputViewport = FScreenPassTextureViewport(SceneTextures.Depth.Resolve, View.ViewRect);

				FScreenPassTexture UpscaledTranslucency = TranslucencyComposition.AddPass(
					GraphBuilder, View, TranslucencyPassResources);

				ensure(View.ViewRect == UpscaledTranslucency.ViewRect);
				ensure(UpscaledTranslucency.Texture == SceneTextures.Color.Target);

				//Invalidate.
				TranslucencyPassResources = FTranslucencyPassResources();
				TranslucencyPassResources.Pass = TranslucencyPass;
			}
			else
			{
				if (TranslucencyPass == ETranslucencyPass::TPT_TranslucencyAfterDOFModulate)
				{
					FTranslucencyPassResources& TranslucencyPassResources = OutTranslucencyResourceMap->Get(ViewIndex, ETranslucencyPass::TPT_TranslucencyAfterDOF);
					ensure(TranslucencyPassResources.ViewRect == ScaledViewRect);
					ensure(TranslucencyPassResources.DepthTexture == SharedDepthTexture);
					TranslucencyPassResources.ColorModulateTexture = SharedColorTexture;
				}
				else if (TranslucencyPass == ETranslucencyPass::TPT_TranslucencyStandardModulate)
				{
					FTranslucencyPassResources& TranslucencyPassResources = OutTranslucencyResourceMap->Get(ViewIndex, ETranslucencyPass::TPT_TranslucencyStandard);
					ensure(TranslucencyPassResources.ViewRect == ScaledViewRect);
					ensure(TranslucencyPassResources.DepthTexture == SharedDepthTexture);
					TranslucencyPassResources.ColorModulateTexture = SharedColorTexture;
				}
				else
				{
					check(!bIsModulate);
				}
			}

			++NumProcessedViews;
		}
		else if (bRenderTranslucencyHold)
		{
			// Main logic:
			// 1. Copy the holdout background visibility before any translucent pass to the translucent holdout texture.
			// 2. Run the TranslucencyViewInner logic to accumulate the background visibility (bv) and path throughput (pt) from back to close.
			// 3. Copy back the alpha channel of the SharedColorTexture.
			// Notes:
			// Since alpha will be polluted by alpha holdout blending mode, we cannot directly compose onto the scene color alpha.
			// with SceneColor.A = SceneColor.A * pt + bv.
			// E.g., alphaholdout material in front of a translucent material.
			// Two step direct compose: bv = HoldoutOpacity*(1 - TranslucentOpacity)
			// This three step copy compose: bv = HoldoutOpacity

			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1, "View%d", ViewIndex);
			
			const FScreenPassTextureViewport SeparateTranslucencyViewport = SeparateTranslucencyDimensions.GetInstancedStereoViewport(View);
			const bool bLumenGIEnabled = Renderer.GetViewPipelineState(View).DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen;

			/** Separate translucency color is either composited immediately or later during post processing. If done immediately, it's because the view doesn't support
			 *  compositing (e.g. we're rendering an underwater view) or because we're downsampling the main translucency pass. In this case, we use a local set of
			 *  textures instead of the external ones passed in.
			 */
			FRDGTextureMSAA SeparateTranslucencyColorTexture = SharedColorTexture;

			{
				FTranslucencyPassResources& TranslucencyPassResources = OutTranslucencyResourceMap->Get(ViewIndex, TranslucencyPass);
				FIntRect CopyRect = TranslucencyPassResources.ViewRect == FIntRect(0,0,0,0) ?  View.ViewRect : TranslucencyPassResources.ViewRect;
				AddCopyBackgroundVisibilityPass(GraphBuilder, View, SceneColorCopyTexture/*SceneTextures.Color.Resolve*/, SeparateTranslucencyColorTexture.Target,
					CopyRect, FCopyBackgroundVisibilityPS::EVisibilityCopyType::FromSceneColor);
			}

			// NOTE: We need to read the depth texture for final fog accumulation, but
			// in the current phase, we don't need depth texture.
			const bool bAlphaHoldoutNeedDepthTest = true;
			FRDGTextureMSAA SeparateTranslucencyDepthTexture;
			if (bAlphaHoldoutNeedDepthTest)
			{
				SeparateTranslucencyDepthTexture = SharedDepthTexture;
			}

			// No scale is needed
			constexpr float ViewportScale = 1.0f;
			// Simply load as we have already initialized the texture.
			constexpr ERenderTargetLoadAction SeparateTranslucencyColorLoadAction = ERenderTargetLoadAction::ELoad;
			constexpr bool bResolveColorTexture = false;

			FOITData OITData = OIT::CreateOITData(GraphBuilder, View, OITPass_SeparateTranslucency);

			RenderTranslucencyViewInner(
				GraphBuilder,
				Renderer,
				View, ViewIndex,
				SeparateTranslucencyViewport,
				ViewportScale,
				SeparateTranslucencyColorTexture,
				SeparateTranslucencyColorLoadAction,
				SeparateTranslucencyDepthTexture.Target,
				CreateTranslucentBasePassUniformBuffer(GraphBuilder, Scene, View, ViewIndex, TranslucentLightingVolumeTextures, SceneColorCopyTexture, SceneTextureSetupMode, bLumenGIEnabled, OITData, NaniteRasterResults, TranslucencyPass),
				TranslucencyPass,
				bResolveColorTexture,
				bRenderInParallel,
				InstanceCullingManager,
				NaniteRasterResults);

			{
				FTranslucencyPassResources& TranslucencyPassResources = OutTranslucencyResourceMap->Get(ViewIndex, TranslucencyPass);
				TranslucencyPassResources.ViewRect = View.ViewRect;
				TranslucencyPassResources.ColorTexture = SharedColorTexture;
				TranslucencyPassResources.DepthTexture = SharedDepthTexture;
			}

			if (OITData.PassType & OITPass_SeparateTranslucency)
			{
				OIT::AddOITComposePass(GraphBuilder, View, OITData, SeparateTranslucencyColorTexture.Target);
			}

			::AddResolveSceneDepthPass(GraphBuilder, View, SeparateTranslucencyDepthTexture);

			if (GetTranslucentHoldoutComposition() == ETranslucentHoldoutCompositionType::BeforeDOF)
			{
				FTranslucencyPassResources& TranslucencyPassResources = OutTranslucencyResourceMap->Get(ViewIndex, TranslucencyPass);

				AddTranslucentHoldoutToSceneColorAlphaPass(GraphBuilder, View, SceneTextures.Color.Target, SceneTextures.Depth.Resolve, TranslucencyPassResources);
			}

			++NumProcessedViews;
		}
		else
		{
			// When rendering translucent meshes under water, we skip modulate passes which are only required when compositing separate translucency passes from render target.
			const bool bSkipPass = bViewIsUnderWater && bIsModulate; 
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1 && !bSkipPass, "View%d", ViewIndex);
			if (bSkipPass)
			{
				return;
			}

			const ERenderTargetLoadAction SceneColorLoadAction = ERenderTargetLoadAction::ELoad;
			const FScreenPassTextureViewport Viewport(SceneTextures.Color.Target, View.ViewRect);
			const float ViewportScale = 1.0f;
			const bool bResolveColorTexture = false;
			const bool bLumenGIEnabled = Renderer.GetViewPipelineState(View).DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen;

			FOITData OITData = OIT::CreateOITData(GraphBuilder, View, OITPass_RegularTranslucency);

			RenderTranslucencyViewInner(
				GraphBuilder,
				Renderer,
				View, ViewIndex,
				Viewport,
				ViewportScale,
				SceneTextures.Color,
				SceneColorLoadAction,
				SceneTextures.Depth.Target,
				CreateTranslucentBasePassUniformBuffer(GraphBuilder, Scene, View, ViewIndex, TranslucentLightingVolumeTextures, SceneColorCopyTexture, SceneTextureSetupMode, bLumenGIEnabled, OITData, NaniteRasterResults, TranslucencyPass),
				TranslucencyPass,
				bResolveColorTexture,
				bRenderInParallel,
				InstanceCullingManager,
				NaniteRasterResults);

			if (OITData.PassType & OITPass_RegularTranslucency)
			{
				OIT::AddOITComposePass(GraphBuilder, View, OITData, SceneTextures.Color.Target);
			}
		}
	}
}

bool IsStandardTranslucencyPassSeparated()
{
	static const auto TranslucencyStandardSeparatedCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Translucency.StandardSeparated"));
	return TranslucencyStandardSeparatedCVar && TranslucencyStandardSeparatedCVar->GetValueOnAnyThread() != 0;
}

bool ShouldDrawInTranslucentBasePass(const FMaterial& Material, ETranslucencyPass::Type TranslucencyPass, float AutoBeforeDOFTranslucencyBoundary)
{
	// Determine the mesh's material and blend mode.
	const EBlendMode BlendMode = Material.GetBlendMode();
	const bool bIsTranslucent = IsTranslucentBlendMode(BlendMode);
	if (!bIsTranslucent)
	{
		return false;
	}

	bool bShouldDraw = false;

	if (!Material.IsDeferredDecal())
	{
		switch (TranslucencyPass)
		{
		case ETranslucencyPass::TPT_TranslucencyStandard:
			bShouldDraw = !Material.IsTranslucencyAfterDOFEnabled() && !Material.IsTranslucencyAfterMotionBlurEnabled();
			if (AutoBeforeDOFTranslucencyBoundary > 0.0f || UPSOPrecacheSettingsManager::GetSettings().bTranslucencyStandardBeforeDOF)
			{
				bShouldDraw = bShouldDraw || (Material.IsTranslucencyAfterDOFEnabled() && BlendMode != BLEND_ColoredTransmittanceOnly);
				bShouldDraw = bShouldDraw || (Material.IsTranslucencyAfterDOFEnabled() && (Material.IsDualBlendingEnabled() || IsModulateBlendMode(BlendMode)));
			}
			break;

		case ETranslucencyPass::TPT_TranslucencyStandardModulate:
			bShouldDraw = !Material.IsTranslucencyAfterDOFEnabled() && !Material.IsTranslucencyAfterMotionBlurEnabled()
				&& (Material.IsDualBlendingEnabled() || IsModulateBlendMode(BlendMode));
			break;

		case ETranslucencyPass::TPT_TranslucencyAfterDOF:
			bShouldDraw = Material.IsTranslucencyAfterDOFEnabled() && BlendMode != BLEND_ColoredTransmittanceOnly;
			break;

			// only dual blended or modulate surfaces need background modulation
		case ETranslucencyPass::TPT_TranslucencyAfterDOFModulate:
			bShouldDraw = Material.IsTranslucencyAfterDOFEnabled() && (Material.IsDualBlendingEnabled() || IsModulateBlendMode(BlendMode));
			break;

		case ETranslucencyPass::TPT_TranslucencyAfterMotionBlur:
			bShouldDraw = Material.IsTranslucencyAfterMotionBlurEnabled();
			break;

		case ETranslucencyPass::TPT_TranslucencyHoldout:
			bShouldDraw = true;
			break;

		case ETranslucencyPass::TPT_AllTranslucency:
			bShouldDraw = true;
			break;
		}
	}

	return bShouldDraw;
}

static bool ShouldRenderTranslucentView(const FViewInfo& View, ETranslucencyView TranslucencyView, ETranslucencyView ViewsToRender)
{
	return View.ShouldRenderView() && EnumHasAnyFlags(TranslucencyView, ViewsToRender);
};

void FDeferredShadingSceneRenderer::RenderTranslucency(
	FDeferredShadingSceneRenderer& Renderer,
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	const FTranslucencyLightingVolumeTextures& TranslucentLightingVolumeTextures,
	FTranslucencyPassResourcesMap* OutTranslucencyResourceMap,
	TArray<FViewInfo>& InViews,
	ETranslucencyView ViewsToRender,
	const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions,
	FInstanceCullingManager& InstanceCullingManager,
	bool bStandardTranslucentCanRenderSeparate,
	const FSingleLayerWaterPrePassResult* InSingleLayerWaterPrePassResult,
	FRDGTextureMSAA& OutSharedDepthTexture,
	const TArrayView<Nanite::FRasterResults>& NaniteRasterResults)
{
	if (!EnumHasAnyFlags(ViewsToRender, ETranslucencyView::UnderWater | ETranslucencyView::AboveWater))
	{
		return;
	}

	const FSceneViewFamily& ViewFamily = *InViews[0].Family;

	RDG_EVENT_SCOPE_STAT(GraphBuilder, Translucency, "RenderTranslucency");
	DynamicRenderScaling::FRDGScope DynamicTranslucencyResolutionScope(GraphBuilder, GDynamicTranslucencyResolution);

	FRDGTextureRef SceneColorCopyTexture = nullptr;
	const bool bIsTranslucentHoldoutEnabled = IsPrimitiveAlphaHoldoutEnabledForAnyView(InViews);

	if (EnumHasAnyFlags(ViewsToRender, ETranslucencyView::AboveWater))
	{
		SceneColorCopyTexture = AddCopySceneColorPass(GraphBuilder, InViews, SceneTextures.Color, /*WithAlpha*/ bIsTranslucentHoldoutEnabled, /*bSkipIfUnderwater*/ true, /*bEncodeSceneColorCopy*/ true);
	}

	// Create a shared depth texture at the correct resolution.
	const bool bIsScalingTranslucency = SeparateTranslucencyDimensions.Scale != 1.0f;
	if (bIsScalingTranslucency)
	{
		const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			SeparateTranslucencyDimensions.Extent,
			PF_DepthStencil,
			FClearValueBinding::DepthFar,
			TexCreate_DepthStencilTargetable | TexCreate_ShaderResource,
			1,
			SeparateTranslucencyDimensions.NumSamples);

		OutSharedDepthTexture = CreateTextureMSAA(
			GraphBuilder, Desc,
			TEXT("Translucency.DepthMS"),
			TEXT("Translucency.Depth"),
			GFastVRamConfig.SeparateTranslucencyModulate); // TODO: this should be SeparateTranslucency, but is what the code was doing

		// Downscale the depth buffer for each individual view, but shared accross all translucencies.
		for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
		{
			FViewInfo& View = InViews[ViewIndex];
			const ETranslucencyView TranslucencyView = GetTranslucencyView(View);

			if (!ShouldRenderTranslucentView(View, TranslucencyView, ViewsToRender))
			{
				continue;
			}

			const FScreenPassTextureViewport SeparateTranslucencyViewport = SeparateTranslucencyDimensions.GetInstancedStereoViewport(View);
			AddDownsampleDepthPass(
				GraphBuilder, View,
				FScreenPassTexture(SceneTextures.Depth.Resolve, View.ViewRect),
				FScreenPassRenderTarget(OutSharedDepthTexture.Target, SeparateTranslucencyViewport.Rect, ViewIndex == 0 ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad),
				EDownsampleDepthFilter::Point);
		}
	}
	else
	{
		// Uses the existing depth buffer for depth testing the translucency.
		OutSharedDepthTexture = SceneTextures.Depth;
	}

	if (ViewFamily.AllowTranslucencyAfterDOF())
	{
		RenderTranslucencyInner(Renderer, GraphBuilder, SceneTextures, TranslucentLightingVolumeTextures, OutTranslucencyResourceMap, OutSharedDepthTexture, InViews, ViewsToRender, SeparateTranslucencyDimensions, SceneColorCopyTexture, ETranslucencyPass::TPT_TranslucencyStandard, InstanceCullingManager, bStandardTranslucentCanRenderSeparate, NaniteRasterResults);
		if (ViewFamily.AllowStandardTranslucencySeparated() && bStandardTranslucentCanRenderSeparate)
		{
			RenderTranslucencyInner(Renderer, GraphBuilder, SceneTextures, TranslucentLightingVolumeTextures, OutTranslucencyResourceMap, OutSharedDepthTexture, InViews, ViewsToRender, SeparateTranslucencyDimensions, SceneColorCopyTexture, ETranslucencyPass::TPT_TranslucencyStandardModulate, InstanceCullingManager, bStandardTranslucentCanRenderSeparate, NaniteRasterResults);
		}

		if (GetHairStrandsComposition() == EHairStrandsCompositionType::AfterTranslucentBeforeTranslucentAfterDOF)
		{
			RenderHairComposition(GraphBuilder, InViews, SceneTextures.Color.Target, SceneTextures.Depth.Target, SceneTextures.Velocity, *OutTranslucencyResourceMap, InSingleLayerWaterPrePassResult);
		}
		RenderTranslucencyInner(Renderer, GraphBuilder, SceneTextures, TranslucentLightingVolumeTextures, OutTranslucencyResourceMap, OutSharedDepthTexture, InViews, ViewsToRender, SeparateTranslucencyDimensions, SceneColorCopyTexture, ETranslucencyPass::TPT_TranslucencyAfterDOF, InstanceCullingManager, bStandardTranslucentCanRenderSeparate, NaniteRasterResults);
		RenderTranslucencyInner(Renderer, GraphBuilder, SceneTextures, TranslucentLightingVolumeTextures, OutTranslucencyResourceMap, OutSharedDepthTexture, InViews, ViewsToRender, SeparateTranslucencyDimensions, SceneColorCopyTexture, ETranslucencyPass::TPT_TranslucencyAfterDOFModulate, InstanceCullingManager, bStandardTranslucentCanRenderSeparate, NaniteRasterResults);
		RenderTranslucencyInner(Renderer, GraphBuilder, SceneTextures, TranslucentLightingVolumeTextures, OutTranslucencyResourceMap, OutSharedDepthTexture, InViews, ViewsToRender, SeparateTranslucencyDimensions, SceneColorCopyTexture, ETranslucencyPass::TPT_TranslucencyAfterMotionBlur, InstanceCullingManager, bStandardTranslucentCanRenderSeparate, NaniteRasterResults);
	}
	else // Otherwise render translucent primitives in a single bucket.
	{
		RenderTranslucencyInner(Renderer, GraphBuilder, SceneTextures, TranslucentLightingVolumeTextures, OutTranslucencyResourceMap, OutSharedDepthTexture, InViews, ViewsToRender, SeparateTranslucencyDimensions, SceneColorCopyTexture, ETranslucencyPass::TPT_AllTranslucency, InstanceCullingManager, bStandardTranslucentCanRenderSeparate, NaniteRasterResults);
	}

	if (bIsTranslucentHoldoutEnabled && SceneColorCopyTexture)
	{
		// Render the translucent holdout background visibility to the alpha channel of SceneColor.
		RenderTranslucencyInner(Renderer, GraphBuilder, SceneTextures, TranslucentLightingVolumeTextures, OutTranslucencyResourceMap, OutSharedDepthTexture, InViews, ViewsToRender, SeparateTranslucencyDimensions, SceneColorCopyTexture, ETranslucencyPass::TPT_TranslucencyHoldout, InstanceCullingManager, bStandardTranslucentCanRenderSeparate, NaniteRasterResults);
	}
}


void FDeferredShadingSceneRenderer::UpscaleTranslucencyIfNeeded(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	ETranslucencyView ViewsToRender,
	FTranslucencyPassResourcesMap* OutTranslucencyResourceMap,
	FRDGTextureMSAA& InSharedDepthTexture)
{
	if (!EnumHasAnyFlags(ViewsToRender, ETranslucencyView::UnderWater | ETranslucencyView::AboveWater))
	{
		return;
	}

	bool bUpscalePostDOFTranslucency = true;
	FRDGTextureRef SharedUpscaledPostDOFTranslucencyColor = nullptr;
	if (bUpscalePostDOFTranslucency)
	{
		const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			SceneTextures.Color.Resolve->Desc.Extent,
			PF_FloatRGBA,
			FClearValueBinding::Black,
			TexCreate_RenderTargetable | TexCreate_ShaderResource);

		SharedUpscaledPostDOFTranslucencyColor = GraphBuilder.CreateTexture(
			Desc, TEXT("Translucency.PostDOF.UpscaledColor"));
	}

	// Upscale to full res.
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];
		const ETranslucencyView TranslucencyView = GetTranslucencyView(View);

		if (!ShouldRenderTranslucentView(View, TranslucencyView, ViewsToRender))
		{
			continue;
		}

		// Upscale the responsive AA into original depth buffer.
		bool bUpscaleResponsiveAA = (
			IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod) &&
			InSharedDepthTexture.Target != SceneTextures.Depth.Target);
		if (bUpscaleResponsiveAA)
		{
			const FScreenPassTextureViewport SeparateTranslucencyViewport = SeparateTranslucencyDimensions.GetInstancedStereoViewport(View);
			AddUpsampleResponsiveAAPass(
				GraphBuilder,
				View,
				FScreenPassTexture(InSharedDepthTexture.Target, SeparateTranslucencyViewport.Rect),
				/* OutputDepthTexture = */ SceneTextures.Depth.Target);
		}

		FTranslucencyPassResources& TranslucencyPassResources = OutTranslucencyResourceMap->Get(ViewIndex, ETranslucencyPass::TPT_TranslucencyAfterDOF);
		if (SharedUpscaledPostDOFTranslucencyColor && TranslucencyPassResources.IsValid() && TranslucencyPassResources.ViewRect.Size() != View.ViewRect.Size() && GetMainTAAPassConfig(View) != EMainTAAPassConfig::TSR)
		{
			FTranslucencyComposition TranslucencyComposition;
			TranslucencyComposition.Operation = FTranslucencyComposition::EOperation::UpscaleOnly;
			TranslucencyComposition.SceneDepth = FScreenPassTexture(SceneTextures.Depth.Resolve, View.ViewRect);
			TranslucencyComposition.OutputViewport = FScreenPassTextureViewport(SceneTextures.Depth.Resolve, View.ViewRect);

			FScreenPassTexture UpscaledTranslucency = TranslucencyComposition.AddPass(
				GraphBuilder, View, TranslucencyPassResources);

			TranslucencyPassResources.ViewRect = UpscaledTranslucency.ViewRect;
			TranslucencyPassResources.ColorTexture = FRDGTextureMSAA(UpscaledTranslucency.Texture);
			TranslucencyPassResources.DepthTexture = FRDGTextureMSAA();
		}
	}
}

FRHIBlendState* GetTranslucentBlendState(const FMaterial& Material, ETranslucencyPass::Type InTranslucencyPassType)
{
	FRHIBlendState* BlendState = nullptr;

	if (Material.IsSubstrateMaterial())
	{
		if (Material.IsDualBlendingEnabled())
		{
			if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyStandard || InTranslucencyPassType == ETranslucencyPass::TPT_AllTranslucency)
			{
				// If we are in the transparency pass (before DoF) we do standard dual blending, and the alpha gets ignored
				// Blend by putting add in target 0 and multiply by background in target 1.
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Source1Color, BO_Add, BF_One, BF_Source1Alpha>::GetRHI();
			}
			else if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterDOF)
			{
				// In the separate pass (after DoF), we want let alpha pass through, and then multiply our color modulation in the after DoF Modulation pass.
				// Alpha is BF_Zero for source and BF_One for dest, which leaves alpha unchanged
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Source1Color, BO_Add, BF_Zero, BF_One>::GetRHI();
			}
			else if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterDOFModulate || InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyStandardModulate)
			{
				// In the separate pass (after DoF) modulate, we want to only darken the target by our multiplication term, and ignore the addition term.
				// For regular dual blending, our function is:
				//     FrameBuffer = MRT0 + MRT1 * FrameBuffer;
				// So we can just remove the MRT0 component and it will modulate as expected.
				// Alpha we will leave unchanged.
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_Source1Color, BO_Add, BF_Zero, BF_One>::GetRHI();
			}
			else if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterMotionBlur)
			{
				// We don't actually currently support color modulation in the post-motion blur pass at the moment, so just do the same as post-DOF for now
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Source1Color, BO_Add, BF_Zero, BF_One>::GetRHI();
			}
		}
		else
		{
			if (Material.GetBlendMode() == BLEND_ColoredTransmittanceOnly)
			{
				// Modulate with the existing scene color, preserve destination alpha.
				BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_DestColor, BF_Zero>::GetRHI();
			}
			else if (Material.GetBlendMode() == BLEND_AlphaHoldout)
			{
				// Blend by holding out the matte shape of the source alpha
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI();
			}
			else
			{
				// We always use pre-multiplied alpha for translucent rendering, that works for any surface with additive color.
				// If a material was requesting dual source blending, the shader will use static platform knowledge to convert colored transmittance to a grey scale transmittance.
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
			}
		}

		if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyHoldout)
		{
			// We render translucent primitives from far to close. For each translucent primitive, 
			// we output the background visibility (bv) contribution in rgb channel, and the path throughput (pt) contribution in the alpha channel
			// E.g., alpha holdout blending has:
			// Src0 = (Opacity, 1 - Opacity)
			// The blend state allows the accumulation of
			// bv = Src0.rgb + (1 - Opacity) * bv
			// pt = pt * (1 - Opacity)
			BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
		}
	}
	else if (Material.GetShadingModels().HasShadingModel(MSM_ThinTranslucent))
	{
		// Special case for dual blending, which is not exposed as a parameter in the material editor
		if (Material.IsDualBlendingEnabled())
		{
			if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyStandard || InTranslucencyPassType == ETranslucencyPass::TPT_AllTranslucency)
			{
				// If we are in the transparency pass (before DoF) we do standard dual blending, and the alpha gets ignored

				// Blend by putting add in target 0 and multiply by background in target 1.
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Source1Color, BO_Add, BF_One, BF_Source1Alpha>::GetRHI();
			}
			else if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterDOF)
			{
				// In the separate pass (after DoF), we want let alpha pass through, and then multiply our color modulation in the after DoF Modulation pass.
				// Alpha is BF_Zero for source and BF_One for dest, which leaves alpha unchanged
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Source1Color, BO_Add, BF_Zero, BF_One>::GetRHI();
			}
			else if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterDOFModulate || InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyStandardModulate)
			{
				// In the separate pass (after DoF) modulate, we want to only darken the target by our multiplication term, and ignore the addition term.
				// For regular dual blending, our function is:
				//     FrameBuffer = MRT0 + MRT1 * FrameBuffer;
				// So we can just remove the MRT0 component and it will modulate as expected.
				// Alpha we will leave unchanged.
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_Source1Color, BO_Add, BF_Zero, BF_One>::GetRHI();
			}
			else if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterMotionBlur)
			{
				// We don't actually currently support color modulation in the post-motion blur pass at the moment, so just do the same as post-DOF for now
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Source1Color, BO_Add, BF_Zero, BF_One>::GetRHI();
			}
		}
		else
		{
			// If unsupported, we still use pre-multiplied alpha but the shader will use the variation converting color transmittance to a grey scale transmittance.
			BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
		}

		if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyHoldout)
		{
			BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
		}
	}
	else
	{
		switch (Material.GetBlendMode())
		{
		default:
		case BLEND_Opaque:
			// Opaque materials are rendered together in the base pass, where the blend state is set at a higher level
			break;
		case BLEND_Masked:
			// Masked materials are rendered together in the base pass, where the blend state is set at a higher level
			break;
		case BLEND_Translucent:
		case BLEND_TranslucentColoredTransmittance:	// When Substrate is disabled, this falls back to simple Translucency.
			if (InTranslucencyPassType != ETranslucencyPass::TPT_TranslucencyHoldout)
			{
				// Note: alpha channel used by separate translucency, storing how much of the background should be added when doing the final composite
				// The Alpha channel is also used by non-separate translucency when rendering to scene captures, which store the final opacity
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
			}
			else
			{
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
			}
			break;
		case BLEND_Additive:
			// Add to the existing scene color
			// Note: alpha channel used by separate translucency, storing how much of the background should be added when doing the final composite
			// The Alpha channel is also used by non-separate translucency when rendering to scene captures, which store the final opacity
			BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
			break;
		case BLEND_Modulate:
			if (InTranslucencyPassType != ETranslucencyPass::TPT_TranslucencyHoldout)
			{
				// Modulate with the existing scene color, preserve destination alpha.
				BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_DestColor, BF_Zero>::GetRHI();
			}
			else
			{
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
			}
			break;
		case BLEND_AlphaComposite:
			if (InTranslucencyPassType != ETranslucencyPass::TPT_TranslucencyHoldout)
			{
				// Blend with existing scene color. New color is already pre-multiplied by alpha.
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
			}
			else
			{
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
			}
			break;
		case BLEND_AlphaHoldout:
			if (InTranslucencyPassType != ETranslucencyPass::TPT_TranslucencyHoldout)
			{
				// Blend by holding out the matte shape of the source alpha
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI();
			}
			else
			{
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
			}
			break;
		};
	}

	return BlendState;
}
