// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/MathFwd.h"
#include "PixelFormat.h"
#include "RHIShaderPlatform.h"
#include "RenderGraphFwd.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "PostProcess/SceneRenderTargets.h"
#include "ScreenPass.h"
#include "SceneView.h"

class FSceneRenderer;
class FViewInfo;
class FMaterial;
struct FTranslucencyPassResources;
struct FRDGTextureMSAA;
struct FLensDistortionLUT;
namespace EMeshPass { enum Type : uint8; }
enum class ETranslucencyView;
namespace ETranslucencyPass { enum Type : int; }

#define DISTORTION_STENCIL_MASK_BIT STENCIL_SANDBOX_MASK

struct FSeparateTranslucencyDimensions
{
	inline FScreenPassTextureViewport GetViewport(FIntRect ViewRect) const
	{
		return FScreenPassTextureViewport(Extent, GetScaledRect(ViewRect, Scale));
	}

	FScreenPassTextureViewport GetInstancedStereoViewport(const FViewInfo& View) const;

	// Extent of the separate translucency targets, if downsampled.
	FIntPoint Extent = FIntPoint::ZeroValue;

	// Amount the view rects should be scaled to match the new separate translucency extent.
	float Scale = 1.0f;

	// The number of MSAA samples to use when creating separate translucency textures.
	uint32 NumSamples = 1;
};

DECLARE_GPU_DRAWCALL_STAT_EXTERN(Translucency);

/** Add Copy SceneColor Pass. */
FRDGTextureRef AddCopySceneColorPass(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureMSAA SceneColor, bool WithAlpha = false, bool bSkipIfUnderwater = true, bool bEncodeSceneColorCopy = true);


/** Add pass to compose the translucent-holdout texture's accumulated background visibility into
 *  scene-color alpha. Scene-color alpha is replaced unconditionally. */
void AddTranslucentHoldoutToSceneColorAlphaPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FTranslucencyPassResources& TranslucencyPassResources
);

/** Variant of AddTranslucentHoldoutToSceneColorAlphaPass that preserves scene-color alpha at
 *  pixels where no translucent-holdout primitive contributed (path throughput == 1) and replaces
 *  it with the accumulated background visibility everywhere else. */
void AddTranslucentHoldoutWithCoverageToSceneColorAlphaPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FTranslucencyPassResources& TranslucencyPassResources
);

/** Converts the translucency pass into the respective mesh pass. */
EMeshPass::Type TranslucencyPassToMeshPass(ETranslucencyPass::Type TranslucencyPass);

/** Converts the translucency pass into a string name. */
const TCHAR* TranslucencyPassToString(ETranslucencyPass::Type TranslucencyPass);

/** Returns the translucency views to render for the requested view. */
ETranslucencyView GetTranslucencyView(const FViewInfo& View);

/** Returns the union of all translucency views to render. */
ETranslucencyView GetTranslucencyViews(TArrayView<const FViewInfo> Views);

/** Computes the translucency dimensions. */
FSeparateTranslucencyDimensions UpdateSeparateTranslucencyDimensions(const FSceneRenderer& SceneRenderer);

/** Check if separate translucency pass is needed for given pass and downsample scale */
bool IsSeparateTranslucencyEnabled(ETranslucencyPass::Type TranslucencyPass, float DownsampleScale);

/** Shared function to get the post DOF texture pixel format and creation flags */
const FRDGTextureDesc GetPostDOFTranslucentTextureDesc(ETranslucencyPass::Type TranslucencyPass, const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions, bool bIsModulate, EShaderPlatform ShaderPlatform);

/** Shared function used to create Post DOF translucent textures */
FRDGTextureMSAA CreatePostDOFTranslucentTexture(FRDGBuilder& GraphBuilder, ETranslucencyPass::Type TranslucencyPass, const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions, bool bIsModulate, EShaderPlatform ShaderPlatform);

bool IsStandardTranslucencyPassSeparated();

bool ShouldDrawInTranslucentBasePass(const FMaterial& Material, ETranslucencyPass::Type TranslucencyPass, float AutoBeforeDOFTranslucencyBoundary);

FViewShaderParameters GetSeparateTranslucencyViewParameters(const FViewInfo& View, const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions, ETranslucencyPass::Type TranslucencyPass);

/** Add a pass to compose separate translucency. */
struct FTranslucencyComposition
{
	enum class EOperation
	{
		UpscaleOnly,
		ComposeToExistingSceneColor,
		ComposeToNewSceneColor,
		ComposeToSceneColorAlpha
	};

	EOperation Operation = EOperation::UpscaleOnly;
	bool bApplyModulateOnly = false;

	FScreenPassTextureSlice SceneColor;
	FScreenPassTexture SceneDepth;

	FScreenPassTextureViewport OutputViewport;
	EPixelFormat OutputPixelFormat = PF_Unknown;

	// [Optional] Lens distortion applied on the scene color.
	const FLensDistortionLUT* LensDistortionLUT = nullptr;

	FScreenPassTexture AddPass(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		const FTranslucencyPassResources& TranslucencyTextures) const;
};

/** Location of the translucent holdout composition into scene color alpha. */
enum class ETranslucentHoldoutCompositionType : uint8
{
	BeforeDOF,
	AfterDOF,
};

/** Get the location of the translucent holdout composition into scene color alpha. */
ETranslucentHoldoutCompositionType GetTranslucentHoldoutComposition();

template<uint32 StencilRef> FRHIDepthStencilState* GetTranslucentPassDepthStencilState(bool bDisableDepthTest)
{
	if (bDisableDepthTest)
	{
		return TStaticDepthStencilState<
				false, CF_Always,
				true , CF_Always, SO_Keep, SO_Keep, SO_Replace,
				false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
				StencilRef, StencilRef>::GetRHI();
	}
	else
	{
		return TStaticDepthStencilState<
				false, CF_DepthNearOrEqual,
				true , CF_Always, SO_Keep, SO_Keep, SO_Replace,
				false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
				StencilRef, StencilRef>::GetRHI();
	}
}

FRHIBlendState* GetTranslucentBlendState(const FMaterial& Material, ETranslucencyPass::Type InTranslucencyPassType);