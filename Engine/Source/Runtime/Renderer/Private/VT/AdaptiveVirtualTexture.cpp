// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdaptiveVirtualTexture.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "VT/AllocatedVirtualTexture.h"
#include "VT/VirtualTexturePhysicalSpace.h"
#include "VT/VirtualTextureScalability.h"
#include "VT/VirtualTextureSpace.h"
#include "VT/VirtualTextureSystem.h"
#include "VirtualTextureDefinitions.h"


static TAutoConsoleVariable<int32> CVarAVTMaxAllocPerFrame(
	TEXT("r.VT.AVT.MaxAllocPerFrame"),
	1,
	TEXT("Max number of allocated VT for adaptive VT to alloc per frame"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAVTMaxFreePerFrame(
	TEXT("r.VT.AVT.MaxFreePerFrame"),
	1,
	TEXT("Max number of allocated VT for adaptive VT to free per frame"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAVTMaxPageResidency(
	TEXT("r.VT.AVT.MaxPageResidency"),
	75,
	TEXT("Percentage of page table to allocate before we start force freeing to make space"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAVTAgeToFree(
	TEXT("r.VT.AVT.AgeToFree"),
	1000,
	TEXT("Number of frames for an allocation to be unused before it is considered for freeing by age"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAVTMinPoolResidencyToFree(
	TEXT("r.VT.AVT.PoolResidencyToFree"),
	85,
	TEXT("We consider freeing allocations by age if page residency percentage of physical pool is greater than this\n")
	TEXT("Only freeing allocations at high residency reduces the likely number of mapped pages to remap, which can be expensive"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAVTLevelIncrement(
	TEXT("r.VT.AVT.LevelIncrement"),
	3,
	TEXT("Number of levels to increment each time we grow an allocated virtual texture"),
	ECVF_RenderThreadSafe
);


/**
 * IVirtualTexture implementation that redirects requests to another IVirtualTexture after having modified vLevel and vAddress.
 * Note that we expect vAddress values only in 32bit range from the VirtualTextureSystem, but we can expand into a genuine 64bit range here to feed our child producer.
 */
class FVirtualTextureAddressRedirect : public IVirtualTexture
{
public:
	FVirtualTextureAddressRedirect(IVirtualTexture* InVirtualTexture, FIntPoint InAddressOffset, int32 InLevelOffset)
		: VirtualTexture(InVirtualTexture)
		, AddressOffset(InAddressOffset)
		, LevelOffset(InLevelOffset)
	{
	}

	virtual ~FVirtualTextureAddressRedirect()
	{
	}

	virtual bool IsPageStreamed(uint8 vLevel, uint32 vAddress) const override
	{
		uint64 X = FMath::ReverseMortonCode2_64(vAddress) + (AddressOffset.X >> (vLevel + LevelOffset));
		uint64 Y = FMath::ReverseMortonCode2_64(vAddress >> 1) + (AddressOffset.Y >> (vLevel + LevelOffset));
		vAddress = FMath::MortonCode2_64(X) | (FMath::MortonCode2_64(Y) << 1);
		vLevel = (uint8)(FMath::Max((int32)vLevel + LevelOffset, 0));

		return VirtualTexture->IsPageStreamed(vLevel, vAddress);
	}

	virtual FVTRequestPageResult RequestPageData(
		FRHICommandListBase& RHICmdList,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		EVTRequestPagePriority Priority
	) override
	{
		uint64 X = FMath::ReverseMortonCode2_64(vAddress) + (AddressOffset.X >> (vLevel + LevelOffset));
		uint64 Y = FMath::ReverseMortonCode2_64(vAddress >> 1) + (AddressOffset.Y >> (vLevel + LevelOffset));
		vAddress = FMath::MortonCode2_64(X) | (FMath::MortonCode2_64(Y) << 1);
		vLevel = (uint8)(FMath::Max((int32)vLevel + LevelOffset, 0));

		return VirtualTexture->RequestPageData(RHICmdList, ProducerHandle, LayerMask, vLevel, vAddress, Priority);
	}

	virtual IVirtualTextureFinalizer* ProducePageData(
		FRHICommandListBase& RHICmdList,
		ERHIFeatureLevel::Type FeatureLevel,
		EVTProducePageFlags Flags,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		uint64 RequestHandle,
		const FVTProduceTargetLayer* TargetLayers
	) override
	{
		uint64 X = FMath::ReverseMortonCode2_64(vAddress) + (AddressOffset.X >> (vLevel + LevelOffset));
		uint64 Y = FMath::ReverseMortonCode2_64(vAddress >> 1) + (AddressOffset.Y >> (vLevel + LevelOffset));
		vAddress = FMath::MortonCode2_64(X) | (FMath::MortonCode2_64(Y) << 1);
		vLevel = (uint8)(FMath::Max((int32)vLevel + LevelOffset, 0));

		return VirtualTexture->ProducePageData(RHICmdList, FeatureLevel, Flags, ProducerHandle, LayerMask, vLevel, vAddress, RequestHandle, TargetLayers);
	}

	virtual void GatherProducePageDataTasks(
		FVirtualTextureProducerHandle const& ProducerHandle,
		FGraphEventArray& InOutTasks) const override
	{
		VirtualTexture->GatherProducePageDataTasks(ProducerHandle, InOutTasks);
	}

	virtual void GatherProducePageDataTasks(
		uint64 RequestHandle,
		FGraphEventArray& InOutTasks) const override
	{
		VirtualTexture->GatherProducePageDataTasks(RequestHandle, InOutTasks);
	}

private:
	IVirtualTexture* VirtualTexture;
	FIntPoint AddressOffset;
	int32 LevelOffset;
};

/** Union to define the layout of our packed allocation requests. */
union FPackedAdaptiveAllocationRequest
{
	uint32 PackedValue = 0;
	struct
	{
		uint32 bIsValid : 2;
		uint32 bIsAllocated : 1;
		uint32 bIsRequest : 1;
		uint32 AllocationOrGridIndex : 24; // Store index in AllocationSlots if bIsAllocated, or GridIndex if not.
		uint32 Space : 4; // Keep in top 4 bits for sorting in QueuePackedAllocationRequests()
	};
};

/** Allocate a virtual texture for a subset of the full adaptive virtual texture. */
IAllocatedVirtualTexture* FAdaptiveVirtualTexture::AllocateVirtualTexture(
	FRHICommandListBase& RHICmdList,
	FVirtualTextureSystem* InSystem,
	FAllocatedVTDescription const& InAllocatedDesc,
	FIntPoint InGridSize,
	uint8 InForcedSpaceID,
	int32 InWidthInTiles,
	int32 InHeightInTiles,
	FIntPoint InAddressOffset,
	int32 InLevelOffset)
{
	FAllocatedVTDescription AllocatedDesc = InAllocatedDesc;

	// We require bPrivateSpace since there can be only one adaptive VT per space.
	ensure(AllocatedDesc.bPrivateSpace);
	AllocatedDesc.bPrivateSpace = true;
	AllocatedDesc.ForceSpaceID = InForcedSpaceID;
	AllocatedDesc.IndirectionTextureSize = FMath::Max(InGridSize.X, InGridSize.Y);
	AllocatedDesc.AdaptiveLevelBias = InLevelOffset;

	for (int32 LayerIndex = 0; LayerIndex < InAllocatedDesc.NumTextureLayers; ++LayerIndex)
	{
		// Test if we have already written layer with a new handle.
		// If we have then we already processed this producer in an ealier layer and have nothing more to do.
		if (AllocatedDesc.ProducerHandle[LayerIndex] != InAllocatedDesc.ProducerHandle[LayerIndex])
		{
			continue;
		}

		FVirtualTextureProducerHandle ProducerHandle = InAllocatedDesc.ProducerHandle[LayerIndex];
		FVirtualTextureProducer* Producer = InSystem->FindProducer(ProducerHandle);
		FVTProducerDescription NewProducerDesc = Producer->GetDescription();
		NewProducerDesc.BlockWidthInTiles = InWidthInTiles;
		NewProducerDesc.BlockHeightInTiles = InHeightInTiles;
		NewProducerDesc.MaxLevel = FMath::CeilLogTwo(FMath::Max(InWidthInTiles, InHeightInTiles));
		
		// Specialize the producer hash by position in the adaptive grid.
		NewProducerDesc.FullNameHash = HashCombine(Producer->GetDescription().FullNameHash, GetTypeHash(InAddressOffset));

		IVirtualTexture* VirtualTextureProducer = Producer->GetVirtualTexture();
		IVirtualTexture* NewVirtualTextureProducer = new FVirtualTextureAddressRedirect(VirtualTextureProducer, InAddressOffset, InLevelOffset);
		FVirtualTextureProducerHandle NewProducerHandle = InSystem->RegisterProducer(RHICmdList, NewProducerDesc, NewVirtualTextureProducer);

		// Copy new producer to all subsequent layers.
		for (int32 WriteLayerIndex = LayerIndex; WriteLayerIndex < InAllocatedDesc.NumTextureLayers; ++WriteLayerIndex)
		{
			if (InAllocatedDesc.ProducerHandle[WriteLayerIndex] == ProducerHandle)
			{
				AllocatedDesc.ProducerHandle[WriteLayerIndex] = NewProducerHandle;
			}
		}
	}

	return InSystem->AllocateVirtualTexture(RHICmdList, AllocatedDesc);
}

/** Destroy an allocated virtual texture and release its producers. */
void FAdaptiveVirtualTexture::DestroyVirtualTexture(FVirtualTextureSystem* InSystem, IAllocatedVirtualTexture* InAllocatedVT, TArray<FVirtualTextureProducerHandle>& OutProducersToRelease)
{
	FAllocatedVTDescription const& Desc = InAllocatedVT->GetDescription();
	for (int32 LayerIndex = 0; LayerIndex < Desc.NumTextureLayers; ++LayerIndex)
	{
		OutProducersToRelease.AddUnique(Desc.ProducerHandle[LayerIndex]);
	}
	InSystem->DestroyVirtualTexture(InAllocatedVT);
}

/** Remaps the page mappings from one allocated virtual texture to another. */
void FAdaptiveVirtualTexture::RemapVirtualTexturePages(FVirtualTextureSystem* InSystem, FAllocatedVirtualTexture* OldAllocatedVT, FAllocatedVirtualTexture* NewAllocatedVT, uint32 InFrame)
{
	const uint32 OldVirtualAddress = OldAllocatedVT->GetVirtualAddress();
	const uint32 NewVirtualAddress = NewAllocatedVT->GetVirtualAddress();

	for (uint32 ProducerIndex = 0u; ProducerIndex < OldAllocatedVT->GetNumUniqueProducers(); ++ProducerIndex)
	{
		check(OldAllocatedVT->GetUniqueProducerMipBias(ProducerIndex) == 0);
		check(NewAllocatedVT->GetUniqueProducerMipBias(ProducerIndex) == 0);

		const FVirtualTextureProducerHandle& OldProducerHandle = OldAllocatedVT->GetUniqueProducerHandle(ProducerIndex);
		const FVirtualTextureProducerHandle& NewProducerHandle = NewAllocatedVT->GetUniqueProducerHandle(ProducerIndex);

		FVirtualTextureProducer* OldProducer = InSystem->FindProducer(OldProducerHandle);
		FVirtualTextureProducer* NewProducer = InSystem->FindProducer(NewProducerHandle);

		if (OldProducer->GetDescription().bPersistentHighestMip)
		{
			InSystem->ForceUnlockAllTiles(OldProducerHandle, OldProducer);
		}

		const uint32 SpaceID = OldAllocatedVT->GetSpaceID();
		const int32 vLevelBias = (int32)NewProducer->GetMaxLevel() - (int32)OldProducer->GetMaxLevel();

		for (uint32 PhysicalGroupIndex = 0u; PhysicalGroupIndex < OldProducer->GetNumPhysicalGroups(); ++PhysicalGroupIndex)
		{
			FVirtualTexturePhysicalSpace* PhysicalSpace = OldProducer->GetPhysicalSpaceForPhysicalGroup(PhysicalGroupIndex);
			FTexturePagePool& PagePool = PhysicalSpace->GetPagePool();

			PagePool.RemapPages(
				InSystem, SpaceID, PhysicalSpace,
				OldProducerHandle, OldVirtualAddress,
				NewProducerHandle, NewVirtualAddress,
				0,
				vLevelBias, vLevelBias, InFrame
			);
		}
	}
}

FAdaptiveVirtualTexture::FAdaptiveVirtualTexture(
	FAdaptiveVTDescription const& InAdaptiveDesc,
	FAllocatedVTDescription const& InAllocatedDesc)
	: AdaptiveDesc(InAdaptiveDesc)
	, AllocatedDesc(InAllocatedDesc)
	, AllocatedVirtualTextureLowMips(nullptr)
	, NumAllocated(0)
{
	MaxLevel = FMath::Max(FMath::CeilLogTwo(AdaptiveDesc.TileCountX), FMath::CeilLogTwo(AdaptiveDesc.TileCountY));

	const int32 AdaptiveGridLevelsX = (int32)FMath::CeilLogTwo(AdaptiveDesc.TileCountX) - AdaptiveDesc.MaxAdaptiveLevel;
	const int32 AdaptiveGridLevelsY = (int32)FMath::CeilLogTwo(AdaptiveDesc.TileCountY) - AdaptiveDesc.MaxAdaptiveLevel;
	ensure(AdaptiveGridLevelsX >= 0 && AdaptiveGridLevelsY >= 0); // Aspect ratio is too big for desired grid size. This will give bad results.

	GridSize = FIntPoint(1 << FMath::Max(AdaptiveGridLevelsX, 0), 1 << FMath::Max(AdaptiveGridLevelsY, 0));
}

void FAdaptiveVirtualTexture::Init(FRHICommandListBase& RHICmdList, FVirtualTextureSystem* InSystem)
{
	// Allocate a low mips virtual texture.
	const int32 LevelOffset = AdaptiveDesc.MaxAdaptiveLevel;
	AllocatedVirtualTextureLowMips = (FAllocatedVirtualTexture*)AllocateVirtualTexture(RHICmdList, InSystem, AllocatedDesc, GridSize, 0xff, GridSize.X, GridSize.Y, FIntPoint::ZeroValue, LevelOffset);
}

void FAdaptiveVirtualTexture::Destroy(FVirtualTextureSystem* InSystem)
{
	DestroyVirtualTexture(InSystem, AllocatedVirtualTextureLowMips, ProducersToRelease);

	for (FAllocation& Allocation : AllocationSlots)
	{
		if (Allocation.AllocatedVT != nullptr)
		{
			DestroyVirtualTexture(InSystem, Allocation.AllocatedVT, ProducersToRelease);
		}
	}

	for (FVirtualTextureProducerHandle Handle : ProducersToRelease)
	{
		InSystem->ReleaseProducer(Handle);
	}

	delete this;
}

IAllocatedVirtualTexture* FAdaptiveVirtualTexture::GetAllocatedVirtualTexture()
{
	return AllocatedVirtualTextureLowMips;
}

int32 FAdaptiveVirtualTexture::GetSpaceID() const
{
	return AllocatedVirtualTextureLowMips->GetSpaceID();
}

void FAdaptiveVirtualTexture::GetProducers(FIntRect const& InTextureRegion, uint32 InMaxLevel, TArray<FProducerInfo>& OutProducerInfos) const
{
	const uint32 NumProducers = AllocatedVirtualTextureLowMips->GetNumUniqueProducers();

	OutProducerInfos.Reserve((NumAllocated + 1) * NumProducers);
	
	// Add producers from persistent allocated virtual texture.
	{
		const uint32 AdaptiveLevelBias = AllocatedVirtualTextureLowMips->GetDescription().AdaptiveLevelBias;
		
		// Only add to output array if we have some relevant mips under the InMaxLevel.
		if (InMaxLevel >= AdaptiveLevelBias)
		{
			const int32 Divisor = 1 << AdaptiveLevelBias;
			const FIntRect RemappedTextureRegion(
				FIntPoint::DivideAndRoundDown(InTextureRegion.Min, Divisor), 
				FIntPoint::DivideAndRoundUp(InTextureRegion.Max, Divisor));
			const uint32 RemappedMaxLevel = InMaxLevel - AdaptiveLevelBias;

			for (uint32 ProducerIndex = 0; ProducerIndex < NumProducers; ++ProducerIndex)
			{
		 		OutProducerInfos.Emplace(FProducerInfo{ AllocatedVirtualTextureLowMips->GetUniqueProducerHandle(ProducerIndex), RemappedTextureRegion, RemappedMaxLevel });
			}
		}
	}

	// Add producers from transient allocated virtual textures.
	for (FAllocation const& Allocation : AllocationSlots)
	{
		if (Allocation.AllocatedVT != nullptr)
		{
			const uint32 AdaptiveLevelBias = Allocation.AllocatedVT->GetDescription().AdaptiveLevelBias;
			if (InMaxLevel >= AdaptiveLevelBias)
			{
				// Get texture region in the full VT space for this allocated VT.
				const uint32 X = Allocation.GridIndex % GridSize.X;
				const uint32 Y = Allocation.GridIndex / GridSize.X;
				const FIntPoint PageSize(AllocatedDesc.TileSize * AdaptiveDesc.TileCountX / GridSize.X, AllocatedDesc.TileSize * AdaptiveDesc.TileCountY / GridSize.Y);
				const FIntPoint PageBase(PageSize.X * X, PageSize.Y * Y);
				const FIntRect AllocationRegion(PageBase - AllocatedDesc.TileBorderSize, PageBase + PageSize + AllocatedDesc.TileBorderSize);

				// Only add to output array if the texture region intersects this allocation region.
				if (AllocationRegion.Intersect(InTextureRegion))
				{
					const int32 Divisor = 1 << AdaptiveLevelBias;
					const FIntRect RemappedTextureRegion(
							FIntPoint::DivideAndRoundDown(InTextureRegion.Min - PageBase, Divisor),
							FIntPoint::DivideAndRoundUp(InTextureRegion.Max - PageBase, Divisor));
					const uint32 RemappedMaxLevel = InMaxLevel - AdaptiveLevelBias;

					for (uint32 ProducerIndex = 0; ProducerIndex < NumProducers; ++ProducerIndex)
					{
						OutProducerInfos.Emplace(FProducerInfo{ Allocation.AllocatedVT->GetUniqueProducerHandle(ProducerIndex), RemappedTextureRegion, RemappedMaxLevel });
					}
				}
			}
		}
	}
}

void FAdaptiveVirtualTexture::GetAllocatedVirtualTextures(FBox2D const& InUVRegion, uint32 InLevel, TArray<FAllocatedInfo>& OutInfos, TArray<uint32>& OutAllocationRequests)
{
	// The persistent allocated virtual texture.
	const int32 RemappedLevel = (int32)AllocatedVirtualTextureLowMips->GetMaxLevel() - (int32)MaxLevel + (int32)InLevel;
	const int32 ClampedRemappedLevel = FMath::Max(RemappedLevel, 0);
	OutInfos.Add(FAllocatedInfo{ AllocatedVirtualTextureLowMips, InUVRegion, (uint32)ClampedRemappedLevel });

	// If the mip level is outside of the persistent allocated virtual texture then find the transient allocated virtual textures.
	if (RemappedLevel < 0)
	{
		const int32 MinGridX = FMath::FloorToInt(FMath::Max(InUVRegion.Min.X, 0.0) * GridSize.X);
		const int32 MaxGridX = FMath::CeilToInt(FMath::Min(InUVRegion.Max.X, 1.0) * GridSize.X);
		const int32 MinGridY = FMath::FloorToInt(FMath::Max(InUVRegion.Min.Y, 0.0) * GridSize.Y);
		const int32 MaxGridY = FMath::CeilToInt(FMath::Min(InUVRegion.Max.Y, 1.0) * GridSize.Y);

		for (int32 Y = MinGridY; Y < MaxGridY; Y++)
		{
			for (int32 X = MinGridX; X < MaxGridX; X++)
			{
				const uint32 GridIndex = X + Y * GridSize.X;
				const uint32 AllocationIndex = GetAllocationIndex(GridIndex);
				if (AllocationIndex != INDEX_NONE)
				{
					FAllocatedVirtualTexture* AllocatedVT = AllocationSlots[AllocationIndex].AllocatedVT;
					const int32 LocalRemappedLevel = RemappedLevel + (int32)AllocatedVT->GetMaxLevel();

					if (LocalRemappedLevel >= 0)
					{
						FVector2D LocalUV0 = InUVRegion.Min * FVector2D(GridSize.X, GridSize.Y) - FVector2D(X, Y);
						FVector2D LocalUV1 = InUVRegion.Max * FVector2D(GridSize.X, GridSize.Y) - FVector2D(X, Y);
						const FBox2D RemappedUVRegion = FBox2D(LocalUV0, LocalUV1).Overlap(FBox2D(FVector2D::Zero(), FVector2D::One()));

						OutInfos.Add(FAllocatedInfo{ AllocatedVT, RemappedUVRegion, (uint32)LocalRemappedLevel });

						continue;
					}
				}

				// The grid location doesn't cover the requested mip level so we create an allocation request.
				FPackedAdaptiveAllocationRequest Request;
				Request.bIsValid = 1;
				Request.bIsAllocated = AllocationIndex == INDEX_NONE ? 0 : 1;
				Request.bIsRequest = 1;
				Request.AllocationOrGridIndex = AllocationIndex == INDEX_NONE ? GridIndex : AllocationIndex;
				Request.Space = GetSpaceID();
				OutAllocationRequests.Add(Request.PackedValue);
			}
		}
	}
}

/** Get hash key for the GridIndexMap. */
static uint16 GetGridIndexHash(int32 InGridIndex)
{
	return MurmurFinalize32(InGridIndex);
}

/** Get hash key for the AllocatedVTMap. */
static uint16 GetAllocatedVTHash(FAllocatedVirtualTexture* InAllocatedVT)
{
	return reinterpret_cast<UPTRINT>(InAllocatedVT) / 16;
}

uint32 FAdaptiveVirtualTexture::GetAllocationIndex(uint32 InGridIndex) const
{
	uint32 Index = GridIndexMap.First(GetGridIndexHash(InGridIndex));
	for (; GridIndexMap.IsValid(Index); Index = GridIndexMap.Next(Index))
	{
		if (AllocationSlots[Index].GridIndex == InGridIndex)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

uint32 FAdaptiveVirtualTexture::GetAllocationIndex(FAllocatedVirtualTexture* InAllocatedVT) const
{
	uint32 Index = AllocatedVTMap.First(GetAllocatedVTHash(InAllocatedVT));
	for (; AllocatedVTMap.IsValid(Index); Index = AllocatedVTMap.Next(Index))
	{
		if (AllocationSlots[Index].AllocatedVT == InAllocatedVT)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

uint32 FAdaptiveVirtualTexture::GetPackedAllocationRequest(uint32 vAddress, uint32 vLevelPlusFixedOffset, uint32 Frame) const
{
	FPackedAdaptiveAllocationRequest Request;
	Request.Space = GetSpaceID();
	Request.bIsRequest = vLevelPlusFixedOffset < UE::HLSL::VirtualTextureFeedbackBias ? 1 : 0;
	Request.bIsValid = 1;

	uint32 vAddressLocal;
	FAllocatedVirtualTexture* AllocatedVT = FVirtualTextureSystem::Get().GetSpace(GetSpaceID())->GetAllocator().Find(vAddress, vAddressLocal);

	if (AllocatedVT == nullptr)
	{
		// Requests are processed a few frames after the GPU requested. It's possible that the VT is no longer allocated.
		return 0;
	}
	else if (AllocatedVT->GetFrameAllocated() > Frame - 3)
	{
		// Don't process any request for a virtual texture that was allocated in the last few frames.
		return 0;
	}
	else if (AllocatedVT == AllocatedVirtualTextureLowMips)
	{
		// Request comes from the low mips allocated VT.
		const uint32 X = FMath::ReverseMortonCode2(vAddressLocal);
		const uint32 Y = FMath::ReverseMortonCode2(vAddressLocal >> 1);
		const uint32 GridIndex = X + Y * GridSize.X;
		const uint32 AllocationIndex = GetAllocationIndex(GridIndex);

		if (AllocationIndex != INDEX_NONE)
		{
			// The higher mips are already allocated but this request came from the low res mips.
			// Do nothing, and if no higher mips are requested then eventually the allocated VT will be evicted.
			return 0;
		}

		Request.bIsAllocated = 0;
		Request.AllocationOrGridIndex = GridIndex;
	}
	else
	{
		const uint32 AllocationIndex = GetAllocationIndex(AllocatedVT);
		check(AllocationIndex != INDEX_NONE);

		Request.bIsAllocated = 1;
		Request.AllocationOrGridIndex = AllocationIndex;

		// If we are allocated at the max level already then we don't want to request a new level.
		if (AllocatedVT->GetMaxLevel() >= AdaptiveDesc.MaxAdaptiveLevel)
		{
			Request.bIsRequest = 0;
		}
	}

	return Request.PackedValue;
}

void FAdaptiveVirtualTexture::QueuePackedAllocationRequests(FVirtualTextureSystem* InSystem, TConstArrayView<uint32> InRequests, uint32 InFrame)
{
	if (InRequests.Num() > 0)
	{
		// Sort in place for batching by SpaceID.
		// We also sort here to help can skip duplicate requests. 
		// It would be better either sort before this call, or to remove duplicates before this call (when gathering requests) so that the sort is cheaper.
		TArray<uint32> SortedRequests;
		SortedRequests = InRequests;
		Algo::Sort(SortedRequests);
		TConstArrayView<uint32> SortedRequestsView(SortedRequests);

		int32 StartRequestIndex = 0;
		FPackedAdaptiveAllocationRequest StartRequest;
		StartRequest.PackedValue = SortedRequestsView[0];

		for (int32 RequestIndex = 0; RequestIndex < SortedRequestsView.Num(); ++RequestIndex)
		{
			FPackedAdaptiveAllocationRequest Request;
			Request.PackedValue = SortedRequestsView[RequestIndex];

			if (Request.Space != StartRequest.Space)
			{
				FAdaptiveVirtualTexture* AdaptiveVT = InSystem->GetAdaptiveVirtualTexture(StartRequest.Space);
				AdaptiveVT->QueuePackedAllocationRequests(SortedRequestsView.Slice(StartRequestIndex, RequestIndex - StartRequestIndex), InFrame);

				StartRequestIndex = RequestIndex;
				StartRequest = Request;
			}
		}

		if (StartRequestIndex < SortedRequestsView.Num())
		{
			FAdaptiveVirtualTexture* AdaptiveVT = InSystem->GetAdaptiveVirtualTexture(StartRequest.Space);
			AdaptiveVT->QueuePackedAllocationRequests(SortedRequestsView.Right(SortedRequestsView.Num() - StartRequestIndex), InFrame);
		}
	}
}

void FAdaptiveVirtualTexture::QueuePackedAllocationRequests(TConstArrayView<uint32> InRequests, uint32 InFrame)
{
	for (int32 RequestIndex = 0; RequestIndex < InRequests.Num(); ++RequestIndex)
	{
		// Skip duplicates.
		if (RequestIndex == 0 || InRequests[RequestIndex] != InRequests[RequestIndex - 1])
		{
			FPackedAdaptiveAllocationRequest Request;
			Request.PackedValue = InRequests[RequestIndex];

			if (Request.bIsAllocated != 0)
			{
				// Already allocated so mark as used. Do this before we process any requests to ensure we don't free before allocating.
				const uint32 AllocationIndex = Request.AllocationOrGridIndex;
				const uint32 MaxVTLevel = AllocationSlots[AllocationIndex].AllocatedVT->GetMaxLevel();

				const uint32 Key = (InFrame << 4) | MaxVTLevel;
				LRUHeap.Update(Key, AllocationIndex);
			}

			if (Request.bIsRequest)
			{
				// Store request to handle in UpdateAllocations()
				RequestsToMap.AddUnique(Request.PackedValue);
			}
		}
	}
}

void FAdaptiveVirtualTexture::Allocate(FRHICommandListBase& RHICmdList, FVirtualTextureSystem* InSystem, uint32 InPackedRequest, uint32 InFrame)
{
	FPackedAdaptiveAllocationRequest Request;
	Request.PackedValue = InPackedRequest;

	// Either reallocate or allocate a new virtual texture depending on the bIsAllocated flag.
	const bool bIsAllocated = Request.bIsAllocated != 0;
	const uint32 AllocationIndex = bIsAllocated ? Request.AllocationOrGridIndex : INDEX_NONE;
	const uint32 GridIndex = bIsAllocated ? AllocationSlots[AllocationIndex].GridIndex : Request.AllocationOrGridIndex;
	FAllocatedVirtualTexture* OldAllocatedVT = bIsAllocated ? AllocationSlots[AllocationIndex].AllocatedVT : nullptr;
	const uint32 CurrentLevel = bIsAllocated ? OldAllocatedVT->GetMaxLevel() : 0;
	const uint32 LevelIncrement = CVarAVTLevelIncrement.GetValueOnRenderThread();
	const uint32 NewLevel = FMath::Min(CurrentLevel + LevelIncrement, AdaptiveDesc.MaxAdaptiveLevel);
	check(NewLevel > CurrentLevel);

	// Check if we have space in the page table to allocate. If not then hopefully we can allocate next frame.
	FVirtualTextureSpace* Space = InSystem->GetSpace(GetSpaceID());
	if (!Space->GetAllocator().TryAlloc(NewLevel))
	{
		return;
	}

	Allocate(RHICmdList, InSystem, GridIndex, AllocationIndex, NewLevel, InFrame);
}

void FAdaptiveVirtualTexture::Allocate(FRHICommandListBase& RHICmdList, FVirtualTextureSystem* InSystem, uint32 InGridIndex, uint32 InAllocationIndex, uint32 InNewLevel, uint32 InFrame)
{
	const uint32 X = InGridIndex % GridSize.X;
	const uint32 Y = InGridIndex / GridSize.X;
	const FIntPoint PageOffset(X * AdaptiveDesc.TileCountX / GridSize.X, Y * AdaptiveDesc.TileCountY / GridSize.Y);
	const int32 LevelOffset = (int32)AdaptiveDesc.MaxAdaptiveLevel - (int32)InNewLevel;

	FAllocatedVirtualTexture* OldAllocatedVT = (InAllocationIndex != INDEX_NONE) ? AllocationSlots[InAllocationIndex].AllocatedVT : nullptr;
	FAllocatedVirtualTexture* NewAllocatedVT = (FAllocatedVirtualTexture*)AllocateVirtualTexture(RHICmdList, InSystem, AllocatedDesc, GridSize, GetSpaceID(), 1 << InNewLevel, 1 << InNewLevel, PageOffset, LevelOffset);

	if (OldAllocatedVT != nullptr)
	{
		// Remap the old allocated virtual texture before destroying it.
		RemapVirtualTexturePages(InSystem, OldAllocatedVT, NewAllocatedVT, InFrame);
		DestroyVirtualTexture(InSystem, OldAllocatedVT, ProducersToRelease);

		// Adjust allocation structures.
		AllocatedVTMap.Remove(GetAllocatedVTHash(OldAllocatedVT), InAllocationIndex);
		AllocatedVTMap.Add(GetAllocatedVTHash(NewAllocatedVT), InAllocationIndex);
		AllocationSlots[InAllocationIndex].AllocatedVT = NewAllocatedVT;

		// Mark allocation as used.
		const uint32 Key = (InFrame << 4) | InNewLevel;
		LRUHeap.Update(Key, InAllocationIndex);

		// Queue indirection texture update unless this allocation slot is already marked as pending.
		if (SlotsPendingRootPageMap.Find(InAllocationIndex) == INDEX_NONE)
		{
			const uint32 vAddress = NewAllocatedVT->GetVirtualAddress();
			const uint32 vAddressX = FMath::ReverseMortonCode2(vAddress);
			const uint32 vAddressY = FMath::ReverseMortonCode2(vAddress >> 1);
			const uint32 PackedIndirectionValue = (1 << 28) | (InNewLevel << 24) | (vAddressY << 12) | vAddressX;
			TextureUpdates.Add(FIndirectionTextureUpdate{ X, Y, PackedIndirectionValue });
		}
	}
	else
	{
		// Add an allocation slot.
		if (FreeSlots.Num() == 0)
		{
			InAllocationIndex = AllocationSlots.Add(FAllocation(InGridIndex, NewAllocatedVT));
		}
		else
		{
			// Reuse a free allocation slot.
			InAllocationIndex = FreeSlots.Pop();
			AllocationSlots[InAllocationIndex].GridIndex = InGridIndex;
			AllocationSlots[InAllocationIndex].AllocatedVT = NewAllocatedVT;
		}

		// Add to pending for later indirection texture update.
		SlotsPendingRootPageMap.Add(InAllocationIndex);

		// Add to allocation structures.
		GridIndexMap.Add(GetGridIndexHash(InGridIndex), InAllocationIndex);
		AllocatedVTMap.Add(GetAllocatedVTHash(NewAllocatedVT), InAllocationIndex);

		const uint32 Key = (InFrame << 4) | InNewLevel;
		LRUHeap.Add(Key, InAllocationIndex);

		NumAllocated++;
	}
}

void FAdaptiveVirtualTexture::Free(FVirtualTextureSystem* InSystem, uint32 InAllocationIndex, uint32 InFrame)
{
	// Destroy allocated virtual texture.
	const uint32 GridIndex = AllocationSlots[InAllocationIndex].GridIndex;
	FAllocatedVirtualTexture* OldAllocatedVT = AllocationSlots[InAllocationIndex].AllocatedVT;
	DestroyVirtualTexture(InSystem, OldAllocatedVT, ProducersToRelease);

	// Remove from all allocation structures.
	GridIndexMap.Remove(GetGridIndexHash(GridIndex), InAllocationIndex);
	AllocatedVTMap.Remove(GetAllocatedVTHash(OldAllocatedVT), InAllocationIndex);
	AllocationSlots[InAllocationIndex] = FAllocation(0, nullptr);
	FreeSlots.Add(InAllocationIndex);
	SlotsPendingRootPageMap.RemoveAllSwap([InAllocationIndex](int32& V) { return V == InAllocationIndex; });

	NumAllocated--;
	check(NumAllocated >= 0);

	// Queue indirection texture update.
	TextureUpdates.Add(FIndirectionTextureUpdate{ GridIndex % GridSize.X, GridIndex / GridSize.X, 0 });
}

bool FAdaptiveVirtualTexture::FreeLRU(FRHICommandListBase& RHICmdList, FVirtualTextureSystem* InSystem, uint32 InFrame, uint32 InFrameAgeToFree)
{
	// Check if top is ready for eviction.
	const uint32 AllocationIndex = LRUHeap.Top();
	check(AllocationIndex != INDEX_NONE);

	const uint32 Key = LRUHeap.GetKey(AllocationIndex);
	const uint32 LastFrameUsed = Key >> 4;
	if (LastFrameUsed + InFrameAgeToFree > InFrame)
	{
		// Nothing is ready for eviction so return false.
		return false;
	}

	// Find next lower level that we have space in the page table for.
	FVirtualTextureSpace* Space = InSystem->GetSpace(GetSpaceID());
	FAllocatedVirtualTexture* OldAllocatedVT = AllocationSlots[AllocationIndex].AllocatedVT;
	const uint32 CurrentLevel = OldAllocatedVT->GetMaxLevel();
	int32 NewLevel = CurrentLevel - 1;
	while (NewLevel > 0)
	{
		if (Space->GetAllocator().TryAlloc(NewLevel))
		{
			break;
		}
		--NewLevel;
	}

	if (NewLevel < 1)
	{
		// No space so completely free allocation.
		LRUHeap.Pop();
		Free(InSystem, AllocationIndex, InFrame);
	}
	else
	{
		// Reallocate to the selected level.
		// Maintain the last used frame so that we can continue to deallocate levels.
		const uint32 GridIndex = AllocationSlots[AllocationIndex].GridIndex;
		Allocate(RHICmdList, InSystem, GridIndex, AllocationIndex, NewLevel, LastFrameUsed);
	}

	return true;
}

void FAdaptiveVirtualTexture::UpdateAllocations(FVirtualTextureSystem* InSystem, FRHICommandListImmediate& RHICmdList, uint32 InFrame)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAdaptiveVirtualTexture::UpdateAllocations);

	if (RequestsToMap.Num() == 0)
	{
		// Free old unused allocations if there is no other work to do.
		const uint32 FrameAgeToFree = CVarAVTAgeToFree.GetValueOnRenderThread();
		if (FrameAgeToFree > 0)
		{
			FVirtualTexturePhysicalSpace const* PhysicalSpace = AllocatedVirtualTextureLowMips->GetPhysicalSpaceForPageTableLayer(0);
			FTexturePagePool const& Pool = PhysicalSpace->GetPagePool();

			const uint32 PageFreeThreshold = FMath::Max(VirtualTextureScalability::GetPageFreeThreshold(), 0u);
			const uint32 FrameMinusThreshold = InFrame > PageFreeThreshold ? InFrame - PageFreeThreshold : 0;
			const uint32 NumVisiblePages = Pool.GetNumVisiblePages(FrameMinusThreshold);
			const uint32 NumPages = Pool.GetNumPages();
			const uint32 MinPoolResidencyToFree = CVarAVTMinPoolResidencyToFree.GetValueOnRenderThread();

			if (NumVisiblePages * 100 > MinPoolResidencyToFree * NumPages)
			{
				const int32 NumToFree = FMath::Min(NumAllocated, CVarAVTMaxFreePerFrame.GetValueOnRenderThread());

				bool bFreeSuccess = true;
				for (int32 FreeCount = 0; bFreeSuccess && FreeCount < NumToFree; FreeCount++)
				{
					bFreeSuccess = FreeLRU(RHICmdList, InSystem, InFrame, FrameAgeToFree);
				}
			}
		}
	}
	else
	{
		// Free to keep within residency threshold.
		FVirtualTextureSpace* Space = InSystem->GetSpace(GetSpaceID());
		const uint32 TotalPages = Space->GetDescription().MaxSpaceSize * Space->GetDescription().MaxSpaceSize;
		const uint32 ResidencyPercent = FMath::Clamp(CVarAVTMaxPageResidency.GetValueOnRenderThread(), 10, 95);
		const uint32 TargetPages = TotalPages * ResidencyPercent / 100;
		const int32 NumToFree = FMath::Min(NumAllocated, CVarAVTMaxFreePerFrame.GetValueOnRenderThread());

		bool bFreeSuccess = true;
		for (int32 FreeCount = 0; bFreeSuccess && FreeCount < NumToFree && Space->GetAllocator().GetNumAllocatedPages() > TargetPages; FreeCount++)
		{
			const uint32 FrameAgeToFree = 15; // Hardcoded threshold. Don't release anything used more recently then this.
			bFreeSuccess = FreeLRU(RHICmdList, InSystem, InFrame, FrameAgeToFree);
		}

		// Process allocation requests.
		const int32 NumToAlloc = CVarAVTMaxAllocPerFrame.GetValueOnRenderThread();

		for (int32 AllocCount = 0; AllocCount < NumToAlloc && RequestsToMap.Num(); AllocCount++)
		{
			// Randomize request order to prevent feedback from top of the view being prioritized.
			int32 RequestIndex = FMath::Rand() % RequestsToMap.Num();
			uint32 PackedRequest = RequestsToMap[RequestIndex];
			Allocate(RHICmdList, InSystem, PackedRequest, InFrame);
			RequestsToMap.RemoveAtSwap(RequestIndex, EAllowShrinking::No);
		}
	}

	// Check if any pending allocation slots are now ready.
	// Pending slots are ones where the virtual texture locked root page(s) remain unmapped.
	// If the root page is unmapped then we may return bad data from a sample.
	for (int32 Index = 0; Index < SlotsPendingRootPageMap.Num(); ++Index)
	{
		const int32 AllocationSlotIndex = SlotsPendingRootPageMap[Index];
		FAllocation const& Allocation = AllocationSlots[AllocationSlotIndex];
		FAllocatedVirtualTexture* AllocatedVT = Allocation.AllocatedVT;
		if (!InSystem->IsPendingRootPageMap(AllocatedVT))
		{
			SlotsPendingRootPageMap.RemoveAtSwap(Index--);

			// Ready for use so that we can now queue the indirection texture update.
			const uint32 vAddress = AllocatedVT->GetVirtualAddress();
			const uint32 vAddressX = FMath::ReverseMortonCode2(vAddress);
			const uint32 vAddressY = FMath::ReverseMortonCode2(vAddress >> 1);
			const uint32 vLevel = AllocatedVT->GetMaxLevel();
			const uint32 PackedIndirectionValue = (1 << 28) | (vLevel << 24) | (vAddressY << 12) | vAddressX;
			const uint32 X = Allocation.GridIndex % GridSize.X;
			const uint32 Y = Allocation.GridIndex / GridSize.X;
			TextureUpdates.Add(FIndirectionTextureUpdate{ X, Y, PackedIndirectionValue });
		}
	}

	// Clear requests
	RequestsToMap.Reset();

	// Release any producers
	for (int32 ProducerIndex = ProducersToRelease.Num() - 1; ProducerIndex >= 0; ProducerIndex--)
	{
		if (InSystem->TryReleaseProducer(ProducersToRelease[ProducerIndex]))
		{
			ProducersToRelease.RemoveAt(ProducerIndex, 1, EAllowShrinking::No);
		}
	}
}

/** RDG parameters for update pass. */
BEGIN_SHADER_PARAMETER_STRUCT(FUpdateAdaptiveIndirectionTextureParameters, )
	RDG_TEXTURE_ACCESS(IndirectionTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

void FAdaptiveVirtualTexture::ApplyPageTableUpdates(FVirtualTextureSystem* InSystem, FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue)
{
	if (TextureUpdates.Num())
	{
		FVirtualTextureSpace* Space = InSystem->GetSpace(GetSpaceID());
		TRefCountPtr<IPooledRenderTarget> RenderTarget = Space->GetPageTableIndirectionRenderTarget();

		FRDGTextureRef Texture = GraphBuilder.RegisterExternalTexture(RenderTarget, ERDGTextureFlags::ForceImmediateFirstBarrier);

		FUpdateAdaptiveIndirectionTextureParameters* Parameters = GraphBuilder.AllocParameters<FUpdateAdaptiveIndirectionTextureParameters>();
		Parameters->IndirectionTexture = Texture;

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("UpdateAdaptiveIndirectionTexture"),
			Parameters,
			ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
			[Texture, MovedTextureUpdates = MoveTemp(TextureUpdates)](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::All());

			// The texture is already in the correct state from RDG, so disable the automatic RHI transitions.
			FRHICommandListScopedAllowExtraTransitions ScopedExtraTransitions(RHICmdList, false);

			//todo[vt]: If we have more than 1 or 2 updates per frame then add a shader to batch updates.
			for (FIndirectionTextureUpdate const& TextureUpdate : MovedTextureUpdates)
			{
				const FUpdateTextureRegion2D Region(TextureUpdate.X, TextureUpdate.Y, 0, 0, 1, 1);
				RHICmdList.UpdateTexture2D(Texture->GetRHI(), 0, Region, 4, (uint8*)&TextureUpdate.Value);
			}
		});
		
		TextureUpdates.Reset();

		ExternalAccessQueue.Add(Texture, ERHIAccess::SRVMask, ERHIPipeline::All);
	}
}
