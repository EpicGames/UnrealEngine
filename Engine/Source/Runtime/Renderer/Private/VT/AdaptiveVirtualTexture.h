// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/BinaryHeap.h"
#include "Containers/HashTable.h"
#include "VirtualTexturing.h"

class FAllocatedVirtualTexture;
class FRDGBuilder;
class FRDGExternalAccessQueue;
class FRHICommandListImmediate;
class FVirtualTextureSystem;
class FRHICommandListBase;

/**
 * Concrete implementation of an adaptive virtual texture.
 * This allocates multiple virtual textures within the same space: one each for a grid of UV ranges, and an additional persistent one for the low resolution mips.
 * We then use an additional page table indirection texture in the shader to select the correct page table address range for our sampled UV.
 * We use the virtual texture feedback to decide when to increase or decrease the resolution of each UV range.
 * When we change resolution for a range we directly remap the page table entires. This removes the cost and any visual glitch from regenerating the pages.
 */
class FAdaptiveVirtualTexture final : public IAdaptiveVirtualTexture
{
public:
	FAdaptiveVirtualTexture(FAdaptiveVTDescription const& InAdaptiveDesc, FAllocatedVTDescription const& InAllocatedDesc);

	/** Initialize the object. This creates the persistent low mips allocated VT. */
	void Init(FRHICommandListBase& RHICmdList, FVirtualTextureSystem* InSystem);

	/** Get a packed allocation key based on a virtual texture feedback request. The virtual texture system collects these opaque keys before queuing them for processing. */
	uint32 GetPackedAllocationRequest(uint32 vAddress, uint32 vLevelPlusFixedOffset, uint32 Frame) const;
	/** Queue a batch of allocation requests. These will be used to reallocate any virtual textures during the next call to UpdateAllocations(). */
	void QueuePackedAllocationRequests(TConstArrayView<uint32> InRequests, uint32 InFrame);
	/** Queue a batch of allocation requests. This static function relays the global requests to the individual object queues. Internally it sorts the requests. */
	static void QueuePackedAllocationRequests(FVirtualTextureSystem* InSystem, TConstArrayView<uint32> InRequests, uint32 InFrame);
	/** Update any allocations based on recent requests. */
	void UpdateAllocations(FVirtualTextureSystem* InSystem, FRHICommandListImmediate& RHICmdList, uint32 InFrame);
	/** Apply any page table updates. */
	void ApplyPageTableUpdates(FVirtualTextureSystem* InSystem, FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue);

	//~ Begin IAdaptiveVirtualTexture Interface.
	virtual IAllocatedVirtualTexture* GetAllocatedVirtualTexture() override;
	virtual int32 GetSpaceID() const override;
	//~ End IAdaptiveVirtualTexture Interface.

	/** Information needed by GetProducers() for all producers for the internally allocated virtual textures. */
	struct FProducerInfo
	{
		FVirtualTextureProducerHandle ProducerHandle;
		FIntRect RemappedTextureRegion;
		uint32 RemappedMaxLevel;
	};
	/** Get internal producers that touch the texture texel region. */
	void GetProducers(FIntRect const& InTextureRegion, uint32 InMaxLevel, TArray<FProducerInfo>& OutProducerInfos) const;

	/** Information needed by GetAllocatedVirtualTextures() for the internally allocated virtual textures. */
	struct FAllocatedInfo
	{
		IAllocatedVirtualTexture* AllocatedVirtualTexture;
		FBox2D RemappedUV;
		uint32 RemappedLevel;
	};
	/** Get internal allocated virtual textures that touch the texture UV region. */
	void GetAllocatedVirtualTextures(FBox2D const& InUVRegion, uint32 InLevel, TArray<FAllocatedInfo>& OutInfos, TArray<uint32>& OutAllocationRequests);

protected:
	//~ Begin IAdaptiveVirtualTexture Interface.
	virtual void Destroy(class FVirtualTextureSystem* InSystem) override;
	//~ End IAdaptiveVirtualTexture Interface.

private:
	/** Lookup the index into AllocationSlots for the GridIndex. Returns INDEX_NONE if it doesn't exist. */
	uint32 GetAllocationIndex(uint32 GridIndex) const;
	/** Lookup the index into AllocationSlots for the AllocatedVT. Returns INDEX_NONE if it doesn't exist. */
	uint32 GetAllocationIndex(FAllocatedVirtualTexture* InAllocatedVT) const;

	/** Allocate a packed request. */
	void Allocate(FRHICommandListBase& RHICmdList, FVirtualTextureSystem* InSystem, uint32 InRequest, uint32 InFrame);
	/** Allocate or reallocate the allocated virtual texture at InGridIndex. */
	void Allocate(FRHICommandListBase& RHICmdList, FVirtualTextureSystem* InSystem, uint32 InGridIndex, uint32 InAllocationIndex, uint32 InNewLevel, uint32 InFrame);
	/** Free an allocated virtual texture. */
	void Free(FVirtualTextureSystem* InSystem, uint32 InAllocationIndex, uint32 InFrame);
	/** Free or reduce and reallocate the least recently used allocation. */
	bool FreeLRU(FRHICommandListBase& RHICmdList, FVirtualTextureSystem* InSystem, uint32 InFrame, uint32 InFrameUnusedThreshold);

	static IAllocatedVirtualTexture* AllocateVirtualTexture(
		FRHICommandListBase& RHICmdList,
		FVirtualTextureSystem* InSystem,
		FAllocatedVTDescription const& InAllocatedDesc,
		FIntPoint InGridSize,
		uint8 InForcedSpaceID,
		int32 InWidthInTiles,
		int32 InHeightInTiles,
		FIntPoint InAddressOffset,
		int32 InLevelOffset);

	static void DestroyVirtualTexture(FVirtualTextureSystem* InSystem, IAllocatedVirtualTexture* InAllocatedVT, TArray<FVirtualTextureProducerHandle>& OutProducersToRelease);
	static void RemapVirtualTexturePages(FVirtualTextureSystem* InSystem, FAllocatedVirtualTexture* OldAllocatedVT, FAllocatedVirtualTexture* NewAllocatedVT, uint32 InFrame);

private:
	/** Adaptive virtual texture description. */
	FAdaptiveVTDescription AdaptiveDesc;
	/** Allocated virtual texture description for the full virtual texture. Used internally to generate descriptions for the sub allocations. */
	FAllocatedVTDescription AllocatedDesc;
	/** Max mip level for the full virtual texture. */
	int32 MaxLevel;
	/** Grid size for the sub allocations. We can have one sub allocation per grid entry. */
	FIntPoint GridSize;

	/** Persistent allocated virtual texture for the low mips. */
	FAllocatedVirtualTexture* AllocatedVirtualTextureLowMips;

	/** Allocation description. */
	struct FAllocation
	{
		FAllocation(int32 InGridIndex, FAllocatedVirtualTexture* InAllocatedVT)
			: GridIndex(InGridIndex)
			, AllocatedVT(InAllocatedVT)
		{}

		/** Grid index is (YPos * GridWidth + XPos). */
		uint32 GridIndex;
		/** Allocated virtual texture. */
		FAllocatedVirtualTexture* AllocatedVT;
	};

	/** Number of valid allocations in AllocationSlots. */
	int32 NumAllocated;
	/** Array of allocation slots. Can contain nulled entries available for reuse. */
	TArray<FAllocation> AllocationSlots;
	/** Indices of free entries in the AllocationSlots array. */
	TArray<int32> FreeSlots;
	/** Map from GridIndex to allocation slot index. */
	FHashTable GridIndexMap;
	/** Map from AllocatedVT pointer to allocation slot index. */
	FHashTable AllocatedVTMap;
	/** Indices to AllocationSlots array for newly allocated virtual textures that are pending their root page before we can use them. */
	TArray<int32> SlotsPendingRootPageMap;
	/** Binary heap to track least recently used entries in AllocationSlots array. Used to decide what slots to evict next. */
	FBinaryHeap<uint32, uint32> LRUHeap;

	/** Array of packed allocation requests to process. */
	TArray<uint32> RequestsToMap;

	/** Description of an update for the indirection texture. */
	struct FIndirectionTextureUpdate
	{
		uint32 X;
		uint32 Y;
		uint32 Value;
	};

	/** Array of indirection texture updates to process. */
	TArray<FIndirectionTextureUpdate> TextureUpdates;

	/** */
	TArray<FVirtualTextureProducerHandle> ProducersToRelease;
};