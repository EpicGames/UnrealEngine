// Copyright Epic Games, Inc. All Rights Reserved.

#include "AllocatedVirtualTexture.h"

#include "Misc/StringBuilder.h"
#include "VT/VirtualTexturePoolConfig.h"
#include "VT/VirtualTextureScalability.h"
#include "VT/VirtualTextureSystem.h"
#include "VT/VirtualTextureSpace.h"
#include "VT/VirtualTexturePhysicalSpace.h"

FAllocatedVirtualTexture::FAllocatedVirtualTexture(
	FRHICommandListBase& RHICmdList,
	FVirtualTextureSystem* InSystem,
	uint32 InFrame,
	const FAllocatedVTDescription& InDesc,
	FVirtualTextureProducer* const* InProducers,
	uint32 InBlockWidthInTiles,
	uint32 InBlockHeightInTiles,
	uint32 InWidthInBlocks,
	uint32 InHeightInBlocks,
	uint32 InDepthInTiles)
	: IAllocatedVirtualTexture(InDesc, InBlockWidthInTiles, InBlockHeightInTiles, InWidthInBlocks, InHeightInBlocks, InDepthInTiles)
	, FrameAllocated(InFrame)
	, Space(nullptr)
{
	FMemory::Memzero(TextureLayers);
	FMemory::Memzero(FallbackColorPerTextureLayer);

	bool bContinuousUpdate = false;

	for (uint32 LayerIndex = 0u; LayerIndex < Description.NumTextureLayers; ++LayerIndex)
	{
		FVirtualTextureProducer const* Producer = InProducers[LayerIndex];
		// Can't have missing entries for null producers if we're not merging duplicate layers
		if (Producer || !Description.bShareDuplicateLayers)
		{
			const uint32 UniqueProducerIndex = AddUniqueProducer(InDesc.ProducerHandle[LayerIndex], Producer);
			const int32 ProducerLayerIndex = InDesc.ProducerLayerIndex[LayerIndex];
			uint32 ProducerPhysicalGroupIndex = 0u;
			FVirtualTexturePhysicalSpace* PhysicalSpace = nullptr;
			if (Producer)
			{
				ProducerPhysicalGroupIndex = Producer->GetPhysicalGroupIndexForTextureLayer(ProducerLayerIndex);
				PhysicalSpace = Producer->GetPhysicalSpaceForPhysicalGroup(ProducerPhysicalGroupIndex);
				bContinuousUpdate |= Producer->GetDescription().bContinuousUpdate;
			}
			const uint32 UniquePhysicalSpaceIndex = AddUniquePhysicalSpace(RHICmdList, PhysicalSpace, UniqueProducerIndex, ProducerPhysicalGroupIndex);
			UniquePageTableLayers[UniquePhysicalSpaceIndex].ProducerTextureLayerMask |= 1 << ProducerLayerIndex;
			const uint8 PageTableLayerLocalIndex = UniquePageTableLayers[UniquePhysicalSpaceIndex].TextureLayerCount++;
			
			TextureLayers[LayerIndex].UniquePageTableLayerIndex = UniquePhysicalSpaceIndex;
			TextureLayers[LayerIndex].PhysicalTextureIndex = PageTableLayerLocalIndex;

			if (Producer)
			{
				FallbackColorPerTextureLayer[LayerIndex] = Producer->GetDescription().LayerFallbackColor[ProducerLayerIndex].ToFColor(false).DWColor();
			}
		}
	}

	// Must have at least 1 valid layer/producer
	check(UniqueProducers.Num() > 0u);

	// Max level of overall allocated VT is limited by size in tiles
	// With multiple layers of different sizes, some layers may have mips smaller than a single tile
	// We can either use the Min or Max of Width/Height to determine the number of mips
	// - Using Max will allow more mips for rectangular VTs, which could potentially reduce aliasing in certain situations
	// - Using Min will relax alignment requirements for the page table allocator, which will tend to reduce overall VRAM usage
	const uint32 FullWidthInTiles = BlockWidthInTiles * WidthInBlocks;
	const uint32 FullHeightInTiles = BlockHeightInTiles * HeightInBlocks;
	MaxLevel = FMath::Min(MaxLevel, FMath::CeilLogTwo(FMath::Min(FullWidthInTiles, FullHeightInTiles)));

	MaxLevel = FMath::Min(MaxLevel, VIRTUALTEXTURE_LOG2_MAX_PAGETABLE_SIZE - 1u);

	// Clamp AdaptiveBias so that each block retains at least 1 tile after shifting.
	const uint32 BlockMinDim = FMath::Min(BlockWidthInTiles, BlockHeightInTiles);
	Description.AllocatedAdaptiveBias = FMath::Min<uint8>(Description.AllocatedAdaptiveBias, FMath::FloorLog2(FMath::Max(1u, BlockMinDim)));
	
	// Clamp AdaptiveBias so that MaxLevel -= AdaptiveBias cannot underflow.
	// A producer may have fewer mips than the block size would suggest.
	Description.AllocatedAdaptiveBias = FMath::Min<uint8>(Description.AllocatedAdaptiveBias, MaxLevel);
	
	// Re-clamp MaxLevel using allocator dimensions
	// The allocator requires CeilLogTwo(AllocMaxDim) >= AllocMaxLevel.
	// This can fail for non-power-of-2 tile counts where shifting by AdaptiveBias loses bits.
	const uint32 AllocMinDim = FMath::Max(1u, FMath::Min(FullWidthInTiles >> Description.AllocatedAdaptiveBias, FullHeightInTiles >> Description.AllocatedAdaptiveBias));
	MaxLevel = FMath::Min(MaxLevel, Description.AllocatedAdaptiveBias + FMath::CeilLogTwo(AllocMinDim));
	
	// Adjust Width/Height/MaxLevel for our adaptive bias.
	// Producer-facing operations will need to use AdaptiveBias back to map to the full resolution texture.
	MaxLevel -= Description.AllocatedAdaptiveBias;
	BlockWidthInTiles = BlockWidthInTiles >> Description.AllocatedAdaptiveBias;
	BlockHeightInTiles = BlockHeightInTiles >> Description.AllocatedAdaptiveBias;
	
	// Get the persistent hash that we will use for identification of AllocatedVT objects across runs.
	PersistentHash = CalculatePersistentHash(InDesc, InProducers);

	// Lock tiles
	LockOrUnlockTiles(InSystem, true);

	// Use 16bit page table entries if all physical spaces are small enough.
	// We gate this on the config support because using both 16bit and 32bit spaces can increase the total number of virtual texture spaces used and there is a hard limit.
	// But private spaces always use their own space, so we ignore the config for that case.
	bool bSupport16BitPageTable = VirtualTexturePool::Support16BitPageTables() || InDesc.bPrivateSpace;
	for (int32 Index = 0; bSupport16BitPageTable && Index < UniquePageTableLayers.Num(); ++Index)
	{
		const FVirtualTexturePhysicalSpace* PhysicalSpace = UniquePageTableLayers[Index].PhysicalSpace;
		if (PhysicalSpace && !PhysicalSpace->DoesSupport16BitPageTable())
		{
			bSupport16BitPageTable = false;
		}
	}

	FVTSpaceDescription SpaceDesc;
	SpaceDesc.Dimensions = InDesc.Dimensions;
	SpaceDesc.NumPageTableLayers = UniquePageTableLayers.Num();
	SpaceDesc.TileSize = InDesc.TileSize;
	SpaceDesc.TileBorderSize = InDesc.TileBorderSize;
	SpaceDesc.bPrivateSpace = InDesc.bPrivateSpace;
	SpaceDesc.bContinuousUpdate = bContinuousUpdate;
	SpaceDesc.MaxSpaceSize = InDesc.MaxSpaceSize > 0 ? InDesc.MaxSpaceSize : SpaceDesc.MaxSpaceSize;
	SpaceDesc.IndirectionTextureSize = InDesc.IndirectionTextureSize;
	SpaceDesc.PageTableFormat = bSupport16BitPageTable ? EVTPageTableFormat::UInt16 : EVTPageTableFormat::UInt32;

	Space = InSystem->AcquireSpace(RHICmdList, SpaceDesc, InDesc.ForceSpaceID, this);
	SpaceID = Space->GetID();
	PageTableFormat = Space->GetPageTableFormat();
}

FAllocatedVirtualTexture::~FAllocatedVirtualTexture()
{
}

void FAllocatedVirtualTexture::AssignVirtualAddress(uint32 vAddress)
{
	checkf(VirtualAddress == ~0u, TEXT("Trying to assign vAddress to AllocatedVT, already assigned"));
	check(vAddress != ~0u);
	VirtualAddress = vAddress;
	VirtualPageX = FMath::ReverseMortonCode2(vAddress);
	VirtualPageY = FMath::ReverseMortonCode2(vAddress >> 1);
}

void FAllocatedVirtualTexture::Destroy(FVirtualTextureSystem* System)
{
	check(NumRefs == 0);

	// Unlock any locked tiles
	LockOrUnlockTiles(System, false);

	// Physical pool needs to evict all pages that belong to this VT
	{
		const uint32 WidthInTiles = GetWidthInTiles();
		const uint32 HeightInTiles = GetHeightInTiles();

		TArray<FVirtualTexturePhysicalSpace*> UniquePhysicalSpaces;
		for (int32 PageTableIndex = 0u; PageTableIndex < UniquePageTableLayers.Num(); ++PageTableIndex)
		{
			if (UniquePageTableLayers[PageTableIndex].PhysicalSpace)
			{
				UniquePhysicalSpaces.Add(UniquePageTableLayers[PageTableIndex].PhysicalSpace);
			}
		}

		for (FVirtualTexturePhysicalSpace* PhysicalSpace : UniquePhysicalSpaces)
		{
			PhysicalSpace->GetPagePool().UnmapAllPagesForSpace(System, Space->GetID(), VirtualAddress, WidthInTiles, HeightInTiles, MaxLevel);
		}

		for (int32 PageTableIndex = 0u; PageTableIndex < UniquePageTableLayers.Num(); ++PageTableIndex)
		{
			if (UniquePageTableLayers[PageTableIndex].PhysicalSpace)
			{
				if (UniquePageTableLayers[PageTableIndex].PhysicalSpace->ReleaseResourceRef() == 0)
				{
					// Clean up page pool before release. Pages should all be unmapped, but may still be allocated. 
					// The physical space can be recycled later, so it's state needs to be clean.
					check(UniquePageTableLayers[PageTableIndex].PhysicalSpace->GetPagePool().GetNumMappedPages() == 0);
					UniquePageTableLayers[PageTableIndex].PhysicalSpace->GetPagePool().EvictAllPages(System);
					UniquePageTableLayers[PageTableIndex].PhysicalSpace->ReleaseResource();
				}
				UniquePageTableLayers[PageTableIndex].PhysicalSpace.SafeRelease();
			}
		}

#if DO_CHECK
		for (uint32 LayerIndex = 0; LayerIndex < Space->GetNumPageTableLayers(); ++LayerIndex)
		{
			const FTexturePageMap& PageMap = Space->GetPageMapForPageTableLayer(LayerIndex);

			TArray<FMappedTexturePage> MappedPages;
			PageMap.GetMappedPagesInRange(VirtualAddress, WidthInTiles, HeightInTiles, MappedPages);
			if (MappedPages.Num() > 0)
			{
				TStringBuilder<2048> Message;
				Message.Appendf(TEXT("Mapped pages remain after releasing AllocatedVT - vAddress: %d, Size: %d x %d, PhysicalSpaces: ["), VirtualAddress, WidthInTiles, HeightInTiles);
				for (FVirtualTexturePhysicalSpace* PhysicalSpace : UniquePhysicalSpaces)
				{
					Message.Appendf(TEXT("%d "), PhysicalSpace->GetID());
				}
				Message.Appendf(TEXT("], MappedPages: ["));

				for (const FMappedTexturePage& MappedPage : MappedPages)
				{
					Message.Appendf(TEXT("(vAddress: %d, PhysicalSpace: %d) "), MappedPage.Page.vAddress, MappedPage.PhysicalSpaceID);
				}
				Message.Appendf(TEXT("]"));
				UE_LOGF(LogVirtualTexturing, Warning, "%ls", Message.ToString());
			}
		}
#endif // DO_CHECK
	}

	Space->FreeVirtualTexture(this);
	System->ReleaseSpace(Space);
}

void FAllocatedVirtualTexture::LockOrUnlockTiles(FVirtualTextureSystem* InSystem, bool bLock) const
{
	// (Un)Lock lowest resolution mip from each producer
	// Depending on the block dimensions of the producers that make up this allocated VT, different allocated VTs may need to lock different low resolution mips from the same producer
	// In the common case where block dimensions match, same mip will be locked by all allocated VTs that make use of the same producer
	for (int32 ProducerIndex = 0u; ProducerIndex < UniqueProducers.Num(); ++ProducerIndex)
	{
		FVirtualTextureProducerHandle ProducerHandle = UniqueProducers[ProducerIndex].Handle;
		FVirtualTextureProducer* Producer = InSystem->FindProducer(ProducerHandle);
		if (Producer && Producer->GetDescription().bPersistentHighestMip)
		{
			const uint32 MipBias = UniqueProducers[ProducerIndex].MipBias;

			// Add AdaptiveBias to get full-VT MaxLevel for producer interaction.
			const uint32 AdjustedMaxLevel = MaxLevel + Description.AllocatedAdaptiveBias;
			check(MipBias <= AdjustedMaxLevel);

			const uint32 Local_vLevel = AdjustedMaxLevel - MipBias;
			checkf(Local_vLevel <= Producer->GetMaxLevel(), TEXT("Invalid Local_vLevel %d for VT producer %s, Producer MaxLevel %d, MipBias %d, AllocatedVT AdjustedMaxLevel %d"),
				Local_vLevel,
				*Producer->GetName().ToString(),
				Producer->GetMaxLevel(),
				MipBias,
				AdjustedMaxLevel);

			const uint32 MipScaleFactor = (1u << Local_vLevel);
			const uint32 RootWidthInTiles = FMath::DivideAndRoundUp(Producer->GetWidthInTiles(), MipScaleFactor);
			const uint32 RootHeightInTiles = FMath::DivideAndRoundUp(Producer->GetHeightInTiles(), MipScaleFactor);

			for (uint32 TileY = 0u; TileY < RootHeightInTiles; ++TileY)
			{
				for (uint32 TileX = 0u; TileX < RootWidthInTiles; ++TileX)
				{
					const uint32 Local_vAddress = FMath::MortonCode2(TileX) | (FMath::MortonCode2(TileY) << 1);
					FVirtualTextureLocalTile Tile(ProducerHandle, Local_vAddress, Local_vLevel);

					const uint32 LocalMipBias = Producer->GetVirtualTexture()->GetLocalMipBias(Local_vLevel, Local_vAddress);
					if (LocalMipBias > 0u)
					{
						Tile.Local_vAddress >>= (LocalMipBias * Description.Dimensions);
						Tile.Local_vLevel += LocalMipBias;
					}

					if (bLock)
					{
						InSystem->LockTile(Tile);
					}
					else
					{
						InSystem->UnlockTile(Tile, Producer);
					}
				}
			}
		}
	}
}

bool FAllocatedVirtualTexture::TryMapLockedTiles(FVirtualTextureSystem* InSystem) const
{
	bool bHasMissingTiles = false;
	for (int32 PageTableLayerIndex = 0u; PageTableLayerIndex < UniquePageTableLayers.Num(); ++PageTableLayerIndex)
	{
		const FPageTableLayerDesc& PageTableLayer = UniquePageTableLayers[PageTableLayerIndex];
		const FProducerDesc& UniqueProducer = UniqueProducers[PageTableLayer.UniqueProducerIndex];
		const FVirtualTextureProducer* Producer = InSystem->FindProducer(UniqueProducer.Handle);
		if (!Producer)
		{
			continue;
		}

		// Note that Local_vLevel is in producer space and must match what LockOrUnlockTiles locked ((MaxLevel+AdaptiveBias) - MipBias).
		const uint32 AdjustedMaxLevel = MaxLevel + Description.AllocatedAdaptiveBias;
		const uint32 WidthInTiles = Producer->GetWidthInTiles();
		const uint32 HeightInTiles = Producer->GetHeightInTiles();
		const uint32 Local_vLevel = FMath::Min(Producer->GetMaxLevel(), AdjustedMaxLevel - UniqueProducer.MipBias);
		const uint32 MipScaleFactor = (1u << Local_vLevel);
		const uint32 RootWidthInTiles = FMath::DivideAndRoundUp(WidthInTiles, MipScaleFactor);
		const uint32 RootHeightInTiles = FMath::DivideAndRoundUp(HeightInTiles, MipScaleFactor);

		FTexturePagePool& PagePool = PageTableLayer.PhysicalSpace->GetPagePool();
		FTexturePageMap& PageMap = Space->GetPageMapForPageTableLayer(PageTableLayerIndex);

		uint32 NumNonResidentPages = 0u;
		for (uint32 TileY = 0u; TileY < RootHeightInTiles; ++TileY)
		{
			for (uint32 TileX = 0u; TileX < RootWidthInTiles; ++TileX)
			{
				const uint32 vAddress = FMath::MortonCode2(VirtualPageX + (TileX << MaxLevel)) | (FMath::MortonCode2(VirtualPageY + (TileY << MaxLevel)) << 1);
				uint32 pAddress = PageMap.FindPageAddress(MaxLevel, vAddress);
				if (pAddress == ~0u)
				{
					uint32 Local_vAddress = FMath::MortonCode2(TileX) | (FMath::MortonCode2(TileY) << 1);

					const uint32 LocalMipBias = Producer->GetVirtualTexture()->GetLocalMipBias(Local_vLevel, Local_vAddress);
					Local_vAddress >>= (LocalMipBias * Description.Dimensions);

					pAddress = PagePool.FindPageAddress(UniqueProducer.Handle, PageTableLayer.ProducerPhysicalGroupIndex, Local_vAddress, Local_vLevel + LocalMipBias);
					if (pAddress != ~0u)
					{
						PagePool.MapPage(Space, PageTableLayer.PhysicalSpace, PageTableLayerIndex, MaxLevel, MaxLevel, vAddress, AdjustedMaxLevel + LocalMipBias, pAddress);
					}
					else
					{
						bHasMissingTiles = true;
						// Mark page table entry as invalid if we can't map.
						PageMap.InvalidateUnmappedRootPage(Space, PageTableLayer.PhysicalSpace, PageTableLayerIndex, MaxLevel, MaxLevel, vAddress, AdjustedMaxLevel + LocalMipBias);
					}
				}
			}
		}
	}

	// Display a message if we've failed to map pages for this after a set number of frames
	// Generally there should be no delay, but if the system is saturated, it's possible that locked pages may not be loaded immediately
	if (bHasMissingTiles && InSystem->GetFrame() > FrameAllocated + 30u)
	{
		UE_LOGF(LogVirtualTexturing, Verbose, "Failed to map lowest resolution mip for AllocatedVT %ls (%d frames)", *Description.Name.ToString(), InSystem->GetFrame() - FrameAllocated);
	}

	return !bHasMissingTiles;
}

uint32 FAllocatedVirtualTexture::CalculatePersistentHash(FAllocatedVTDescription const& InDesc, FVirtualTextureProducer* const* InProducers) const
{
	uint32 Hash = 0;

	for (uint32 LayerIndex = 0u; LayerIndex < InDesc.NumTextureLayers; ++LayerIndex)
	{
		FVirtualTextureProducer const* Producer = InProducers[LayerIndex];
		if (Producer != nullptr)
		{
			Hash = HashCombine(Hash, LayerIndex);
			Hash = HashCombine(Hash, Producer->GetDescription().FullNameHash);
		}
	}

	Hash = HashCombine(Hash, GetTypeHash(InDesc.TileSize));
	Hash = HashCombine(Hash, GetTypeHash(InDesc.TileBorderSize));
	Hash = HashCombine(Hash, GetTypeHash(InDesc.Dimensions));
	Hash = HashCombine(Hash, GetTypeHash(InDesc.NumTextureLayers));
	Hash = HashCombine(Hash, GetTypeHash(InDesc.AllocatedAdaptiveBias));
	Hash = HashCombine(Hash, GetTypeHash(InDesc.PackedFlags));

	return Hash;
}

uint32 FAllocatedVirtualTexture::AddUniqueProducer(FVirtualTextureProducerHandle const& InHandle, const FVirtualTextureProducer* InProducer)
{
	for (int32 Index = 0u; Index < UniqueProducers.Num(); ++Index)
	{
		if (UniqueProducers[Index].Handle == InHandle)
		{
			return Index;
		}
	}
	const uint32 Index = UniqueProducers.AddDefaulted();
	check(Index < VIRTUALTEXTURE_SPACE_MAXLAYERS);
	
	uint32 MipBias = 0u;
	if (InProducer)
	{
		const FVTProducerDescription& ProducerDesc = InProducer->GetDescription();
		// maybe these values should just be set by producers, rather than also set on AllocatedVT desc
		check(ProducerDesc.Dimensions == Description.Dimensions);
		check(ProducerDesc.TileSize == Description.TileSize);
		check(ProducerDesc.TileBorderSize == Description.TileBorderSize);

		const uint32 MipBiasX = FMath::CeilLogTwo(BlockWidthInTiles / ProducerDesc.BlockWidthInTiles);
		const uint32 MipBiasY = FMath::CeilLogTwo(BlockHeightInTiles / ProducerDesc.BlockHeightInTiles);
		check(ProducerDesc.BlockWidthInTiles << MipBiasX == BlockWidthInTiles);
		check(ProducerDesc.BlockHeightInTiles << MipBiasY == BlockHeightInTiles);

		// If the producer aspect ratio doesn't match the aspect ratio for the AllocatedVT, there's no way to choose a 100% mip bias
		// By chossing the minimum of X/Y bias, we'll effectively crop this producer to match the aspect ratio of the AllocatedVT
		// This case can happen as base materials will choose to group VTs together into a stack as long as all the textures assigned in the base material share the same aspect ratio
		// But it's possible for a MI to overide some of thse textures such that the aspect ratios no longer match
		// This will be fine for some cases, especially if the common case where the mismatched texture is a small dummy texture with a constant color
		MipBias = FMath::Min(MipBiasX, MipBiasY);

		MaxLevel = FMath::Max<uint32>(MaxLevel, ProducerDesc.MaxLevel + MipBias);
	}

	UniqueProducers[Index].Handle = InHandle;
	UniqueProducers[Index].MipBias = MipBias;
	
	return Index;
}

uint32 FAllocatedVirtualTexture::AddUniquePhysicalSpace(FRHICommandListBase& InRHICmdList, FVirtualTexturePhysicalSpace* InPhysicalSpace, uint32 InUniqueProducerIndex, uint32 InProducerPhysicalSpaceIndex)
{
	if (Description.bShareDuplicateLayers)
	{
		for (int32 Index = 0u; Index < UniquePageTableLayers.Num(); ++Index)
		{
			if (UniquePageTableLayers[Index].PhysicalSpace == InPhysicalSpace &&
				UniquePageTableLayers[Index].UniqueProducerIndex == InUniqueProducerIndex &&
				UniquePageTableLayers[Index].ProducerPhysicalGroupIndex == InProducerPhysicalSpaceIndex)
			{
				return Index;
			}
		}
	}
	const uint32 Index = UniquePageTableLayers.AddDefaulted();
	check(Index < VIRTUALTEXTURE_SPACE_MAXLAYERS);

	UniquePageTableLayers[Index].PhysicalSpace = InPhysicalSpace;
	UniquePageTableLayers[Index].UniqueProducerIndex = InUniqueProducerIndex;
	UniquePageTableLayers[Index].ProducerPhysicalGroupIndex = InProducerPhysicalSpaceIndex;
	UniquePageTableLayers[Index].ProducerTextureLayerMask = 0;
	UniquePageTableLayers[Index].TextureLayerCount = 0;

	if (InPhysicalSpace && InPhysicalSpace->AddResourceRef() == 1)
	{
		InPhysicalSpace->InitResource(InRHICmdList);
	}

	return Index;
}

uint32 FAllocatedVirtualTexture::GetNumPageTableTextures() const
{
	return Space->GetNumPageTableTextures();
}

FRHITexture* FAllocatedVirtualTexture::GetPageTableTexture(uint32 InPageTableIndex) const
{
	return Space->GetPageTableTexture(InPageTableIndex);
}

FRHITexture* FAllocatedVirtualTexture::GetPageTableIndirectionTexture() const
{
	return Space->GetPageTableIndirectionTexture();
}

uint32 FAllocatedVirtualTexture::GetPhysicalTextureSize(uint32 InLayerIndex) const
{
	if (InLayerIndex < Description.NumTextureLayers)
	{
		const FVirtualTexturePhysicalSpace* PhysicalSpace = UniquePageTableLayers[TextureLayers[InLayerIndex].UniquePageTableLayerIndex].PhysicalSpace;
		return PhysicalSpace ? PhysicalSpace->GetTextureSize() : 0u;
	}
	return 0u;
}

FRHITexture* FAllocatedVirtualTexture::GetPhysicalTexture(uint32 InLayerIndex) const
{
	if (InLayerIndex < Description.NumTextureLayers)
	{
		if (const FVirtualTexturePhysicalSpace* PhysicalSpace = UniquePageTableLayers[TextureLayers[InLayerIndex].UniquePageTableLayerIndex].PhysicalSpace)
		{
			TRefCountPtr<IPooledRenderTarget> PooledRenderTarget = PhysicalSpace->GetPhysicalTexture(TextureLayers[InLayerIndex].PhysicalTextureIndex);
			if (PooledRenderTarget.IsValid())
			{
				return PooledRenderTarget->GetRHI();
			}
		}
	}
	return nullptr;
}

FRHIShaderResourceView* FAllocatedVirtualTexture::GetPhysicalTextureSRV(uint32 InLayerIndex, bool bSRGB) const
{
	if (InLayerIndex < Description.NumTextureLayers)
	{
		const FVirtualTexturePhysicalSpace* PhysicalSpace = UniquePageTableLayers[TextureLayers[InLayerIndex].UniquePageTableLayerIndex].PhysicalSpace;
		return PhysicalSpace ? PhysicalSpace->GetPhysicalTextureSRV(TextureLayers[InLayerIndex].PhysicalTextureIndex, bSRGB) : nullptr;
	}
	return nullptr;
}

uint32 FAllocatedVirtualTexture::GetResidencyMipMapBiasGroup() const
{
	// For correctness we should fill a 4 bit mask of groups from all layers.
	// But that approach also requires extra shader maths to accumulate the max bias from all 4 groups in the mask.
	// So for now we are just take the group from layer 0 and pass it up directly.
	FVirtualTexturePhysicalSpace* PhysicalSpace = GetPhysicalSpaceForPageTableLayer(0);
	return PhysicalSpace != nullptr ? PhysicalSpace->GetDescriptionExt().ResidencyMipMapBiasGroup : 0;
}

static inline uint32 BitcastFloatToUInt32(float v)
{
	const union
	{
		float FloatValue;
		uint32 UIntValue;
	} u = { v };
	return u.UIntValue;
}

void FAllocatedVirtualTexture::GetPackedPageTableUniform(FUintVector4* OutUniform) const
{
	// Force invalidate the page table when the space is no longer in the supported feedback range.
	// This will cause us to sample the fallback color only.
	if (UNLIKELY(SpaceID >= VIRTUALTEXTURE_MAX_FEEDBACK_SPACES))
	{
		OutUniform[0] = OutUniform[1] = FUintVector4(0, 0, 0, 0);
		return;
	}

	const uint32 vPageX = VirtualPageX;
	const uint32 vPageY = VirtualPageY;
	const uint32 vPageSize = GetVirtualTileSize();
	const uint32 PageBorderSize = GetTileBorderSize();
	const uint32 WidthInPages = GetWidthInTiles();
	const uint32 HeightInPages = GetHeightInTiles();
	const uint32 vPageTableMipBias = FMath::FloorLog2(vPageSize);
	const uint32 MipBiasGroup = GetResidencyMipMapBiasGroup();
	const uint32 AdaptiveLevelBias = Description.AdaptiveLevelBias;
	const uint32 PageTable32Bit = (GetPageTableFormat() == EVTPageTableFormat::UInt32) ? 1 : 0;

	// Here MaxAnisotropy only controls the VT mip level selection
	// We don't need to limit this value based on border size, and we can add this factor in even if HW anisotropic filtering is disabled
	const uint32 MaxAnisotropy = VirtualTextureScalability::GetMaxAnisotropy();
	const uint32 MaxAnisotropyLog2 = (MaxAnisotropy > 0u) ? FMath::FloorLog2(MaxAnisotropy) : 0u;

	// make sure everything fits in the allocated number of bits
	checkSlow(vPageX < 4096u);
	checkSlow(vPageY < 4096u);
	checkSlow(vPageTableMipBias < 16u);
	checkSlow(MipBiasGroup < 4u);
	checkSlow(MaxLevel < 16u);
	checkSlow(AdaptiveLevelBias < 16u);
	checkSlow(Description.AllocatedAdaptiveBias < 16u);

	OutUniform[0].X = BitcastFloatToUInt32(1.0f / (float)WidthInBlocks);
	OutUniform[0].Y = BitcastFloatToUInt32(1.0f / (float)HeightInBlocks);

	OutUniform[0].Z = BitcastFloatToUInt32((float)WidthInPages);
	OutUniform[0].W = BitcastFloatToUInt32((float)HeightInPages);

	OutUniform[1].X = BitcastFloatToUInt32((float)MaxAnisotropyLog2);
	OutUniform[1].Y = vPageX | (vPageY << 12) | (vPageTableMipBias << 24) | (MipBiasGroup << 28);
	OutUniform[1].Z = MaxLevel | (AdaptiveLevelBias << 4) | (PageTable32Bit << 8) | (Description.AllocatedAdaptiveBias << 9);
	OutUniform[1].W = (SpaceID << 28);
}

void FAllocatedVirtualTexture::GetPackedUniform(FUintVector4* OutUniform, uint32 LayerIndex) const
{
	const uint32 PhysicalTextureSize = GetPhysicalTextureSize(LayerIndex);
	if (PhysicalTextureSize > 0u)
	{
		const uint32 vPageSize = GetVirtualTileSize();
		const uint32 PageBorderSize = GetTileBorderSize();
		const float RcpPhysicalTextureSize = 1.0f / float(PhysicalTextureSize);
		const uint32 pPageSize = vPageSize + PageBorderSize * 2u;

		OutUniform->X = FallbackColorPerTextureLayer[LayerIndex];
		OutUniform->Y = BitcastFloatToUInt32((float)vPageSize * RcpPhysicalTextureSize);
		OutUniform->Z = BitcastFloatToUInt32((float)PageBorderSize * RcpPhysicalTextureSize);
		OutUniform->W = BitcastFloatToUInt32((float)pPageSize * RcpPhysicalTextureSize);
	}
	else
	{
		OutUniform->X = 0u;
		OutUniform->Y = 0u;
		OutUniform->Z = 0u;
		OutUniform->W = 0u;
	}
}
