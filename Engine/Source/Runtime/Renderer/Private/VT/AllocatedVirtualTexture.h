// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTextureShared.h"
#include "RendererInterface.h"
#include "VirtualTexturing.h"
#include "VirtualTexturePhysicalSpace.h"
#include "HAL/ThreadSafeCounter.h"

class FVirtualTextureSpace;
class FVirtualTexturePhysicalSpace;
class FVirtualTextureProducer;
class FVirtualTextureSystem;

class FAllocatedVirtualTexture final : public IAllocatedVirtualTexture
{
public:
	FAllocatedVirtualTexture(
		FRHICommandListBase& RHICmdList,
		FVirtualTextureSystem* InSystem,
		uint32 InFrame,
		const FAllocatedVTDescription& InDesc,
		FVirtualTextureProducer* const* InProducers,
		uint32 InBlockWidthInTiles,
		uint32 InBlockHeightInTiles,
		uint32 InWidthInBlocks,
		uint32 InHeightInBlocks,
		uint32 InDepthInTiles);

	virtual ~FAllocatedVirtualTexture();

	inline uint32 GetFrameAllocated() const { return FrameAllocated; }

	void AssignVirtualAddress(uint32 vAddress);

	void LockOrUnlockTiles(FVirtualTextureSystem* InSystem, bool bLock) const;

	// begin IAllocatedVirtualTexture
	virtual uint32 GetPersistentHash() const override { return PersistentHash; }
	virtual uint32 GetNumPageTableTextures() const override;
	virtual FRHITexture* GetPageTableTexture(uint32 InPageTableIndex) const override;
	virtual FRHITexture* GetPageTableIndirectionTexture() const override;
	virtual uint32 GetPhysicalTextureSize(uint32 InLayerIndex) const override;
	virtual FRHITexture* GetPhysicalTexture(uint32 InLayerIndex) const override;
	virtual FRHIShaderResourceView* GetPhysicalTextureSRV(uint32 InLayerIndex, bool bSRGB) const override;
	virtual void GetPackedPageTableUniform(FUintVector4* OutUniform) const override;
	virtual void GetPackedUniform(FUintVector4* OutUniform, uint32 LayerIndex) const override;
	virtual void Destroy(FVirtualTextureSystem* InSystem) override;
	virtual bool TryMapLockedTiles(FVirtualTextureSystem* InSystem) const override;
	// end IAllocatedVirtualTexture

	inline uint32 GetNumUniqueProducers() const { return UniqueProducers.Num(); }
	inline const FVirtualTextureProducerHandle& GetUniqueProducerHandle(uint32 InProducerIndex) const { return UniqueProducers[InProducerIndex].Handle; }
	inline uint8 GetUniqueProducerMipBias(uint32 InProducerIndex) const { return UniqueProducers[InProducerIndex].MipBias; }

	inline uint32 GetNumPageTableLayers() const { return UniquePageTableLayers.Num(); }
	inline FVirtualTexturePhysicalSpace* GetPhysicalSpaceForPageTableLayer(uint32 InLayerIndex) const { return UniquePageTableLayers[InLayerIndex].PhysicalSpace; }
	inline uint32 GetProducerIndexForPageTableLayer(uint32 InLayerIndex) const { return UniquePageTableLayers[InLayerIndex].UniqueProducerIndex; }
	inline uint32 GetProducerTextureLayerMaskForPageTableLayer(uint32 InLayerIndex) const { return UniquePageTableLayers[InLayerIndex].ProducerTextureLayerMask; }
	inline uint32 GetProducerPhysicalGroupIndexForPageTableLayer(uint32 InLayerIndex) const { return UniquePageTableLayers[InLayerIndex].ProducerPhysicalGroupIndex; }

private:
	uint32 CalculatePersistentHash(FAllocatedVTDescription const& InDesc, FVirtualTextureProducer* const* InProducers) const;
	uint32 AddUniqueProducer(FVirtualTextureProducerHandle const& InHandle, const FVirtualTextureProducer* InProducer);
	uint32 AddUniquePhysicalSpace(FRHICommandListBase& RHICmdList, FVirtualTexturePhysicalSpace* InPhysicalSpace, uint32 InUniqueProducerIndex, uint32 InProducerPhysicalSpaceIndex);
	uint32 GetResidencyMipMapBiasGroup() const;

	uint32 FrameAllocated;
	uint32 PersistentHash;

	FVirtualTextureSpace* Space;

	// Unique producers in this AllocatedVT.
	struct FProducerDesc
	{
		FVirtualTextureProducerHandle Handle;
		uint8 MipBias;
	};
	TArray<FProducerDesc> UniqueProducers;

	// Unique page table layers in this AllocatedVT.
	// We create a page table layer for each physical space group in each producer.
	struct FPageTableLayerDesc
	{
		TRefCountPtr<FVirtualTexturePhysicalSpace> PhysicalSpace;
		uint8 UniqueProducerIndex;
		uint8 ProducerPhysicalGroupIndex;
		uint16 ProducerTextureLayerMask;
		uint8 TextureLayerCount;
	};
	TArray<FPageTableLayerDesc> UniquePageTableLayers;

	// Texture layers in this AllocatedVT.
	// Each texture layer maps onto one page table layer.
	// Each texture layer needs a unique index within the page table layer to describe the unique physical texture it references.
	struct FTextureLayerDesc
	{
		uint8 UniquePageTableLayerIndex;
		uint8 PhysicalTextureIndex;
	};
	FTextureLayerDesc TextureLayers[VIRTUALTEXTURE_SPACE_MAXLAYERS];

	// The fallback color to use on each texture layer when unmapped.
	// This is stored as linear UNorm instead of full FLinearColor since that is what the shader uses.
	// That means that HDR formats will be clamped.
	uint32 FallbackColorPerTextureLayer[VIRTUALTEXTURE_SPACE_MAXLAYERS];
};
