// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/MeshPaintVirtualTextureSceneExtension.h"

#include "GlobalRenderResources.h"
#include "ScenePrivate.h"
#include "SceneUniformBuffer.h"
#include "ShaderParameterMacros.h"
#include "VT/MeshPaintVirtualTexture.h"

IMPLEMENT_SCENE_EXTENSION(FMeshPaintVirtualTextureSceneExtension);

bool FMeshPaintVirtualTextureSceneExtension::ShouldCreateExtension(FScene& InScene)
{
	return MeshPaintVirtualTexture::IsSupported(InScene.GetShaderPlatform());
}

ISceneExtensionRenderer* FMeshPaintVirtualTextureSceneExtension::CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags)
{
	return new FRenderer(InSceneRenderer);
}

BEGIN_SHADER_PARAMETER_STRUCT(FMeshPaintTextureParameters, RENDERER_API)
	SHADER_PARAMETER_TEXTURE(Texture2D<uint4>, PageTableTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D<float4>, PhysicalTexture)
	SHADER_PARAMETER(FUintVector4, PackedUniform)
END_SHADER_PARAMETER_STRUCT()

DECLARE_SCENE_UB_STRUCT(FMeshPaintTextureParameters, MeshPaint, RENDERER_API)

static void GetMeshPaintParameters(MeshPaintVirtualTexture::FUniformParams const& InParameters, FMeshPaintTextureParameters& OutParameters)
{
	OutParameters.PageTableTexture = InParameters.PageTableTexture ? InParameters.PageTableTexture : GBlackUintTexture->TextureRHI;
	OutParameters.PhysicalTexture = InParameters.PhysicalTexture ? InParameters.PhysicalTexture : GBlackTextureWithSRV->TextureRHI;
	OutParameters.PackedUniform = InParameters.PackedUniform;
}

static void GetDefaultMeshPaintParameters(FMeshPaintTextureParameters& OutParameters, FRDGBuilder& GraphBuilder)
{
	MeshPaintVirtualTexture::FUniformParams DefaultParameters;
	GetMeshPaintParameters(DefaultParameters, OutParameters);
}

IMPLEMENT_SCENE_UB_STRUCT(FMeshPaintTextureParameters, MeshPaint, GetDefaultMeshPaintParameters);

void FMeshPaintVirtualTextureSceneExtension::FRenderer::UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer)
{
	FMeshPaintTextureParameters Parameters;
	GetMeshPaintParameters(MeshPaintVirtualTexture::GetUniformParams(), Parameters);
	SceneUniformBuffer.Set(SceneUB::MeshPaint, Parameters);
}
