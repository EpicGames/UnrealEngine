// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneExtensions.h"

class FMeshPaintVirtualTextureSceneExtension : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FMeshPaintVirtualTextureSceneExtension);

public:
	using ISceneExtension::ISceneExtension;

	static bool ShouldCreateExtension(FScene& Scene);

	//~ Begin ISceneExtension Interface.
	virtual ISceneExtensionRenderer* CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags) override;
	//~ End ISceneExtension Interface.

protected:
	class FRenderer : public ISceneExtensionRenderer
	{
		DECLARE_SCENE_EXTENSION_RENDERER(FRenderer, FMeshPaintVirtualTextureSceneExtension);

	public:
		using ISceneExtensionRenderer::ISceneExtensionRenderer;
		//~ Begin ISceneExtensionRenderer Interface.
		virtual void UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer) override;
		//~ End ISceneExtensionRenderer Interface.
	};
};
