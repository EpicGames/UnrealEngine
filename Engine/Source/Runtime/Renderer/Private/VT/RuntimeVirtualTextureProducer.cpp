// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeVirtualTextureProducer.h"

#include "RendererInterface.h"
#include "VT/RuntimeVirtualTextureRender.h"
#include "VT/RuntimeVirtualTextureSceneProxy.h"
#include "RenderGraphBuilder.h"
#include "UObject/Class.h"

FRuntimeVirtualTextureFinalizer::FRuntimeVirtualTextureFinalizer(
	FVTProducerDescription const& InDesc, 
	int32 InRuntimeVirtualTextureId,
	ERuntimeVirtualTextureMaterialType InMaterialType, 
	bool InClearTextures, 
	FSceneInterface* InScene, 
	FTransform const& InUVToWorld,
	FBox const& InWorldBounds,
	FVector4f const& InCustomMaterialData)
	: Desc(InDesc)
	, RuntimeVirtualTextureId(InRuntimeVirtualTextureId)
	, MaterialType(InMaterialType)
	, bClearTextures(InClearTextures)
	, Scene(InScene)
	, UVToWorld(InUVToWorld)
	, WorldBounds(InWorldBounds)
	, CustomMaterialData(InCustomMaterialData)
{
}

bool FRuntimeVirtualTextureFinalizer::IsReady()
{
	return RuntimeVirtualTexture::IsSceneReadyToRender(Scene);
}

void FRuntimeVirtualTextureFinalizer::AddTile(FTileEntry& Tile)
{
	Tiles.Add(Tile);
}

UE_TRACE_EVENT_BEGIN(Cpu, RuntimeVirtualTextureFinalizerRenderFinalize, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Priority) // Note : these strings are actually static (display names of EVTProducerPriority), so we could avoid using a string here, but Insights doesn't have support for FNames or enums yet
UE_TRACE_EVENT_FIELD(int32, NumTiles)
UE_TRACE_EVENT_FIELD(int32, NumBatches)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, NumTilesPerLevel) // Note : this could be a int32[] but Insights doesn't support displaying that in a generic way so we have to store it as a string for now
UE_TRACE_EVENT_END()

namespace RuntimeVirtualTexture::Private
{

TStaticArray<FString, static_cast<int32>(EVTProducerPriority::Count)> GetProducerPriorityDisplayStrings()
{
	TStaticArray<FString, static_cast<int32>(EVTProducerPriority::Count)> EnumStrings;
	for (uint8 EnumValue = 0; EnumValue < static_cast<uint8>(EVTProducerPriority::Count); ++EnumValue)
	{
		EnumStrings[EnumValue] = UEnum::GetDisplayValueAsText(static_cast<EVTProducerPriority>(EnumValue)).ToString();
	}
	return EnumStrings;
}

} // namespace RuntimeVirtualTexture::Private

void FRuntimeVirtualTextureFinalizer::RenderFinalize(FRDGBuilder& GraphBuilder, ISceneRenderer* SceneRenderer)
{
	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	RuntimeVirtualTexture::FRenderPageBatchDesc RenderPageBatchDesc;
	RenderPageBatchDesc.SceneRenderer = SceneRenderer;
	RenderPageBatchDesc.RuntimeVirtualTextureId = RuntimeVirtualTextureId;
	RenderPageBatchDesc.UVToWorld = UVToWorld;
	RenderPageBatchDesc.WorldBounds = WorldBounds;
	RenderPageBatchDesc.MaterialType = MaterialType;
	RenderPageBatchDesc.MaxLevel = Desc.MaxLevel;
	RenderPageBatchDesc.bClearTextures = bClearTextures;
	RenderPageBatchDesc.bIsThumbnails = false;
	RenderPageBatchDesc.FixedColor = FLinearColor::Transparent;
	RenderPageBatchDesc.CustomMaterialData = CustomMaterialData;
	
#if CPUPROFILERTRACE_ENABLED
	TArray<int32, TInlineAllocator<16>> NumTilesPerLevel;
	NumTilesPerLevel.AddDefaulted(16);
#endif // CPUPROFILERTRACE_ENABLED

	for (int LayerIndex = 0; LayerIndex < RuntimeVirtualTexture::MaxTextureLayers; ++LayerIndex)
	{
		RenderPageBatchDesc.Targets[LayerIndex].PooledRenderTarget = Tiles[0].Targets[LayerIndex].PooledRenderTarget;
	}

	int32 BatchSize = 0;
	for (auto Entry : Tiles)
	{
		bool bBreakBatchForTextures = false;
		for (int LayerIndex = 0; LayerIndex < RuntimeVirtualTexture::MaxTextureLayers; ++LayerIndex)
		{
			// This should never happen which is why we don't bother sorting to maximize batch size
			bBreakBatchForTextures |= (RenderPageBatchDesc.Targets[LayerIndex].PooledRenderTarget != Entry.Targets[LayerIndex].PooledRenderTarget);
		}

		if (BatchSize == RuntimeVirtualTexture::MaxRenderPageBatch || bBreakBatchForTextures)
		{
			RenderPageBatchDesc.NumPageDescs = BatchSize;
			Batches.Add(RuntimeVirtualTexture::InitPageBatch(GraphBuilder, RenderPageBatchDesc));
			BatchSize = 0;
		}

		if (bBreakBatchForTextures)
		{
			for (int LayerIndex = 0; LayerIndex < RuntimeVirtualTexture::MaxTextureLayers; ++LayerIndex)
			{
				RenderPageBatchDesc.Targets[LayerIndex].PooledRenderTarget = Entry.Targets[LayerIndex].PooledRenderTarget;
			}
		}

		RuntimeVirtualTexture::FRenderPageDesc& RenderPageDesc = RenderPageBatchDesc.PageDescs[BatchSize++];

		const float X = (float)FMath::ReverseMortonCode2_64(Entry.vAddress);
		const float Y = (float)FMath::ReverseMortonCode2_64(Entry.vAddress >> 1);
		const float DivisorX = (float)Desc.BlockWidthInTiles / (float)(1 << Entry.vLevel);
		const float DivisorY = (float)Desc.BlockHeightInTiles / (float)(1 << Entry.vLevel);

		const FVector2D UV(X / DivisorX, Y / DivisorY);
		const FVector2D UVSize(1.f / DivisorX, 1.f / DivisorY);
		const FVector2D UVBorder = UVSize * ((float)Desc.TileBorderSize / (float)Desc.TileSize);
		const FBox2D UVRange(UV - UVBorder, UV + UVSize + UVBorder);

		RenderPageDesc.vLevel = Entry.vLevel;
		RenderPageDesc.UVRange = UVRange;

#if CPUPROFILERTRACE_ENABLED
		if (Entry.vLevel >= NumTilesPerLevel.Num())
		{
			NumTilesPerLevel.AddDefaulted(Entry.vLevel - NumTilesPerLevel.Num() + 1);
		}
		++NumTilesPerLevel[Entry.vLevel];
#endif // CPUPROFILERTRACE_ENABLED

		const int32 TileSize = Desc.TileSize + 2 * Desc.TileBorderSize;
		for (int LayerIndex = 0; LayerIndex < RuntimeVirtualTexture::MaxTextureLayers; ++LayerIndex)
		{
			const FIntPoint DestinationRectStart0(Entry.Targets[LayerIndex].pPageLocation.X * TileSize, Entry.Targets[LayerIndex].pPageLocation.Y * TileSize);
			RenderPageDesc.DestRect[LayerIndex] = FIntRect(DestinationRectStart0, DestinationRectStart0 + FIntPoint(TileSize, TileSize));
		}
	}

	if (BatchSize > 0)
	{
		RenderPageBatchDesc.NumPageDescs = BatchSize;
		Batches.Add(RuntimeVirtualTexture::InitPageBatch(GraphBuilder, RenderPageBatchDesc));
	}

#if CPUPROFILERTRACE_ENABLED
	auto GetNumTilesPerLevelToString = [&]() -> FString
		{
			int32 MinMipLevel = NumTilesPerLevel.IndexOfByPredicate([](int32 InNumTiles) { return (InNumTiles > 0); });
			check(MinMipLevel != INDEX_NONE);
			int32 MaxMipLevel = NumTilesPerLevel.FindLastByPredicate([](int32 InNumTiles) { return (InNumTiles > 0); });
			check(MaxMipLevel != INDEX_NONE);
			// Trim the last zeroes to reduce the size of the trace string
			if (MaxMipLevel + 1 < NumTilesPerLevel.Num())
			{
				NumTilesPerLevel.RemoveAt(MaxMipLevel + 1, NumTilesPerLevel.Num() - MaxMipLevel - 1);
			}
			// Trim the first zeroes to reduce the size of the trace string even more
			if (MinMipLevel > 0)
			{
				NumTilesPerLevel.RemoveAt(0, MinMipLevel);
			}

			FString NumTilesPerLevelString = FString::Printf(TEXT("Mips[%i-%i] "), MinMipLevel, MaxMipLevel);

			for (int32 NumTiles : NumTilesPerLevel)
			{
				NumTilesPerLevelString += FString::Printf(TEXT("%i,"), NumTiles);
			}
			// Remove the trailing comma
			NumTilesPerLevelString.LeftChopInline(1);
			return NumTilesPerLevelString;
		};

	// static store for fast retrieval : 
	static TStaticArray<FString, static_cast<int32>(EVTProducerPriority::Count)> EnumStrings = RuntimeVirtualTexture::Private::GetProducerPriorityDisplayStrings();

	UE_TRACE_LOG_SCOPED_T(Cpu, RuntimeVirtualTextureFinalizerRenderFinalize, CpuChannel)
		<< RuntimeVirtualTextureFinalizerRenderFinalize.Name(*Desc.Name.ToString())
		<< RuntimeVirtualTextureFinalizerRenderFinalize.Priority(*EnumStrings[static_cast<int32>(Desc.Priority)])
		<< RuntimeVirtualTextureFinalizerRenderFinalize.NumTiles(Tiles.Num())
		<< RuntimeVirtualTextureFinalizerRenderFinalize.NumBatches(Batches.Num())
		<< RuntimeVirtualTextureFinalizerRenderFinalize.NumTilesPerLevel(*GetNumTilesPerLevelToString());
#endif // CPUPROFILERTRACE_ENABLED

	for (RuntimeVirtualTexture::FBatchRenderContext const* Batch : Batches)
	{
		RuntimeVirtualTexture::RenderPageBatch(GraphBuilder, *Batch);
	}
}

void FRuntimeVirtualTextureFinalizer::Finalize(FRDGBuilder& GraphBuilder)
{
	for (RuntimeVirtualTexture::FBatchRenderContext const* Batch : Batches)
	{
		RuntimeVirtualTexture::FinalizePageBatch(GraphBuilder, *Batch);
	}

	Tiles.SetNumUnsafeInternal(0);
	Batches.SetNumUnsafeInternal(0);
}

FRuntimeVirtualTextureProducer::FRuntimeVirtualTextureProducer(
	FVTProducerDescription const& InDesc,
	int32 InRuntimeVirtualTextureId,
	ERuntimeVirtualTextureMaterialType InMaterialType,
	bool InClearTextures,
	FSceneInterface* InScene,
	FTransform const& InUVToWorld,
	FBox const& InWorldBounds)
	: Finalizer(InDesc, InRuntimeVirtualTextureId, InMaterialType, InClearTextures, InScene, InUVToWorld, InWorldBounds, FVector4f::Zero())
{
}

FRuntimeVirtualTextureProducer::FRuntimeVirtualTextureProducer(
	FVTProducerDescription const& InDesc, 
	int32 InRuntimeVirtualTextureId,
	ERuntimeVirtualTextureMaterialType InMaterialType,
	bool InClearTextures, 
	FSceneInterface* InScene, 
	FTransform const& InUVToWorld,
	FBox const& InWorldBounds,
	FVector4f const& InCustomMaterialData)
	: Finalizer(InDesc, InRuntimeVirtualTextureId, InMaterialType, InClearTextures, InScene, InUVToWorld, InWorldBounds, InCustomMaterialData)
{
}

FVTRequestPageResult FRuntimeVirtualTextureProducer::RequestPageData(
	FRHICommandListBase& RHICmdList,
	const FVirtualTextureProducerHandle& ProducerHandle,
	uint8 LayerMask,
	uint8 vLevel,
	uint64 vAddress,
	EVTRequestPagePriority Priority)
{
	// Note that when the finalizer is not ready (outside of the Begin/End Scene Render) we return the Saturated status here.
	// This is to indicate that the RVT can't render at this time (because we require the GPU Scene to be up to date).
	// This will happen for DrawTileMesh() style rendering used by material/HLOD baking.
	// It's best to avoid sampling RVT in material baking, but if it is necessary then an option is to have streaming mips built and enabled.
	FVTRequestPageResult result;
	result.Handle = 0;
	result.Status = Finalizer.IsReady() ? EVTRequestPageStatus::Available : EVTRequestPageStatus::Saturated;
	return result;
}

IVirtualTextureFinalizer* FRuntimeVirtualTextureProducer::ProducePageData(
	FRHICommandListBase& RHICmdList,
	ERHIFeatureLevel::Type FeatureLevel,
	EVTProducePageFlags Flags,
	const FVirtualTextureProducerHandle& ProducerHandle,
	uint8 LayerMask,
	uint8 vLevel,
	uint64 vAddress,
	uint64 RequestHandle,
	const FVTProduceTargetLayer* TargetLayers)
{
	FRuntimeVirtualTextureFinalizer::FTileEntry Tile;
	Tile.vAddress = vAddress;
	Tile.vLevel = vLevel;

	// Partial layer masks can happen when one layer has more physical space available so that old pages are evicted at different rates.
	// We currently render all layers even for these partial requests. That might be considered inefficient?
	// But since the problem is avoided by setting bSinglePhysicalSpace on the URuntimeVirtualTexture we can live with it.

	for (int LayerIndex = 0; LayerIndex < RuntimeVirtualTexture::MaxTextureLayers; ++LayerIndex)
	{
		if (TargetLayers[LayerIndex].PooledRenderTarget != nullptr)
		{
			Tile.Targets[LayerIndex] = TargetLayers[LayerIndex];
		}
	}

	Finalizer.AddTile(Tile);

	return &Finalizer;
}
