// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "VirtualTexturing.h"
#include "VT/RuntimeVirtualTextureEnum.h"

enum class ERuntimeVirtualTextureMaterialType : uint8;
class FRHITexture;
class FSceneInterface;
namespace RuntimeVirtualTexture { class FBatchRenderContext; }

/** IVirtualTextureFinalizer implementation that renders the virtual texture pages on demand. */
class FRuntimeVirtualTextureFinalizer : public IVirtualTextureFinalizer
{
public:
	FRuntimeVirtualTextureFinalizer(
		FVTProducerDescription const& InDesc, 
		int32 InRuntimeVirtualTextureId,
		ERuntimeVirtualTextureMaterialType InMaterialType, 
		bool InClearTextures, 
		FSceneInterface* InScene, 
		FTransform const& InUVToWorld,
		FBox const& InWorldBounds,
		FVector4f const& InCustomMaterialData);

	virtual ~FRuntimeVirtualTextureFinalizer() = default;

	/** A description for a single tile to render. */
	struct FTileEntry
	{
		FVTProduceTargetLayer Targets[RuntimeVirtualTexture::MaxTextureLayers];
		uint64 vAddress = 0;
		uint8 vLevel = 0;
	};

	/** Returns false if we don't yet have everything we need to render a VT page. */
	bool IsReady();

	/** Add a tile to the finalize queue. */
	void AddTile(FTileEntry& Tile);

	//~ Begin IVirtualTextureFinalizer Interface.
	virtual void RenderFinalize(FRDGBuilder& GraphBuilder, ISceneRenderer* SceneRenderer) override;
	virtual void Finalize(FRDGBuilder& GraphBuilder) override;
	//~ End IVirtualTextureFinalizer Interface.

private:
	/** Description of our virtual texture. */
	const FVTProducerDescription Desc;
	/** Object ID of our virtual texture. */
	int32 RuntimeVirtualTextureId;
	/** Contents of virtual texture layer stack. */
	ERuntimeVirtualTextureMaterialType MaterialType;
	/** Clear before render flag. */
	bool bClearTextures;
	/** Scene that the virtual texture is placed within. */
	FSceneInterface* Scene;
	/** Transform from UV space to world space. */
	FTransform UVToWorld;
	/** Bounds of runtime virtual texture volume in world space. */
	FBox WorldBounds;
	/** Custom material data for the runtime virtual texture. */
	FVector4f CustomMaterialData;
	/** Array of tiles in the queue to finalize. */
	TArray<FTileEntry> Tiles;
	/** Array of batch render contexts used during finalize. */
	TArray<RuntimeVirtualTexture::FBatchRenderContext const*> Batches;
};

/** IVirtualTexture implementation that is handling runtime rendered page data requests. */
class FRuntimeVirtualTextureProducer : public IVirtualTexture
{
public:
	UE_DEPRECATED(5.6, "Use constructor that takes InCustomMaterialData.")
	RENDERER_API FRuntimeVirtualTextureProducer(
		FVTProducerDescription const& InDesc, 
		int32 InRuntimeVirtualTextureId,
		ERuntimeVirtualTextureMaterialType InMaterialType, 
		bool InClearTextures, 
		FSceneInterface* InScene, 
		FTransform const& InUVToWorld,
		FBox const& InWorldBounds);

	RENDERER_API FRuntimeVirtualTextureProducer(
		FVTProducerDescription const& InDesc,
		int32 InRuntimeVirtualTextureId,
		ERuntimeVirtualTextureMaterialType InMaterialType,
		bool InClearTextures,
		FSceneInterface* InScene,
		FTransform const& InUVToWorld,
		FBox const& InWorldBounds,
		FVector4f const& InCustomMaterialData);

	virtual ~FRuntimeVirtualTextureProducer() = default;

	//~ Begin IVirtualTexture Interface.
	virtual bool IsPageStreamed(uint8 vLevel, uint32 vAddress) const override
	{
		return false;
	}

	virtual FVTRequestPageResult RequestPageData(
		FRHICommandListBase& RHICmdList,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		EVTRequestPagePriority Priority
	) override;

	virtual IVirtualTextureFinalizer* ProducePageData(
		FRHICommandListBase& RHICmdList,
		ERHIFeatureLevel::Type FeatureLevel,
		EVTProducePageFlags Flags,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		uint64 RequestHandle,
		const FVTProduceTargetLayer* TargetLayers
	) override;
	//~ End IVirtualTexture Interface.

private:
	FRuntimeVirtualTextureFinalizer Finalizer;
};
