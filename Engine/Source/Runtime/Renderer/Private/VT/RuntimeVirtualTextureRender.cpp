// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/RuntimeVirtualTextureRender.h"

#include "ComponentRecreateRenderStateContext.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "EngineModule.h"
#include "GlobalShader.h"
#include "GPUScene.h"
#include "MaterialShader.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "PostProcess/SceneRenderTargets.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RenderCaptureInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderUtils.h"
#include "RHIResourceUtils.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "ShaderPlatformCachedIniValue.h"
#include "ShaderBaseClasses.h"
#include "SimpleMeshDrawCommandPass.h"
#include "StaticMeshBatch.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/RuntimeVirtualTextureSceneExtension.h"
#include "VT/RuntimeVirtualTextureSceneProxy.h"

CSV_DECLARE_CATEGORY_EXTERN(VirtualTexturing);

DECLARE_DWORD_COUNTER_STAT(TEXT("Num pages rendered"), STAT_RenderedPages, STATGROUP_VirtualTexturing);

namespace RuntimeVirtualTexture
{
	static FAutoConsoleVariableDeprecated CVarVTMipColors_Deprecated(TEXT("r.VT.RVT.MipColors"), TEXT("r.VT.Borders"), TEXT("5.7"));

	static TAutoConsoleVariable<int32> CVarVTHighQualityPerPixelHeight(
		TEXT("r.VT.RVT.HighQualityPerPixelHeight"),
		1,
		TEXT("Use higher quality sampling of per pixel heightmaps when rendering to Runtime Virtual Texture.\n"),
		ECVF_ReadOnly);

	static TAutoConsoleVariable<int32> CVarVTDirectCompress(
		TEXT("r.VT.RVT.DirectCompress"),
		1,
		TEXT("Compress texture data direct to the physical texture on platforms that support it."),
		ECVF_RenderThreadSafe);

	static TAutoConsoleVariable<bool> CVarVTApplyPageCorruptionFix(
		TEXT("r.VT.RVT.PageCorruptionFix"),
		true,
		TEXT("Apply change that has been found to fix some rare page corruption on PC.\n"),
		ECVF_RenderThreadSafe);

    int32 RenderCaptureNextRVTPagesDraws = 0;
    static FAutoConsoleVariableRef CVarRenderCaptureNextRVTPagesDraws(
	    TEXT("r.VT.RenderCaptureNextPagesDraws"),
	    RenderCaptureNextRVTPagesDraws,
	    TEXT("Trigger a render capture during the next RVT RenderPages draw calls."));

	static TAutoConsoleVariable<bool> CVarRVTAstc(
		TEXT("r.VT.RVT.ASTC"),
		0,
		TEXT("Use ASTC compression instead of ETC2 when the hardware supports it."),
		ECVF_ReadOnly);

	static TAutoConsoleVariable<bool> CVarRVTAstcHigh(
		TEXT("r.VT.RVT.ASTC.High"),
		0,
		TEXT("When using ASTC compression, produce higher quality output at roughly 2x the time spent encoding."),
		ECVF_Default);

	BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FEtcParameters, )
		SHADER_PARAMETER_ARRAY(FVector4f, ALPHA_DISTANCE_TABLES, [16])
		SHADER_PARAMETER_ARRAY(FVector4f, RGB_DISTANCE_TABLES, [8])
	END_GLOBAL_SHADER_PARAMETER_STRUCT()

	IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FEtcParameters, "EtcParameters");
	
	static TGlobalResource<FGlobalDynamicReadBuffer> RuntimeVirtualReadBuffer;

	class FEtcParametersUniformBuffer : public TUniformBuffer<FEtcParameters>
	{
		typedef TUniformBuffer<FEtcParameters> Super;
	public:
		FEtcParametersUniformBuffer()
		{
			FEtcParameters Parameters;
			Parameters.ALPHA_DISTANCE_TABLES[0] = FVector4f(2, 5, 8, 14);
			Parameters.ALPHA_DISTANCE_TABLES[1] = FVector4f(2, 6, 9, 12);
			Parameters.ALPHA_DISTANCE_TABLES[2] = FVector4f(1, 4, 7, 12);
			Parameters.ALPHA_DISTANCE_TABLES[3] = FVector4f(1, 3, 5, 12);
			Parameters.ALPHA_DISTANCE_TABLES[4] = FVector4f(2, 5, 7, 11);
			Parameters.ALPHA_DISTANCE_TABLES[5] = FVector4f(2, 6, 8, 10);
			Parameters.ALPHA_DISTANCE_TABLES[6] = FVector4f(3, 6, 7, 10);
			Parameters.ALPHA_DISTANCE_TABLES[7] = FVector4f(2, 4, 7, 10);
			Parameters.ALPHA_DISTANCE_TABLES[8] = FVector4f(1, 5, 7, 9);
			Parameters.ALPHA_DISTANCE_TABLES[9] = FVector4f(1, 4, 7, 9);
			Parameters.ALPHA_DISTANCE_TABLES[10] = FVector4f(1, 3, 7, 9);
			Parameters.ALPHA_DISTANCE_TABLES[11] = FVector4f(1, 4, 6, 9);
			Parameters.ALPHA_DISTANCE_TABLES[12] = FVector4f(2, 3, 6, 9);
			Parameters.ALPHA_DISTANCE_TABLES[13] = FVector4f(0, 1, 2, 9);
			Parameters.ALPHA_DISTANCE_TABLES[14] = FVector4f(3, 5, 7, 8);
			Parameters.ALPHA_DISTANCE_TABLES[15] = FVector4f(2, 4, 6, 8);

			Parameters.RGB_DISTANCE_TABLES[0] = FVector4f(-8, -2, 2, 8);
			Parameters.RGB_DISTANCE_TABLES[1] = FVector4f(-17, -5, 5, 17);
			Parameters.RGB_DISTANCE_TABLES[2] = FVector4f(-29, -9, 9, 29);
			Parameters.RGB_DISTANCE_TABLES[3] = FVector4f(-42, -13, 13, 42);
			Parameters.RGB_DISTANCE_TABLES[4] = FVector4f(-60, -18, 18, 60);
			Parameters.RGB_DISTANCE_TABLES[5] = FVector4f(-80, -24, 24, 80);
			Parameters.RGB_DISTANCE_TABLES[6] = FVector4f(-106, -33, 33, 106);
			Parameters.RGB_DISTANCE_TABLES[7] = FVector4f(-183, -47, 47, 183);

			SetContentsNoUpdate(Parameters);
		}
	};

	const TUniformBufferRef<FEtcParameters>& GetEtcParametersUniformBufferRef()
	{
		static TGlobalResource<FEtcParametersUniformBuffer> EtcParametersUniformBuffer;
		return EtcParametersUniformBuffer.GetUniformBufferRef();
	}

	static const uint8 TritsToInteger[243] = 
	{
		0, 1, 2,
		4, 5, 6,
		8, 9, 10,

		16, 17, 18,
		20, 21, 22,
		24, 25, 26,

		3, 7, 15,
		19, 23, 27,
		12, 13, 14,

		32, 33, 34,
		36, 37, 38,
		40, 41, 42,

		48, 49, 50,
		52, 53, 54,
		56, 57, 58,

		35, 39, 47,
		51, 55, 59,
		44, 45, 46,

		64, 65, 66,
		68, 69, 70,
		72, 73, 74,

		80, 81, 82,
		84, 85, 86,
		88, 89, 90,

		67, 71, 79,
		83, 87, 91,
		76, 77, 78,

		128, 129, 130,
		132, 133, 134,
		136, 137, 138,

		144, 145, 146,
		148, 149, 150,
		152, 153, 154,

		131, 135, 143,
		147, 151, 155,
		140, 141, 142,

		160, 161, 162,
		164, 165, 166,
		168, 169, 170,

		176, 177, 178,
		180, 181, 182,
		184, 185, 186,

		163, 167, 175,
		179, 183, 187,
		172, 173, 174,

		192, 193, 194,
		196, 197, 198,
		200, 201, 202,

		208, 209, 210,
		212, 213, 214,
		216, 217, 218,

		195, 199, 207,
		211, 215, 219,
		204, 205, 206,

		96, 97, 98,
		100, 101, 102,
		104, 105, 106,

		112, 113, 114,
		116, 117, 118,
		120, 121, 122,

		99, 103, 111,
		115, 119, 123,
		108, 109, 110,

		224, 225, 226,
		228, 229, 230,
		232, 233, 234,

		240, 241, 242,
		244, 245, 246,
		248, 249, 250,

		227, 231, 239,
		243, 247, 251,
		236, 237, 238,

		28, 29, 30,
		60, 61, 62,
		92, 93, 94,

		156, 157, 158,
		188, 189, 190,
		220, 221, 222,

		31, 63, 127,
		159, 191, 255,
		252, 253, 254
	};

	static const uint8 QuintsToInteger[125] =
	{
		0, 1, 2, 3, 4,
		8, 9, 10, 11, 12,
		16, 17, 18, 19, 20,
		24, 25, 26, 27, 28,
		5, 13, 21, 29, 6,

		32, 33, 34, 35, 36,
		40, 41, 42, 43, 44,
		48, 49, 50, 51, 52,
		56, 57, 58, 59, 60,
		37, 45, 53, 61, 14,

		64, 65, 66, 67, 68,
		72, 73, 74, 75, 76,
		80, 81, 82, 83, 84,
		88, 89, 90, 91, 92,
		69, 77, 85, 93, 22,

		96, 97, 98, 99, 100,
		104, 105, 106, 107, 108,
		112, 113, 114, 115, 116,
		120, 121, 122, 123, 124,
		101, 109, 117, 125, 30,

		102, 103, 70, 71, 38,
		110, 111, 78, 79, 46,
		118, 119, 86, 87, 54,
		126, 127, 94, 95, 62,
		39, 47, 55, 63, 31
	};

	// from [ARM:astc-encoder] quantization_and_transfer_table quant_and_xfer_tables
	#define WEIGHT_QUANTIZE_NUM 32
	static const uint8 ScrambleTable[12 * WEIGHT_QUANTIZE_NUM] = {
		// quantization method 0, range 0..1
		//{
			0, 1,
			0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 1, range 0..2
		//{
			0, 1, 2,
			0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 2, range 0..3
		//{
			0, 1, 2, 3,
			0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 3, range 0..4
		//{
			0, 1, 2, 3, 4,
			0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 4, range 0..5
		//{
			0, 2, 4, 5, 3, 1,
			0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 5, range 0..7
		//{
			0, 1, 2, 3, 4, 5, 6, 7,
			0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 6, range 0..9
		//{
			0, 2, 4, 6, 8, 9, 7, 5, 3, 1,
			0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 7, range 0..11
		//{
			0, 4, 8, 2, 6, 10, 11, 7, 3, 9, 5, 1,
			0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 8, range 0..15
		//{
			0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15,
			0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 9, range 0..19
		//{
			0, 4, 8, 12, 16, 2, 6, 10, 14, 18, 19, 15, 11, 7, 3, 17, 13, 9, 5, 1,
			0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 10, range 0..23
		//{
			0, 8, 16, 2, 10, 18, 4, 12, 20, 6, 14, 22, 23, 15, 7, 21, 13, 5, 19,
			11, 3, 17, 9, 1, 0, 0, 0, 0, 0, 0, 0, 0,
		//},
		// quantization method 11, range 0..31
		//{
			0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19,
			20, 21, 22, 23, 24, 25, 26, 27, 28, 29, 30, 31,
		//}
	};

	static const uint8 ColorScrambleTable48[256] = {
		  0,   0,   0,  16,  16,  16,  16,  16,  32,  32,  32,  32,  32,  32,   2,   2,
		  2,   2,   2,  18,  18,  18,  18,  18,  34,  34,  34,  34,  34,  34,   4,   4,
		  4,   4,   4,  20,  20,  20,  20,  20,  20,  36,  36,  36,  36,  36,   6,   6,
		  6,   6,   6,  22,  22,  22,  22,  22,  22,  38,  38,  38,  38,  38,  38,   8,
		  8,   8,   8,   8,  24,  24,  24,  24,  24,  40,  40,  40,  40,  40,  40,  10,
		 10,  10,  10,  10,  26,  26,  26,  26,  26,  42,  42,  42,  42,  42,  42,  12,
		 12,  12,  12,  12,  28,  28,  28,  28,  28,  28,  44,  44,  44,  44,  44,  14,
		 14,  14,  14,  14,  30,  30,  30,  30,  30,  30,  46,  46,  46,  46,  46,  46,
		 47,  47,  47,  47,  47,  47,  31,  31,  31,  31,  31,  31,  15,  15,  15,  15,
		 15,  45,  45,  45,  45,  45,  29,  29,  29,  29,  29,  29,  13,  13,  13,  13,
		 13,  43,  43,  43,  43,  43,  43,  27,  27,  27,  27,  27,  11,  11,  11,  11,
		 11,  41,  41,  41,  41,  41,  41,  25,  25,  25,  25,  25,   9,   9,   9,   9,
		  9,  39,  39,  39,  39,  39,  39,  23,  23,  23,  23,  23,  23,   7,   7,   7,
		  7,   7,  37,  37,  37,  37,  37,  21,  21,  21,  21,  21,  21,   5,   5,   5,
		  5,   5,  35,  35,  35,  35,  35,  35,  19,  19,  19,  19,  19,   3,   3,   3,
		  3,   3,  33,  33,  33,  33,  33,  33,  17,  17,  17,  17,  17,   1,   1,   1 
	};

	static const uint8 ColorScrambleTable80[256] = {
		0,   0,  16,  16,  16,  32,  32,  32,  48,  48,  48,  64,  64,  64,  64,   2,
		2,   2,  18,  18,  18,  34,  34,  34,  50,  50,  50,  66,  66,  66,  66,   4,
		4,   4,  20,  20,  20,  36,  36,  36,  52,  52,  52,  52,  68,  68,  68,   6,
		6,   6,  22,  22,  22,  38,  38,  38,  54,  54,  54,  54,  70,  70,  70,   8,
		8,   8,  24,  24,  24,  40,  40,  40,  40,  56,  56,  56,  72,  72,  72,  10,
		10,  10,  26,  26,  26,  42,  42,  42,  42,  58,  58,  58,  74,  74,  74,  12,
		12,  12,  28,  28,  28,  28,  44,  44,  44,  60,  60,  60,  76,  76,  76,  14,
		14,  14,  30,  30,  30,  30,  46,  46,  46,  62,  62,  62,  78,  78,  78,  78,
		79,  79,  79,  79,  63,  63,  63,  47,  47,  47,  31,  31,  31,  31,  15,  15,
		15,  77,  77,  77,  61,  61,  61,  45,  45,  45,  29,  29,  29,  29,  13,  13,
		13,  75,  75,  75,  59,  59,  59,  43,  43,  43,  43,  27,  27,  27,  11,  11,
		11,  73,  73,  73,  57,  57,  57,  41,  41,  41,  41,  25,  25,  25,   9,   9,
		9,  71,  71,  71,  55,  55,  55,  55,  39,  39,  39,  23,  23,  23,   7,   7,
		7,  69,  69,  69,  53,  53,  53,  53,  37,  37,  37,  21,  21,  21,   5,   5,
		5,  67,  67,  67,  67,  51,  51,  51,  35,  35,  35,  19,  19,  19,   3,   3,
		3,  65,  65,  65,  65,  49,  49,  49,  33,  33,  33,  17,  17,  17,   1,   1
	};

	static const uint8 ColorScrambleTable192[256] = {
		  0,  64, 128, 128,   2,  66, 130, 130,   4,  68, 132, 132,   6,  70, 134, 134,
		  8,  72, 136, 136,  10,  74, 138, 138,  12,  76, 140, 140,  14,  78, 142, 142,
		 16,  80, 144, 144,  18,  82, 146, 146,  20,  84, 148, 148,  22,  86, 150, 150,
		 24,  88, 152, 152,  26,  90, 154, 154,  28,  92, 156, 156,  30,  94, 158, 158,
		 32,  96, 160, 160,  34,  98, 162, 162,  36, 100, 164, 164,  38, 102, 166, 166,
		 40, 104, 168, 168,  42, 106, 170, 170,  44, 108, 172, 172,  46, 110, 174, 174,
		 48, 112, 176, 176,  50, 114, 178, 178,  52, 116, 180, 180,  54, 118, 182, 182,
		 56, 120, 184, 184,  58, 122, 186, 186,  60, 124, 188, 188,  62, 126, 190, 190,
		191, 191, 127,  63, 189, 189, 125,  61, 187, 187, 123,  59, 185, 185, 121,  57,
		183, 183, 119,  55, 181, 181, 117,  53, 179, 179, 115,  51, 177, 177, 113,  49,
		175, 175, 111,  47, 173, 173, 109,  45, 171, 171, 107,  43, 169, 169, 105,  41,
		167, 167, 103,  39, 165, 165, 101,  37, 163, 163,  99,  35, 161, 161,  97,  33,
		159, 159,  95,  31, 157, 157,  93,  29, 155, 155,  91,  27, 153, 153,  89,  25,
		151, 151,  87,  23, 149, 149,  85,  21, 147, 147,  83,  19, 145, 145,  81,  17,
		143, 143,  79,  15, 141, 141,  77,  13, 139, 139,  75,  11, 137, 137,  73,   9,
		135, 135,  71,   7, 133, 133,  69,   5, 131, 131,  67,   3, 129, 129,  65,   1
	};

	BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FAstcParameters, )
		SHADER_PARAMETER_SRV(Buffer<uint>, TritsToInteger)
		SHADER_PARAMETER_SRV(Buffer<uint>, QuintsToInteger)
		SHADER_PARAMETER_SRV(Buffer<uint>, ScrambleTable)
		SHADER_PARAMETER_SRV(Buffer<uint>, ColorScrambleTable48)
		SHADER_PARAMETER_SRV(Buffer<uint>, ColorScrambleTable80)
		SHADER_PARAMETER_SRV(Buffer<uint>, ColorScrambleTable192)
	END_GLOBAL_SHADER_PARAMETER_STRUCT()

	IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FAstcParameters, "AstcParameters");

	class FAstcParametersUniformBuffer : public TUniformBuffer<FAstcParameters>
	{
		typedef TUniformBuffer<FAstcParameters> Super;
	public:
		virtual void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			FAstcParameters Parameters;

			{
				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex<uint8>(TEXT("TritsToInteger"), 243)
					.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
					.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
				Parameters.TritsToInteger = RHICmdList.CreateShaderResourceView(
					UE::RHIResourceUtils::CreateBufferWithArray(RHICmdList, CreateDesc, TritsToInteger),
					FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8_UINT));
			}
			{
				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex<uint8>(TEXT("QuintsToInteger"), 125)
					.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
					.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
				Parameters.QuintsToInteger = RHICmdList.CreateShaderResourceView(
					UE::RHIResourceUtils::CreateBufferWithArray(RHICmdList, CreateDesc, QuintsToInteger),
					FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8_UINT));
			}
			{
				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex<uint8>(TEXT("ScrambleTable"), 12 * WEIGHT_QUANTIZE_NUM)
					.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
					.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
				Parameters.ScrambleTable = RHICmdList.CreateShaderResourceView(
					UE::RHIResourceUtils::CreateBufferWithArray(RHICmdList, CreateDesc, ScrambleTable),
					FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8_UINT));
			}
			{
				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex<uint8>(TEXT("ColorScrambleTable48"), 256)
					.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
					.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
				Parameters.ColorScrambleTable48 = RHICmdList.CreateShaderResourceView(
					UE::RHIResourceUtils::CreateBufferWithArray(RHICmdList, CreateDesc, ColorScrambleTable48),
					FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8_UINT));
			}
			{
				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex<uint8>(TEXT("ColorScrambleTable80"), 256)
					.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
					.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
				Parameters.ColorScrambleTable80 = RHICmdList.CreateShaderResourceView(
					UE::RHIResourceUtils::CreateBufferWithArray(RHICmdList, CreateDesc, ColorScrambleTable80),
					FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8_UINT));
			}
			{
				const FRHIBufferCreateDesc CreateDesc =
					FRHIBufferCreateDesc::CreateVertex<uint8>(TEXT("ColorScrambleTable192"), 256)
					.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
					.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);
				Parameters.ColorScrambleTable192 = RHICmdList.CreateShaderResourceView(
					UE::RHIResourceUtils::CreateBufferWithArray(RHICmdList, CreateDesc, ColorScrambleTable192),
					FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R8_UINT));
			}

			SetContentsNoUpdate(Parameters);
			Super::InitRHI(RHICmdList);
		}
	};

	const TUniformBufferRef<FAstcParameters>& GetAstcParametersUniformBufferRef()
	{
		static TGlobalResource<FAstcParametersUniformBuffer> AstcParametersUniformBuffer;
		return AstcParametersUniformBuffer.GetUniformBufferRef();
	}

	bool UseEtcProfile(EShaderPlatform ShaderPlatform)
	{
		switch (ShaderPlatform)
		{
		case SP_METAL_ES3_1_IOS:
		case SP_METAL_SM5_IOS:
		case SP_METAL_SM6_IOS:
		case SP_METAL_SIM:
		case SP_METAL_ES3_1_TVOS:
		case SP_METAL_SM5_TVOS:
		case SP_VULKAN_ES3_1_ANDROID:
		case SP_OPENGL_ES3_1_ANDROID:
		case SP_VULKAN_SM5_ANDROID:
			return true;
		default:
			break;
		}
		return false;
	}

	bool UseAstcProfile(EShaderPlatform ShaderPlatform)
	{
		if (!CVarRVTAstc.GetValueOnAnyThread()) {
			return false;
		}
		switch (ShaderPlatform)
		{
		case SP_METAL_ES3_1_IOS:
		case SP_METAL_SM5_IOS:
		case SP_METAL_SM6_IOS:
		case SP_METAL_SIM:
		case SP_METAL_ES3_1_TVOS:
		case SP_METAL_SM5_TVOS:
		case SP_VULKAN_ES3_1_ANDROID:
		case SP_OPENGL_ES3_1_ANDROID:
		case SP_VULKAN_SM5_ANDROID:
			return true;
		default:
			break;
		}
		return false;
	}

	bool UseAstcHighProfile(EShaderPlatform ShaderPlatform)
	{
		return UseAstcProfile(ShaderPlatform) && CVarRVTAstcHigh.GetValueOnAnyThread();

	}

	/** For platforms that do not support 2-channel images, write 64bit compressed texture outputs into RGBA16 instead of RG32. */
	bool UseRGBA16(EShaderPlatform ShaderPlatform)
	{
		return IsOpenGLPlatform(ShaderPlatform);
	}

	/** Parameters used when writing to the virtual texture. */
	BEGIN_UNIFORM_BUFFER_STRUCT(FRuntimeVirtualTexturePassParameters, )
		SHADER_PARAMETER(FVector4f, MipLevel)
		SHADER_PARAMETER(FVector4f, CustomMaterialData)
		SHADER_PARAMETER(FVector4f, FixedColor)
		SHADER_PARAMETER(FVector4f, MipColorAndSize)
		SHADER_PARAMETER(FVector2f, PackHeight)
	END_UNIFORM_BUFFER_STRUCT()

	/** Uniform buffer for writing to the virtual texture. We reuse the DeferredDecals UB slot, which can't be used at the same time. This avoids the overhead of a new slot. */
	IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FRuntimeVirtualTexturePassParameters, "RuntimeVirtualTexturePassParameters", DeferredDecals);

	/** Mesh material shader for writing to the virtual texture. */
	class FShader_VirtualTextureMaterialDraw : public FMeshMaterialShader
	{
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
			SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
			SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRuntimeVirtualTexturePassParameters, RuntimeVirtualTexturePassParameters)
			SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
		{
			return UseVirtualTexturing(Parameters.Platform) &&
				(Parameters.MaterialParameters.bHasRuntimeVirtualTextureOutput || Parameters.MaterialParameters.bIsDefaultMaterial);
		}

		static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("IS_VIRTUAL_TEXTURE_MATERIAL"), 1);

			static FShaderPlatformCachedIniValue<bool> HighQualityPerPixelHeightValue(TEXT("r.VT.RVT.HighQualityPerPixelHeight"));
			const bool bHighQualityPerPixelHeight = (HighQualityPerPixelHeightValue.Get((EShaderPlatform)Parameters.Platform) != 0);
			OutEnvironment.SetDefine(TEXT("PER_PIXEL_HEIGHTMAP_HQ"), bHighQualityPerPixelHeight ? 1 : 0);
		}

		FShader_VirtualTextureMaterialDraw()
		{}

		FShader_VirtualTextureMaterialDraw(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
			: FMeshMaterialShader(Initializer)
		{
		}
	};


	/** Specialization for ERuntimeVirtualTextureMaterialType::BaseColor */
	class FMaterialPolicy_BaseColor
	{
	public:
		static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
		{
			return RuntimeVirtualTexture::IsMaterialTypeSupported(ERuntimeVirtualTextureMaterialType::BaseColor, Parameters.Platform);
		}

		static void ModifyCompilationEnvironment(FShaderCompilerEnvironment& OutEnvironment)
		{
			OutEnvironment.SetDefine(TEXT("OUT_BASECOLOR"), 1);
		}

		static FRHIBlendState* GetBlendState(uint8 OutputAttributeMask)
		{
			return TStaticBlendState< CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One >::GetRHI();
		}
	};

	/** Specialization for ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular */
	class FMaterialPolicy_BaseColorNormalSpecular
	{
	private:
		/** Compile time helper to build blend state from the connected output attribute mask. */
		static constexpr EColorWriteMask GetColorMaskFromAttributeMask(uint8 AttributeMask, uint8 RenderTargetIndex)
		{
			// Color mask in the output render targets for each of the relevant attributes in ERuntimeVirtualTextureAttributeType
			const EColorWriteMask AttributeMasks[][3] = {
				{ CW_RGBA, CW_NONE, CW_NONE }, // BaseColor
				{ CW_NONE, EColorWriteMask(CW_RED | CW_GREEN | CW_ALPHA), EColorWriteMask(CW_BLUE | CW_ALPHA) }, // Normal
				{ CW_NONE, CW_NONE, EColorWriteMask(CW_GREEN | CW_ALPHA) }, // Roughness
				{ CW_NONE, CW_NONE, EColorWriteMask(CW_RED | CW_ALPHA) }, // Specular
				{ CW_NONE, EColorWriteMask(CW_BLUE | CW_ALPHA), CW_NONE }, // Mask
			};

			// Combine the color masks for this AttributeMask
			EColorWriteMask ColorWriteMask = CW_NONE;
			for (int32 i = 0; i < 5; ++i)
			{
				if (AttributeMask & (1 << i))
				{
					ColorWriteMask = EColorWriteMask(ColorWriteMask | AttributeMasks[i][RenderTargetIndex]);
				}
			}
			return ColorWriteMask;
		}

		/** Helper to convert the connected output attribute mask to a blend state with a color mask for these attributes. */
		template< uint32 AttributeMask >
		static FRHIBlendState* TGetBlendStateFromAttributeMask()
		{
			return TStaticBlendState<
				GetColorMaskFromAttributeMask(AttributeMask, 0), BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,
				GetColorMaskFromAttributeMask(AttributeMask, 1), BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,
				GetColorMaskFromAttributeMask(AttributeMask, 2), BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	>::GetRHI();
		}

		/** Runtime conversion of attribute mask to static blend state. */
		static FRHIBlendState* GetBlendStateImpl(uint8 AttributeMask)
		{
			// We have 5 relevant bits in the attribute mask. Any more and this would get painful...
			switch (AttributeMask & 0x1f)
			{
			case 1: return TGetBlendStateFromAttributeMask<1>();
			case 2: return TGetBlendStateFromAttributeMask<2>();
			case 3: return TGetBlendStateFromAttributeMask<3>();
			case 4: return TGetBlendStateFromAttributeMask<4>();
			case 5: return TGetBlendStateFromAttributeMask<5>();
			case 6: return TGetBlendStateFromAttributeMask<6>();
			case 7: return TGetBlendStateFromAttributeMask<7>();
			case 8: return TGetBlendStateFromAttributeMask<8>();
			case 9: return TGetBlendStateFromAttributeMask<9>();
			case 10: return TGetBlendStateFromAttributeMask<10>();
			case 11: return TGetBlendStateFromAttributeMask<11>();
			case 12: return TGetBlendStateFromAttributeMask<12>();
			case 13: return TGetBlendStateFromAttributeMask<13>();
			case 14: return TGetBlendStateFromAttributeMask<14>();
			case 15: return TGetBlendStateFromAttributeMask<15>();
			case 16: return TGetBlendStateFromAttributeMask<16>();
			case 17: return TGetBlendStateFromAttributeMask<17>();
			case 18: return TGetBlendStateFromAttributeMask<18>();
			case 19: return TGetBlendStateFromAttributeMask<19>();
			case 20: return TGetBlendStateFromAttributeMask<20>();
			case 21: return TGetBlendStateFromAttributeMask<21>();
			case 22: return TGetBlendStateFromAttributeMask<22>();
			case 23: return TGetBlendStateFromAttributeMask<23>();
			case 24: return TGetBlendStateFromAttributeMask<24>();
			case 25: return TGetBlendStateFromAttributeMask<25>();
			case 26: return TGetBlendStateFromAttributeMask<26>();
			case 27: return TGetBlendStateFromAttributeMask<27>();
			case 28: return TGetBlendStateFromAttributeMask<28>();
			case 29: return TGetBlendStateFromAttributeMask<29>();
			case 30: return TGetBlendStateFromAttributeMask<30>();
			default: return TGetBlendStateFromAttributeMask<31>();
			}
		}

	public:
		static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
		{
			return RuntimeVirtualTexture::IsMaterialTypeSupported(ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular, Parameters.Platform);
		}

		static void ModifyCompilationEnvironment(FShaderCompilerEnvironment& OutEnvironment)
		{
			OutEnvironment.SetDefine(TEXT("OUT_BASECOLOR_NORMAL_SPECULAR"), 1);
		}

		static FRHIBlendState* GetBlendState(uint8 OutputAttributeMask)
		{
			return GetBlendStateImpl(OutputAttributeMask);
		}
	};

	/** Specialization for ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness */
	class FMaterialPolicy_BaseColorNormalRoughness
	{
	private:
		/** Compile time helper to build blend state from the connected output attribute mask. */
		static constexpr EColorWriteMask GetColorMaskFromAttributeMask(uint8 AttributeMask, uint8 RenderTargetIndex)
		{
			// Color mask in the output render targets for each of the relevant attributes in ERuntimeVirtualTextureAttributeType
			const EColorWriteMask AttributeMasks[][2] = {
				{ CW_RGBA, CW_NONE}, // BaseColor
				{ CW_NONE, EColorWriteMask(CW_RED| CW_BLUE | CW_ALPHA)}, // Normal
				{ CW_NONE, EColorWriteMask(CW_GREEN | CW_ALPHA)}, // Roughness
				{ CW_NONE, CW_NONE}, // Specular
				{ CW_NONE, CW_NONE}, // Mask
			};

			// Combine the color masks for this AttributeMask
			EColorWriteMask ColorWriteMask = CW_NONE;
			for (int32 i = 0; i < 5; ++i)
			{
				if (AttributeMask & (1 << i))
				{
					ColorWriteMask = EColorWriteMask(ColorWriteMask | AttributeMasks[i][RenderTargetIndex]);
				}
			}
			return ColorWriteMask;
		}
		/** Helper to convert the connected output attribute mask to a blend state with a color mask for these attributes. */
		template< uint32 AttributeMask >
		static FRHIBlendState* TGetBlendStateFromAttributeMask()
		{
			return TStaticBlendState<
				GetColorMaskFromAttributeMask(AttributeMask, 0), BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,
				// normal XY is stored in R and B channels, and the Sign of Z is considered always positive
				GetColorMaskFromAttributeMask(AttributeMask, 1), BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
		}
		/** Runtime conversion of attribute mask to static blend state. */
		static FRHIBlendState* GetBlendStateImpl(uint8 AttributeMask)
		{
			// We have 5 relevant bits in the attribute mask. Any more and this would get painful...
			switch (AttributeMask & 0x1f)
			{
			case 1: return TGetBlendStateFromAttributeMask<1>();
			case 2: return TGetBlendStateFromAttributeMask<2>();
			case 3: return TGetBlendStateFromAttributeMask<3>();
			case 4: return TGetBlendStateFromAttributeMask<4>();
			case 5: return TGetBlendStateFromAttributeMask<5>();
			case 6: return TGetBlendStateFromAttributeMask<6>();
			case 7: return TGetBlendStateFromAttributeMask<7>();
			case 8: return TGetBlendStateFromAttributeMask<8>();
			case 9: return TGetBlendStateFromAttributeMask<9>();
			case 10: return TGetBlendStateFromAttributeMask<10>();
			case 11: return TGetBlendStateFromAttributeMask<11>();
			case 12: return TGetBlendStateFromAttributeMask<12>();
			case 13: return TGetBlendStateFromAttributeMask<13>();
			case 14: return TGetBlendStateFromAttributeMask<14>();
			case 15: return TGetBlendStateFromAttributeMask<15>();
			case 16: return TGetBlendStateFromAttributeMask<16>();
			case 17: return TGetBlendStateFromAttributeMask<17>();
			case 18: return TGetBlendStateFromAttributeMask<18>();
			case 19: return TGetBlendStateFromAttributeMask<19>();
			case 20: return TGetBlendStateFromAttributeMask<20>();
			case 21: return TGetBlendStateFromAttributeMask<21>();
			case 22: return TGetBlendStateFromAttributeMask<22>();
			case 23: return TGetBlendStateFromAttributeMask<23>();
			case 24: return TGetBlendStateFromAttributeMask<24>();
			case 25: return TGetBlendStateFromAttributeMask<25>();
			case 26: return TGetBlendStateFromAttributeMask<26>();
			case 27: return TGetBlendStateFromAttributeMask<27>();
			case 28: return TGetBlendStateFromAttributeMask<28>();
			case 29: return TGetBlendStateFromAttributeMask<29>();
			case 30: return TGetBlendStateFromAttributeMask<30>();
			default: return TGetBlendStateFromAttributeMask<31>();
			}
		}

	public:
		static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
		{
			return RuntimeVirtualTexture::IsMaterialTypeSupported(ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness, Parameters.Platform);
		}
		
		static void ModifyCompilationEnvironment(FShaderCompilerEnvironment& OutEnvironment)
		{
			OutEnvironment.SetDefine(TEXT("OUT_BASECOLOR_NORMAL_ROUGHNESS"), 1);
		}
		static FRHIBlendState* GetBlendState(uint8 OutputAttributeMask)
		{
			return GetBlendStateImpl(OutputAttributeMask);
		}
	};

	/** Specialization for ERuntimeVirtualTextureMaterialType::Mask4 */
	class FMaterialPolicy_Mask4
	{
	public:
		static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
		{
			return RuntimeVirtualTexture::IsMaterialTypeSupported(ERuntimeVirtualTextureMaterialType::Mask4, Parameters.Platform);
		}

		static void ModifyCompilationEnvironment(FShaderCompilerEnvironment& OutEnvironment)
		{
			OutEnvironment.SetDefine(TEXT("OUT_MASK4"), 1);
		}

		static FRHIBlendState* GetBlendState(uint8 OutputAttributeMask)
		{
			return TStaticBlendState<
				CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,
				CW_RED, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One >::GetRHI();
		}
	};

	/** Specialization for ERuntimeVirtualTextureMaterialType::WorldHeight */
	class FMaterialPolicy_WorldHeight
	{
	public:
		static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
		{
			return RuntimeVirtualTexture::IsMaterialTypeSupported(ERuntimeVirtualTextureMaterialType::WorldHeight, Parameters.Platform);
		}

		static void ModifyCompilationEnvironment(FShaderCompilerEnvironment& OutEnvironment)
		{
			OutEnvironment.SetDefine(TEXT("OUT_WORLDHEIGHT"), 1);
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_R32_FLOAT);
		}

		static FRHIBlendState* GetBlendState(uint8 OutputAttributeMask)
		{
			return TStaticBlendState< CW_RED, BO_Max, BF_One, BF_One, BO_Add, BF_One, BF_One >::GetRHI();
		}
	};

	/** Specialization for ERuntimeVirtualTextureMaterialType::Displacement */
	class FMaterialPolicy_Displacement
	{
	public:
		static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
		{
			return RuntimeVirtualTexture::IsMaterialTypeSupported(ERuntimeVirtualTextureMaterialType::Displacement, Parameters.Platform);
		}

		static void ModifyCompilationEnvironment(FShaderCompilerEnvironment& OutEnvironment)
		{
			OutEnvironment.SetDefine(TEXT("OUT_DISPLACEMENT"), 1);
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_A32B32G32R32F);
		}

		static FRHIBlendState* GetBlendState(uint8 OutputAttributeMask)
		{
			return TStaticBlendState< CW_RED, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One >::GetRHI();
		}
	};


	/** Vertex shader derivation of material shader. Templated on policy for virtual texture layout. */
	template< class MaterialPolicy >
	class FShader_VirtualTextureMaterialDraw_VS : public FShader_VirtualTextureMaterialDraw
	{
	public:
		DECLARE_SHADER_TYPE(FShader_VirtualTextureMaterialDraw_VS, MeshMaterial);

		FShader_VirtualTextureMaterialDraw_VS()
		{}

		FShader_VirtualTextureMaterialDraw_VS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
			: FShader_VirtualTextureMaterialDraw(Initializer)
		{}

		static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
		{
			return FShader_VirtualTextureMaterialDraw::ShouldCompilePermutation(Parameters) && MaterialPolicy::ShouldCompilePermutation(Parameters);
		}

		static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FShader_VirtualTextureMaterialDraw::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			MaterialPolicy::ModifyCompilationEnvironment(OutEnvironment);
		}
	};

	/** Pixel shader derivation of material shader. Templated on policy for virtual texture layout. */
	template< class MaterialPolicy >
	class FShader_VirtualTextureMaterialDraw_PS : public FShader_VirtualTextureMaterialDraw
	{
	public:
		DECLARE_SHADER_TYPE(FShader_VirtualTextureMaterialDraw_PS< MaterialPolicy >, MeshMaterial);

		FShader_VirtualTextureMaterialDraw_PS()
		{}

		FShader_VirtualTextureMaterialDraw_PS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
			: FShader_VirtualTextureMaterialDraw(Initializer)
		{}

		static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FShader_VirtualTextureMaterialDraw::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			MaterialPolicy::ModifyCompilationEnvironment(OutEnvironment);
		}
	};

	// If we change this macro or add additional policy types then we need to update GetRuntimeVirtualTextureShaderTypes() in LandscapeRender.cpp
	// That code is used to filter out unnecessary shader variations
#define IMPLEMENT_VIRTUALTEXTURE_SHADER_TYPE(PolicyType, PolicyName) \
	typedef FShader_VirtualTextureMaterialDraw_VS<PolicyType> TVirtualTextureVS##PolicyName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TVirtualTextureVS##PolicyName, TEXT("/Engine/Private/VirtualTextureMaterial.usf"), TEXT("MainVS"), SF_Vertex); \
	typedef FShader_VirtualTextureMaterialDraw_PS<PolicyType> TVirtualTexturePS##PolicyName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TVirtualTexturePS##PolicyName, TEXT("/Engine/Private/VirtualTextureMaterial.usf"), TEXT("MainPS"), SF_Pixel);

	IMPLEMENT_VIRTUALTEXTURE_SHADER_TYPE(FMaterialPolicy_BaseColor, BaseColor);
	IMPLEMENT_VIRTUALTEXTURE_SHADER_TYPE(FMaterialPolicy_BaseColorNormalRoughness, BaseColorNormalRoughness);
	IMPLEMENT_VIRTUALTEXTURE_SHADER_TYPE(FMaterialPolicy_BaseColorNormalSpecular, BaseColorNormalSpecular);
	IMPLEMENT_VIRTUALTEXTURE_SHADER_TYPE(FMaterialPolicy_Mask4, Mask4);
	IMPLEMENT_VIRTUALTEXTURE_SHADER_TYPE(FMaterialPolicy_WorldHeight, WorldHeight);
	IMPLEMENT_VIRTUALTEXTURE_SHADER_TYPE(FMaterialPolicy_Displacement, Displacement);

	/** Structure to localize the setup of our render graph based on the virtual texture setup. */
	struct FRenderGraphSetup
	{
		static void SetupRenderTargetsInfo(ERuntimeVirtualTextureMaterialType MaterialType, ERHIFeatureLevel::Type FeatureLevel, bool bLQFormat, FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo)
		{
			const ETextureCreateFlags RTCreateFlags = TexCreate_RenderTargetable | TexCreate_ShaderResource;
			const ETextureCreateFlags RTSrgbFlags = TexCreate_SRGB;

			switch (MaterialType)
			{
			case ERuntimeVirtualTextureMaterialType::BaseColor:
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags | RTSrgbFlags, RenderTargetsInfo);
				break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
				AddRenderTargetInfo(bLQFormat ? PF_R5G6B5_UNORM : PF_B8G8R8A8, RTCreateFlags, RenderTargetsInfo);
				AddRenderTargetInfo(bLQFormat ? PF_R5G6B5_UNORM : PF_B8G8R8A8, RTCreateFlags, RenderTargetsInfo);
				break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags | RTSrgbFlags, RenderTargetsInfo);
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags, RenderTargetsInfo);
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags, RenderTargetsInfo);
				break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags | RTSrgbFlags, RenderTargetsInfo);
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags, RenderTargetsInfo);
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags, RenderTargetsInfo);
				break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags | RTSrgbFlags, RenderTargetsInfo);
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags, RenderTargetsInfo);
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags, RenderTargetsInfo);
				break;
			case ERuntimeVirtualTextureMaterialType::Mask4:
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags, RenderTargetsInfo);
				AddRenderTargetInfo(PF_B8G8R8A8, RTCreateFlags, RenderTargetsInfo);
				break;
			case ERuntimeVirtualTextureMaterialType::WorldHeight:
			case ERuntimeVirtualTextureMaterialType::Displacement:
				AddRenderTargetInfo(PF_G16, RTCreateFlags, RenderTargetsInfo);
				break;
			}
		}

		/** Initializer description for the graph setup. */
		struct FInitDesc
		{
			ERHIFeatureLevel::Type FeatureLevel;
			ERuntimeVirtualTextureMaterialType MaterialType;
			FIntPoint TextureSize;
			int32 PageCount = 1;
			TArray<TRefCountPtr<IPooledRenderTarget>> OutputTargets;
			bool bClearTextures = false;
			bool bIsThumbnails = false;

			/** Initialize from a page batch description. */
			FInitDesc(FRenderPageBatchDesc const& InDesc)
			{
				check(InDesc.SceneRenderer != nullptr && InDesc.SceneRenderer->GetScene() != nullptr);
				FeatureLevel = InDesc.SceneRenderer->GetScene()->GetFeatureLevel();
				MaterialType = InDesc.MaterialType;
				PageCount = InDesc.NumPageDescs;
				TextureSize = InDesc.PageDescs[0].DestRect[0].Size();
				OutputTargets.Add(InDesc.Targets[0].PooledRenderTarget);
				OutputTargets.Add(InDesc.Targets[1].PooledRenderTarget);
				OutputTargets.Add(InDesc.Targets[2].PooledRenderTarget);
				bClearTextures = InDesc.bClearTextures;
				bIsThumbnails = InDesc.bIsThumbnails;
			}
		};

		/** CreateTextureDesc() creates a texture2Darray if we have page batch size > 1 or a simple texture2D otherwise. */
		static FRDGTextureDesc CreateTextureDesc(FIntPoint Size, EPixelFormat Format, FClearValueBinding ClearValue, ETextureCreateFlags Flags, uint16 ArraySize)
		{
			if (ArraySize > 1)
			{
				return FRDGTextureDesc::Create2DArray(Size, Format, ClearValue, Flags | TexCreate_TargetArraySlicesIndependently, ArraySize);
			}
			else
			{
				return FRDGTextureDesc::Create2D(Size, Format, ClearValue, Flags);
			}
		}

		/** CreateTextureSRV() createss an SRV for a single slice if Texture is a texture array. */
		static FRDGTextureSRVRef CreateTextureSRV(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture, int32 ArraySlice)
		{
			if (Texture == nullptr)
			{
				return nullptr;
			}
			if (ArraySlice >= 0)
			{
				return GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForSlice(Texture, ArraySlice));
			}
			return GraphBuilder.CreateSRV(Texture);
		}

		/** Initialize the graph setup. */
		void Init(FRDGBuilder& GraphBuilder, FInitDesc const& Desc)
		{
			const EPixelFormat OutputFormat0 = Desc.OutputTargets[0].IsValid() ? Desc.OutputTargets[0]->GetRHI()->GetFormat() : PF_Unknown;

			bRenderPass = OutputFormat0 != PF_Unknown;
			bCopyThumbnailPass = bRenderPass && Desc.bIsThumbnails;
			const bool bCompressedFormat = GPixelFormats[OutputFormat0].BlockSizeX == 4 && GPixelFormats[OutputFormat0].BlockSizeY == 4;
			const bool bLQFormat = OutputFormat0 == PF_R5G6B5_UNORM;
			bCompressPass = bRenderPass && !bCopyThumbnailPass && bCompressedFormat;
			bCopyPass = bRenderPass && !bCopyThumbnailPass && !bCompressPass && 
				(Desc.MaterialType == ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular || Desc.MaterialType == ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg || Desc.MaterialType == ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg || Desc.MaterialType == ERuntimeVirtualTextureMaterialType::Mask4);
			
			// Use direct aliasing for compression pass on platforms that support it.
			bDirectAliasing = bCompressedFormat && GRHISupportsUAVFormatAliasing && CVarVTDirectCompress.GetValueOnRenderThread() != 0;

			// ForceImmediateFirstBarrier so that UAV transitions for the output targets aren't hoisted above
			// Finalize() and into RenderFinalize() or earlier where they will be incorrect for virtual texture sampling.
			const ERDGTextureFlags ExternalTextureFlags = ERDGTextureFlags::ForceImmediateFirstBarrier;

			// Some problems happen when we don't use ERenderTargetLoadAction::EClear:
			// * Some RHI need explicit flag to avoid a fast clear (TexCreate_NoFastClear).
			// * DX12 RHI has a bug with RDG transient allocator (UE-173023) so we use TexCreate_Shared to avoid that.
			const ETextureCreateFlags RTNoClearHackFlags = TexCreate_NoFastClear | TexCreate_Shared;

			const ETextureCreateFlags RTClearFlags = Desc.bClearTextures ? TexCreate_None : RTNoClearHackFlags;
			const ETextureCreateFlags RTCreateFlags = TexCreate_RenderTargetable | TexCreate_ShaderResource | RTClearFlags;
			const ETextureCreateFlags RTSrgbFlags = TexCreate_SRGB;

			const EPixelFormat Compressed64BitFormat = UseRGBA16(GMaxRHIShaderPlatform) ? PF_R16G16B16A16_UINT : PF_R32G32_UINT;
			const EPixelFormat Compressed128BitFormat = PF_R32G32B32A32_UINT;

			const int32 PageCount = Desc.PageCount;

			switch (Desc.MaterialType)
			{
			case ERuntimeVirtualTextureMaterialType::BaseColor:
				if (bRenderPass)
				{
					OutputAlias0 = RenderTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags | RTSrgbFlags, PageCount), TEXT("RenderTexture0"));
				}
				if (bCompressPass)
				{
					if (bDirectAliasing)
					{
						CompressTexture0 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[0], ExternalTextureFlags);
						CompressTextureUAV0_64bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0, 0, Compressed64BitFormat));
						OutputAlias0 = nullptr;
					}
					else
					{
						OutputAlias0 = CompressTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed64BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture0"));
						CompressTextureUAV0_64bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0));
					}
				}
				if (bCopyThumbnailPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags | RTSrgbFlags, PageCount), TEXT("CopyTexture0"));
				}
				break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
				if (bRenderPass)
				{
					OutputAlias0 = RenderTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, bLQFormat ? PF_R5G6B5_UNORM : PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("RenderTexture0"));
					OutputAlias1 = RenderTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, bLQFormat ? PF_R5G6B5_UNORM : PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("RenderTexture1"));
				}
				if (bCompressPass)
				{
					if (bDirectAliasing)
					{
						CompressTexture0 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[0], ExternalTextureFlags);
						CompressTextureUAV0_64bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0, 0, Compressed64BitFormat));
						CompressTexture1 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[1], ExternalTextureFlags);
						CompressTextureUAV1_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture1, 0, Compressed128BitFormat));
						OutputAlias0 = OutputAlias1 = nullptr;
					}
					else
					{
						OutputAlias0 = CompressTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed64BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture0"));
						CompressTextureUAV0_64bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0));
						OutputAlias1 = CompressTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed128BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture1"));
						CompressTextureUAV1_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture1));
					}
				}
				if (bCopyThumbnailPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, bLQFormat ? PF_R5G6B5_UNORM : PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture0"));
					OutputAlias1 = nullptr;
				}
				break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
				if (bRenderPass)
				{
					OutputAlias0 = RenderTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags | RTSrgbFlags, PageCount), TEXT("RenderTexture0"));
					RenderTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::DefaultNormal8Bit, RTCreateFlags, PageCount), TEXT("RenderTexture1"));
					RenderTexture2 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::DefaultNormal8Bit, RTCreateFlags, PageCount), TEXT("RenderTexture2"));
				}
				if (bCompressPass)
				{
					if (bDirectAliasing)
					{
						CompressTexture0 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[0], ExternalTextureFlags);
						CompressTextureUAV0_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0, 0, Compressed128BitFormat));
						CompressTexture1 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[1], ExternalTextureFlags);
						CompressTextureUAV1_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture1, 0, Compressed128BitFormat));
						OutputAlias0 = OutputAlias1 = nullptr;
					}
					else
					{
						OutputAlias0 = CompressTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed128BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture0"));
						CompressTextureUAV0_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0));
						OutputAlias1 = CompressTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed128BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture1"));
						CompressTextureUAV1_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture1));
					}
				}
				if (bCopyPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags | RTSrgbFlags, PageCount), TEXT("CopyTexture0"));
					OutputAlias1 = CopyTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture1"));
				}
				if (bCopyThumbnailPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags | RTSrgbFlags, PageCount), TEXT("CopyTexture0"));
				}
				break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
				if (bRenderPass)
				{
					OutputAlias0 = RenderTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags | RTSrgbFlags, PageCount), TEXT("RenderTexture0"));
					RenderTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::DefaultNormal8Bit, RTCreateFlags, PageCount), TEXT("RenderTexture1"));
					RenderTexture2 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::DefaultNormal8Bit, RTCreateFlags, PageCount), TEXT("RenderTexture2"));
				}
				if (bCompressPass)
				{
					if (bDirectAliasing)
					{
						CompressTexture0 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[0], ExternalTextureFlags);
						CompressTextureUAV0_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0, 0, Compressed128BitFormat));
						CompressTexture1 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[1], ExternalTextureFlags);
						CompressTextureUAV1_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture1, 0, Compressed128BitFormat));
						CompressTexture2 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[2], ExternalTextureFlags);
						CompressTextureUAV2_64bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture2, 0, Compressed64BitFormat));

						OutputAlias0 = OutputAlias1 = OutputAlias2 = nullptr;
					}
					else
					{
						OutputAlias0 = CompressTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed128BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture0"));
						CompressTextureUAV0_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0));
						OutputAlias1 = CompressTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed128BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture1"));
						CompressTextureUAV1_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture1));
						OutputAlias2 = CompressTexture2 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed64BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture2"));
						CompressTextureUAV2_64bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture2));
					}
				}
				if (bCopyPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture0"));
					OutputAlias1 = CopyTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture1"));
					OutputAlias2 = CopyTexture2 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture2"));
				}
				if (bCopyThumbnailPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags | RTSrgbFlags, PageCount), TEXT("CopyTexture0"));
				}
				break;
			case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
				if (bRenderPass)
				{
					OutputAlias0 = RenderTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags | RTSrgbFlags, PageCount), TEXT("RenderTexture0"));
					RenderTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::DefaultNormal8Bit, RTCreateFlags, PageCount), TEXT("RenderTexture1"));
					RenderTexture2 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::DefaultNormal8Bit, RTCreateFlags, PageCount), TEXT("RenderTexture2"));
				}
				if (bCompressPass)
				{
					if (bDirectAliasing)
					{
						CompressTexture0 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[0], ExternalTextureFlags);
						CompressTextureUAV0_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0, 0, Compressed128BitFormat));
						CompressTexture1 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[1], ExternalTextureFlags);
						CompressTextureUAV1_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture1, 0, Compressed128BitFormat));
						CompressTexture2 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[2], ExternalTextureFlags);
						CompressTextureUAV2_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture2, 0, Compressed128BitFormat));

						OutputAlias0 = OutputAlias1 = OutputAlias2 = nullptr;
					}
					else
					{
						OutputAlias0 = CompressTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed128BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture0"));
						CompressTextureUAV0_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0));
						OutputAlias1 = CompressTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed128BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture1"));
						CompressTextureUAV1_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture1));
						OutputAlias2 = CompressTexture2 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed128BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture2"));
						CompressTextureUAV2_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture2));
					}
				}
				if (bCopyPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture0"));
					OutputAlias1 = CopyTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture1"));
					OutputAlias2 = CopyTexture2 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture2"));
				}
				if (bCopyThumbnailPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags | RTSrgbFlags, PageCount), TEXT("CopyTexture0"));
				}
				break;
			case ERuntimeVirtualTextureMaterialType::Mask4:
				if (bRenderPass)
				{
					OutputAlias0 = RenderTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("RenderTexture0"));
					OutputAlias1 = RenderTexture1 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("RenderTexture1"));
				}
				if (bCompressPass)
				{
					if (bDirectAliasing)
					{
						CompressTexture0 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[0], ExternalTextureFlags);
						CompressTextureUAV0_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0, 0, Compressed128BitFormat));
						OutputAlias0 = nullptr;
					}
					else
					{
						OutputAlias0 = CompressTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed128BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture0"));
						CompressTextureUAV0_128bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0));
					}
				}
				if (bCopyPass || bCopyThumbnailPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture0"));
				}
				break;
			case ERuntimeVirtualTextureMaterialType::WorldHeight:
				if (bRenderPass)
				{
					OutputAlias0 = RenderTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_G16, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("RenderTexture0"));
				}
				if (bCopyThumbnailPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture0"));
				}
				break;
			case ERuntimeVirtualTextureMaterialType::Displacement:
				if (bRenderPass)
				{
					OutputAlias0 = RenderTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_G16, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("RenderTexture0"));
				}
				if (bCompressPass)
				{
					if (bDirectAliasing)
					{
						CompressTexture0 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[0], ExternalTextureFlags);
						CompressTextureUAV0_64bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0, 0, Compressed64BitFormat));
						OutputAlias0 = nullptr;
					}
					else
					{
						OutputAlias0 = CompressTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize / 4, Compressed64BitFormat, FClearValueBinding::None, TexCreate_UAV, PageCount), TEXT("CompressTexture0"));
						CompressTextureUAV0_64bit = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CompressTexture0));
					}
				}
				if (bCopyThumbnailPass)
				{
					OutputAlias0 = CopyTexture0 = GraphBuilder.CreateTexture(CreateTextureDesc(Desc.TextureSize, PF_B8G8R8A8, FClearValueBinding::Black, RTCreateFlags, PageCount), TEXT("CopyTexture0"));
				}
				break;
			}

			if (OutputAlias0 && Desc.OutputTargets[0])
			{
				TargetTexture0 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[0], ExternalTextureFlags);
			}
			if (OutputAlias1 && Desc.OutputTargets[1])
			{
				TargetTexture1 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[1], ExternalTextureFlags);
			}
			if (OutputAlias2 && Desc.OutputTargets[2])
			{
				TargetTexture2 = GraphBuilder.RegisterExternalTexture(Desc.OutputTargets[2], ExternalTextureFlags);
			}
		}

		/** Flags to express what passes we need for this virtual texture layout. */
		bool bRenderPass = false;
		bool bCompressPass = false;
		bool bCopyPass = false;
		bool bCopyThumbnailPass = false;
		bool bDirectAliasing = false;

		/** Render graph textures needed for this virtual texture layout. */
		FRDGTextureRef RenderTexture0 = nullptr;
		FRDGTextureRef RenderTexture1 = nullptr;
		FRDGTextureRef RenderTexture2 = nullptr;
		FRDGTextureRef CompressTexture0 = nullptr;
		FRDGTextureRef CompressTexture1 = nullptr;
		FRDGTextureRef CompressTexture2 = nullptr;
		FRDGTextureUAVRef CompressTextureUAV0_64bit = nullptr;
		FRDGTextureUAVRef CompressTextureUAV1_64bit = nullptr;
		FRDGTextureUAVRef CompressTextureUAV2_64bit = nullptr;
		FRDGTextureUAVRef CompressTextureUAV0_128bit = nullptr;
		FRDGTextureUAVRef CompressTextureUAV1_128bit = nullptr;
		FRDGTextureUAVRef CompressTextureUAV2_128bit = nullptr;
		FRDGTextureRef CopyTexture0 = nullptr;
		FRDGTextureRef CopyTexture1 = nullptr;
		FRDGTextureRef CopyTexture2 = nullptr;

		/** Aliases to one of the render/compress/copy textures. This is what we will Copy into the final physical texture. */
		FRDGTextureRef OutputAlias0 = nullptr;
		FRDGTextureRef OutputAlias1 = nullptr;
		FRDGTextureRef OutputAlias2 = nullptr;
		/** If we have output aliases, then these will containg the final physical texture targets. */
		FRDGTextureRef TargetTexture0 = nullptr;
		FRDGTextureRef TargetTexture1 = nullptr;
		FRDGTextureRef TargetTexture2 = nullptr;
	};

	/** 
	 * Context for rendering a batch of pages. 
	 * Holds the batch description and the render graph allocations.
	 * Allows us to maintain state across RenderFinalize() and Finalize() calls.
	 */
	class FBatchRenderContext
	{
	public:
		FRenderGraphSetup GraphSetup;
		FRenderPageBatchDesc BatchDesc;
		bool bAllowCachedMeshDrawCommands = true;
		bool bAllowMipDebug = true;
	};

	/** Mesh processor for rendering static meshes to the virtual texture */
	class FRuntimeVirtualTextureMeshProcessor : public FSceneRenderingAllocatorObject<FRuntimeVirtualTextureMeshProcessor>, public FMeshPassProcessor
	{
	public:
		FRuntimeVirtualTextureMeshProcessor(const FScene* InScene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InView, FMeshPassDrawListContext* InDrawListContext)
			: FMeshPassProcessor(EMeshPass::VirtualTexture, InScene, InFeatureLevel, InView, InDrawListContext)
		{
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
		}

	private:
		bool TryAddMeshBatch(
			const FMeshBatch& RESTRICT MeshBatch,
			uint64 BatchElementMask,
			const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
			int32 StaticMeshId,
			const FMaterialRenderProxy* MaterialRenderProxy,
			const FMaterial* Material)
		{
			const uint8 OutputAttributeMask = Material->IsDefaultMaterial() ? 0xff : Material->GetRuntimeVirtualTextureOutputAttibuteMask_RenderThread();

			if (OutputAttributeMask != 0)
			{
				switch ((ERuntimeVirtualTextureMaterialType)MeshBatch.RuntimeVirtualTextureMaterialType)
				{
				case ERuntimeVirtualTextureMaterialType::BaseColor:
					return Process<FMaterialPolicy_BaseColor>(MeshBatch, BatchElementMask, StaticMeshId, OutputAttributeMask, PrimitiveSceneProxy, *MaterialRenderProxy, *Material);
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
					return Process<FMaterialPolicy_BaseColorNormalRoughness>(MeshBatch, BatchElementMask, StaticMeshId, OutputAttributeMask, PrimitiveSceneProxy, *MaterialRenderProxy, *Material);
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
				case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
					return Process<FMaterialPolicy_BaseColorNormalSpecular>(MeshBatch, BatchElementMask, StaticMeshId, OutputAttributeMask, PrimitiveSceneProxy, *MaterialRenderProxy, *Material);
				case ERuntimeVirtualTextureMaterialType::Mask4:
					return Process<FMaterialPolicy_Mask4>(MeshBatch, BatchElementMask, StaticMeshId, OutputAttributeMask, PrimitiveSceneProxy, *MaterialRenderProxy, *Material);
				case ERuntimeVirtualTextureMaterialType::WorldHeight:
					return Process<FMaterialPolicy_WorldHeight>(MeshBatch, BatchElementMask, StaticMeshId, OutputAttributeMask, PrimitiveSceneProxy, *MaterialRenderProxy, *Material);
				case ERuntimeVirtualTextureMaterialType::Displacement:
					return Process<FMaterialPolicy_Displacement>(MeshBatch, BatchElementMask, StaticMeshId, OutputAttributeMask, PrimitiveSceneProxy, *MaterialRenderProxy, *Material);
				default:
					break;
				}
			}

			return true;
		}

		template<class MaterialPolicy>
		bool Process(
			const FMeshBatch& MeshBatch,
			uint64 BatchElementMask,
			int32 StaticMeshId,
			uint8 OutputAttributeMask,
			const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
			const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
			const FMaterial& RESTRICT MaterialResource)
		{
			const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

			TMeshProcessorShaders<
				FShader_VirtualTextureMaterialDraw_VS< MaterialPolicy >,
				FShader_VirtualTextureMaterialDraw_PS< MaterialPolicy > > VirtualTexturePassShaders;

			FMaterialShaderTypes ShaderTypes;
			ShaderTypes.AddShaderType<FShader_VirtualTextureMaterialDraw_VS< MaterialPolicy>>();
			ShaderTypes.AddShaderType<FShader_VirtualTextureMaterialDraw_PS< MaterialPolicy>>();

			FMaterialShaders Shaders;
			if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
			{
				return false;
			}

			Shaders.TryGetVertexShader(VirtualTexturePassShaders.VertexShader);
			Shaders.TryGetPixelShader(VirtualTexturePassShaders.PixelShader);

			DrawRenderState.SetBlendState(MaterialPolicy::GetBlendState(OutputAttributeMask));

			const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
			ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(MaterialResource, OverrideSettings);
			ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(MaterialResource, OverrideSettings);

			FMeshMaterialShaderElementData ShaderElementData;
			ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

			FMeshDrawCommandSortKey SortKey;
			SortKey.Translucent.MeshIdInPrimitive = MeshBatch.MeshIdInPrimitive;
			SortKey.Translucent.Distance = 0;
			SortKey.Translucent.Priority = (uint16)((int32)PrimitiveSceneProxy->GetTranslucencySortPriority() - (int32)SHRT_MIN);

			BuildMeshDrawCommands(
				MeshBatch,
				BatchElementMask,
				PrimitiveSceneProxy,
				MaterialRenderProxy,
				MaterialResource,
				DrawRenderState,
				VirtualTexturePassShaders,
				MeshFillMode,
				MeshCullMode,
				SortKey,
				EMeshPassFeatures::Default,
				ShaderElementData);

			return true;
		}

		template<class MaterialPolicy>
		void CollectPSOInitializersInternal(
			const FPSOPrecacheVertexFactoryData& VertexFactoryData,
			const FMaterial& RESTRICT MaterialResource,
			const ERasterizerFillMode& MeshFillMode,
			const ERasterizerCullMode& MeshCullMode,
			uint8 OutputAttributeMask,
			ERuntimeVirtualTextureMaterialType MaterialType,
			TArray<FPSOPrecacheData>& PSOInitializers)
		{			
			FMaterialShaderTypes ShaderTypes;
			ShaderTypes.AddShaderType<FShader_VirtualTextureMaterialDraw_VS< MaterialPolicy>>();
			ShaderTypes.AddShaderType<FShader_VirtualTextureMaterialDraw_PS< MaterialPolicy>>();
			FMaterialShaders Shaders;
			if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactoryData.VertexFactoryType, Shaders))
			{
				return;
			}

			TMeshProcessorShaders<
				FShader_VirtualTextureMaterialDraw_VS< MaterialPolicy >,
				FShader_VirtualTextureMaterialDraw_PS< MaterialPolicy > > VirtualTexturePassShaders;
			Shaders.TryGetVertexShader(VirtualTexturePassShaders.VertexShader);
			Shaders.TryGetPixelShader(VirtualTexturePassShaders.PixelShader);

			FMeshPassProcessorRenderState PSODrawRenderState(DrawRenderState);
			PSODrawRenderState.SetBlendState(MaterialPolicy::GetBlendState(OutputAttributeMask));

			const bool bLQQuality = false;
			FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
			RenderTargetsInfo.NumSamples = 1;
			FRenderGraphSetup::SetupRenderTargetsInfo(MaterialType, FeatureLevel, bLQQuality, RenderTargetsInfo);
			AddGraphicsPipelineStateInitializer(
				VertexFactoryData,
				MaterialResource,
				PSODrawRenderState,
				RenderTargetsInfo,
				VirtualTexturePassShaders,
				MeshFillMode,
				MeshCullMode,
				PT_TriangleList,
				EMeshPassFeatures::Default,
				true /*bRequired*/,
				PSOInitializers);
		}

	public:
		virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final
		{
			if (MeshBatch.bRenderToVirtualTexture)
			{
				const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
				while (MaterialRenderProxy)
				{
					const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
					if (Material && Material->GetRenderingThreadShaderMap())
					{
						if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material))
						{
							break;
						}
					}

					MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
				}
			}
		}

		virtual void CollectPSOInitializers(
			const FSceneTexturesConfig& SceneTexturesConfig, 
			const FMaterial& Material, 
			const FPSOPrecacheVertexFactoryData& VertexFactoryData,
			const FPSOPrecacheParams& PreCacheParams, 
			TArray<FPSOPrecacheData>& PSOInitializers) override final
		{
			EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
			if (!UseVirtualTexturing(ShaderPlatform))
			{
				return;
			}

			const uint8 OutputAttributeMask = Material.IsDefaultMaterial() ? 0xff : Material.GetRuntimeVirtualTextureOutputAttibuteMask_GameThread();

			if (OutputAttributeMask != 0)
			{
				const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
				const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
				const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

				// Tried checking which virtual textures are used on primitive component at PSO level, but if only those types are precached
				// then quite a few hitches can be seen - if we want to reduce the amount of PSOs to precache here then better investigation
				// is needed what types should be compiled (currently there are around 300+ PSOs coming from virtual textures after level loading)
				CollectPSOInitializersInternal<FMaterialPolicy_BaseColor>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, OutputAttributeMask, ERuntimeVirtualTextureMaterialType::BaseColor, PSOInitializers);
				CollectPSOInitializersInternal<FMaterialPolicy_BaseColorNormalRoughness>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, OutputAttributeMask, ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness, PSOInitializers);
				CollectPSOInitializersInternal<FMaterialPolicy_BaseColorNormalSpecular>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, OutputAttributeMask, ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular, PSOInitializers);
				CollectPSOInitializersInternal<FMaterialPolicy_Mask4>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, OutputAttributeMask, ERuntimeVirtualTextureMaterialType::BaseColor, PSOInitializers);
				CollectPSOInitializersInternal<FMaterialPolicy_WorldHeight>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, OutputAttributeMask, ERuntimeVirtualTextureMaterialType::WorldHeight, PSOInitializers);
				CollectPSOInitializersInternal<FMaterialPolicy_Displacement>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, OutputAttributeMask, ERuntimeVirtualTextureMaterialType::Displacement, PSOInitializers);
			}
		}

	private:
		FMeshPassProcessorRenderState DrawRenderState;
	};

	/** Mesh collector for all dynamic mesh batches */
	class FDynamicMeshCollector
	{
	public:
		FDynamicMeshCollector(FRHICommandList& RHICmdList, FSceneRenderingBulkObjectAllocator& Allocator, FScene* Scene, FViewInfo* View)
			: View(View)
			, Scene(Scene)
			, Collector(GMaxRHIFeatureLevel, Allocator)
		{
			DynamicVertexBuffer.Init(RHICmdList);
			DynamicIndexBuffer.Init(RHICmdList);
			Collector.Start(RHICmdList, DynamicVertexBuffer, DynamicIndexBuffer, RuntimeVirtualReadBuffer);

			// Create a new primitive collector for this page set only
			if (Scene->GPUScene.IsEnabled())
			{
				View->DynamicPrimitiveCollector = FGPUScenePrimitiveCollector(Scene->GPUScene.GetCurrentDynamicContext());
			}

			Collector.AddViewMeshArrays(
				View,
				&MeshBatchAndRelevances, 
				&SimpleElements,
				&View->DynamicPrimitiveCollector
			);
		}

		/** Commit and submit batches */
		void Submit(FRDGBuilder& GraphBuilder, FRuntimeVirtualTextureMeshProcessor& Processor, ERuntimeVirtualTextureMaterialType MaterialType)
		{
			Collector.Finish();
			DynamicVertexBuffer.Commit();
			DynamicIndexBuffer.Commit();

			if (Scene->GPUScene.IsEnabled())
			{
				View->DynamicPrimitiveCollector.Commit();
			}

			// Avoid uploads on empty submissions
			if (MeshBatchAndRelevances.IsEmpty())
			{
				return;
			}

			Scene->GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, *View);

			// Process all batches marked for virtual texturing
			for (FMeshBatchAndRelevance& MeshBatch : MeshBatchAndRelevances)
			{
				if (MeshBatch.Mesh->bRenderToVirtualTexture && MeshBatch.Mesh->RuntimeVirtualTextureMaterialType == static_cast<uint32>(MaterialType))
				{
					Processor.AddMeshBatch(*MeshBatch.Mesh, ~0ull, MeshBatch.PrimitiveSceneProxy);
				}
			}
		}

		void SetPrimitive(FPrimitiveSceneProxy* Proxy, FHitProxyId HitProxyId)
		{
			Collector.SetPrimitive(Proxy, HitProxyId);
		}

		FMeshElementCollector& GetCollector()
		{
			return Collector;
		}

	private:
		FViewInfo* View;
		FScene*    Scene;

		/** Collection states, simple elements ignored */
		FMeshElementCollector      Collector;
		FGlobalDynamicVertexBuffer DynamicVertexBuffer;
		FGlobalDynamicIndexBuffer  DynamicIndexBuffer;
		FSimpleElementCollector    SimpleElements;

		/** All collected batches */
		TArray<FMeshBatchAndRelevance, SceneRenderingAllocator> MeshBatchAndRelevances;
	};

	/** Registration for virtual texture command caching pass */
	FMeshPassProcessor* CreateRuntimeVirtualTexturePassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
	{
		return new FRuntimeVirtualTextureMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext);
	}

	REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(VirtualTexturePass, CreateRuntimeVirtualTexturePassProcessor, EShadingPath::Deferred, EMeshPass::VirtualTexture, EMeshPassFlags::CachedMeshCommands);
	REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(VirtualTexturePassMobile, CreateRuntimeVirtualTexturePassProcessor, EShadingPath::Mobile, EMeshPass::VirtualTexture, EMeshPassFlags::CachedMeshCommands);


	/** Collect meshes to draw. */
	void GatherMeshesToDraw(FDynamicPassMeshDrawListContext* DynamicMeshPassContext, FRDGBuilder& GraphBuilder, FScene* Scene, FViewInfo* View, ERuntimeVirtualTextureMaterialType MaterialType, int32 RuntimeVirtualTextureId, uint8 vLevel, uint8 MaxLevel, bool bAllowCachedMeshDrawCommands)
	{
		// Cached draw command collectors
		const FCachedPassMeshDrawList& SceneDrawList = Scene->CachedDrawLists[EMeshPass::VirtualTexture];

		// Uncached mesh processor
		FRuntimeVirtualTextureMeshProcessor MeshProcessor(Scene, Scene->GetFeatureLevel(), View, DynamicMeshPassContext);

		// Pre-calculate view factors used for culling
		const float RcpWorldSize = 1.f / (View->ViewMatrices.GetClipToView().M[0][0]);
		const float WorldToPixel = View->ViewRect.Width() * RcpWorldSize;

		TArray<int32> PrimitiveIndices;
		if (FRuntimeVirtualTextureSceneExtension const* SceneExtension = Scene->GetExtensionPtr<FRuntimeVirtualTextureSceneExtension>())
		{
			SceneExtension->GetPrimitivesForRuntimeVirtualTexture(Scene, RuntimeVirtualTextureId, PrimitiveIndices);
		}

		// Lazily created on first usage
		FDynamicMeshCollector* DynamicCollector = nullptr;

		// Set of views for dynamic collection
		TArray<const FSceneView*> Views{View};

		for (const int32 PrimitiveIndex : PrimitiveIndices)
 		{
			//todo[vt]: In our case we know that frustum is an oriented box so investigate cheaper test for intersecting that
			const FSphere SphereBounds = Scene->PrimitiveBounds[PrimitiveIndex].BoxSphereBounds.GetSphere();
			if (!View->ViewFrustum.IntersectSphere(SphereBounds.Center, SphereBounds.W))
			{
				continue;
			}

			FPrimitiveSceneInfo* RESTRICT PrimitiveSceneInfo = Scene->Primitives[PrimitiveIndex];

			// Cull primitives according to mip level or pixel coverage
			const FPrimitiveRuntimeVirtualTextureLodInfo LodInfo = PrimitiveSceneInfo->GetRuntimeVirtualTextureLodInfo();
			if (LodInfo.CullMethod == 0)
			{
				if (MaxLevel - vLevel < LodInfo.CullValue)
				{
					continue;
				}
			}
			else
			{
				// Note that we use 2^MinPixelCoverage as that scales linearly with mip extents
				int32 PixelCoverage = FMath::FloorToInt(2.f * SphereBounds.W * WorldToPixel);
				if (PixelCoverage < (1 << LodInfo.CullValue))
				{
					continue;
				}
			}

			FMeshDrawCommandPrimitiveIdInfo IdInfo = PrimitiveSceneInfo->GetMDCIdInfo();

			// Calculate Lod for current mip
			const float AreaRatio = 2.f * SphereBounds.W * RcpWorldSize;
			const int32 CurFirstLODIdx = PrimitiveSceneInfo->Proxy->GetCurrentFirstLODIdx_RenderThread();
			const int32 MinLODIdx = FMath::Max((int32)LodInfo.MinLod, CurFirstLODIdx);
			const int32 MaxLODIdx = FMath::Max((int32)LodInfo.MaxLod, CurFirstLODIdx);
			const int32 LodBias = (int32)LodInfo.LodBias - FPrimitiveRuntimeVirtualTextureLodInfo::LodBiasOffset;
			const int32 LodIndex = FMath::Clamp<int32>(LodBias - FMath::FloorToInt(FMath::Log2(AreaRatio)), MinLODIdx, MaxLODIdx);

			// Process meshes
			for (int32 MeshIndex = 0; MeshIndex < PrimitiveSceneInfo->StaticMeshes.Num(); ++MeshIndex)
			{
				FStaticMeshBatchRelevance const& StaticMeshRelevance = PrimitiveSceneInfo->StaticMeshRelevances[MeshIndex];
				if (StaticMeshRelevance.bRenderToVirtualTexture && StaticMeshRelevance.GetLODIndex() == LodIndex && StaticMeshRelevance.RuntimeVirtualTextureMaterialType == (uint32)MaterialType)
				{
					bool bCachedDraw = false;
					if (bAllowCachedMeshDrawCommands && StaticMeshRelevance.bSupportsCachingMeshDrawCommands)
					{
						// Use cached draw command
						const int32 StaticMeshCommandInfoIndex = StaticMeshRelevance.GetStaticMeshCommandInfoIndex(EMeshPass::VirtualTexture);
						if (StaticMeshCommandInfoIndex >= 0)
						{
							FCachedMeshDrawCommandInfo& CachedMeshDrawCommand = PrimitiveSceneInfo->StaticMeshCommandInfos[StaticMeshCommandInfoIndex];

							const FMeshDrawCommand* MeshDrawCommand = CachedMeshDrawCommand.StateBucketId >= 0
								? &Scene->CachedMeshDrawCommandStateBuckets[EMeshPass::VirtualTexture].GetByElementId(CachedMeshDrawCommand.StateBucketId).Key
								: &SceneDrawList.MeshDrawCommands[CachedMeshDrawCommand.CommandIndex];

							FVisibleMeshDrawCommand NewVisibleMeshDrawCommand;
							NewVisibleMeshDrawCommand.Setup(
								MeshDrawCommand,
								IdInfo,
								CachedMeshDrawCommand.StateBucketId,
								CachedMeshDrawCommand.MeshFillMode,
								CachedMeshDrawCommand.MeshCullMode,
								CachedMeshDrawCommand.Flags,
								CachedMeshDrawCommand.SortKey,
								CachedMeshDrawCommand.CullingPayload,
								EMeshDrawCommandCullingPayloadFlags::NoScreenSizeCull);

							DynamicMeshPassContext->AddVisibleMeshDrawCommand(NewVisibleMeshDrawCommand);
							bCachedDraw = true;
						}
					}

					if (!bCachedDraw)
					{
						// No cached draw command was available. Process the mesh batch.
						uint64 BatchElementMask = ~0ull;
						MeshProcessor.AddMeshBatch(PrimitiveSceneInfo->StaticMeshes[MeshIndex], BatchElementMask, Scene->PrimitiveSceneProxies[PrimitiveIndex]);
					}
				}
			}

			// It's not entirely accurate to equate the dynamic view relevance here, so, assume dynamic
			// mesh batches if no static were supplied. For now at least.
			if (PrimitiveSceneInfo->StaticMeshes.IsEmpty())
			{
				// Lazy create
				if (!DynamicCollector)
				{
					DynamicCollector = View->Allocator.Create<FDynamicMeshCollector>(GraphBuilder.RHICmdList, View->Allocator, Scene, View);
				}
				
				// Collect all dynamic batches
				DynamicCollector->SetPrimitive(PrimitiveSceneInfo->Proxy, PrimitiveSceneInfo->DefaultDynamicHitProxyId);
				PrimitiveSceneInfo->Proxy->GetDynamicMeshElements(Views, *View->Family, 0x1, DynamicCollector->GetCollector());
			}
		}

		// Process collected batches
		if (DynamicCollector)
		{
			DynamicCollector->Submit(GraphBuilder, MeshProcessor, MaterialType);
		}
	}

	/** BC Compression compute shader */
	class FShader_VirtualTextureCompress : public FGlobalShader
	{
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(FUintVector4, SourceRect)
			SHADER_PARAMETER_SCALAR_ARRAY(int32, DestPos, [MaxRenderPageBatch * MaxTextureLayers * 2])
			SHADER_PARAMETER_STRUCT_REF(FEtcParameters, EtcParameters)
			SHADER_PARAMETER_STRUCT_REF(FAstcParameters, AstcParameters)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, RenderTexture0)
			SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler0)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, RenderTexture1)
			SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler1)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, RenderTexture2)
			SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler2)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint2>, OutCompressTexture0_64bit)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint2>, OutCompressTexture1_64bit)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint2>, OutCompressTexture2_64bit)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint4>, OutCompressTexture0_128bit)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint4>, OutCompressTexture1_128bit)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint4>, OutCompressTexture2_128bit)
		END_SHADER_PARAMETER_STRUCT()

		class FUseSrcTextureArray : SHADER_PERMUTATION_BOOL("USE_SRC_TEXTURE_ARRAY");
		class FUseDstTextureArray : SHADER_PERMUTATION_BOOL("USE_DST_TEXTURE_ARRAY");
		class FAstcHighProfile : SHADER_PERMUTATION_BOOL("ASTC_HIGH_PROFILE");
		using FPermutationDomain = TShaderPermutationDomain<FUseSrcTextureArray, FUseDstTextureArray, FAstcHighProfile>;

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			
			OutEnvironment.SetDefine(TEXT("ETC_PROFILE"), UseEtcProfile(Parameters.Platform) ? 1 : 0);
			OutEnvironment.SetDefine(TEXT("ASTC_PROFILE"), UseAstcProfile(Parameters.Platform) ? 1 : 0);
			OutEnvironment.SetDefine(TEXT("PACK_RG32_RGBA16"), UseRGBA16(Parameters.Platform) ? 1 : 0);
			
			OutEnvironment.SetDefine(TEXT("MAX_BATCH_SIZE"), MaxRenderPageBatch);
			OutEnvironment.SetDefine(TEXT("MAX_DST_LAYERS"), MaxTextureLayers);

			const FPermutationDomain PermutationVector(Parameters.PermutationId);
			const bool bUseSrcTextureArray = PermutationVector.Get<FUseSrcTextureArray>();
			OutEnvironment.SetDefine(TEXT("BLOCK_COMPRESS_SRC_TEXTURE_ARRAY"), bUseSrcTextureArray ? 1 : 0);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			FPermutationDomain PermutationVector(Parameters.PermutationId);
			if (!PermutationVector.Get<FUseSrcTextureArray>() && PermutationVector.Get<FUseDstTextureArray>())
			{
				// No compress pass goes from simple source texture to destination array texture.
				return false;
			}
			if (PermutationVector.Get<FAstcHighProfile>())
			{
				return UseAstcHighProfile(Parameters.Platform);
			}
			return true;
		}

		FShader_VirtualTextureCompress()
		{}

		FShader_VirtualTextureCompress(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
			: FGlobalShader(Initializer)
		{
			Bindings.BindForLegacyShaderParameters(this, Initializer.PermutationId, Initializer.ParameterMap, *FParameters::FTypeInfo::GetStructMetadata());
		}
	};

	template< ERuntimeVirtualTextureMaterialType MaterialType >
	class FShader_VirtualTextureCompress_CS : public FShader_VirtualTextureCompress
	{
	public:
		typedef FShader_VirtualTextureCompress_CS< MaterialType > ClassName; // typedef is only so that we can use in DECLARE_SHADER_TYPE macro
		DECLARE_SHADER_TYPE( ClassName, Global );

		FShader_VirtualTextureCompress_CS()
		{}

		FShader_VirtualTextureCompress_CS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
			: FShader_VirtualTextureCompress(Initializer)
		{}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return UseVirtualTexturing(Parameters.Platform) && RuntimeVirtualTexture::IsMaterialTypeSupported(MaterialType, Parameters.Platform);
		}
	};

	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCompress_CS< ERuntimeVirtualTextureMaterialType::BaseColor >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CompressBaseColorCS"), SF_Compute);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCompress_CS< ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CompressBaseColorNormalSpecularCS"), SF_Compute);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCompress_CS< ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CompressBaseColorNormalRoughnessCS"), SF_Compute);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCompress_CS< ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CompressBaseColorNormalSpecularYCoCgCS"), SF_Compute);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCompress_CS< ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CompressBaseColorNormalSpecularMaskYCoCgCS"), SF_Compute);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCompress_CS< ERuntimeVirtualTextureMaterialType::Mask4 >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CompressMask4CS"), SF_Compute);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCompress_CS< ERuntimeVirtualTextureMaterialType::Displacement >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CompressDisplacementCS"), SF_Compute);


	/** Add the BC compression pass to the graph. */
	template< ERuntimeVirtualTextureMaterialType MaterialType >
	void AddCompressPass(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FShader_VirtualTextureCompress::FParameters* Parameters, FIntVector GroupCount, bool bDirectAliasing)
	{
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);

		FShader_VirtualTextureCompress::FPermutationDomain PermutationVector;
		PermutationVector.Set<FShader_VirtualTextureCompress::FUseSrcTextureArray>(GroupCount.Z > 1);
		PermutationVector.Set<FShader_VirtualTextureCompress::FUseDstTextureArray>(GroupCount.Z > 1 && !bDirectAliasing);
		PermutationVector.Set<FShader_VirtualTextureCompress::FAstcHighProfile>(UseAstcHighProfile(GShaderPlatformForFeatureLevel[FeatureLevel]));
		TShaderMapRef< FShader_VirtualTextureCompress_CS< MaterialType > > ComputeShader(GlobalShaderMap, PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("VirtualTextureCompress"),
			ComputeShader, Parameters, GroupCount);
	}

	/** Set up the BC compression pass for the given MaterialType. */
	void AddCompressPass(
		FRDGBuilder& GraphBuilder, 
		ERHIFeatureLevel::Type FeatureLevel, 
		FShader_VirtualTextureCompress::FParameters* Parameters, 
		FIntPoint TextureSize, 
		int32 NumSlices, 
		ERuntimeVirtualTextureMaterialType MaterialType, 
		bool bDirectAliasing)
	{
		const FIntVector GroupCount(((TextureSize.X / 4) + 7) / 8, ((TextureSize.Y / 4) + 7) / 8, NumSlices);

		// Dispatch using the shader variation for our MaterialType
		switch (MaterialType)
		{
		case ERuntimeVirtualTextureMaterialType::BaseColor:
			AddCompressPass<ERuntimeVirtualTextureMaterialType::BaseColor>(GraphBuilder, FeatureLevel, Parameters, GroupCount, bDirectAliasing);
			break;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
			AddCompressPass<ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular>(GraphBuilder, FeatureLevel, Parameters, GroupCount, bDirectAliasing);
			break;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
			AddCompressPass<ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness>(GraphBuilder, FeatureLevel, Parameters, GroupCount, bDirectAliasing);
			break;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
			AddCompressPass<ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg>(GraphBuilder, FeatureLevel, Parameters, GroupCount, bDirectAliasing);
			break;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
			AddCompressPass<ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg>(GraphBuilder, FeatureLevel, Parameters, GroupCount, bDirectAliasing);
			break;
		case ERuntimeVirtualTextureMaterialType::Mask4:
			AddCompressPass<ERuntimeVirtualTextureMaterialType::Mask4>(GraphBuilder, FeatureLevel, Parameters, GroupCount, bDirectAliasing);
			break;
		case ERuntimeVirtualTextureMaterialType::Displacement:
			AddCompressPass<ERuntimeVirtualTextureMaterialType::Displacement>(GraphBuilder, FeatureLevel, Parameters, GroupCount, bDirectAliasing);
			break;
		}
	}


	/** Copy shaders are used when compression is disabled. These are used to ensure that the channel layout is the same as with compression. */
	class FShader_VirtualTextureCopy : public FGlobalShader
	{
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			RENDER_TARGET_BINDING_SLOTS()
			SHADER_PARAMETER(FIntVector4, DestRect)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, RenderTexture0)
			SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler0)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, RenderTexture1)
			SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler1)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, RenderTexture2)
			SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler2)
		END_SHADER_PARAMETER_STRUCT()

		FShader_VirtualTextureCopy()
		{}

		FShader_VirtualTextureCopy(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
			: FGlobalShader(Initializer)
		{
			Bindings.BindForLegacyShaderParameters(this, Initializer.PermutationId, Initializer.ParameterMap, *FParameters::FTypeInfo::GetStructMetadata());
		}

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
			OutEnvironment.SetDefine(TEXT("MAX_BATCH_SIZE"), 1);
			OutEnvironment.SetDefine(TEXT("MAX_DST_LAYERS"), 1);
		}
	};

	class FShader_VirtualTextureCopy_VS : public FShader_VirtualTextureCopy
	{
	public:
		DECLARE_SHADER_TYPE(FShader_VirtualTextureCopy_VS, Global);

		FShader_VirtualTextureCopy_VS()
		{}

		FShader_VirtualTextureCopy_VS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
			: FShader_VirtualTextureCopy(Initializer)
		{}
	};

	IMPLEMENT_SHADER_TYPE(, FShader_VirtualTextureCopy_VS, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CopyVS"), SF_Vertex);

	template< ERuntimeVirtualTextureMaterialType MaterialType >
	class FShader_VirtualTextureCopy_PS : public FShader_VirtualTextureCopy
	{
	public:
		typedef FShader_VirtualTextureCopy_PS< MaterialType > ClassName; // typedef is only so that we can use in DECLARE_SHADER_TYPE macro
		DECLARE_SHADER_TYPE(ClassName, Global);

		FShader_VirtualTextureCopy_PS()
		{}

		FShader_VirtualTextureCopy_PS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
			: FShader_VirtualTextureCopy(Initializer)
		{}
	};

	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCopy_PS< ERuntimeVirtualTextureMaterialType::BaseColor >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CopyBaseColorPS"), SF_Pixel);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCopy_PS< ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CopyBaseColorNormalSpecularPS"), SF_Pixel);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCopy_PS< ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CopyBaseColorNormalSpecularYCoCgPS"), SF_Pixel);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCopy_PS< ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CopyBaseColorNormalSpecularMaskYCoCgPS"), SF_Pixel);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCopy_PS< ERuntimeVirtualTextureMaterialType::Mask4 >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CopyMask4PS"), SF_Pixel);
	IMPLEMENT_SHADER_TYPE(template<>, FShader_VirtualTextureCopy_PS< ERuntimeVirtualTextureMaterialType::WorldHeight >, TEXT("/Engine/Private/VirtualTextureCompress.usf"), TEXT("CopyWorldHeightPS"), SF_Pixel);


	/** Add the copy pass to the graph. */
	template< ERuntimeVirtualTextureMaterialType MaterialType >
	void AddCopyPass(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FShader_VirtualTextureCopy::FParameters* Parameters, FIntPoint TextureSize)
	{
		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);
		TShaderMapRef< FShader_VirtualTextureCopy_VS > VertexShader(GlobalShaderMap);
		TShaderMapRef< FShader_VirtualTextureCopy_PS< MaterialType > > PixelShader(GlobalShaderMap);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("VirtualTextureCopy"),
			Parameters,
			ERDGPassFlags::Raster,
			[VertexShader, PixelShader, Parameters, TextureSize](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *Parameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);

			RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, TextureSize[0], TextureSize[1], 1.0f);
			RHICmdList.DrawIndexedPrimitive(GTwoTrianglesIndexBuffer.IndexBufferRHI, 0, 0, 4, 0, 2, 1);
		});
	}

	/** Set up the copy pass for the given MaterialType. */
	void AddCopyPass(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FShader_VirtualTextureCopy::FParameters* Parameters, FIntPoint TextureSize, ERuntimeVirtualTextureMaterialType MaterialType)
	{
		switch (MaterialType)
		{
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
			AddCopyPass<ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular>(GraphBuilder, FeatureLevel, Parameters, TextureSize);
			break;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
			AddCopyPass<ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg>(GraphBuilder, FeatureLevel, Parameters, TextureSize);
			break;
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
			AddCopyPass<ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg>(GraphBuilder, FeatureLevel, Parameters, TextureSize);
			break;
		case ERuntimeVirtualTextureMaterialType::Mask4:
			AddCopyPass<ERuntimeVirtualTextureMaterialType::Mask4>(GraphBuilder, FeatureLevel, Parameters, TextureSize);
			break;
		}
	}

	/** Set up the copy pass for the given MaterialType. */
	void AddCopyThumbnailPass(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FShader_VirtualTextureCopy::FParameters* Parameters, FIntPoint TextureSize, ERuntimeVirtualTextureMaterialType MaterialType)
	{
		switch (MaterialType)
		{
		case ERuntimeVirtualTextureMaterialType::BaseColor:
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Roughness:
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular:
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_YCoCg:
		case ERuntimeVirtualTextureMaterialType::BaseColor_Normal_Specular_Mask_YCoCg:
			AddCopyPass<ERuntimeVirtualTextureMaterialType::BaseColor>(GraphBuilder, FeatureLevel, Parameters, TextureSize);
			break;
		case ERuntimeVirtualTextureMaterialType::Mask4:
			AddCopyPass<ERuntimeVirtualTextureMaterialType::Mask4>(GraphBuilder, FeatureLevel, Parameters, TextureSize);
			break;
		case ERuntimeVirtualTextureMaterialType::WorldHeight:
		case ERuntimeVirtualTextureMaterialType::Displacement:
			AddCopyPass<ERuntimeVirtualTextureMaterialType::WorldHeight>(GraphBuilder, FeatureLevel, Parameters, TextureSize);
			break;
		}
	}

	/** Mesh render pass prologue to set the viewport. Also applies a page corruption workaround when that is enabled. */
	void MeshPassPrologue(FRHICommandList& RHICmdList, FIntRect const& InViewRect, EShaderPlatform InShaderPlatform)
	{
#if PLATFORM_WINDOWS && !PLATFORM_CPU_ARM_FAMILY
		static bool bIsNVidia = IsRHIDeviceNVIDIA();
		if (bIsNVidia && CVarVTApplyPageCorruptionFix.GetValueOnRenderThread())
		{
			// Workaround fix for an issue where runtime virtual texture page corruption causes square artifacts from corrupt pages.
			// Repro of the bug in cooked builds is rare. But there is a reliable editor repro when editing layered materials parameters: UE-323864. 
			// It's been found that inserting a single call to ID3D12GraphicsCommandList::SetPipelineState() before rendering any RVT pages resolves the issue.
			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderMapRef<FShader_VirtualTextureCopy_VS> VertexShader(GlobalShaderMap);
			TShaderMapRef<FShader_VirtualTextureCopy_PS<ERuntimeVirtualTextureMaterialType::BaseColor>> PixelShader(GlobalShaderMap);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			GraphicsPSOInit.RenderTargetsEnabled = 1;
			GraphicsPSOInit.RenderTargetFormats[0] = PF_B8G8R8A8;
			GraphicsPSOInit.RenderTargetFlags[0] = TexCreate_None;
			GraphicsPSOInit.NumSamples = 1;
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0, EApplyRendertargetOption::DoNothing);
		}
#endif

		RHICmdList.SetViewport(static_cast<float>(InViewRect.Min.X), static_cast<float>(InViewRect.Min.Y), 0.0f, static_cast<float>(InViewRect.Max.X), static_cast<float>(InViewRect.Max.Y), 1.0f);
	}

	/** Get the debug color to use for a given mip level. Returns InDefaultColor if mip debugging is disabled. */
	FLinearColor GetDebugMipLevelColor(bool bInAllowMipDebug, uint32 InLevel, uint32 InTileAndBorderSize)
	{
		if (!bInAllowMipDebug)
		{
			return FLinearColor::Black;
		}

		static const auto CVarBorders = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VT.Borders"));
		const int32 DebugBorderSize = CVarBorders ? CVarBorders->GetValueOnRenderThread() : 0;
		if (DebugBorderSize <= 0)
		{
			return FLinearColor::Black;
		}

		static const auto CVarBordersMip = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VT.Borders.Mip"));
		const int32 DebugBorderMip = CVarBordersMip ? CVarBordersMip->GetValueOnRenderThread() : -1;
		if (DebugBorderMip >= 0 && DebugBorderMip != InLevel)
		{
			return FLinearColor::Black;
		}

		static const uint32 MipColors[] = {
			0x00FFFFFF, 0x00FFFF00, 0x0000FFFF, 0x0000FF00, 0x00FF00FF, 0x00FF0000, 0x000000FF,
			0x00808080, 0x00808000, 0x00008080, 0x00008000, 0x00800080, 0x00800000, 0x00000080 };

		InLevel = FMath::Min<uint32>(InLevel, sizeof(MipColors) / sizeof(MipColors[0]) - 1);
		FLinearColor Color = FLinearColor(FColor(MipColors[InLevel]));
		
		// Mip border size is stored in alpha channel.
		// Need to cover requested border size plus the tile border. Assume tile border is the difference above some power of 2 tile size.
		const uint32 TileBorderSize = (InTileAndBorderSize - (1 << FMath::FloorLog2(InTileAndBorderSize)));
		Color.A = 1.0f - float(TileBorderSize + DebugBorderSize * 2) / float(InTileAndBorderSize);

		return Color;
	}

	/** 
	 * Render a single page from a batch. 
	 * todo[vt]: Can we add some batch rendering mesh pass where all prerequesite BuildRenderingCommands/Compute phases are batched and then all Graphics draws are batched.
	 */
	void RenderPage(FRDGBuilder& GraphBuilder, FBatchRenderContext const& BatchRenderContext, int32 PageIndex)
	{
		CSV_CUSTOM_STAT(VirtualTexturing, RenderedPages, 1, ECsvCustomStatOp::Accumulate);
		INC_DWORD_STAT_BY(STAT_RenderedPages, 1);

		FRenderGraphSetup const& GraphSetup = BatchRenderContext.GraphSetup;
		FRenderPageBatchDesc const& BatchDesc = BatchRenderContext.BatchDesc;
		FRenderPageDesc const& PageDesc = BatchDesc.PageDescs[PageIndex];
		FScene* Scene = BatchDesc.SceneRenderer->GetScene();

		// Initialize the view required for the material render pass
		FSceneViewFamily::ConstructionValues ViewFamilyInit(nullptr, Scene, FEngineShowFlags(ESFIM_Game));
		ViewFamilyInit.SetTime(FGameTime());
		FViewFamilyInfo& ViewFamily = *GraphBuilder.AllocObject<FViewFamilyInfo>(ViewFamilyInit);
		ViewFamily.SetSceneRenderer(BatchDesc.SceneRenderer);

		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily = &ViewFamily;

		const FIntPoint TextureSize = PageDesc.DestRect[0].Size();
		ViewInitOptions.SetViewRectangle(FIntRect(FIntPoint(0, 0), TextureSize));

		FBox2D const& UVRange = PageDesc.UVRange;
		const FVector UVCenter = FVector(UVRange.GetCenter(), 0.f);
		FTransform const& UVToWorld = BatchDesc.UVToWorld;
		const FVector CameraLookAt = UVToWorld.TransformPosition(UVCenter);
		const float BoundBoxZ = UVToWorld.GetScale3D().Z;
		const FVector CameraPos = CameraLookAt + BoundBoxZ * UVToWorld.GetUnitAxis(EAxis::Z);
		ViewInitOptions.ViewOrigin = CameraPos;

		const float OrthoWidth = UVToWorld.GetScaledAxis(EAxis::X).Size() * UVRange.GetExtent().X;
		const float OrthoHeight = UVToWorld.GetScaledAxis(EAxis::Y).Size() * UVRange.GetExtent().Y;

		const FTransform WorldToUVRotate(UVToWorld.GetRotation().Inverse());
		ViewInitOptions.ViewRotationMatrix = WorldToUVRotate.ToMatrixNoScale() * FMatrix(
			FPlane(1, 0, 0, 0),
			FPlane(0, -1, 0, 0),
			FPlane(0, 0, -1, 0),
			FPlane(0, 0, 0, 1));

		const float NearPlane = 0;
		const float FarPlane = BoundBoxZ;
		const float ZScale = 1.0f / (FarPlane - NearPlane);
		const float ZOffset = -NearPlane;
		ViewInitOptions.ProjectionMatrix = FReversedZOrthoMatrix(OrthoWidth, OrthoHeight, ZScale, ZOffset);

		const uint8 vLevel = PageDesc.vLevel;
		const uint8 MaxLevel = BatchDesc.MaxLevel;
		const FVector4f MipLevelParameter = FVector4f((float)vLevel, (float)MaxLevel, OrthoWidth / (float)TextureSize.X, OrthoHeight / (float)TextureSize.Y);
		
		FBox const& WorldBounds = BatchDesc.WorldBounds;
		const float HeightRange = FMath::Max<float>(WorldBounds.Max.Z - WorldBounds.Min.Z, 1.f);
		const FVector2D WorldHeightPackParameter = FVector2D(1.f / HeightRange, -WorldBounds.Min.Z / HeightRange);

		ViewInitOptions.BackgroundColor = FLinearColor::Black;
		ViewInitOptions.OverlayColor = FLinearColor::White;

		FViewInfo* View = GraphBuilder.AllocObject<FViewInfo>(ViewInitOptions);
		ViewFamily.Views.Add(View);

		View->bIsVirtualTexture = true;
		View->ViewRect = View->UnconstrainedViewRect;
		View->CachedViewUniformShaderParameters = MakeUnique<FViewUniformShaderParameters>();
		View->SetupUniformBufferParameters(nullptr, 0, *View->CachedViewUniformShaderParameters);
		View->ViewUniformBuffer = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(*View->CachedViewUniformShaderParameters, UniformBuffer_SingleFrame);

		{
			RenderCaptureInterface::FScopedCapture RenderCapture((RenderCaptureNextRVTPagesDraws != 0), GraphBuilder, TEXT("RenderRVTPage"));
			RenderCaptureNextRVTPagesDraws = FMath::Max(RenderCaptureNextRVTPagesDraws - 1, 0);

			ERenderTargetLoadAction LoadAction = BatchDesc.bClearTextures ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ENoAction;
			FShader_VirtualTextureMaterialDraw::FParameters* PassParameters = GraphBuilder.AllocParameters<FShader_VirtualTextureMaterialDraw::FParameters>();
			PassParameters->View = View->ViewUniformBuffer;
			PassParameters->Scene = BatchDesc.SceneRenderer->GetSceneUniformBufferRef(GraphBuilder);

			FRuntimeVirtualTexturePassParameters* RuntimeVirtualTexturePassParameters = GraphBuilder.AllocParameters<FRuntimeVirtualTexturePassParameters>();
			RuntimeVirtualTexturePassParameters->MipLevel = MipLevelParameter;
			RuntimeVirtualTexturePassParameters->CustomMaterialData = BatchDesc.CustomMaterialData;
			RuntimeVirtualTexturePassParameters->FixedColor = BatchDesc.FixedColor;
			RuntimeVirtualTexturePassParameters->MipColorAndSize = GetDebugMipLevelColor(BatchRenderContext.bAllowMipDebug, vLevel, TextureSize.X);
			RuntimeVirtualTexturePassParameters->PackHeight = FVector2f(WorldHeightPackParameter);	// LWC_TODO: Precision loss
 			PassParameters->RuntimeVirtualTexturePassParameters = GraphBuilder.CreateUniformBuffer(RuntimeVirtualTexturePassParameters);

			PassParameters->RenderTargets[0] = GraphSetup.RenderTexture0 ? FRenderTargetBinding(GraphSetup.RenderTexture0, LoadAction, 0, PageIndex) : FRenderTargetBinding();
			PassParameters->RenderTargets[1] = GraphSetup.RenderTexture1 ? FRenderTargetBinding(GraphSetup.RenderTexture1, LoadAction, 0, PageIndex) : FRenderTargetBinding();
			PassParameters->RenderTargets[2] = GraphSetup.RenderTexture2 ? FRenderTargetBinding(GraphSetup.RenderTexture2, LoadAction, 0, PageIndex) : FRenderTargetBinding();

			const int32 RuntimeVirtualTextureId = BatchDesc.RuntimeVirtualTextureId;
			const ERuntimeVirtualTextureMaterialType MaterialType = BatchDesc.MaterialType;
			const bool bAllowCachedMeshDrawCommands = BatchRenderContext.bAllowCachedMeshDrawCommands;

			AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, *View, nullptr, RDG_EVENT_NAME("VirtualTextureDraw"), ERDGPassFlags::Raster | ERDGPassFlags::NeverMerge,
	    	[Scene, &GraphBuilder, View, MaterialType, RuntimeVirtualTextureId, vLevel, MaxLevel, bAllowCachedMeshDrawCommands](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
	    	{
	    		GatherMeshesToDraw(DynamicMeshPassContext, GraphBuilder, Scene, View, MaterialType, RuntimeVirtualTextureId, vLevel, MaxLevel, bAllowCachedMeshDrawCommands);
	    	}, 
			[ViewRect = View->ViewRect, ShaderPlatform = Scene->GetShaderPlatform()](FRHICommandList& RHICmdList)
			{
				MeshPassPrologue(RHICmdList, ViewRect, ShaderPlatform);
			});
		}
	}

	/** 
	 * Copy a single rendered page doing any attribute packing.
	 * This path is rarely used, since most use cases want to compress the results of rendering.
	 * We use a pixel shader, but could use a compute shader which batches multiple pages.
	 */
	void CopyPage(FRDGBuilder& GraphBuilder, FBatchRenderContext const& BatchRenderContext, int32 PageIndex)
	{
		FRenderGraphSetup const& GraphSetup = BatchRenderContext.GraphSetup;
		FRenderPageBatchDesc const& BatchDesc = BatchRenderContext.BatchDesc;
		FRenderPageDesc const& PageDesc = BatchDesc.PageDescs[PageIndex];
		const int32 ArraySlice = BatchDesc.NumPageDescs > 1 ? PageIndex : -1;
		FScene* Scene = BatchDesc.SceneRenderer->GetScene();
		const FIntPoint TextureSize = PageDesc.DestRect[0].Size();
		
		FShader_VirtualTextureCopy::FParameters* PassParameters = GraphBuilder.AllocParameters<FShader_VirtualTextureCopy::FParameters>();
		PassParameters->RenderTargets[0] = GraphSetup.CopyTexture0 ? FRenderTargetBinding(GraphSetup.CopyTexture0, ERenderTargetLoadAction::ENoAction, 0, PageIndex) : FRenderTargetBinding();
		PassParameters->RenderTargets[1] = GraphSetup.CopyTexture1 ? FRenderTargetBinding(GraphSetup.CopyTexture1, ERenderTargetLoadAction::ENoAction, 0, PageIndex) : FRenderTargetBinding();
		PassParameters->RenderTargets[2] = GraphSetup.CopyTexture2 ? FRenderTargetBinding(GraphSetup.CopyTexture2, ERenderTargetLoadAction::ENoAction, 0, PageIndex) : FRenderTargetBinding();
		PassParameters->DestRect = FIntVector4(0, 0, TextureSize.X, TextureSize.Y);
		PassParameters->RenderTexture0 = FRenderGraphSetup::CreateTextureSRV(GraphBuilder, GraphSetup.RenderTexture0, ArraySlice);
		PassParameters->TextureSampler0 = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->RenderTexture1 = FRenderGraphSetup::CreateTextureSRV(GraphBuilder, GraphSetup.RenderTexture1, ArraySlice);
		PassParameters->TextureSampler1 = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->RenderTexture2 = FRenderGraphSetup::CreateTextureSRV(GraphBuilder, GraphSetup.RenderTexture2, ArraySlice);
		PassParameters->TextureSampler2 = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		if (GraphSetup.bCopyPass)
		{
			AddCopyPass(GraphBuilder, Scene->GetFeatureLevel(), PassParameters, TextureSize, BatchDesc.MaterialType);
		}
		else
		{
			AddCopyThumbnailPass(GraphBuilder, Scene->GetFeatureLevel(), PassParameters, TextureSize, BatchDesc.MaterialType);
		}
	}

	/** Compress all pages in a batch. */
	void CompressPages(FRDGBuilder& GraphBuilder, FBatchRenderContext const& BatchRenderContext)
	{
		FRenderGraphSetup const& GraphSetup = BatchRenderContext.GraphSetup;
		FRenderPageBatchDesc const& BatchDesc = BatchRenderContext.BatchDesc;

		FScene* Scene = BatchDesc.SceneRenderer->GetScene();
		const FIntPoint TextureSize = BatchDesc.PageDescs[0].DestRect[0].Size();

		FShader_VirtualTextureCompress::FParameters* PassParameters = GraphBuilder.AllocParameters<FShader_VirtualTextureCompress::FParameters>();
		PassParameters->SourceRect = FUintVector4(0, 0, TextureSize.X, TextureSize.Y);
		PassParameters->EtcParameters = GetEtcParametersUniformBufferRef();
		PassParameters->AstcParameters = GetAstcParametersUniformBufferRef();
		PassParameters->RenderTexture0 = GraphSetup.RenderTexture0;
		PassParameters->TextureSampler0 = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->RenderTexture1 = GraphSetup.RenderTexture1;
		PassParameters->TextureSampler1 = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->RenderTexture2 = GraphSetup.RenderTexture2;
		PassParameters->TextureSampler2 = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->OutCompressTexture0_64bit = GraphSetup.CompressTextureUAV0_64bit;
		PassParameters->OutCompressTexture1_64bit = GraphSetup.CompressTextureUAV1_64bit;
		PassParameters->OutCompressTexture2_64bit = GraphSetup.CompressTextureUAV2_64bit;
		PassParameters->OutCompressTexture0_128bit = GraphSetup.CompressTextureUAV0_128bit;
		PassParameters->OutCompressTexture1_128bit = GraphSetup.CompressTextureUAV1_128bit;
		PassParameters->OutCompressTexture2_128bit = GraphSetup.CompressTextureUAV2_128bit;

		for (int32 PageIndex = 0; PageIndex < BatchDesc.NumPageDescs; ++PageIndex)
		{
			FRenderPageDesc const& PageDesc = BatchDesc.PageDescs[PageIndex];
			for (int32 LayerIndex = 0; LayerIndex < MaxTextureLayers; ++LayerIndex)
			{
				const int32 WriteIndex = (PageIndex * MaxTextureLayers + LayerIndex) * 2;

				// Direct aliasing case assumes needs to adjust dest position for BC block size.
				const int32 DestPosX = GraphSetup.bDirectAliasing ? PageDesc.DestRect[LayerIndex].Min.X / 4 : 0;
				const int32 DestPosY = GraphSetup.bDirectAliasing ? PageDesc.DestRect[LayerIndex].Min.Y / 4 : 0;

				GET_SCALAR_ARRAY_ELEMENT(PassParameters->DestPos, WriteIndex) = DestPosX;
				GET_SCALAR_ARRAY_ELEMENT(PassParameters->DestPos, WriteIndex + 1) = DestPosY;
			}
		}

		AddCompressPass(GraphBuilder, Scene->GetFeatureLevel(), PassParameters, TextureSize, BatchDesc.NumPageDescs, BatchDesc.MaterialType, GraphSetup.bDirectAliasing);
	}
	
	/** Copy all pages in a batch to the final output textures. */
	void CopyPagesToOutput(FRDGBuilder& GraphBuilder, FBatchRenderContext const& BatchRenderContext)
	{
		FRenderGraphSetup const& GraphSetup = BatchRenderContext.GraphSetup;
		if (GraphSetup.OutputAlias0 == nullptr && GraphSetup.OutputAlias1 == nullptr && GraphSetup.OutputAlias2 == nullptr)
		{
			return;
		}

		FRenderPageBatchDesc const& BatchDesc = BatchRenderContext.BatchDesc;
		const FRDGTextureRef SourceTexture[MaxTextureLayers] = { GraphSetup.OutputAlias0, GraphSetup.OutputAlias1, GraphSetup.OutputAlias2 };
		const FRDGTextureRef DestTexture[MaxTextureLayers] = { GraphSetup.TargetTexture0, GraphSetup.TargetTexture1, GraphSetup.TargetTexture2 };
		const FIntVector CopySize = SourceTexture[0] ? SourceTexture[0]->Desc.GetSize() : FIntVector(0, 0, 0);

		for (int32 PageIndex = 0; PageIndex < BatchDesc.NumPageDescs; ++PageIndex)
		{
			FRenderPageDesc const& PageDesc = BatchDesc.PageDescs[PageIndex];

			for (int32 LayerIndex = 0; LayerIndex < MaxTextureLayers; ++LayerIndex)
			{
				if (SourceTexture[LayerIndex] != nullptr && DestTexture[LayerIndex] != nullptr)
				{
					FRHICopyTextureInfo CopyInfo;
					CopyInfo.Size = CopySize;
					CopyInfo.SourceSliceIndex = PageIndex;
					CopyInfo.DestPosition = FIntVector(PageDesc.DestRect[LayerIndex].Min.X, PageDesc.DestRect[LayerIndex].Min.Y, 0);

					AddCopyTexturePass(GraphBuilder, SourceTexture[LayerIndex], DestTexture[LayerIndex], CopyInfo);
				}
			}
		}
	}

	bool IsSceneReadyToRender(FSceneInterface* Scene)
	{
		return Scene != nullptr && Scene->GetRenderScene() != nullptr && Scene->GetRenderScene()->GPUScene.IsRendering();
	}

	FBatchRenderContext const* InitPageBatch(FRDGBuilder& GraphBuilder, FRenderPageBatchDesc const& InDesc)
	{
		FBatchRenderContext* Context = GraphBuilder.AllocObject<FBatchRenderContext>();
		Context->GraphSetup.Init(GraphBuilder, FRenderGraphSetup::FInitDesc(InDesc));
		Context->BatchDesc = InDesc;
		return Context;
	}

	void RenderPageBatch(FRDGBuilder& GraphBuilder, FBatchRenderContext const& InContext)
	{
		FRenderGraphSetup const& GraphSetup = InContext.GraphSetup;
		FRenderPageBatchDesc const& Desc = InContext.BatchDesc;

		if (GraphSetup.bRenderPass)
		{
			for (int32 PageIndex = 0; PageIndex < Desc.NumPageDescs; ++PageIndex)
			{
				RenderPage(GraphBuilder, InContext, PageIndex);
			}
		}
		
		if (GraphSetup.bCopyPass || GraphSetup.bCopyThumbnailPass)
		{
			for (int32 PageIndex = 0; PageIndex < Desc.NumPageDescs; ++PageIndex)
			{
				CopyPage(GraphBuilder, InContext, PageIndex);
			}
		}

		// Batch compress pages now if not direct aliasing the final output texture.
		// This can reduce the memory high water mark.
		// If we are direct aliasing then we must defer compression to FinalizePageBatch().
		if (GraphSetup.bCompressPass && !InContext.GraphSetup.bDirectAliasing)
		{
			CompressPages(GraphBuilder, InContext);
		}
	}

	void FinalizePageBatch(FRDGBuilder& GraphBuilder, FBatchRenderContext const& InContext)
	{
		FRenderGraphSetup const& GraphSetup = InContext.GraphSetup;

		if (GraphSetup.bCompressPass && GraphSetup.bDirectAliasing)
		{
			CompressPages(GraphBuilder, InContext);
		}
		
		if (!GraphSetup.bDirectAliasing)
		{
			CopyPagesToOutput(GraphBuilder, InContext);
		}
	}

	void RenderPages(FRDGBuilder& GraphBuilder, FRenderPageBatchDesc const& InDesc)
	{
		FBatchRenderContext Context;
		Context.GraphSetup.Init(GraphBuilder, FRenderGraphSetup::FInitDesc(InDesc));
		Context.BatchDesc = InDesc;

		// Disable MDC caching for this standalone path because we can't guarantee that primitives 
		// associated with the scene have been recreated (e.g. by World->SendAllEndOfFrameUpdates()).
		Context.bAllowCachedMeshDrawCommands = false;
		// Disable mip color debugging for standalone path used for baking streaming low mips.
		Context.bAllowMipDebug = false;

		RenderPageBatch(GraphBuilder, Context);
		FinalizePageBatch(GraphBuilder, Context);
	}

	/** This function is deprecated! */
	uint32 GetRuntimeVirtualTextureSceneIndex_GameThread(URuntimeVirtualTextureComponent* InComponent)
	{
		if (InComponent == nullptr || InComponent->GetScene() == nullptr || InComponent->GetVirtualTexture() == nullptr)
		{
			return ~0u;
		}

		int32 SceneIndex = 0;
		FSceneInterface* SceneInterface = InComponent->GetScene();
		int32 RuntimeVirtualTextureId = InComponent->GetVirtualTexture()->GetUniqueID();

		ENQUEUE_RENDER_COMMAND(GetSceneIndexCommand)(
			[&SceneIndex, SceneInterface, RuntimeVirtualTextureId](FRHICommandListImmediate& RHICmdList)
		{
			if (FScene* Scene = SceneInterface->GetRenderScene())
			{
				SceneIndex = Scene->RuntimeVirtualTextures.IndexOfByPredicate([RuntimeVirtualTextureId] (FRuntimeVirtualTextureSceneProxy const* SceneProxy)
				{
					return SceneProxy->RuntimeVirtualTextureId == RuntimeVirtualTextureId;
				});
			}
		});
		FlushRenderingCommands();
		return SceneIndex;
	}
}
