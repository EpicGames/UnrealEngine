// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/RuntimeVirtualTextureSceneExtension.h"

#include "RenderUtils.h"
#include "ScenePrivate.h"

IMPLEMENT_SCENE_EXTENSION(FRuntimeVirtualTextureSceneExtension);

/** Persistant scene data stored by the extension. */
struct FRuntimeVirtualTextureSceneExtensionData
{
	/** A bit array and count of active persistant primitive ids for a single runtime virtual texture. */
	struct FActivePrimitives
	{
		TBitArray<> BitArray;
		int32 SetBitCount = 0;
	};
	
	/** Map of runtime virtual texture id to entry data. */
	TMap<int32, FActivePrimitives> RuntimeVirtualTextureMap;
};

FRuntimeVirtualTextureSceneExtension::FRuntimeVirtualTextureSceneExtension(FScene& InScene)
	: ISceneExtension(InScene)
	, Data(new FRuntimeVirtualTextureSceneExtensionData)
{
}

bool FRuntimeVirtualTextureSceneExtension::ShouldCreateExtension(FScene& InScene)
{
	return UseVirtualTexturing(InScene.GetShaderPlatform());
}

void FRuntimeVirtualTextureSceneExtension::GetPrimitivesForRuntimeVirtualTexture(FScene const* InScene, int32 InRuntimeVirtualTextureId, TArray<int32>& OutPrimitiveIndices) const
{
	if (FRuntimeVirtualTextureSceneExtensionData::FActivePrimitives* Found = Data->RuntimeVirtualTextureMap.Find(InRuntimeVirtualTextureId))
	{
		OutPrimitiveIndices.Reserve(Found->SetBitCount);

		int32 Index = Found->BitArray.Find(true);
		while (Index != INDEX_NONE)
		{
			OutPrimitiveIndices.Add(InScene->GetPrimitiveIndex(FPersistentPrimitiveIndex{Index}));
			Index = Found->BitArray.FindFrom(true, Index + 1);
		}
	}
}

class FRuntimeVirtualTextureSceneExtensionUpdater : public ISceneExtensionUpdater
{
	DECLARE_SCENE_EXTENSION_UPDATER(FRenderer, FRuntimeVirtualTextureSceneExtension);

public:
	FRuntimeVirtualTextureSceneExtensionUpdater(FRuntimeVirtualTextureSceneExtensionData& InData) 
		: Data(InData)
	{
	}

	virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override
	{
		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.RemovedPrimitiveSceneInfos)
		{
			if (PrimitiveSceneInfo->bWritesRuntimeVirtualTexture)
			{
				if (FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy)
				{
					const FPersistentPrimitiveIndex PersistentIndex = PrimitiveSceneInfo->GetPersistentIndex();

					for (int32 RuntimeVirtualTextureId : Proxy->GetRuntimeVirtualTextureIds())
					{
						FRuntimeVirtualTextureSceneExtensionData::FActivePrimitives& Entry = Data.RuntimeVirtualTextureMap.FindChecked(RuntimeVirtualTextureId);
						ensure(Entry.BitArray[PersistentIndex.Index] == true);
						Entry.BitArray[PersistentIndex.Index] = false;
						Entry.SetBitCount--;
					}
				}
			}
		}
	}

	virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override
	{
		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.AddedPrimitiveSceneInfos)
		{
			if (PrimitiveSceneInfo->bWritesRuntimeVirtualTexture)
			{
				if (FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy)
				{
					const FPersistentPrimitiveIndex PersistentIndex = PrimitiveSceneInfo->GetPersistentIndex();

					for (int32 RuntimeVirtualTextureId : Proxy->GetRuntimeVirtualTextureIds())
					{
						FRuntimeVirtualTextureSceneExtensionData::FActivePrimitives& Entry = Data.RuntimeVirtualTextureMap.FindOrAdd(RuntimeVirtualTextureId);
						Entry.BitArray.PadToNum(PersistentIndex.Index + 1, false);
						ensure(Entry.BitArray[PersistentIndex.Index] == false);
						Entry.BitArray[PersistentIndex.Index] = true;
						Entry.SetBitCount++;
					}
				}
			}
		}
	}

private:
	/** Reference to the owner's extension data. */
	FRuntimeVirtualTextureSceneExtensionData& Data;
};

ISceneExtensionUpdater* FRuntimeVirtualTextureSceneExtension::CreateUpdater()
{
	return new FRuntimeVirtualTextureSceneExtensionUpdater(*Data);
}
