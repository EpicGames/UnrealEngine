// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneExtensions.h"

/** Scene extension for storing information about runtime virtual textures in the scene. */
class FRuntimeVirtualTextureSceneExtension : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FRuntimeVirtualTextureSceneExtension);

public:
	using ISceneExtension::ISceneExtension;

	FRuntimeVirtualTextureSceneExtension(FScene& InScene);

	//~ Begin ISceneExtension Interface.
	static bool ShouldCreateExtension(FScene& Scene);
	virtual ISceneExtensionUpdater* CreateUpdater() override;
	//~ End ISceneExtension Interface.

	/** Get a list of scene primitive indices for a given runtime virtual texture. */
	void GetPrimitivesForRuntimeVirtualTexture(FScene const* InScene, int32 InRuntimeVirtualTextureId, TArray<int32>& OutPrimitiveIndices) const;

private:
	TUniquePtr<struct FRuntimeVirtualTextureSceneExtensionData> Data;
};
