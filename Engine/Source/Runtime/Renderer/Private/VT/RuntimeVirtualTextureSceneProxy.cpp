// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/RuntimeVirtualTextureSceneProxy.h"

#include "Components/RuntimeVirtualTextureComponent.h"
#include "RendererOnScreenNotification.h"
#include "SceneInterface.h"
#include "SceneUtils.h"
#include "VirtualTextureSystem.h"
#include "VT/RuntimeVirtualTextureEnum.h"
#include "VT/RuntimeVirtualTexture.h"
#include "VT/RuntimeVirtualTextureProducer.h"
#include "VT/VirtualTexture.h"
#include "VT/VirtualTextureBuilder.h"
#include "VT/VirtualTextureScalability.h"
#include "RenderingThread.h"

#define LOCTEXT_NAMESPACE "VirtualTexture"

FRuntimeVirtualTextureSceneProxy::FRuntimeVirtualTextureSceneProxy(URuntimeVirtualTextureComponent* InComponent)
{
	// Evaluate the flags used to hide primitives writing to this virtual texture.
	InComponent->GetHidePrimitiveSettings(bHidePrimitivesInEditor, bHidePrimitivesInGame);

	if (InComponent->GetVirtualTexture() != nullptr)
	{
		if (InComponent->IsEnabledInScene())
		{
			URuntimeVirtualTexture::FInitSettings InitSettings;
			InitSettings.TileCountBias = InComponent->IsScalable() ? VirtualTextureScalability::GetRuntimeVirtualTextureSizeBias(InComponent->GetScalabilityGroup()) : 0;

			VirtualTexture = InComponent->GetVirtualTexture();
			RuntimeVirtualTextureId = VirtualTexture->GetUniqueID();

			Transform = InComponent->GetComponentTransform();
			Bounds = InComponent->Bounds.GetBox();
			const FVector4f CustomMaterialData = InComponent->GetCustomMaterialData();

			// The producer description is calculated using the transform to determine the aspect ratio
			FVTProducerDescription ProducerDesc;
			VirtualTexture->GetProducerDescription(ProducerDesc, InitSettings, Transform);

			MaterialType = VirtualTexture->GetMaterialType();
			const bool bClearTextures = VirtualTexture->GetClearTextures();

			// Get streaming texture if it is valid.
			UVirtualTexture2D* StreamingTexture = nullptr;

			FSceneInterface* SceneInterface = InComponent->GetScene();
			const EShadingPath ShadingPath = SceneInterface ? SceneInterface->GetShadingPath() : EShadingPath::Deferred;
			if (InComponent->IsStreamingLowMips(ShadingPath))
			{
				if (InComponent->IsStreamingTextureInvalid(ShadingPath))
				{
#if !UE_BUILD_SHIPPING
					// Notify that streaming texture is invalid since this can cause performance regression.
					const FString Name = InComponent->GetPathName();
					OnScreenWarningDelegateHandle = FRendererOnScreenNotification::Get().AddLambda([Name](FCoreDelegates::FSeverityMessageMap& OutMessages)
					{
						OutMessages.Add(
							FCoreDelegates::EOnScreenMessageSeverity::Warning,
							FText::Format(LOCTEXT("SVTInvalid", "Runtime Virtual Texture '{0}' streaming mips needs to be rebuilt."), FText::FromString(Name)));
					});
#endif
				}
				else
				{
					StreamingTexture = InComponent->GetStreamingTexture()->GetVirtualTexture(ShadingPath);
				}
			}

			// The producer object created here will be passed into the virtual texture system which will take ownership.
			IVirtualTexture* Producer = nullptr;

			// Create a producer for the streaming low mips. 
			// This is bound with the main producer so that one allocated VT can use both runtime or streaming producers dependent on mip level.
			if (StreamingTexture == nullptr)
			{
				// Create the runtime virtual texture producer.
				Producer = new FRuntimeVirtualTextureProducer(ProducerDesc, RuntimeVirtualTextureId, MaterialType, bClearTextures, SceneInterface, Transform, Bounds, CustomMaterialData);

				// We only need to dirty flush up to the producer description MaxLevel which accounts for the RemoveLowMips
				MaxDirtyLevel = ProducerDesc.MaxLevel;
			}
			else
			{
				// Create the streaming virtual texture producer.
				FVTProducerDescription StreamingProducerDesc;
				IVirtualTexture* StreamingProducer = RuntimeVirtualTexture::CreateStreamingTextureProducer(StreamingTexture, ProducerDesc, StreamingProducerDesc);

				// Copy the layer fallback colors from the streaming virtual texture.
				for (uint32 LayerIndex = 0u; LayerIndex < ProducerDesc.NumTextureLayers; ++LayerIndex)
				{
					ProducerDesc.LayerFallbackColor[LayerIndex] = StreamingProducerDesc.LayerFallbackColor[LayerIndex];
				}

				if (InComponent->IsStreamingLowMipsOnly())
				{
					// Clamp the the runtime virtual texture producer dimensions to the streaming virtual texture dimensions.
					// This will force to only using streaming pages.
					ProducerDesc.BlockWidthInTiles = StreamingProducerDesc.BlockWidthInTiles;
					ProducerDesc.BlockHeightInTiles = StreamingProducerDesc.BlockHeightInTiles;
					ProducerDesc.MaxLevel = StreamingProducerDesc.MaxLevel;
				}

				// Create the runtime virtual texture producer.
				Producer = new FRuntimeVirtualTextureProducer(ProducerDesc, RuntimeVirtualTextureId, MaterialType, bClearTextures, SceneInterface, Transform, Bounds, CustomMaterialData);

				// Bind the runtime virtual texture producer to the streaming producer.
				const int32 NumLevels = (int32)FMath::CeilLogTwo(FMath::Max(ProducerDesc.BlockWidthInTiles, ProducerDesc.BlockHeightInTiles));
				const int32 NumStreamingLevels = (int32)FMath::CeilLogTwo(FMath::Max(StreamingProducerDesc.BlockWidthInTiles, StreamingProducerDesc.BlockHeightInTiles));
				ensure(NumLevels >= NumStreamingLevels);
				const int32 TransitionLevel = NumLevels - NumStreamingLevels;

				Producer = RuntimeVirtualTexture::BindStreamingTextureProducer(Producer, StreamingProducer, TransitionLevel);
				
				// Any dirty flushes don't need to flush the streaming mips (they only change with a build step).
				MaxDirtyLevel = TransitionLevel - 1;
			}

			// Store effective virtual texture size used when calculating dirty regions.
			VirtualTextureSize = FIntPoint(ProducerDesc.BlockWidthInTiles * ProducerDesc.TileSize, ProducerDesc.BlockHeightInTiles * ProducerDesc.TileSize);

			// The Initialize() call will allocate the virtual texture by spawning work on the render thread.
			VirtualTexture->Initialize(Producer, ProducerDesc, Transform, Bounds,
				URuntimeVirtualTexture::FOnReInitDelegate::CreateRaw(this, &FRuntimeVirtualTextureSceneProxy::MarkUnused));

			bAdaptive = VirtualTexture->GetAdaptivePageTable();

			for (int32 Index = 0; Index < ERuntimeVirtualTextureShaderUniform_Count; ++Index)
			{
				ShaderUniforms[Index] = VirtualTexture->GetUniformParameter(Index);
			}

			// Store the ProducerHandle, SpaceID and AllocatedVirtualTexture immediately after virtual texture is initialized.
			ENQUEUE_RENDER_COMMAND(GetProducerHandle)(
				[this, VirtualTexturePtr = VirtualTexture](FRHICommandList& RHICmdList)
			{
				ProducerHandle = VirtualTexturePtr->GetProducerHandle();
				AllocatedVirtualTexture = VirtualTexturePtr->GetAllocatedVirtualTexture();
				SpaceID = AllocatedVirtualTexture->GetSpaceID();
			});
		}
		else
		{
			// When not enabled, ensure that the RVT asset has no allocated VT.
			// In PIE this handles removing the RVT from the editor scene.
			InComponent->GetVirtualTexture()->Release();
		}
	}
}

FRuntimeVirtualTextureSceneProxy::~FRuntimeVirtualTextureSceneProxy()
{
#if !UE_BUILD_SHIPPING
	FRendererOnScreenNotification::Get().Remove(OnScreenWarningDelegateHandle);
#endif
}

void FRuntimeVirtualTextureSceneProxy::Release()
{
	if (VirtualTexture != nullptr)
	{
		VirtualTexture->Release();
		VirtualTexture = nullptr;
	}
}

void FRuntimeVirtualTextureSceneProxy::MarkUnused()
{
	ENQUEUE_RENDER_COMMAND(GetProducerHandle)(
		[this](FRHICommandList& RHICmdList)
	{
		VirtualTexture = nullptr;
		ProducerHandle = {};
		AllocatedVirtualTexture = nullptr;
		SpaceID = -1;
	});
}

/** Transform world bounds into Virtual Texture UV space. */
static FBox2D GetUVRectFromWorldBounds(FTransform const& InTransform, FBoxSphereBounds const& InBounds)
{
	const FVector O = InTransform.GetTranslation();
	const FVector U = InTransform.GetUnitAxis(EAxis::X) * 1.f / InTransform.GetScale3D().X;
	const FVector V = InTransform.GetUnitAxis(EAxis::Y) * 1.f / InTransform.GetScale3D().Y;
	const FVector P = InBounds.GetSphere().Center - O;
	const FVector2D UVCenter = FVector2D(FVector::DotProduct(P, U), FVector::DotProduct(P, V));
	const float Scale = FMath::Max(1.f / InTransform.GetScale3D().X, 1.f / InTransform.GetScale3D().Y);
	const float UVRadius = InBounds.GetSphere().W * Scale;
	const FVector2D UVExtent(UVRadius, UVRadius);

	return FBox2D(UVCenter - UVExtent, UVCenter + UVExtent);
}

void FRuntimeVirtualTextureSceneProxy::Dirty(FBoxSphereBounds const& InBounds, EVTInvalidatePriority InInvalidatePriority)
{
	// If Producer handle is not initialized yet it's safe to do nothing because we won't have rendered anything to the VT that needs flushing.
	if (ProducerHandle.PackedValue != 0)
	{
		const FBox2D UVRect = GetUVRectFromWorldBounds(Transform, InBounds);

		// Convert to Texel coordinate space
		const FIntRect TextureRect(0, 0, VirtualTextureSize.X, VirtualTextureSize.Y);
		FIntRect TexelRect(
			FMath::FloorToInt(UVRect.Min.X * VirtualTextureSize.X),
			FMath::FloorToInt(UVRect.Min.Y * VirtualTextureSize.Y),
			FMath::CeilToInt(UVRect.Max.X * VirtualTextureSize.X),
			FMath::CeilToInt(UVRect.Max.Y * VirtualTextureSize.Y));
		TexelRect.Clip(TextureRect);

		// Only add rect if it has some area
		if (TexelRect.Min != TexelRect.Max)
		{
			FDirtyRect DirtyRect{ .Rect = TexelRect, .InvalidatePriority = InInvalidatePriority };
			const bool bFirst = DirtyRects.Add(DirtyRect) == 0;
			if (bFirst)
			{
				CombinedDirtyRect = DirtyRect;
			}
			else
			{
				CombinedDirtyRect.Union(DirtyRect);
			}
		}
	}
}

void FRuntimeVirtualTextureSceneProxy::FlushDirtyPages()
{
	// Don't do any work if we won't mark anything dirty.
	if (MaxDirtyLevel >= 0 && CombinedDirtyRect.Rect.Width() != 0 && CombinedDirtyRect.Rect.Height() != 0)
	{
		// Keeping visible pages mapped reduces update flicker due to the latency in the unmap/feedback/map sequence.
		// But it potentially creates more page update work since more pages may get updated.
		const uint32 MaxAgeToKeepMapped = VirtualTextureScalability::GetKeepDirtyPageMappedFrameThreshold();

		//todo[vt]: 
		// Profile to work out best heuristic for when we should use the CombinedDirtyRect
		// Also consider using some other structure to represent dirty area such as a course 2D bitfield
		bool bCombinedFlush = (DirtyRects.Num() > 2 || CombinedDirtyRect.Rect == FIntRect(0, 0, VirtualTextureSize.X, VirtualTextureSize.Y));
		// Don't use the combined rect if one of the dirty rects is prioritized, because that would would leave all of the pages being covered by the combined rect to get prioritized, 
		//  which would give exactly the opposite result of what we're trying to achieve, since we need to keep the number of prioritized pages to remain low :  
		bCombinedFlush &= (CombinedDirtyRect.InvalidatePriority == EVTInvalidatePriority::Normal);

		if (bCombinedFlush)
		{
			FVirtualTextureSystem::Get().FlushCache(ProducerHandle, SpaceID, CombinedDirtyRect.Rect, MaxDirtyLevel, MaxAgeToKeepMapped, EVTInvalidatePriority::Normal);
		}
		else
		{
			for (const FDirtyRect& DirtyRect : DirtyRects)
			{
				FVirtualTextureSystem::Get().FlushCache(ProducerHandle, SpaceID, DirtyRect.Rect, MaxDirtyLevel, MaxAgeToKeepMapped, DirtyRect.InvalidatePriority);
			}
		}
	}

	DirtyRects.Reset();
	CombinedDirtyRect = FDirtyRect();
}

void FRuntimeVirtualTextureSceneProxy::RequestPreload(FBoxSphereBounds const& InBounds, int32 InLevel)
{
	// If Producer handle is not initialized yet it's safe to do nothing.
	if (ProducerHandle.PackedValue != 0)
	{
		const FBox2D UVRect = GetUVRectFromWorldBounds(Transform, InBounds);
		FVirtualTextureSystem::Get().RequestTiles(AllocatedVirtualTexture, FVector2D::One(), FVector2D::Zero(), FVector2D::One(), UVRect.Min, UVRect.Max, InLevel);
	}
}

FVector4 FRuntimeVirtualTextureSceneProxy::GetUniformParameter(ERuntimeVirtualTextureShaderUniform UniformName) const
{
	const int32 UniformIndex = (int32)UniformName;

	if (UniformIndex < ERuntimeVirtualTextureShaderUniform_Count)
	{
		return ShaderUniforms[UniformIndex];
	}

	checkNoEntry();
	return FVector4::Zero();
}

#undef LOCTEXT_NAMESPACE
