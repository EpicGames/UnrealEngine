// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTexturing.h"
#include "VirtualTextureEnum.h"
#include "VT/RuntimeVirtualTextureEnum.h"

class URuntimeVirtualTexture;
class URuntimeVirtualTextureComponent;

/** Scene proxy for the URuntimeVirtualTextureComponent. Manages a runtime virtual texture in the renderer scene. */
class FRuntimeVirtualTextureSceneProxy
{
public:
	/** Constructor initializes resources for the URuntimeVirtualTexture associated with the provided component. */
	FRuntimeVirtualTextureSceneProxy(URuntimeVirtualTextureComponent* InComponent);
	~FRuntimeVirtualTextureSceneProxy();

	/**
	 * Release the object and it's associated runtime virtual texture resources. 
	 * Call this on the main thread before deferring deletion to happen on the render thread.
	 */
	void Release();

	/** Mark the object as no longer used. This can happen in the edge case where two components in a scene use the same runtime virtual texture asset. */
	void MarkUnused();

	/**
	 * Mark an area of the associated runtime virtual texture as dirty. 
	 * @param Bounds World-space bounds of the area of the runtime virtual texture to invalidate
	 * @param Priority Allows the affected pages to get processed in priority. This allows increased responsiveness when 
	 *  there are more pages being updated than can be handled in a given frame (when throttling)
	 */
	void Dirty(FBoxSphereBounds const& Bounds, EVTInvalidatePriority InvalidatePriority);

	/** Flush the cached physical pages of the virtual texture at the given world space bounds. */
	void FlushDirtyPages();

	/** Request preload of an area of the associated runtime virtual texture at a given mip level. */
	void RequestPreload(FBoxSphereBounds const& Bounds, int32 Level);

	/** Accessors */
	bool IsAdaptive() const { return bAdaptive; }
	ERuntimeVirtualTextureMaterialType GetMaterialType() const { return MaterialType; }
	const FBox& GetBounds() const { return Bounds; }
	FVector4 GetUniformParameter(ERuntimeVirtualTextureShaderUniform UniformName) const;
	const IAllocatedVirtualTexture* GetAllocatedVirtualTexture() const { return AllocatedVirtualTexture; }

	/** Index in FScene::RuntimeVirtualTextures. */
	int32 SceneIndex = -1;
	
	/** Unique object ID of the runtime virtual texture used to filter proxies to render to it.  */
	int32 RuntimeVirtualTextureId = -1;

	/** Hide primitives in the main pass in editor mode. */
	bool bHidePrimitivesInEditor = false;
	/** Hide primitives in the main pass in game mode. */
	bool bHidePrimitivesInGame = false;

private:
	/** Pointer to linked URuntimeVirtualTexture. Not for dereferencing, just for pointer comparison. */
	URuntimeVirtualTexture* VirtualTexture = nullptr;

	/** Whether this is an adaptive virtual texture. */
	bool bAdaptive = false;
	/** Material type for the URuntimeVirtualTexture object. */
	ERuntimeVirtualTextureMaterialType MaterialType = ERuntimeVirtualTextureMaterialType::Count;
	/** UVToWorld transform for the URuntimeVirtualTexture object. */
	FTransform Transform = FTransform::Identity;
	/** Component world bounds. */
	FBox Bounds = FBox(ForceInit);
	/** Virtual texture size of the URuntimeVirtualTexture object. */
	FIntPoint VirtualTextureSize = FIntPoint::ZeroValue;
	/** Shader uniforms cached from the URuntimeVirtualTexture object. */
	FVector4 ShaderUniforms[ERuntimeVirtualTextureShaderUniform_Count];

	/** Handle for the producer that this Proxy initialized. Used only for invalidation logic. */
	FVirtualTextureProducerHandle ProducerHandle;
	/** Allocated virtual texture that this Proxy initialized. Used only for preload logic. */
	IAllocatedVirtualTexture* AllocatedVirtualTexture = nullptr;
	/** Space ID used by the virtual texture. Used only for invalidation logic. */
	int32 SpaceID = -1;

	/** Maximum mip level to mark dirty. Can be less than the virtual texture's MaxLevel if we have streaming mips. */
	int32 MaxDirtyLevel = 0;

	struct FDirtyRect
	{
		FIntRect Rect;
		EVTInvalidatePriority InvalidatePriority = EVTInvalidatePriority::Normal;

		void Union(const FDirtyRect& InOther)
		{
			Rect.Union(InOther.Rect);
			InvalidatePriority = static_cast<EVTInvalidatePriority>(FMath::Max(static_cast<uint8>(InvalidatePriority), static_cast<uint8>(InOther.InvalidatePriority)));
		}
	};
	/** Array of dirty rectangles to process at the next flush. */
	TArray<FDirtyRect> DirtyRects;
	/** Combined dirty rectangle to process at the next flush. */
	FDirtyRect CombinedDirtyRect;

	/** DelegateHandle for on screen warning messages. */
	FDelegateHandle OnScreenWarningDelegateHandle;
};
