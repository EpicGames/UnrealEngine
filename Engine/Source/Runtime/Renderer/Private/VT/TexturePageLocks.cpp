// Copyright Epic Games, Inc. All Rights Reserved.

#include "TexturePageLocks.h"

FTexturePageLocks::FTexturePageLocks()
{
}

uint32 FTexturePageLocks::AcquireIndex()
{
	if (FreeIndices.Num() > 0)
	{
		return FreeIndices.Pop();
	}
	LockedTiles.AddDefaulted();
	const uint32 Index = LockCounts.AddZeroed();
	check(LockedTiles.Num() == LockCounts.Num());
	return Index;
}

bool FTexturePageLocks::Lock(const FVirtualTextureLocalTile& Tile)
{
	const uint16 Hash = Tile.GetMurmurHash();
	for (uint32 Index = TileHash.First(Hash); TileHash.IsValid(Index); Index = TileHash.Next(Index))
	{
		if (LockedTiles[Index] == Tile)
		{
			const uint32 PrevLockCount = LockCounts[Index];
			check(PrevLockCount > 0u);
			check(PrevLockCount < 0xffff);
			LockCounts[Index] = PrevLockCount + 1u;
			return false;
		}
	}

	const uint32 Index = AcquireIndex();
	LockedTiles[Index] = Tile;
	LockCounts[Index] = 1u;
	TileHash.Add(Hash, Index);
	ProducerToTileIndex.Add(MurmurFinalize32(Tile.PackedProducerHandle), Index);
	return true;
}

bool FTexturePageLocks::Unlock(const FVirtualTextureLocalTile& Tile)
{
	const uint16 Hash = Tile.GetMurmurHash();
	for (uint32 Index = TileHash.First(Hash); TileHash.IsValid(Index); Index = TileHash.Next(Index))
	{
		if (LockedTiles[Index] == Tile)
		{
			const uint32 PrevLockCount = LockCounts[Index];
			check(PrevLockCount > 0u);
			if (PrevLockCount == 1u)
			{
				// no longer locked
				FreeIndices.Add(Index);
				TileHash.Remove(Hash, Index);
				ProducerToTileIndex.Remove(MurmurFinalize32(Tile.PackedProducerHandle), Index);
				LockCounts[Index] = 0u;
				LockedTiles[Index].PackedValue = 0u;
				return true;
			}

			// still locked
			LockCounts[Index] = PrevLockCount - 1u;
			return false;
		}
	}

	// Possible that we're trying to unlock a tile that's already been force-unlocked because the producer was destroyed
	return false;
}

void FTexturePageLocks::ForceUnlockAll(const FVirtualTextureProducerHandle& ProducerHandle, TArray<FVirtualTextureLocalTile>& OutUnlockedTiles)
{
	TArray<int32, TInlineAllocator<32>> TileIndicesToUnlock;
	
	const uint32 Hash = MurmurFinalize32(ProducerHandle.PackedValue);
	for (uint32 Index = ProducerToTileIndex.First(Hash); ProducerToTileIndex.IsValid(Index); Index = ProducerToTileIndex.Next(Index))
	{
		if (LockedTiles[Index].PackedProducerHandle == ProducerHandle.PackedValue && LockCounts[Index] > 0u)
		{
			TileIndicesToUnlock.Add(Index);
		}
	}

	for (int32 Index : TileIndicesToUnlock)
	{
		const FVirtualTextureLocalTile& Tile = LockedTiles[Index];
		check(Tile.GetProducerHandle() == ProducerHandle);
		OutUnlockedTiles.Add(Tile);
		FreeIndices.Add(Index);
		TileHash.Remove(Tile.GetMurmurHash(), Index); 
		ProducerToTileIndex.Remove(MurmurFinalize32(Tile.PackedProducerHandle), Index);
		LockCounts[Index] = 0u;
		LockedTiles[Index].PackedValue = 0u;
	}
}

bool FTexturePageLocks::IsLocked(const FVirtualTextureLocalTile& Tile) const
{
	const uint16 Hash = Tile.GetMurmurHash();
	for (uint32 Index = TileHash.First(Hash); TileHash.IsValid(Index); Index = TileHash.Next(Index))
	{
		if (LockedTiles[Index] == Tile)
		{
			// lock count needs to be at least 1 if it's in the hash table
			check(LockCounts[Index] > 0u);
			return true;
		}
	}

	return false;
}
