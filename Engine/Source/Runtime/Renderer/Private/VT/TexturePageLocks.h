// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/HashTable.h"
#include "VirtualTexturing.h"

/**
 * This tracks lock counts for individual VT tiles
 * We need to track lock counts for all tiles, since different allocated VTs may try to lock/unlock the same pages,
 * and we need a way to know when a given tile is no longer locked by anything
 */
class FTexturePageLocks
{
public:
	FTexturePageLocks();

	/** Lock/Unlock return true if the lock state was changed */
	bool Lock(const FVirtualTextureLocalTile& Tile);
	bool Unlock(const FVirtualTextureLocalTile& Tile);

	/** Force-unlock all tiles using the given producer, returns list of all tiles that were unlocked */
	void ForceUnlockAll(const FVirtualTextureProducerHandle& ProducerHandle, TArray<FVirtualTextureLocalTile>& OutUnlockedTiles);

	bool IsLocked(const FVirtualTextureLocalTile& Tile) const;

private:
	uint32 AcquireIndex();

	FHashTable TileHash;
	FHashTable ProducerToTileIndex;
	TArray<FVirtualTextureLocalTile> LockedTiles;
	TArray<uint16> LockCounts;
	TArray<uint32> FreeIndices;
};
