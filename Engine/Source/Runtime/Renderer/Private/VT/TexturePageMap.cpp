// Copyright Epic Games, Inc. All Rights Reserved.

#include "TexturePageMap.h"
#include "VirtualTextureSpace.h"
#include "VirtualTexturePhysicalSpace.h"
#include "VirtualTextureSystem.h"

FORCEINLINE uint32 EncodeSortKey(uint8 vLevel, uint32 vAddress)
{
	uint32 Key;
	Key = vAddress << 0;
	Key |= (uint32)vLevel << 24;
	return Key;
}

FORCEINLINE void DecodeSortKey(uint32 Key, uint8& vLevel, uint32& vAddress)
{
	vAddress = (Key >> 0) & 0x00ffffffu;
	vLevel = (Key >> 24);
}


FTexturePageMap::FTexturePageMap()
	: LayerIndex(0u)
	, vDimensions(0u)
	, HashTable(4096u)
	, MappedPageCount(0u)
	, SortedKeysDirty(false)
{
}

FTexturePageMap::~FTexturePageMap()
{
}

void FTexturePageMap::Initialize(uint32 InSize, uint32 InLayerIndex, uint32 InDimensions)
{
	Pages.Empty(InSize + PageListHead_Count);
	for (uint32 ListHeadIndex = 0u; ListHeadIndex < PageListHead_Count; ++ListHeadIndex)
	{
		FPageEntry& ListHead = Pages.AddDefaulted_GetRef();
		ListHead.NextIndex = ListHead.PrevIndex = ListHeadIndex;
	}

	LayerIndex = InLayerIndex;
	vDimensions = InDimensions;
	HashTable.Resize(InSize);
	SortedKeys.Reserve(InSize);
}

uint32 FTexturePageMap::FindPageIndex(uint8 vLogSize, uint32 vAddress) const
{
	const FTexturePage CheckPage(vLogSize, vAddress);
	const uint16 Hash = MurmurFinalize32(CheckPage.Packed);
	for (uint32 PageIndex = HashTable.First(Hash); HashTable.IsValid(PageIndex); PageIndex = HashTable.Next(PageIndex))
	{
		const FPageEntry& Entry = Pages[PageIndex];
		if (Entry.Page == CheckPage)
		{
			return PageIndex;
		}
	}

	return ~0u;
}

uint32 FTexturePageMap::FindPageAddress(uint8 vLogSize, uint32 vAddress) const
{
	const uint32 Index = FindPageIndex(vLogSize, vAddress);
	return Index != ~0u ? Pages[Index].pAddress : ~0u;
}

FPhysicalSpaceIDAndAddress FTexturePageMap::FindPagePhysicalSpaceIDAndAddress(uint8 vLogSize, uint32 vAddress) const
{
	const FTexturePage CheckPage(vLogSize, vAddress);
	const uint16 Hash = MurmurFinalize32(CheckPage.Packed);
	return FindPagePhysicalSpaceIDAndAddress(CheckPage, Hash);
}

uint32 FTexturePageMap::FindNearestPageIndex(uint8 vLogSize, uint32 vAddress, uint8 MaxLevel) const
{
	while (vLogSize <= MaxLevel)
	{
		const uint32 Index = FindPageIndex(vLogSize, vAddress);
		if (Index != ~0u)
		{
			return Index;
		}

		vLogSize++;
		vAddress &= ~0u << (vDimensions * vLogSize);
	}

	return ~0u;
}

uint32 FTexturePageMap::FindNearestPageAddress(uint8 vLogSize, uint32 vAddress) const
{
	const uint32 Index = FindNearestPageIndex(vLogSize, vAddress, VIRTUALTEXTURE_LOG2_MAX_PAGETABLE_SIZE - 1u);
	return Index != ~0u ? Pages[Index].pAddress : ~0u;
}

uint32 FTexturePageMap::FindNearestPageLevel(uint8 vLogSize, uint32 vAddress) const
{
	const uint32 Index = FindNearestPageIndex(vLogSize, vAddress, VIRTUALTEXTURE_LOG2_MAX_PAGETABLE_SIZE - 1u);
	if (Index != ~0u)
	{
		return Pages[Index].Local_vLevel;
	}
	return 0xff;
}

void FTexturePageMap::UnmapPage(FVirtualTextureSystem* System, FVirtualTextureSpace* Space, uint8 vLogSize, uint32 vAddress, bool bMapAncestorPage)
{
	const uint32 PageIndex = FindPageIndex(vLogSize, vAddress);
	check(PageIndex != ~0u);

	const FTexturePage Page(vLogSize, vAddress);
	const FPageEntry& Entry = Pages[PageIndex];
	check(Entry.Page == Page);

	// Unmap old page
	const uint16 Hash = MurmurFinalize32(Page.Packed);
	HashTable.Remove(Hash, PageIndex);

	if (bMapAncestorPage)
	{
		const uint32 Parent_vLogSize = vLogSize + 1;
		const uint32 Parent_vAddress = vAddress & (~0u << (vDimensions * Parent_vLogSize));
		const uint32 AncestorIndex = FindNearestPageIndex(Parent_vLogSize, Parent_vAddress, Entry.MaxLevel);
		if (AncestorIndex != ~0u)
		{
			// Root page should typically be locked in memory, so we should always find some valid ancestor pAddress, unless the entire VT is being released
			// No reason to queue a page table update to invalid pAddress, just leave it alone for now, it will be updated when the page is remapped
			const FPageEntry& AncestorPage = Pages[AncestorIndex];
			check(AncestorPage.PackedProducerHandle == Entry.PackedProducerHandle);
			check(AncestorPage.PhysicalSpaceID == Entry.PhysicalSpaceID);
			const FVirtualTexturePhysicalSpace* PhysicalSpace = System->GetPhysicalSpace(AncestorPage.PhysicalSpaceID);
			Space->QueueUpdate(LayerIndex, vLogSize, vAddress, AncestorPage.Local_vLevel, PhysicalSpace->GetPhysicalLocation(AncestorPage.pAddress));
		}
	}
	

	// Deal with case where SortedAddIndexes has not been processed yet.
	bool bFoundInAddIndexes = false;
	for (int32 AddIndex = 0; AddIndex < SortedAddIndexes.Num(); ++AddIndex)
	{
		if ((SortedAddIndexes[AddIndex] & 0xffffffff) == PageIndex)
		{
			bFoundInAddIndexes = true;
			SortedAddIndexes.RemoveAtSwap(AddIndex, EAllowShrinking::No);
			break;
		}
	}

	if (!bFoundInAddIndexes)
	{
		const uint32 OldKey = EncodeSortKey(vLogSize, vAddress);
		const uint32 OldIndex = LowerBound(0, SortedKeys.Num(), OldKey, ~0u);
		check(SortedKeys[OldIndex] == OldKey); // make sure we actually found the key (should always exist, since we're removing it)
		checkSlow(UpperBound(0, SortedKeys.Num(), OldKey, ~0u) == OldIndex + 1u); // make sure key only exists once
		checkSlow(!SortedSubIndexes.Contains(OldIndex)); // make sure we're not somehow removing the same key twice

		SortedSubIndexes.Add(OldIndex);
	}

	RemovePageFromList(PageIndex);
	AddPageToList(PageListHead_Unmapped, PageIndex);

	check(MappedPageCount > 0u);
	--MappedPageCount;

	SortedKeysDirty = true;

}

void FTexturePageMap::MapPage(FVirtualTextureSpace* Space, FVirtualTexturePhysicalSpace* PhysicalSpace, uint32 PackedProducerHandle, uint8 MaxLevel, uint8 vLogSize, uint32 vAddress, uint8 Local_vLevel, uint16 pAddress)
{
#if DO_GUARD_SLOW
	const uint32 PrevPageIndex = FindPageIndex(vLogSize, vAddress);
	checkSlow(PrevPageIndex == ~0u);
#endif // DO_GUARD_SLOW

#if DO_CHECK
	// If there's already an ancestor page mapped, make sure it's from the same producer/physical space
	if (vLogSize < MaxLevel)
	{
		const uint32 Parent_vLogSize = vLogSize + 1;
		const uint32 Parent_vAddress = vAddress & (~0u << (vDimensions * Parent_vLogSize));
		const uint32 AncestorIndex = FindNearestPageIndex(Parent_vLogSize, Parent_vAddress, MaxLevel);
		if (AncestorIndex != ~0u)
		{
			const FPageEntry& AncestorPage = Pages[AncestorIndex];
			check(AncestorPage.PackedProducerHandle == PackedProducerHandle);
			check(AncestorPage.PhysicalSpaceID == PhysicalSpace->GetID());
		}
	}
#endif // DO_CHECK

	const FTexturePage Page(vLogSize, vAddress);
	const uint32 PageIndex = AcquirePage();
	check(PageIndex > 0u);
	FPageEntry& Entry = Pages[PageIndex];
	Entry.Page = Page;
	Entry.PackedProducerHandle = PackedProducerHandle;
	Entry.pAddress = pAddress;
	Entry.MaxLevel = MaxLevel;
	Entry.Local_vLevel = Local_vLevel;
	Entry.PhysicalSpaceID = PhysicalSpace->GetID();

	++MappedPageCount;
	AddPageToList(PageListHead_Mapped, PageIndex); // Add to list of allocated pages

	{
		const uint32 NewKey = EncodeSortKey(vLogSize, vAddress);
		const uint32 NewIndex = UpperBound(0, SortedKeys.Num(), NewKey, ~0u);
		SortedAddIndexes.Add(((uint64)NewIndex << 32) | PageIndex);

		// Map new page
		const uint16 Hash = MurmurFinalize32(Page.Packed);
		HashTable.Add(Hash, PageIndex);
		Space->QueueUpdate(LayerIndex, vLogSize, vAddress, Local_vLevel, PhysicalSpace->GetPhysicalLocation(pAddress));
	}

	SortedKeysDirty = true;
}

void FTexturePageMap::InvalidateUnmappedRootPage(FVirtualTextureSpace* Space, FVirtualTexturePhysicalSpace* PhysicalSpace, uint32 PackedProducerHandle, uint8 MaxLevel, uint8 vLogSize, uint32 vAddress, uint8 Local_vLevel)
{
	// Page should not be mapped.
	check(FindPageIndex(vLogSize, vAddress) == ~0u);
	// Queue clear page table entry.
	Space->QueueUpdate(LayerIndex, vLogSize, vAddress, Local_vLevel, FIntVector::ZeroValue);
}

void FTexturePageMap::GetMappedPagesInRange(uint32 vAddress, uint32 Width, uint32 Height, TArray<FMappedTexturePage>& OutMappedPages) const
{
	const uint32 vTileX0 = FMath::ReverseMortonCode2(vAddress);
	const uint32 vTileY0 = FMath::ReverseMortonCode2(vAddress >> 1);
	const uint32 vTileX1 = vTileX0 + Width;
	const uint32 vTileY1 = vTileY0 + Height;
	const uint32 vAddressMax = FMath::MortonCode2(vTileX1) | (FMath::MortonCode2(vTileY1) << 1);

	uint32 PageIndex = Pages[PageListHead_Mapped].NextIndex;
	uint32 CheckPageCount = 0u;
	while (PageIndex != PageListHead_Mapped)
	{
		const FPageEntry& Entry = Pages[PageIndex];
		if (Entry.Page.vAddress >= vAddress && Entry.Page.vAddress < vAddressMax)
		{
			const uint32 vTileX = FMath::ReverseMortonCode2(Entry.Page.vAddress);
			const uint32 vTileY = FMath::ReverseMortonCode2(Entry.Page.vAddress >> 1);
			if (vTileX >= vTileX0 &&
				vTileY >= vTileY0 &&
				vTileX < vTileX1 &&
				vTileY < vTileY1)
			{
				FMappedTexturePage& OutPage = OutMappedPages.AddDefaulted_GetRef();
				OutPage.Page = Entry.Page;
				OutPage.pAddress = Entry.pAddress;
				OutPage.PhysicalSpaceID = Entry.PhysicalSpaceID;
				OutPage.Local_vLevel = Entry.Local_vLevel;
			}
		}

		PageIndex = Entry.NextIndex;
		++CheckPageCount;
	}
	check(MappedPageCount == CheckPageCount);
}

// Must call this before the below functions so that SortedKeys is up to date.
inline void FTexturePageMap::BuildSortedKeys()
{
	checkSlow(SortedSubIndexes.Num() || SortedAddIndexes.Num());

	SortedSubIndexes.Sort();
	SortedAddIndexes.Sort(
		[this](const uint64& A, const uint64& B)
	{
		const FPageEntry& PageA = Pages[(uint32)A];
		const FPageEntry& PageB = Pages[(uint32)B];

		uint32 KeyA = EncodeSortKey(PageA.Page.vLogSize, PageA.Page.vAddress);
		uint32 KeyB = EncodeSortKey(PageB.Page.vLogSize, PageB.Page.vAddress);

		return KeyA < KeyB;
	});

	// Copy version
	Exchange(SortedKeys, UnsortedKeys);
	Exchange(SortedAddresses, UnsortedAddresses);

	uint32 NumUnsorted = UnsortedKeys.Num();
	SortedKeys.SetNum(NumUnsorted + SortedAddIndexes.Num() - SortedSubIndexes.Num(), EAllowShrinking::No);
	SortedAddresses.SetNum(NumUnsorted + SortedAddIndexes.Num() - SortedSubIndexes.Num(), EAllowShrinking::No);

	int32 SubI = 0;
	int32 AddI = 0;
	int32 UnsortedI = 0;
	int32 SortedI = 0;

	while (SortedI < SortedKeys.Num())
	{
		const uint32 SubIndex = SubI < SortedSubIndexes.Num() ? SortedSubIndexes[SubI] : NumUnsorted;
		const uint32 AddIndex = AddI < SortedAddIndexes.Num() ? (SortedAddIndexes[AddI] >> 32) : NumUnsorted;
		const uint32 MinIndex = FMath::Min(SubIndex, AddIndex);

		check(MinIndex >= (uint32)UnsortedI);
		if (MinIndex > (uint32)UnsortedI)
		{
			const uint32 Interval = MinIndex - UnsortedI;
			FMemory::Memcpy(&SortedKeys[SortedI], &UnsortedKeys[UnsortedI], Interval * sizeof(uint32));
			FMemory::Memcpy(&SortedAddresses[SortedI], &UnsortedAddresses[UnsortedI], Interval * sizeof(FPhysicalSpaceIDAndAddress));

			UnsortedI += Interval;
			SortedI += Interval;

			if (SortedI >= SortedKeys.Num())
				break;
		}

		if (SubIndex < AddIndex)
		{
			checkSlow(SubI < SortedSubIndexes.Num());

			// Skip hole
			UnsortedI++;
			SubI++;
		}
		else
		{
			checkSlow(AddI < SortedAddIndexes.Num());

			// Add new updated page
			const uint32 PageIndex = (uint32)SortedAddIndexes[AddI];
			const FPageEntry& Entry = Pages[PageIndex];
			SortedKeys[SortedI] = EncodeSortKey(Entry.Page.vLogSize, Entry.Page.vAddress);
			SortedAddresses[SortedI] = FPhysicalSpaceIDAndAddress(Entry.PhysicalSpaceID, Entry.pAddress);

			SortedI++;
			AddI++;
		}
	}

	SortedSubIndexes.Reset();
	SortedAddIndexes.Reset();

	SortedKeysDirty = false;
}

void FTexturePageMap::ReleaseUnmappedPages()
{
	uint32 PageIndex = Pages[PageListHead_Unmapped].NextIndex;
	uint32 CheckUnmappedCount = 0u;
	while (PageIndex != PageListHead_Unmapped)
	{
		FPageEntry& Entry = Pages[PageIndex];
		const uint32 NextPageIndex = Entry.NextIndex;
		Entry.Page.Packed = ~0u;
		Entry.Packed = ~0u;
		RemovePageFromList(PageIndex);
		AddPageToList(PageListHead_Free, PageIndex);
		PageIndex = NextPageIndex;
		++CheckUnmappedCount;
	}
	check(Pages[PageListHead_Unmapped].NextIndex == PageListHead_Unmapped);
}

// Binary search lower bound
// Similar to std::lower_bound
// Range [Min,Max)
uint32 FTexturePageMap::LowerBound(uint32 Min, uint32 Max, uint32 SearchKey, uint32 Mask) const
{
	while (Min != Max)
	{
		uint32 Mid = Min + (Max - Min) / 2;
		uint32 Key = SortedKeys[Mid] & Mask;

		if (SearchKey <= Key)
			Max = Mid;
		else
			Min = Mid + 1;
	}

	return Min;
}

// Binary search upper bound
// Similar to std::upper_bound
// Range [Min,Max)
uint32 FTexturePageMap::UpperBound(uint32 Min, uint32 Max, uint32 SearchKey, uint32 Mask) const
{
	while (Min != Max)
	{
		uint32 Mid = Min + (Max - Min) / 2;
		uint32 Key = SortedKeys[Mid] & Mask;

		if (SearchKey < Key)
			Max = Mid;
		else
			Min = Mid + 1;
	}

	return Min;
}

// Binary search equal range
// Similar to std::equal_range
// Range [Min,Max)
uint64 FTexturePageMap::EqualRange(uint32 Min, uint32 Max, uint32 SearchKey, uint32 Mask) const
{
	while (Min != Max)
	{
		uint32 Mid = Min + (Max - Min) / 2;
		uint32 Key = SortedKeys[Mid] & Mask;

		if (SearchKey < Key)
		{
			Max = Mid;
		}
		else if (SearchKey > Key)
		{
			Min = Mid + 1;
		}
		else
		{	// Range straddles Mid. Search both sides and return.
			Min = LowerBound(Min, Mid, SearchKey, Mask);
			Max = UpperBound(Mid + 1, Max, SearchKey, Mask);
			return Min | ((uint64)Max << 32);
		}
	}

	return 0;
}

void FTexturePageMap::RefreshEntirePageTable(FVirtualTextureSystem* System, TArray< FPageTableUpdate >* Output)
{
	if (SortedKeysDirty)
	{
		BuildSortedKeys();
	}

	for (int i = SortedKeys.Num() - 1; i >= 0; i--)
	{
		FPageTableUpdate Update;
		DecodeSortKey(SortedKeys[i], Update.vLevel, Update.vAddress);
		const FPhysicalSpaceIDAndAddress& PhysicalAddress = SortedAddresses[i];
		const FVirtualTexturePhysicalSpace* PhysicalSpace = System->GetPhysicalSpace(PhysicalAddress.PhysicalSpaceID);
		Update.pTileLocation = PhysicalSpace->GetPhysicalLocation(PhysicalAddress.pAddress);
		Update.vLogSize = Update.vLevel;

		for (int Mip = Update.vLevel; Mip >= 0; Mip--)
		{
			Output[Mip].Add(Update);
		}
	}

	ReleaseUnmappedPages();
}

/*
======================
Update entry in page table for this page and entries for all of its unmapped descendants.

If no mapped descendants then this is a single square per mip.
If there are mapped descendants then draw those on top using painters algorithm.
Outputs list of FPageTableUpdate which will be drawn on the GPU to the page table.
======================
*/
void FTexturePageMap::ExpandPageTableUpdatePainters(FVirtualTextureSystem* System, FPageTableUpdate Update, TArray< FPageTableUpdate >* Output)
{
	if (SortedKeysDirty)
	{
		BuildSortedKeys();
	}

	static TArray< FPageTableUpdate > LoopOutput;

	LoopOutput.Reset();

	uint8  vLogSize = Update.vLogSize;
	uint32 vAddress = Update.vAddress;

	Output[vLogSize].Add(Update);

	// Start with input quad
	LoopOutput.Add(Update);

	uint32 SearchRange = SortedKeys.Num();

	for (uint32 Mip = vLogSize; Mip > 0; )
	{
		Mip--;
		uint32 SearchKey = EncodeSortKey(Mip, vAddress);
		uint32 Mask = ~0u << (vDimensions * vLogSize);

		uint64 DescendantRange = EqualRange(0, SearchRange, SearchKey, Mask);
		if (DescendantRange != 0)
		{
			uint32 DescendantMin = (uint32)DescendantRange;
			uint32 DescendantMax = DescendantRange >> 32;

			// List is sorted by level so lower levels must be earlier in the list than what we found.
			SearchRange = DescendantMin;

			for (uint32 DescendantIndex = DescendantMin; DescendantIndex < DescendantMax; DescendantIndex++)
			{
				checkSlow(SearchKey == (SortedKeys[DescendantIndex] & Mask));

				FPageTableUpdate Descendant;
				uint8 Descendant_Level;
				DecodeSortKey(SortedKeys[DescendantIndex], Descendant_Level, Descendant.vAddress);
				const FPhysicalSpaceIDAndAddress& PhysicalAddress = SortedAddresses[DescendantIndex];
				const FVirtualTexturePhysicalSpace* PhysicalSpace = System->GetPhysicalSpace(PhysicalAddress.PhysicalSpaceID);
				Descendant.pTileLocation = PhysicalSpace->GetPhysicalLocation(PhysicalAddress.pAddress);

				Descendant.vLevel = Mip;
				Descendant.vLogSize = Mip;
				checkSlow(Descendant_Level == Mip);

				// Mask out low bits
				uint32 Ancestor_vAddress = Descendant.vAddress & (~0u << (vDimensions * vLogSize));
				checkSlow(Ancestor_vAddress == vAddress);

				LoopOutput.Add(Descendant);
			}
		}

		Output[Mip].Append(LoopOutput);
	}

	ReleaseUnmappedPages();
}

/*
======================
Update entry in page table for this page and entries for all of its unmapped descendants.

If no mapped descendants then this is a single square per mip.
If there are mapped descendants then break it up into many squares in quadtree order with holes for any already mapped pages.
Outputs list of FPageTableUpdate which will be drawn on the GPU to the page table.
======================
*/
void FTexturePageMap::ExpandPageTableUpdateMasked(FVirtualTextureSystem* System, FPageTableUpdate Update, TArray< FPageTableUpdate >* Output)
{
	if (SortedKeysDirty)
	{
		BuildSortedKeys();
	}

	static TArray< FPageTableUpdate > LoopInput;
	static TArray< FPageTableUpdate > LoopOutput;
	static TArray< FPageTableUpdate > Stack;

	LoopInput.Reset();
	LoopOutput.Reset();
	checkSlow(Stack.Num() == 0);

	uint8  vLogSize = Update.vLogSize;
	uint32 vAddress = Update.vAddress;

	Output[vLogSize].Add(FPageTableUpdate(Update));

	// Start with input quad
	LoopOutput.Add(Update);

	uint32 SearchRange = SortedKeys.Num();

	for (uint32 Mip = vLogSize; Mip > 0; )
	{
		Mip--;
		uint32 SearchKey = EncodeSortKey(Mip, vAddress);
		uint32 Mask = ~0u << (vDimensions * vLogSize);

		uint64 DescendantRange = EqualRange(0, SearchRange, SearchKey, Mask);
		if (DescendantRange != 0)
		{
			uint32 DescendantMin = (uint32)DescendantRange;
			uint32 DescendantMax = DescendantRange >> 32;

			// List is sorted by level so lower levels must be earlier in the list than what we found.
			SearchRange = DescendantMin;

			// Ping-pong input and output
			Exchange(LoopInput, LoopOutput);
			LoopOutput.Reset();
			int32 InputIndex = 0;

			Update = LoopInput[InputIndex++];

			for (uint32 DescendantIndex = DescendantMin; DescendantIndex < DescendantMax; )
			{
				checkSlow(SearchKey == (SortedKeys[DescendantIndex] & Mask));

				FPageTableUpdate Descendant;
				uint8 Descendant_Level;
				DecodeSortKey(SortedKeys[DescendantIndex], Descendant_Level, Descendant.vAddress);
				const FPhysicalSpaceIDAndAddress& PhysicalAddress = SortedAddresses[DescendantIndex];
				const FVirtualTexturePhysicalSpace* PhysicalSpace = System->GetPhysicalSpace(PhysicalAddress.PhysicalSpaceID);
				Descendant.pTileLocation = PhysicalSpace->GetPhysicalLocation(PhysicalAddress.pAddress);

				Descendant.vLevel = Mip;
				Descendant.vLogSize = Mip;
				checkSlow(Descendant_Level == Mip);

				// Mask out low bits
				uint32 Ancestor_vAddress = Descendant.vAddress & (~0u << (vDimensions * vLogSize));
				checkSlow(Ancestor_vAddress == vAddress);

				uint32 UpdateSize = 1 << (vDimensions * Update.vLogSize);
				uint32 DescendantSize = 1 << (vDimensions * Descendant.vLogSize);

				checkSlow(Update.vLogSize >= Mip);

				Update.Check(vDimensions);
				Descendant.Check(vDimensions);

				// Find if Update intersects with Descendant

				// Is update past descendant?
				if (Update.vAddress > Descendant.vAddress)
				{
					checkSlow(Update.vAddress >= Descendant.vAddress + DescendantSize);
					// Move to next descendant
					DescendantIndex++;
					continue;
				}
				// Is update quad before descendant quad and doesn't intersect?
				else if (Update.vAddress + UpdateSize <= Descendant.vAddress)
				{
					// Output this update and fetch next
					LoopOutput.Add(Update);
				}
				// Does update quad equal descendant quad?
				else if (Update.vAddress == Descendant.vAddress &&
					Update.vLogSize == Descendant.vLogSize)
				{
					// Move to next descendant
					DescendantIndex++;
					// Toss this update and fetch next
				}
				else
				{
					checkSlow(Update.vLogSize > Mip);

					// Update intersects with Descendant but isn't the same size
					// Split update into 4 for 2D, 8 for 3D
					Update.vLogSize--;
					for (uint32 Sibling = (1 << vDimensions) - 1; Sibling > 0; Sibling--)
					{
						Stack.Push(FPageTableUpdate(Update, Sibling, vDimensions));
					}
					continue;
				}

				// Fetch next update
				if (Stack.Num())
				{
					Update = Stack.Pop(EAllowShrinking::No);
				}
				else if (InputIndex < LoopInput.Num())
				{
					Update = LoopInput[InputIndex++];
				}
				else
				{
					// No more input
					Update.vLogSize = 0xff;
					break;
				}
			}

			// If update was still being worked with add it
			if (Update.vLogSize != 0xff)
			{
				LoopOutput.Add(Update);
			}
			// Add remaining stack to output
			while (Stack.Num())
			{
				LoopOutput.Add(Stack.Pop(EAllowShrinking::No));
			}
			// Add remaining input to output
			LoopOutput.Append(LoopInput.GetData() + InputIndex, LoopInput.Num() - InputIndex);
		}

		if (LoopOutput.Num() == 0)
		{
			// Completely masked out by descendants
			break;
		}
		else
		{
			Output[Mip].Append(LoopOutput);
		}
	}

	ReleaseUnmappedPages();
}
