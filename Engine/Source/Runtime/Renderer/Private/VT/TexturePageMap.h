// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTextureShared.h"
#include "Containers/BinaryHeap.h"
#include "Containers/HashTable.h"

class FVirtualTextureSystem;
class FVirtualTextureSpace;
class FVirtualTexturePhysicalSpace;

// 4k x 4k virtual pages
// 256 x 256 physical pages
union FTexturePage
{
	FTexturePage() : Packed(~0u) {}
	FTexturePage(uint8 InLogSize, uint32 InAddress) : vAddress(InAddress), vLogSize(InLogSize) {}

	uint32 Packed;
	struct
	{
		// Address is Morton order, relative to mip 0
		uint32 vAddress : 24;
		uint32 vLogSize : 8;
	};
};
static_assert(sizeof(FTexturePage) == sizeof(uint32), "Bad packing");
FORCEINLINE bool operator==(const FTexturePage& Lhs, const FTexturePage& Rhs) { return Lhs.Packed == Rhs.Packed; }
FORCEINLINE bool operator!=(const FTexturePage& Lhs, const FTexturePage& Rhs) { return Lhs.Packed != Rhs.Packed; }

union FPhysicalSpaceIDAndAddress
{
	FPhysicalSpaceIDAndAddress() : Packed(~0u) {}
	FPhysicalSpaceIDAndAddress(uint16 InPhysicalSpaceID, uint16 InAddress) : PhysicalSpaceID(InPhysicalSpaceID), pAddress(InAddress) {}
	uint32 Packed;
	struct 
	{
		uint16 PhysicalSpaceID;
		uint16 pAddress;
	};
};
FORCEINLINE bool operator==(const FPhysicalSpaceIDAndAddress& Lhs, const FPhysicalSpaceIDAndAddress& Rhs) { return Lhs.Packed == Rhs.Packed; }
FORCEINLINE bool operator!=(const FPhysicalSpaceIDAndAddress& Lhs, const FPhysicalSpaceIDAndAddress& Rhs) { return Lhs.Packed != Rhs.Packed; }

struct FMappedTexturePage
{
	FTexturePage Page;
	uint32 pAddress : 16;
	uint32 PhysicalSpaceID : 12;
	uint32 Local_vLevel : 4;
};

/**
 * Manages a single layer of a VT page table, contains mappings of virtual->physical address
 * Pages should not be directly mapped/unmapped from this class, this should instead go through FTexturePagePool
 * In the context of page mappings, vLogSize and vLevel refer to 2 similar but slightly different things
 * - vLogSize is the mip level of the virtual address space being mapped (from the allocated VT)
 * - vLevel is the mip level of the producer that's being mapped (somethings called Local_vLevel)
 * - These are often the same value, but can be different in certain situations.
 *   For example when unmapping a page, the ancestor page with a higher vLevel is mapped to the same address at vLogSize
 *   When different layers have different sizes, mip bias will cause lower vLevel pages to be mapped to address at vLogSize
 */
class FTexturePageMap
{
public:
	FTexturePageMap();
	~FTexturePageMap();

	void Initialize(uint32 InSize, uint32 InLayerIndex, uint32 InDimensions);

	uint32		GetSize() const { return Pages.Num(); }

	/**
	 * Find the physical address for the given virtual address.
	 * Returns ~0u if not found.
	 */
	uint32		FindPageAddress(uint8 vLogSize, uint32 vAddress) const;

	FPhysicalSpaceIDAndAddress FindPagePhysicalSpaceIDAndAddress(const FTexturePage& CheckPage, uint16 Hash) const;

	FPhysicalSpaceIDAndAddress FindPagePhysicalSpaceIDAndAddress(uint8 vLogSize, uint32 vAddress) const;

	/**
	* Find the best matching physical address along the mipmap fall back chain for the given virtual address.
	* Returns ~0u if none found at all.
	*/
	uint32		FindNearestPageAddress(uint8 vLogSize, uint32 vAddress) const;

	uint32		FindNearestPageLevel(uint8 vLogSize, uint32 vAddress) const;

	/**
	 * Unmap the physical address from any virtual address it was mapped to before.
	 */
	void		UnmapPage(FVirtualTextureSystem* System, FVirtualTextureSpace* Space, uint8 vLogSize, uint32 vAddress, bool bMapAncestorPage);

	/**
	* Map the physical address to a specific virtual address.
	*/
	void		MapPage(FVirtualTextureSpace* Space, FVirtualTexturePhysicalSpace* PhysicalSpace, uint32 PackedProducerHandle, uint8 MaxLevel, uint8 vLogSize, uint32 vAddress, uint8 Local_vLevel, uint16 pAddress);

	void		GetMappedPagesInRange(uint32 vAddress, uint32 Width, uint32 Height, TArray<FMappedTexturePage>& OutMappedPages) const;

	void		RefreshEntirePageTable(FVirtualTextureSystem* System, TArray< FPageTableUpdate >* Output);
	void		ExpandPageTableUpdatePainters(FVirtualTextureSystem* System, FPageTableUpdate Update, TArray< FPageTableUpdate >* Output);
	void		ExpandPageTableUpdateMasked(FVirtualTextureSystem* System, FPageTableUpdate Update, TArray< FPageTableUpdate >* Output);

	void		InvalidateUnmappedRootPage(FVirtualTextureSpace* Space, FVirtualTexturePhysicalSpace* PhysicalSpace, uint32 PackedProducerHandle, uint8 MaxLevel, uint8 vLogSize, uint32 vAddress, uint8 Local_vLevel);

private:
	void		BuildSortedKeys();
	void        ReleaseUnmappedPages();

	uint32		LowerBound(uint32 Min, uint32 Max, uint32 SearchKey, uint32 Mask) const;
	uint32		UpperBound(uint32 Min, uint32 Max, uint32 SearchKey, uint32 Mask) const;
	uint64		EqualRange(uint32 Min, uint32 Max, uint32 SearchKey, uint32 Mask) const;

	uint32		FindPageIndex(uint8 vLogSize, uint32 vAddress) const;
	uint32		FindNearestPageIndex(uint8 vLogSize, uint32 vAddress, uint8 MaxLevel) const;

	uint32		LayerIndex;
	uint32		vDimensions;

	enum EPageListHead
	{
		PageListHead_Free,
		PageListHead_Mapped,
		PageListHead_Unmapped,

		PageListHead_Count,
	};

	struct FPageEntry
	{
		FTexturePage Page;
		uint32 NextIndex;
		uint32 PrevIndex;
		union
		{
			uint64 Packed;
			struct 
			{
				uint32 PackedProducerHandle;
				uint32 pAddress : 16;
				uint32 PhysicalSpaceID : 8;
				uint32 MaxLevel : 4;
				uint32 Local_vLevel : 4;
			};
		};
	};

	void RemovePageFromList(uint32 Index)
	{
		FPageEntry& Page = Pages[Index];
		Pages[Page.PrevIndex].NextIndex = Page.NextIndex;
		Pages[Page.NextIndex].PrevIndex = Page.PrevIndex;
		Page.NextIndex = Page.PrevIndex = Index;
	}

	void AddPageToList(uint32 HeadIndex, uint32 Index)
	{
		FPageEntry& Head = Pages[HeadIndex];
		FPageEntry& Page = Pages[Index];
		check(Index >= PageListHead_Count); // make sure we're not trying to add a list head to another list

		// make sure we're not currently in any list
		check(Page.NextIndex == Index);
		check(Page.PrevIndex == Index);

		Page.NextIndex = HeadIndex;
		Page.PrevIndex = Head.PrevIndex;
		Pages[Head.PrevIndex].NextIndex = Index;
		Head.PrevIndex = Index;
	}

	uint32 AcquirePage()
	{
		FPageEntry& FreeHead = Pages[PageListHead_Free];
		uint32 Index = FreeHead.NextIndex;
		if (Index != 0u)
		{
			RemovePageFromList(Index);
			return Index;
		}

		Index = Pages.AddDefaulted();
		FPageEntry& Page = Pages[Index];
		Page.NextIndex = Page.PrevIndex = Index;
		return Index;
	}

	TArray<FPageEntry>	Pages;
	FHashTable			HashTable;
	uint32				MappedPageCount;

	TArray< uint32 >						UnsortedKeys;
	TArray< FPhysicalSpaceIDAndAddress >	UnsortedAddresses;
	TArray< uint32 >						SortedKeys;
	TArray< FPhysicalSpaceIDAndAddress >	SortedAddresses;
	bool									SortedKeysDirty;

	TArray< uint32 >	SortedSubIndexes;
	TArray< uint64 >	SortedAddIndexes;
};

inline FPhysicalSpaceIDAndAddress FTexturePageMap::FindPagePhysicalSpaceIDAndAddress(const FTexturePage& CheckPage, uint16 Hash) const
{
	for (uint32 PageIndex = HashTable.First(Hash); HashTable.IsValid(PageIndex); PageIndex = HashTable.Next(PageIndex))
	{
		const FPageEntry& Entry = Pages[PageIndex];
		if (Entry.Page == CheckPage)
		{
			return FPhysicalSpaceIDAndAddress(Entry.PhysicalSpaceID, Entry.pAddress);
		}
	}
	return FPhysicalSpaceIDAndAddress();
}
