// Copyright Epic Games, Inc. All Rights Reserved.

#include "TexturePagePool.h"

#include "VirtualTextureSpace.h"
#include "VirtualTextureSystem.h"
#include "VirtualTextureEnum.h"

// Reserve pAddress=0 to indicate unmapped page table entry.
const uint32 FTexturePagePool::NumReservedPages = 1u;

FTexturePagePool::FTexturePagePool()
	: PageHash(16u * 1024)
	, NumPages(0u)
	, NumPagesMapped(0u)
	, NumPagesAllocated(NumReservedPages)
{
}

FTexturePagePool::~FTexturePagePool()
{}

void FTexturePagePool::Initialize(uint32 InNumPages)
{
	NumPages = InNumPages;
	Pages.AddZeroed(InNumPages);
	PageHash.Resize(InNumPages);

	FreeHeap.Resize(InNumPages, InNumPages);

	for (uint32 i = NumReservedPages; i < InNumPages; i++)
	{
		FreeHeap.Add(0, i);
	}

	// Initialize list head for each page, plus one for free list
	PageMapping.AddUninitialized(InNumPages + 1u);
	for (uint32 i = 0; i <= InNumPages; i++)
	{
		FPageMapping& Mapping = PageMapping[i];
		FMemory::Memset(Mapping, 0xff);
		Mapping.NextIndex = Mapping.PrevIndex = i;
	}
}

void FTexturePagePool::EvictAllPages(FVirtualTextureSystem* System)
{
	TArray<uint16> PagesToEvict;
	while (FreeHeap.Num() > 0u)
	{
		const uint16 pAddress = FreeHeap.Top();
		FreeHeap.Pop();
		PagesToEvict.Add(pAddress);
	}

	for (int32 i = 0; i < PagesToEvict.Num(); i++)
	{
		UnmapAllPages(System, PagesToEvict[i], true);
		FreeHeap.Add(0, PagesToEvict[i]);
	}
}

void FTexturePagePool::UnmapAllPagesForSpace(FVirtualTextureSystem* System, uint8 SpaceID, uint32 vAddress, uint32 Width, uint32 Height, uint32 MaxLevel)
{
	check(Width > 0u);
	check(Height > 0u);
	checkf((vAddress & (0xffffffff << (MaxLevel * 2u))) == vAddress, TEXT("vAddress %08X is not aligned to max level %d"), vAddress, MaxLevel);

	const uint32 vTileX0 = FMath::ReverseMortonCode2(vAddress);
	const uint32 vTileY0 = FMath::ReverseMortonCode2(vAddress >> 1);
	const uint32 vTileX1 = vTileX0 + Width;
	const uint32 vTileY1 = vTileY0 + Height;
	const uint32 vAddressMax = FMath::MortonCode2(vTileX1) | (FMath::MortonCode2(vTileY1) << 1);

	// walk through all of our current mapping entries, and unmap any that belong to the current space
	for (int32 MappingIndex = NumPages + 1; MappingIndex < PageMapping.Num(); ++MappingIndex)
	{
		const FPageMapping& Mapping = PageMapping[MappingIndex];
		if (Mapping.PageTableLayerIndex != 0xff &&
			Mapping.SpaceID == SpaceID &&
			Mapping.vLogSize <= MaxLevel &&
			Mapping.vAddress >= vAddress &&
			Mapping.vAddress < vAddressMax)
		{
			check((Mapping.vAddress & (0xffffffff << (Mapping.vLogSize * 2u))) == Mapping.vAddress);

			const uint32 vTileX = FMath::ReverseMortonCode2(Mapping.vAddress);
			const uint32 vTileY = FMath::ReverseMortonCode2(Mapping.vAddress >> 1);

			if (vTileX >= vTileX0 &&
				vTileY >= vTileY0 &&
				vTileX < vTileX1 &&
				vTileY < vTileY1)
			{
				// MaxLevel should match, if this mapping actually belongs to the region we're trying to unmap
				check(Mapping.MaxLevel == MaxLevel);
				// we're unmapping all pages for space, so don't try to map any ancestor pages...they'll be unmapped as well
				UnmapPageMapping(System, MappingIndex, false);
			}
		}
	}
}

void FTexturePagePool::EvictPages(FVirtualTextureSystem* System, const FVirtualTextureProducerHandle& ProducerHandle)
{
	TArray<uint32, TInlineAllocator<256>> ToEvict;
	const uint32 Hash = MurmurFinalize32(ProducerHandle.PackedValue);
	for (uint32 pAddress = ProducerToPageIndex.First(Hash); ProducerToPageIndex.IsValid(pAddress); pAddress = ProducerToPageIndex.Next(pAddress))
	{
		const FPageEntry& PageEntry = Pages[pAddress];
		if (PageEntry.PackedProducerHandle == ProducerHandle.PackedValue)
		{
			ToEvict.Add(pAddress);
		}
	}

	for (uint32 pAddress : ToEvict)
	{
		UnmapAllPages(System, pAddress, false);
		checkf(FreeHeap.IsPresent(pAddress), TEXT("If the address is present in the FreeHeap already, then there's an issue because we expect all locked pages to have been unlocked earlier by a call to ForceUnlockAllTiles"));
		FreeHeap.Update(0, pAddress);
	}
}

void FTexturePagePool::EvictPages(
	FVirtualTextureSystem* System, 
	FVirtualTextureProducerHandle const& ProducerHandle, 
	FVTProducerDescription const& Desc, 
	FIntRect const& TextureRegion, 
	uint32 MaxLevelToEvict, 
	uint32 MinFrameToKeepMapped,
	EVTInvalidatePriority InvalidatePriority,
	TSet<FVirtualTextureLocalTileRequest>& OutDirtyMapped)
{
	TArray<uint32, TInlineAllocator<256>> ToEvict;
	const uint32 Hash = MurmurFinalize32(ProducerHandle.PackedValue);
	for (uint32 pAddress = ProducerToPageIndex.First(Hash); ProducerToPageIndex.IsValid(pAddress); pAddress = ProducerToPageIndex.Next(pAddress))
	{
		const FPageEntry& PageEntry = Pages[pAddress];
		if (PageEntry.PackedProducerHandle == ProducerHandle.PackedValue)
		{
			const uint32 vAddress = Pages[pAddress].Local_vAddress;
			const uint32 vLevel = Pages[pAddress].Local_vLevel;

			if (vLevel <= MaxLevelToEvict)
			{
				const int32 TileSize = Desc.TileSize << vLevel;
				const int32 X = FMath::ReverseMortonCode2(vAddress) * TileSize;
				const int32 Y = FMath::ReverseMortonCode2(vAddress >> 1) * TileSize;
				const int32 TileBorderSize = Desc.TileBorderSize << vLevel;
				const FIntRect PageRect(X - TileBorderSize, Y - TileBorderSize, X + TileSize + TileBorderSize, Y + TileSize + TileBorderSize);

				if (!(PageRect.Min.X > TextureRegion.Max.X) && !(TextureRegion.Min.X > PageRect.Max.X) && !(PageRect.Min.Y > TextureRegion.Max.Y) && !(TextureRegion.Min.Y > PageRect.Max.Y))
				{
					if (!FreeHeap.IsPresent(pAddress))
					{
						// Locked pages aren't unmapped but are added to the dirty output array.
						AddOrMergeTileRequest(FVirtualTextureLocalTileRequest(FVirtualTextureLocalTile(ProducerHandle, vAddress, vLevel), Desc.Priority, InvalidatePriority), OutDirtyMapped);
					}
					else
					{
						const uint32 PageFrame = FreeHeap.GetKey(pAddress) >> 4;
						if (PageFrame >= MinFrameToKeepMapped)
						{
							// Visible pages aren't unmapped but are added to the dirty output array.
							AddOrMergeTileRequest(FVirtualTextureLocalTileRequest(FVirtualTextureLocalTile(ProducerHandle, vAddress, vLevel), Desc.Priority, InvalidatePriority), OutDirtyMapped);
						}
						else
						{
							ToEvict.Add(pAddress);
						}
					}
				}
			}
		}
	}

	for (uint32 pAddress : ToEvict)
	{
		UnmapAllPages(System, pAddress, true);
		FreeHeap.Update(0, pAddress);
	}
}

void FTexturePagePool::GetAllLockedPages(FVirtualTextureSystem* System, TSet<FVirtualTextureLocalTile>& OutPages)
{
	OutPages.Reserve(OutPages.Num() + GetNumLockedPages());

	for (uint32 i = NumReservedPages; i < NumPages; ++i)
	{
		if (!FreeHeap.IsPresent(i))
		{
			OutPages.Add(FVirtualTextureLocalTile(FVirtualTextureProducerHandle(Pages[i].PackedProducerHandle), Pages[i].Local_vAddress, Pages[i].Local_vLevel));
		}
	}
}

FVirtualTextureLocalTile FTexturePagePool::GetLocalTileFromPhysicalAddress(uint16 pAddress)
{
	return FVirtualTextureLocalTile(FVirtualTextureProducerHandle(Pages[pAddress].PackedProducerHandle), Pages[pAddress].Local_vAddress, Pages[pAddress].Local_vLevel);
}

bool FTexturePagePool::AnyFreeAvailable( uint32 Frame, uint32 FreeThreshold) const
{
	if( FreeHeap.Num() > 0 )
	{
		// Keys include vLevel to help prevent parent before child ordering
		const uint16 pAddress = FreeHeap.Top();
		const uint32 PageFrame = FreeHeap.GetKey(pAddress) >> 4;
		// Don't free any pages that were touched this frame
		return PageFrame + FreeThreshold < Frame;
	}

	return false;
}

uint16 FTexturePagePool::GetPageHash(const FPageEntry& Entry)
{
	return (uint16)MurmurFinalize64(Entry.PackedValue);
}

uint32 FTexturePagePool::FindPageAddress(const FVirtualTextureProducerHandle& ProducerHandle, uint8 GroupIndex, uint32 Local_vAddress, uint8 Local_vLevel) const
{
	FPageEntry CheckPage;
	CheckPage.PackedProducerHandle = ProducerHandle.PackedValue;
	CheckPage.Local_vAddress = Local_vAddress;
	CheckPage.Local_vLevel = Local_vLevel;
	CheckPage.GroupIndex = GroupIndex;

	const uint16 Hash = GetPageHash(CheckPage);
	for (uint32 PageIndex = PageHash.First(Hash); PageHash.IsValid(PageIndex); PageIndex = PageHash.Next(PageIndex))
	{
		const FPageEntry& PageEntry = Pages[PageIndex];
		if (PageEntry.PackedValue == CheckPage.PackedValue)
		{
			return PageIndex;
		}
	}

	return ~0u;
}

uint32 FTexturePagePool::FindNearestPageAddress(const FVirtualTextureProducerHandle& ProducerHandle, uint8 GroupIndex, uint32 Local_vAddress, uint8 Local_vLevel, uint8 MaxLevel) const
{
	while (Local_vLevel <= MaxLevel)
	{
		const uint32 pAddress = FindPageAddress(ProducerHandle, GroupIndex, Local_vAddress, Local_vLevel);
		if (pAddress != ~0u)
		{
			return pAddress;
		}

		++Local_vLevel;
		Local_vAddress >>= 2;
	}
	return ~0u;
}

uint32 FTexturePagePool::FindNearestPageLevel(const FVirtualTextureProducerHandle& ProducerHandle, uint8 GroupIndex, uint32 Local_vAddress, uint8 Local_vLevel) const
{
	while (Local_vLevel < 16u)
	{
		const uint32 pAddress = FindPageAddress(ProducerHandle, GroupIndex, Local_vAddress, Local_vLevel);
		if (pAddress != ~0u)
		{
			return Pages[pAddress].Local_vLevel;
		}

		++Local_vLevel;
		Local_vAddress >>= 2;
	}
	return ~0u;
}

uint32 FTexturePagePool::Alloc(FVirtualTextureSystem* System, uint32 Frame, const FVirtualTextureProducerHandle& ProducerHandle, uint8 GroupIndex, uint32 Local_vAddress, uint8 Local_vLevel, bool bLock)
{
	check(ProducerHandle.PackedValue != 0u);
	checkSlow(FindPageAddress(ProducerHandle, GroupIndex, Local_vAddress, Local_vLevel) == ~0u);

	// Grab the LRU free page
	const uint16 pAddress = FreeHeap.Top();
	FPageEntry& PageEntry = Pages[pAddress];

	// If the LRU page is allocated, that means the pool must be 100% allocated
	check(PageEntry.PackedProducerHandle == 0u || NumPagesAllocated == NumPages);

	// Unmap any previous usage
	UnmapAllPages(System, pAddress, true);

	// Mark the page as used for the given producer
	PageEntry.PackedProducerHandle = ProducerHandle.PackedValue;
	PageEntry.Local_vAddress = Local_vAddress;
	PageEntry.Local_vLevel = Local_vLevel;
	PageEntry.GroupIndex = GroupIndex;
	PageHash.Add(GetPageHash(PageEntry), pAddress);
	ProducerToPageIndex.Add(MurmurFinalize32(ProducerHandle.PackedValue), pAddress);

	if (bLock)
	{
		FreeHeap.Pop();
	}
	else
	{
		FreeHeap.Update((Frame << 4) + (Local_vLevel & 0xf), pAddress);
	}

	++NumPagesAllocated;
	check(NumPagesAllocated <= NumPages);

	return pAddress;
}

void FTexturePagePool::Free(FVirtualTextureSystem* System, uint16 pAddress)
{
	UnmapAllPages(System, pAddress, true);

	if (FreeHeap.IsPresent(pAddress))
	{
		FreeHeap.Update(0u, pAddress);
	}
	else
	{
		FreeHeap.Add(0u, pAddress);
	}
}

void FTexturePagePool::Unlock(uint32 Frame, uint16 pAddress)
{
	const FPageEntry& PageEntry = Pages[pAddress];
	FreeHeap.Add((Frame << 4) + PageEntry.Local_vLevel, pAddress);
}

void FTexturePagePool::Lock(uint16 pAddress)
{
	// 'Remove' checks IsPresent(), so this will be a nop if address is already locked
	FreeHeap.Remove(pAddress);
}

void FTexturePagePool::UpdateUsage(uint32 Frame, uint16 pAddress)
{
	if (FreeHeap.IsPresent(pAddress))
	{
		const FPageEntry& PageEntry = Pages[pAddress];
		FreeHeap.Update((Frame << 4) + PageEntry.Local_vLevel, pAddress);
	}
}

uint32 FTexturePagePool::GetNumVisiblePages(uint32 Frame) const
{
	uint32 Count = 0;
	for (uint32 i = NumReservedPages; i < NumPages; ++i)
	{
		if (FreeHeap.IsPresent(i))
		{
			uint32 Key = FreeHeap.GetKey(i);
			if ((Key >> 4) > Frame)
			{
				Count ++;
			}
		}
		else
		{
			// Consider all locked pages as visible
			Count++;
		}
	}

	return Count;
}

void FTexturePagePool::CollectProducerCounts(TMap<uint32, FUintVector2>& OutProducerCountMap) const
{
	for (uint32 i = NumReservedPages; i < NumPages; ++i)
	{
		const uint32 PackedProducerHandle = Pages[i].PackedProducerHandle;
		if (PackedProducerHandle != 0u)
		{
			const uint32 IsLocked = FreeHeap.IsPresent(i) ? 0u : 1u;
			OutProducerCountMap.FindOrAdd(PackedProducerHandle) += FUintVector2(1u, IsLocked);
		}
	}
}


void FTexturePagePool::MapPage(FVirtualTextureSpace* Space, FVirtualTexturePhysicalSpace* PhysicalSpace, uint8 PageTableLayerIndex, uint8 MaxLevel, uint8 vLogSize, uint32 vAddress, uint8 Local_vLevel, uint16 pAddress)
{
	check(pAddress >= NumReservedPages);
	check(pAddress < NumPages);
	const FPageEntry& PageEntry = Pages[pAddress];
	checkf(PageEntry.PackedProducerHandle != 0u, TEXT("Trying to map pAddress %04x that hasn't been allocated"), pAddress);

	FTexturePageMap& PageMap = Space->GetPageMapForPageTableLayer(PageTableLayerIndex);
	PageMap.MapPage(Space, PhysicalSpace, PageEntry.PackedProducerHandle, MaxLevel, vLogSize, vAddress, Local_vLevel, pAddress);

	++NumPagesMapped;

	const uint32 MappingIndex = AcquireMapping();
	AddMappingToList(pAddress, MappingIndex);
	FPageMapping& Mapping = PageMapping[MappingIndex];
	Mapping.SpaceID = Space->GetID();
	Mapping.vAddress = vAddress;
	Mapping.vLogSize = vLogSize;
	Mapping.MaxLevel = MaxLevel;
	Mapping.PageTableLayerIndex = PageTableLayerIndex;
}

void FTexturePagePool::UnmapPageMapping(FVirtualTextureSystem* System, uint32 MappingIndex, bool bMapAncestorPage)
{
	FPageMapping& Mapping = PageMapping[MappingIndex];
	FVirtualTextureSpace* Space = System->GetSpace(Mapping.SpaceID);
	FTexturePageMap& PageMap = Space->GetPageMapForPageTableLayer(Mapping.PageTableLayerIndex);

	PageMap.UnmapPage(System, Space, Mapping.vLogSize, Mapping.vAddress, bMapAncestorPage);

	check(NumPagesMapped > 0u);
	--NumPagesMapped;

	Mapping.vAddress = 0x00ffffff;
	Mapping.vLogSize = 0x0f;
	Mapping.SpaceID = 0x0f;
	Mapping.MaxLevel = 0x0f;
	Mapping.PageTableLayerIndex = 0xff;

	ReleaseMapping(MappingIndex);
}

void FTexturePagePool::UnmapAllPages(FVirtualTextureSystem* System, uint16 pAddress, bool bMapAncestorPages)
{
	FPageEntry& PageEntry = Pages[pAddress];
	if (PageEntry.PackedProducerHandle != 0u)
	{
		check(NumPagesAllocated > NumReservedPages);
		--NumPagesAllocated;
		PageHash.Remove(GetPageHash(PageEntry), pAddress);
		ProducerToPageIndex.Remove(MurmurFinalize32(PageEntry.PackedProducerHandle), pAddress);
		PageEntry.PackedValue = 0u;
	}

	// unmap the page from all of its current mappings
	uint32 MappingIndex = PageMapping[pAddress].NextIndex;
	while (MappingIndex != pAddress)
	{
		FPageMapping& Mapping = PageMapping[MappingIndex];
		const uint32 NextIndex = Mapping.NextIndex;
		UnmapPageMapping(System, MappingIndex, bMapAncestorPages);

		MappingIndex = NextIndex;
	}

	check(PageMapping[pAddress].NextIndex == pAddress); // verify the list is properly empty
}

void FTexturePagePool::RemapPages(
	FVirtualTextureSystem* System, uint8 SpaceID, FVirtualTexturePhysicalSpace* PhysicalSpace,
	FVirtualTextureProducerHandle const& OldProducerHandle, uint32 OldVirtualAddress,
	FVirtualTextureProducerHandle const& NewProducerHandle, uint32 NewVirtualAddress,
	int32 OldAllocatedAdaptiveBias,
	int32 vLevelChange_vAddress, int32 vLevelChange_Producer, uint32 Frame)
{
	const uint32 OldProducerHash = MurmurFinalize32(OldProducerHandle.PackedValue);
	const uint32 NewProducerHash = MurmurFinalize32(NewProducerHandle.PackedValue);

	const uint32 OldBaseX = FMath::ReverseMortonCode2(OldVirtualAddress);
	const uint32 OldBaseY = FMath::ReverseMortonCode2(OldVirtualAddress >> 1);
	const uint32 NewBaseX = FMath::ReverseMortonCode2(NewVirtualAddress);
	const uint32 NewBaseY = FMath::ReverseMortonCode2(NewVirtualAddress >> 1);

	TArray<uint32, TInlineAllocator<256>> ToRemap;
	const uint32 Hash = MurmurFinalize32(OldProducerHandle.PackedValue);
	for (uint32 pAddress = ProducerToPageIndex.First(Hash); ProducerToPageIndex.IsValid(pAddress); pAddress = ProducerToPageIndex.Next(pAddress))
	{
		const FPageEntry& PageEntry = Pages[pAddress];
		if (PageEntry.PackedProducerHandle == OldProducerHandle.PackedValue)
		{
			ToRemap.Add(pAddress);
		}
	}

	for (uint32 pAddress : ToRemap)
	{
		FPageEntry& PageEntry = Pages[pAddress];
		check(PageEntry.PackedProducerHandle == OldProducerHandle.PackedValue);

		if ((int32)PageEntry.Local_vLevel + vLevelChange_Producer < 0 || (int32)PageEntry.Local_vLevel + vLevelChange_vAddress < OldAllocatedAdaptiveBias)
		{
			// Remap removes this level 
			UnmapAllPages(System, pAddress, false);
			// Queue page for recycling
			FreeHeap.Update(0, pAddress);
		}
		else
		{
			// Directly modify page entry for new producer
			PageHash.Remove(GetPageHash(PageEntry), pAddress);
			PageEntry.PackedProducerHandle = NewProducerHandle.PackedValue;
			PageEntry.Local_vLevel += vLevelChange_Producer;
			PageHash.Add(GetPageHash(PageEntry), pAddress);

			ProducerToPageIndex.Remove(OldProducerHash, pAddress);
			ProducerToPageIndex.Add(NewProducerHash, pAddress);

			if (FreeHeap.IsPresent(pAddress))
			{
				FreeHeap.Update((Frame << 4) + PageEntry.Local_vLevel, pAddress);
			}

			// Go through mappings and modify directly.
			uint32 MappingIndex = PageMapping[pAddress].NextIndex;
			while (MappingIndex != pAddress)
			{
				FPageMapping& Mapping = PageMapping[MappingIndex];

				FVirtualTextureSpace* Space = System->GetSpace(Mapping.SpaceID);
				FTexturePageMap& PageMap = Space->GetPageMapForPageTableLayer(Mapping.PageTableLayerIndex);

				PageMap.UnmapPage(System, Space, Mapping.vLogSize, Mapping.vAddress, false);

				const uint32 XLocal = FMath::ReverseMortonCode2(Mapping.vAddress) - OldBaseX;
				const uint32 YLocal = FMath::ReverseMortonCode2(Mapping.vAddress >> 1) - OldBaseY;
				const uint32 X = NewBaseX + ((vLevelChange_vAddress >= 0) ? (XLocal << vLevelChange_vAddress) : (XLocal >> -vLevelChange_vAddress));
				const uint32 Y = NewBaseY + ((vLevelChange_vAddress >= 0) ? (YLocal << vLevelChange_vAddress) : (YLocal >> -vLevelChange_vAddress));

				Mapping.vAddress = FMath::MortonCode2(X) | (FMath::MortonCode2(Y) << 1);
				Mapping.vLogSize = (int32)Mapping.vLogSize + vLevelChange_vAddress;
				Mapping.MaxLevel = (int32)Mapping.MaxLevel + vLevelChange_vAddress;

				const int32 vLevel = PageEntry.Local_vLevel; // Deal with any producer mip bias?
				PageMap.MapPage(Space, PhysicalSpace, NewProducerHandle.PackedValue, Mapping.MaxLevel, Mapping.vLogSize, Mapping.vAddress, vLevel, pAddress);

				MappingIndex = Mapping.NextIndex;
			}
		}
	}
}
