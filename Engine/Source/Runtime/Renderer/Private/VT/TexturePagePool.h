// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTextureShared.h"
#include "Containers/BinaryHeap.h"
#include "Containers/HashTable.h"

union FVirtualTextureLocalTile;
class FVirtualTexturePhysicalSpace;
union FVirtualTextureProducerHandle;
class FVirtualTextureSpace;
class FVirtualTextureSystem;
struct FVTProducerDescription;

/**
 * Manages a pool of texture pages, backed by a large GPU texture atlas.
 * Pages can be allocated for a particular virtual texture, and mapped into any number of virtual pages tables.
 * FTexturePagePool tracks the VT that owns the allocation for each page, and maintains a list of page table mappings for each allocated page.
 * In order to maintain page table mappings, this class works closely with FTexturePageMap, which tracks mappings for a single layer of a given page table
 */
class FTexturePagePool
{
public:
	FTexturePagePool();
	~FTexturePagePool();

	void Initialize(uint32 InNumPages);

	uint32 GetNumPages() const { return FMath::Max(NumPages, NumReservedPages) - NumReservedPages; }
	uint32 GetNumLockedPages() const { return GetNumPages() - FreeHeap.Num(); }
	uint32 GetNumMappedPages() const { return NumPagesMapped; }
	uint32 GetNumAllocatedPages() const { return NumPagesAllocated; }

	/**
	 * Reset the page pool. This can be used to flush any caches.
	 */
	void EvictAllPages(FVirtualTextureSystem* System);

	/**
	 * Unmap/remove any pages that were allocated by the given producer
	 */
	void EvictPages(FVirtualTextureSystem* System, const FVirtualTextureProducerHandle& ProducerHandle);

	/**
	 * Unmap/remove any pages that were allocated by the given producer and are inside the TextureRegion.
	 * Outputs the locked pages that can't be unmapped to the OutLocked array.
	 */
	void EvictPages(FVirtualTextureSystem* System, FVirtualTextureProducerHandle const& ProducerHandle, FVTProducerDescription const& Desc, FIntRect const& TextureRegion, uint32 MaxLevelToEvict, 
					uint32 MinFrameToKeepMapped, EVTInvalidatePriority InvalidatePriority, TSet<union FVirtualTextureLocalTileRequest>& OutDirtyMapped);

	/**
	* Unmap all pages from the given address range in a space. Pages will remain resident in the pool, but no longer by mapped to any page table.
	*/
	void UnmapAllPagesForSpace(FVirtualTextureSystem* System, uint8 SpaceID, uint32 vAddress, uint32 Width, uint32 Height, uint32 MaxLevel);

	/**
	 * Remap physical pages from one producer to another.
	 * @param OldAllocatedAdaptiveBias The allocated adaptive bias of the old virtual texture, used for level bounds checking
	 * @param vLevelChange_vAddress Applied physical level change, used for base remapping
	 * @param vLevelChange_Producer Applied producer (i.e., local) level change
	 * @param Frame The current frame, used for LRU updates
	 */
	void RemapPages(
		FVirtualTextureSystem* System, uint8 SpaceID, FVirtualTexturePhysicalSpace* PhysicalSpace,
		FVirtualTextureProducerHandle const& ProducerHandleOld, uint32 OldVirtualAddress,
		FVirtualTextureProducerHandle const& ProducerHandleNew, uint32 NewVirtualAddress,
		int32 OldAllocatedAdaptiveBias,
		int32 vLevelChange_vAddress, int32 vLevelChange_Producer, uint32 Frame
	);

	/**
	 * Get descriptions of the locked pages in this pool
	 */
	void GetAllLockedPages(FVirtualTextureSystem* System, TSet<union FVirtualTextureLocalTile>& OutPages);

	FVirtualTextureLocalTile GetLocalTileFromPhysicalAddress(uint16 pAddress);
	
	/** Get the local vLevel of the page allocated at the given physical address */
	inline uint8 GetLocalLevelForAddress(uint16 pAddress) const { check(Pages[pAddress].PackedValue != 0u); return Pages[pAddress].Local_vLevel; }

	/** Get the packed producer handle of the page allocated at the given physical address */
	inline uint32 GetPackedProducerHandleForAddress(uint16 pAddress) const { check(Pages[pAddress].PackedValue != 0u); return Pages[pAddress].PackedProducerHandle; }

	/**
	 * Check if there are any free pages available at the moment.
	 */
	bool AnyFreeAvailable( uint32 Frame, uint32 FreeThreshold ) const;

	/**
	 * Find physical address of the page allocated for the given VT address, or ~0 if not allocated
	 */
	uint32		FindPageAddress(const FVirtualTextureProducerHandle& ProducerHandle, uint8 GroupIndex, uint32 Local_vAddress, uint8 Local_vLevel) const;

	/**
	 * Find the physical address of the allocated page that's closest to the given page, or ~0 if not found
	 */
	uint32		FindNearestPageAddress(const FVirtualTextureProducerHandle& ProducerHandle, uint8 GroupIndex, uint32 Local_vAddress, uint8 Local_vLevel, uint8 MaxLevel) const;

	/**
	 * Find the level of the allocated page that's closest to the given page, or ~0 if not found
	 */
	uint32		FindNearestPageLevel(const FVirtualTextureProducerHandle& ProducerHandle, uint8 GroupIndex, uint32 Local_vAddress, uint8 Local_vLevel) const;
	
	/**
	 * Allocate a physical address
	 * This allocation will be owned by the given VT producer, and if successful, may be mapped into virtual page tables
	 * Assuming the pool is full, the returned physical address will first be unmapped from anything that was previously using it
	 * @param Frame The current frame, used to manage LRU allocations
	 * @param ProducerHandle Handle of the VT producer requesting this page
	 * @param GroupIndex Physical group in the producer
	 * @param Local_vAddress Virtual address relative to the given producer
	 * @param Local_vLevel Mip level in the producer
	 * @param bLock Should the allocation be locked; locked allocations will never be evicted
	 */
	uint32		Alloc(FVirtualTextureSystem* System, uint32 Frame, const FVirtualTextureProducerHandle& ProducerHandle, uint8 GroupIndex, uint32 Local_vAddress, uint8 Local_vLevel, bool bLock);

	/**
	 * Marks the given physical address as free, will be unlocked if needed, moved to top of LRU list, no longer associated with any producer
	 */
	void		Free(FVirtualTextureSystem* System, uint16 pAddress);

	/**
	 * Mark a physical address as locked, so it will not be evicted.
	 */
	void        Lock(uint16 pAddress);

	/**
	 * Unlock the given physical address
	 */
	void		Unlock(uint32 Frame, uint16 pAddress);

	/**
	 * Marks the given physical address as used on this frame.
	 * This means it's guaranteed not to be evicted later on this frame, and less likely to be evicted on future frames (LRU pages are evicted first)
	 */
	void		UpdateUsage(uint32 Frame, uint16 pAddress);

	/**
	 * Returns the number of pages marked as used since a given frame. 
	 */
	uint32		GetNumVisiblePages(uint32 Frame) const;

	/**
	 * Returns a map between producer handle and a uint2 containing number of tiles and number of locked tils that the producer uses in the pool.
	 */
	void		CollectProducerCounts(TMap<uint32, FUintVector2>& OutProducerCountMap) const;

	/**
	* Map the physical address to a specific virtual address.
	*/
	void		MapPage(FVirtualTextureSpace* Space, FVirtualTexturePhysicalSpace* PhysicalSpace, uint8 PageTableLayerIndex, uint8 MaxLevel, uint8 vLogSize, uint32 vAddress, uint8 Local_vLevel, uint16 pAddress);

private:
	// Allocate 24 bits to store next/prev indices, pack layer index into 8 bits
	static const uint32 PAGE_MAPPING_CAPACITY = 0x00ffffff;

	struct FPageMapping
	{
		uint32 vAddress : 24;
		uint32 vLogSize : 4;
		uint32 MaxLevel : 4;

		uint32 NextIndex : 24;
		uint32 SpaceID : 8;

		uint32 PrevIndex : 24;
		uint32 PageTableLayerIndex : 8;
	};

	union FPageEntry
	{
		uint64 PackedValue;
		struct 
		{
			uint32 PackedProducerHandle;
			uint32 Local_vAddress : 24;
			uint32 Local_vLevel : 4;
			uint32 GroupIndex : 4;
		};
	};

	static uint16 GetPageHash(const FPageEntry& Entry);

	void UnmapPageMapping(FVirtualTextureSystem* System, uint32 MappingIndex, bool bMapAncestorPage);
	void UnmapAllPages(FVirtualTextureSystem* System, uint16 pAddress, bool bMapAncestorPages);

	void RemoveMappingFromList(uint32 Index)
	{
		FPageMapping& Mapping = PageMapping[Index];
		PageMapping[Mapping.PrevIndex].NextIndex = Mapping.NextIndex;
		PageMapping[Mapping.NextIndex].PrevIndex = Mapping.PrevIndex;
		Mapping.NextIndex = Mapping.PrevIndex = Index;
	}

	void AddMappingToList(uint32 HeadIndex, uint32 Index)
	{
		FPageMapping& Head = PageMapping[HeadIndex];
		FPageMapping& Mapping = PageMapping[Index];
		check(Index > NumPages); // make sure we're not trying to add a list head to another list
		check(Index <= PAGE_MAPPING_CAPACITY);

		// make sure we're not currently in any list
		check(Mapping.NextIndex == Index);
		check(Mapping.PrevIndex == Index);

		Mapping.NextIndex = HeadIndex;
		Mapping.PrevIndex = Head.PrevIndex;
		PageMapping[Head.PrevIndex].NextIndex = Index;
		Head.PrevIndex = Index;
	}

	uint32 AcquireMapping()
	{
		const uint32 FreeHeadIndex = NumPages;
		FPageMapping& FreeHead = PageMapping[FreeHeadIndex];
		uint32 Index = FreeHead.NextIndex;
		if (Index != FreeHeadIndex)
		{
			RemoveMappingFromList(Index);
			return Index;
		}

		Index = PageMapping.AddDefaulted();
		check(Index <= PAGE_MAPPING_CAPACITY);
		FPageMapping& Mapping = PageMapping[Index];
		Mapping.NextIndex = Mapping.PrevIndex = Index;
		return Index;
	}

	void ReleaseMapping(uint32 Index)
	{
		const uint32 FreeHeadIndex = NumPages;
		RemoveMappingFromList(Index);
		AddMappingToList(FreeHeadIndex, Index);
	}

	FBinaryHeap<uint32, uint16> FreeHeap;

	FHashTable PageHash;
	FHashTable ProducerToPageIndex;
	TArray<FPageEntry> Pages;

	// Holds linked lists of mappings for each  physical page in the pool
	// Indices [0, NumPages) hold the list head for list of mappings for each page
	// Index 'NumPages' holds the list head for the free list
	// Additional indices are list elements belonging to one of the prior lists
	TArray<FPageMapping> PageMapping;

	uint32 NumPages;
	uint32 NumPagesMapped;
	uint32 NumPagesAllocated;

	static const uint32 NumReservedPages;
};
