// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTextureShared.h"
#include "Containers/HashTable.h"

class FUniquePageList
{
public:
			FUniquePageList();

	void	Initialize();

	void	Add( uint32 Page, uint32 Count );

	uint32	GetNum() const					{ return NumPages; }
	uint32	GetPage(uint32 Index) const		{ checkSlow(Index < NumPages); const uint32 SortedIndex = bSorted ? SortOrder[Index] : Index; checkSlow(SortedIndex < NumPages); return Pages[SortedIndex]; }
	uint32	GetCount(uint32 Index) const	{ checkSlow(Index < NumPages); const uint32 SortedIndex = bSorted ? SortOrder[Index] : Index; checkSlow(SortedIndex < NumPages); return Counts[SortedIndex]; }

	void	MergePages(const FUniquePageList* RESTRICT Other);

	void	SortPages();

private:
	enum
	{
		HashSize		= 16*1024,
		MaxUniquePages	= 8*1024,
	};

	bool bInitialized;
	bool bSorted;
	uint32 NumPages;
	uint32 MaxNumCollisions;
	uint16 HashIndices[HashSize];
	uint32 Pages[ MaxUniquePages ];
	uint16 Counts[ MaxUniquePages ];
	uint16 SortOrder[ MaxUniquePages ];
};

inline FUniquePageList::FUniquePageList()
	: bInitialized( false )
	, bSorted( false )
	, NumPages( 0 )
	, MaxNumCollisions( 0 )
{}

inline void FUniquePageList::Initialize()
{
	if (!bInitialized)
	{
		FMemory::Memset(HashIndices, 0xff);
		bInitialized = true;
	}
}

inline void FUniquePageList::Add( uint32 Page, uint32 Count )
{
	uint32 HashIndex = MurmurFinalize32(Page) & (HashSize - 1u);
	uint32 NumCollisions = 0u;
	while (true)
	{
		uint32 PageIndex = HashIndices[HashIndex];
		if (PageIndex == 0xffff)
		{
			if (NumPages < MaxUniquePages)
			{
				PageIndex = NumPages++;
				HashIndices[HashIndex] = PageIndex;
				Pages[PageIndex] = Page;
				Counts[PageIndex] = Count;
			}
			break;
		}
		else if (Pages[PageIndex] == Page)
		{
			const uint32 PrevCount = Counts[PageIndex];
			Counts[PageIndex] = FMath::Min<uint32>(PrevCount + Count, 0xffff);
			break;
		}
		HashIndex = (HashIndex + 1u) & (HashSize - 1u);
		++NumCollisions;
	}
	bSorted = false;
#if DO_GUARD_SLOW
	MaxNumCollisions = FMath::Max(MaxNumCollisions, NumCollisions);
#endif // DO_GUARD_SLOW
}

inline void FUniquePageList::MergePages(const FUniquePageList* RESTRICT Other)
{
	for (uint32 Index = 0u; Index < Other->NumPages; ++Index)
	{
		Add(Other->Pages[Index], Other->Counts[Index]);
	}
}

inline void FUniquePageList::SortPages()
{
	for (uint32 Index = 0; Index < NumPages; ++Index)
	{
		SortOrder[Index] = Index;
	}
	Algo::Sort(MakeArrayView(SortOrder, NumPages), [this](uint32 A, uint32 B) { return Counts[A] == Counts[B] ? A < B : Counts[A] > Counts[B]; });

	bSorted = true;
}
