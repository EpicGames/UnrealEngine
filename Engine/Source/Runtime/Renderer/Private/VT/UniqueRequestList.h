// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/HashTable.h"
#include "VirtualTexturing.h"
#include "VirtualTextureSystem.h"
#include "VirtualTextureProducer.h"
#include "VirtualTexturePhysicalSpace.h"

union FMappingRequest
{
	inline FMappingRequest() {}
	inline FMappingRequest(uint16 InLoadIndex, uint8 InPhysicalGroupIndex, uint8 InSpaceID, uint8 InPageTableLayerIndex, uint32 InMaxLevel, uint32 InAddress, uint8 InLevel, uint8 InLocalLevel)
		: vAddress(InAddress), vLevel(InLevel), SpaceID(InSpaceID), LoadRequestIndex(InLoadIndex), Local_vLevel(InLocalLevel), ProducerPhysicalGroupIndex(InPhysicalGroupIndex), PageTableLayerIndex(InPageTableLayerIndex), MaxLevel(InMaxLevel)
	{}

	uint64 PackedValue;
	struct
	{
		uint32 vAddress : 24;
		uint32 vLevel : 4;
		uint32 SpaceID : 4;
		uint32 LoadRequestIndex : 16;
		uint32 Local_vLevel : 4;
		uint32 ProducerPhysicalGroupIndex : 4;
		uint32 PageTableLayerIndex : 4;
		uint32 MaxLevel : 4;
	};
};
static_assert(sizeof(FMappingRequest) == sizeof(uint64), "Bad packing");
inline bool operator==(const FMappingRequest& Lhs, const FMappingRequest& Rhs) { return Lhs.PackedValue == Rhs.PackedValue; }
inline bool operator!=(const FMappingRequest& Lhs, const FMappingRequest& Rhs) { return Lhs.PackedValue != Rhs.PackedValue; }

union FDirectMappingRequest
{
	inline FDirectMappingRequest() {}
	inline FDirectMappingRequest(uint8 InSpaceID, uint16 InPhysicalSpaceID, uint8 InPageTableLayerIndex, uint32 InMaxLevel, uint32 InAddress, uint8 InLevel, uint8 InLocalLevel, uint16 InPhysicalAddress)
		: vAddress(InAddress), vLevel(InLevel), SpaceID(InSpaceID), pAddress(InPhysicalAddress), PhysicalSpaceID(InPhysicalSpaceID), Local_vLevel(InLocalLevel), MaxLevel(InMaxLevel), PageTableLayerIndex(InPageTableLayerIndex), Pad(0u)
	{}

	uint32 PackedValue[3];
	struct
	{
		uint32 vAddress : 24;
		uint32 vLevel : 4;
		uint32 SpaceID : 4;

		uint32 pAddress : 16;
		uint32 PhysicalSpaceID : 8;
		uint32 Local_vLevel : 4;
		uint32 MaxLevel : 4;

		uint32 PageTableLayerIndex : 4;
		uint32 Pad : 28;
	};
};
static_assert(sizeof(FDirectMappingRequest) == sizeof(uint32) * 3, "Bad packing");
inline bool operator==(const FDirectMappingRequest& Lhs, const FDirectMappingRequest& Rhs) { return Lhs.PackedValue[0] == Rhs.PackedValue[0] && Lhs.PackedValue[1] == Rhs.PackedValue[1] && Lhs.PackedValue[2] == Rhs.PackedValue[2]; }
inline bool operator!=(const FDirectMappingRequest& Lhs, const FDirectMappingRequest& Rhs) { return !operator==(Lhs, Rhs); }

class FUniqueRequestList
{
public:
	explicit FUniqueRequestList(FConcurrentLinearBulkObjectAllocator& Allocator)
		: LoadRequestHash(NoInit)
		, MappingRequestHash(NoInit)
		, DirectMappingRequestHash(NoInit)
		, LoadRequests(Allocator.CreateArray<FVirtualTextureLocalTileRequest>(LoadRequestCapacity))
		, MappingRequests(Allocator.CreateArray<FMappingRequest>(MappingRequestCapacity))
		, DirectMappingRequests(Allocator.CreateArray<FDirectMappingRequest>(DirectMappingRequestCapacity))
		, ContinuousUpdateRequests(Allocator.CreateArray<FVirtualTextureLocalTileRequest>(LoadRequestCapacity))
		, AdaptiveAllocationsRequests(Allocator.MallocArray<uint32>(LoadRequestCapacity))
		, LoadRequestCount(Allocator.MallocArray<uint16>(LoadRequestCapacity))
		, LoadRequestGroupMask(Allocator.MallocArray<uint8>(LoadRequestCapacity))
		, LoadRequestFlags(Allocator.MallocArray<FLoadRequestFlags>(LoadRequestCapacity))
		, NumLoadRequests(0u)
		, NumLockRequests(0u)
		, NumNonStreamingLoadRequests(0u)
		, NumMappingRequests(0u)
		, NumDirectMappingRequests(0u)
		, NumContinuousUpdateRequests(0u)
		, NumAdaptiveAllocationRequests(0u)
	{
	}

	inline void Initialize()
	{
		LoadRequestHash.Clear();
		MappingRequestHash.Clear();
		DirectMappingRequestHash.Clear();
		ContinuousUpdateRequestHash.Clear();
	}

	inline void Reset(bool bResetContinousUpdates)
	{
		LoadRequestHash.Clear();
		MappingRequestHash.Clear();
		DirectMappingRequestHash.Clear();
		NumLoadRequests = 0;
		NumLockRequests = 0;
		NumNonStreamingLoadRequests = 0;
		NumMappingRequests = 0;
		NumDirectMappingRequests = 0;
		NumAdaptiveAllocationRequests = 0;

		if (bResetContinousUpdates)
		{
			NumContinuousUpdateRequests = 0;
			ContinuousUpdateRequestHash.Clear();
		}
	}

	inline uint32 GetNumLoadRequests() const { return NumLoadRequests; }
	inline uint32 GetNumNonStreamingLoadRequests() const { return NumNonStreamingLoadRequests; }
	inline uint32 GetNumMappingRequests() const { return NumMappingRequests; }
	inline uint32 GetNumDirectMappingRequests() const { return NumDirectMappingRequests; }
	inline uint32 GetNumContinuousUpdateRequests() const { return NumContinuousUpdateRequests; }
	inline uint32 GetNumAdaptiveAllocationRequests() const { return NumAdaptiveAllocationRequests; }

	inline const FVirtualTextureLocalTileRequest& GetLoadRequest(uint32 i) const { checkSlow(i < NumLoadRequests); return LoadRequests[i]; }
	inline const FMappingRequest& GetMappingRequest(uint32 i) const { checkSlow(i < NumMappingRequests); return MappingRequests[i]; }
	inline const FDirectMappingRequest& GetDirectMappingRequest(uint32 i) const { checkSlow(i < NumDirectMappingRequests); return DirectMappingRequests[i]; }
	inline const FVirtualTextureLocalTileRequest& GetContinuousUpdateRequest(uint32 i) const { checkSlow(i < NumContinuousUpdateRequests); return ContinuousUpdateRequests[i]; }
	inline const uint32& GetAdaptiveAllocationRequest(uint32 i) const { checkSlow(i < NumAdaptiveAllocationRequests); return AdaptiveAllocationsRequests[i]; }
	
	inline uint8 GetGroupMask(uint32 i) const { checkSlow(i < NumLoadRequests); return LoadRequestGroupMask[i]; }
	inline bool IsLocked(uint32 i) const { checkSlow(i < NumLoadRequests); return i < NumLockRequests; }

	uint16 AddLoadRequest(const FVirtualTextureLocalTileRequest& TileRequest, uint8 GroupMask, uint16 Count, bool bStreamingRequest);
	uint16 LockLoadRequest(const FVirtualTextureLocalTileRequest& TileRequest, uint8 GroupMask, bool bStreamingRequest);

	void AddMappingRequest(uint16 LoadRequestIndex, uint8 ProducerPhysicalGroupIndex, uint8 SpaceID, uint8 PageTableLayerIndex, uint32 MaxLevel, uint32 vAddress, uint8 vLevel, uint8 Local_vLevel);

	void AddDirectMappingRequest(uint8 InSpaceID, uint16 InPhysicalSpaceID, uint8 InPageTableLayerIndex, uint32 MaxLevel, uint32 InAddress, uint8 InLevel, uint8 InLocalLevel, uint16 InPhysicalAddress);
	void AddDirectMappingRequest(const FDirectMappingRequest& Request);

	void AddContinuousUpdateRequest(const FVirtualTextureLocalTileRequest& Request);

	void AddAdaptiveAllocationRequest(uint32 Request);

	void MergeRequests(const FUniqueRequestList* RESTRICT Other, FConcurrentLinearBulkObjectAllocator& Allocator);

	void SortRequests(FVirtualTextureProducerCollection& Producers, FConcurrentLinearBulkObjectAllocator& Allocator, uint32 MaxNonStreamingLoadRequests, uint32 MaxStreamingLoadRequests, bool bUseCombinedLimit, bool bSortByPriority);

private:
	static const uint32 LoadRequestCapacity = 4u * 1024;
	static const uint32 MappingRequestCapacity = 8u * 1024 - 256u;
	static const uint32 DirectMappingRequestCapacity = MappingRequestCapacity;
	static const uint32 ContinuousUpdateRequestCapacity = LoadRequestCapacity;
	static const uint32 AdaptiveAllocationRequestCapacity = LoadRequestCapacity;

	TStaticHashTable<1024u, LoadRequestCapacity> LoadRequestHash;
	TStaticHashTable<1024u, MappingRequestCapacity> MappingRequestHash;
	TStaticHashTable<512u, DirectMappingRequestCapacity> DirectMappingRequestHash;
	TStaticHashTable<1024u, ContinuousUpdateRequestCapacity> ContinuousUpdateRequestHash;

	FVirtualTextureLocalTileRequest* LoadRequests;
	FMappingRequest* MappingRequests;
	FDirectMappingRequest* DirectMappingRequests;
	FVirtualTextureLocalTileRequest* ContinuousUpdateRequests;
	uint32* AdaptiveAllocationsRequests;
	
	uint16* LoadRequestCount;
	uint8* LoadRequestGroupMask;

	struct FLoadRequestFlags
	{
		uint8 bLocked : 1;
		uint8 bStreaming : 1;
		uint8 Padding : 6;

		FLoadRequestFlags(bool bInLocked, bool bInStreaming)
			: bLocked(bInLocked)
			, bStreaming(bInStreaming)
		{}
	};
	FLoadRequestFlags* LoadRequestFlags;

	uint32 NumLoadRequests;
	uint32 NumLockRequests;
	uint32 NumNonStreamingLoadRequests;
	uint32 NumMappingRequests;
	uint32 NumDirectMappingRequests;
	uint32 NumContinuousUpdateRequests;
	uint32 NumAdaptiveAllocationRequests;
};

inline uint16 FUniqueRequestList::AddLoadRequest(const FVirtualTextureLocalTileRequest& TileRequest, uint8 GroupMask, uint16 Count, bool bStreamingRequest)
{
	const uint16 Hash = TileRequest.GetMurmurHash();
	check(GroupMask != 0u);
	for (uint16 Index = LoadRequestHash.First(Hash); LoadRequestHash.IsValid(Index); Index = LoadRequestHash.Next(Index))
	{
		if (TileRequest == LoadRequests[Index])
		{
			LoadRequests[Index].MergeWith(TileRequest);

			check(LoadRequestFlags[Index].bStreaming == bStreamingRequest);

			LoadRequestCount[Index] = FMath::Min<uint32>((uint32)LoadRequestCount[Index] + Count, MAX_uint16);
			LoadRequestGroupMask[Index] |= GroupMask;
			return Index;
		}
	}

	if (NumLoadRequests < LoadRequestCapacity)
	{
		const uint32 Index = NumLoadRequests++;
		LoadRequestHash.Add(Hash, Index);
		LoadRequests[Index] = TileRequest;
		LoadRequestCount[Index] = Count;
		LoadRequestGroupMask[Index] = GroupMask;
		LoadRequestFlags[Index] = FLoadRequestFlags(/*bInLocked = */false, bStreamingRequest);
		return Index;
	}
	return 0xffff;
}

inline uint16 FUniqueRequestList::LockLoadRequest(const FVirtualTextureLocalTileRequest& TileRequest, uint8 GroupMask, bool bStreamingRequest)
{
	const uint16 Hash = TileRequest.GetMurmurHash();
	check(GroupMask != 0u);

	for (uint16 Index = LoadRequestHash.First(Hash); LoadRequestHash.IsValid(Index); Index = LoadRequestHash.Next(Index))
	{
		if (TileRequest == LoadRequests[Index])
		{
			LoadRequests[Index].MergeWith(TileRequest);

			FLoadRequestFlags& Flags = LoadRequestFlags[Index];
			check(bStreamingRequest == Flags.bStreaming);

			if (!Flags.bLocked)
			{
				Flags.bLocked = true;
				LoadRequestCount[Index] = MAX_uint16;
				++NumLockRequests;
			}
			LoadRequestGroupMask[Index] |= GroupMask;
			return Index;
		}
	}

	if (NumLoadRequests < LoadRequestCapacity)
	{
		const uint32 Index = NumLoadRequests++;
		LoadRequestHash.Add(Hash, Index);
		LoadRequests[Index] = TileRequest;
		LoadRequestCount[Index] = MAX_uint16;
		LoadRequestGroupMask[Index] = GroupMask;
		LoadRequestFlags[Index] = FLoadRequestFlags(/*bInLocked = */true, bStreamingRequest);
		++NumLockRequests;
		return Index;
	}

	return 0xffff;
}

inline void FUniqueRequestList::AddMappingRequest(uint16 LoadRequestIndex, uint8 ProducerPhysicalGroupIndex, uint8 SpaceID, uint8 PageTableLayerIndex, uint32 MaxLevel, uint32 vAddress, uint8 vLevel, uint8 Local_vLevel)
{
	check(LoadRequestIndex < NumLoadRequests);
	const FMappingRequest Request(LoadRequestIndex, ProducerPhysicalGroupIndex, SpaceID, PageTableLayerIndex, MaxLevel, vAddress, vLevel, Local_vLevel);
	const uint16 Hash = static_cast<uint16>(MurmurFinalize64(Request.PackedValue));

	for (uint16 Index = MappingRequestHash.First(Hash); MappingRequestHash.IsValid(Index); Index = MappingRequestHash.Next(Index))
	{
		if (Request == MappingRequests[Index])
		{
			return;
		}
	}

	if (NumMappingRequests < MappingRequestCapacity)
	{
		const uint32 Index = NumMappingRequests++;
		MappingRequestHash.Add(Hash, Index);
		MappingRequests[Index] = Request;
	}
}

inline void FUniqueRequestList::AddDirectMappingRequest(uint8 InSpaceID, uint16 InPhysicalSpaceID, uint8 InPageTableLayerIndex, uint32 InMaxLevel, uint32 InAddress, uint8 InLevel, uint8 InLocalLevel, uint16 InPhysicalAddress)
{
	const FDirectMappingRequest Request(InSpaceID, InPhysicalSpaceID, InPageTableLayerIndex, InMaxLevel, InAddress, InLevel, InLocalLevel, InPhysicalAddress);
	AddDirectMappingRequest(Request);
}

inline void FUniqueRequestList::AddDirectMappingRequest(const FDirectMappingRequest& Request)
{
	const uint16 Hash = Murmur32({ Request.PackedValue[0], Request.PackedValue[1], Request.PackedValue[2] });
	for (uint16 Index = DirectMappingRequestHash.First(Hash); DirectMappingRequestHash.IsValid(Index); Index = DirectMappingRequestHash.Next(Index))
	{
		if (Request == DirectMappingRequests[Index])
		{
			return;
		}
	}

	if (NumDirectMappingRequests < DirectMappingRequestCapacity)
	{
		const uint32 Index = NumDirectMappingRequests++;
		DirectMappingRequestHash.Add(Hash, Index);
		DirectMappingRequests[Index] = Request;
	}
}

inline void FUniqueRequestList::AddContinuousUpdateRequest(const FVirtualTextureLocalTileRequest& Request)
{
	const uint16 Hash = Request.GetMurmurHash();
	for (uint16 Index = ContinuousUpdateRequestHash.First(Hash); ContinuousUpdateRequestHash.IsValid(Index); Index = ContinuousUpdateRequestHash.Next(Index))
	{
		if (Request == ContinuousUpdateRequests[Index])
		{
			ContinuousUpdateRequests[Index].MergeWith(Request);
			return;
		}
	}

	if (NumContinuousUpdateRequests < ContinuousUpdateRequestCapacity)
	{
		const uint32 Index = NumContinuousUpdateRequests++;
		ContinuousUpdateRequestHash.Add(Hash, Index);
		ContinuousUpdateRequests[Index] = Request;
	}
}

inline void FUniqueRequestList::AddAdaptiveAllocationRequest(uint32 Request)
{
	if (NumAdaptiveAllocationRequests < AdaptiveAllocationRequestCapacity)
	{
		AdaptiveAllocationsRequests[NumAdaptiveAllocationRequests++] = Request;
	}
}

inline void FUniqueRequestList::MergeRequests(const FUniqueRequestList* RESTRICT Other, FConcurrentLinearBulkObjectAllocator& Allocator)
{
	uint16* LoadRequestIndexRemap = Allocator.MallocArray<uint16>(Other->NumLoadRequests);

	for (uint32 Index = 0u; Index < Other->NumLoadRequests; ++Index)
	{
		if (Other->IsLocked(Index))
		{
			LoadRequestIndexRemap[Index] = LockLoadRequest(Other->GetLoadRequest(Index), Other->LoadRequestGroupMask[Index], Other->LoadRequestFlags[Index].bStreaming);
		}
		else
		{
			LoadRequestIndexRemap[Index] = AddLoadRequest(Other->GetLoadRequest(Index), Other->LoadRequestGroupMask[Index], Other->LoadRequestCount[Index], Other->LoadRequestFlags[Index].bStreaming);
		}
	}

	for (uint32 Index = 0u; Index < Other->NumMappingRequests; ++Index)
	{
		const FMappingRequest& Request = Other->GetMappingRequest(Index);
		check(Request.LoadRequestIndex < Other->NumLoadRequests);
		const uint16 LoadRequestIndex = LoadRequestIndexRemap[Request.LoadRequestIndex];
		if (LoadRequestIndex != 0xffff)
		{
			AddMappingRequest(LoadRequestIndex, Request.ProducerPhysicalGroupIndex, Request.SpaceID, Request.PageTableLayerIndex, Request.MaxLevel, Request.vAddress, Request.vLevel, Request.Local_vLevel);
		}
	}

	for (uint32 Index = 0u; Index < Other->NumDirectMappingRequests; ++Index)
	{
		AddDirectMappingRequest(Other->GetDirectMappingRequest(Index));
	}

	for (uint32 Index = 0u; Index < Other->NumContinuousUpdateRequests; ++Index)
	{
		AddContinuousUpdateRequest(Other->GetContinuousUpdateRequest(Index));
	}

	for (uint32 Index = 0u; Index < Other->NumAdaptiveAllocationRequests; ++Index)
	{
		AddAdaptiveAllocationRequest(Other->GetAdaptiveAllocationRequest(Index));
	}
}

inline void FUniqueRequestList::SortRequests(FVirtualTextureProducerCollection& Producers, FConcurrentLinearBulkObjectAllocator& Allocator, uint32 MaxNonStreamingLoadRequests, uint32 MaxStreamingLoadRequests, bool bUseCombinedLimit, bool bSortByPriority)
{
	if (bUseCombinedLimit)
	{
		MaxNonStreamingLoadRequests += MaxStreamingLoadRequests;
		MaxStreamingLoadRequests = 0;
	}

	// Compute priority of each load request
	uint32 CheckNumLockRequests = 0u;
	uint32 NumNonStreamingLockRequests = 0u;
	uint32 NumStreamingNonLockRequests = 0u;

	FVTRequestPriorityAndIndex* SortedKeys = Allocator.CreateArray<FVTRequestPriorityAndIndex>(NumLoadRequests);
	for (uint32 i = 0u; i < NumLoadRequests; ++i)
	{
		const uint32 Count = LoadRequestCount[i];
		const FLoadRequestFlags Flags = LoadRequestFlags[i];

		// Try to load higher mips first
		const FVirtualTextureLocalTileRequest& LoadRequest = GetLoadRequest(i);
		const uint32 PagePriority = (Count * (1u + LoadRequest.GetTile().Local_vLevel));

		const bool bStreaming = !bUseCombinedLimit && Flags.bStreaming;
		if (Flags.bLocked)
		{
			NumNonStreamingLockRequests += bStreaming ? 0 : 1;
			++CheckNumLockRequests;
		}
		else if (bStreaming)
		{
			++NumStreamingNonLockRequests;
		}
		SortedKeys[i] = FVTRequestPriorityAndIndex(
			i,
			/*bInLocked = */(Flags.bLocked != 0),
			/*bInStreaming = */bStreaming,
			bSortByPriority ? LoadRequest.GetProducerPriority() : static_cast<EVTProducerPriority>(0),
			bSortByPriority ? LoadRequest.GetInvalidatePriority() : static_cast<EVTInvalidatePriority>(0),
			PagePriority);
	}
	checkSlow(CheckNumLockRequests == NumLockRequests);

	// Sort so highest priority requests are at the front of the list. 
	//  Important note : the rest of the algorithm assumes locked requests come first, then streaming requests, then the rest :
	Algo::Sort(MakeArrayView(SortedKeys, NumLoadRequests));

	// Clamp number of load requests to maximum, but also ensure all lock requests are considered
	const uint32 NumStreamingLockRequests = NumLockRequests - NumNonStreamingLockRequests;
	const uint32 NumStreamingRequests = NumStreamingNonLockRequests + NumStreamingLockRequests;
	const uint32 NumNonStreamingRequests = NumLoadRequests - NumStreamingRequests;

	const uint32 NewNumNonStreamingRequests = FMath::Min(NumNonStreamingRequests, FMath::Max(NumNonStreamingLockRequests, MaxNonStreamingLoadRequests));
	const uint32 NewNumStreamingRequests = FMath::Min(NumStreamingRequests, FMath::Max(NumStreamingLockRequests, MaxStreamingLoadRequests));
	const uint32 NewNumLoadRequests = NewNumNonStreamingRequests + NewNumStreamingRequests;

	// Re-index load request list, using sorted indices
	FVirtualTextureLocalTileRequest* SortedLoadRequests = Allocator.CreateArray<FVirtualTextureLocalTileRequest>(NewNumLoadRequests);
	uint8* SortedGroupMask = Allocator.MallocArray<uint8>(NewNumLoadRequests);
	FLoadRequestFlags* SortedFlags = Allocator.MallocArray<FLoadRequestFlags>(NewNumLoadRequests);
	uint16* LoadIndexToSortedLoadIndex = Allocator.MallocArray<uint16>(NumLoadRequests);
	FMemory::Memset(LoadIndexToSortedLoadIndex, 0xff, NumLoadRequests * sizeof(uint16));

	uint32 WriteIndex = 0;
	auto CopyRequestToSorted = [this, &WriteIndex, SortedKeys, SortedLoadRequests, SortedGroupMask, SortedFlags, LoadIndexToSortedLoadIndex](uint32 SortedIndex)
		{
			const uint32 OldIndex = SortedKeys[SortedIndex].Index;
			checkSlow(OldIndex < NumLoadRequests);

			SortedLoadRequests[WriteIndex] = LoadRequests[OldIndex];
			SortedGroupMask[WriteIndex] = LoadRequestGroupMask[OldIndex];
			SortedFlags[WriteIndex] = LoadRequestFlags[OldIndex];
			LoadIndexToSortedLoadIndex[OldIndex] = WriteIndex;

			++WriteIndex;
		};

	for (uint32 SortedIndex = 0u; SortedIndex < NumLockRequests; ++SortedIndex)
	{
		CopyRequestToSorted(SortedIndex);
		checkfSlow((SortedKeys[SortedIndex].GetPriorityKey().Locked != 0) && SortedFlags[WriteIndex - 1].bLocked, TEXT("If this asserts, then the sorting is invalid : the code assumes locked requests are the ones at the beginning of the sorted list"));
	}

	for (uint32 i = 0u; i < NewNumStreamingRequests - NumStreamingLockRequests; ++i)
	{
		const uint32 SortedIndex = NumLockRequests + i;
		CopyRequestToSorted(SortedIndex);
		checkfSlow((SortedKeys[SortedIndex].GetPriorityKey().Streaming != 0) && !SortedFlags[WriteIndex - 1].bLocked, TEXT("If this asserts, then the sorting is invalid : the code assumes locked requests are the ones at the beginning of the sorted list"));
	}

	for (uint32 i = 0u; i < NewNumNonStreamingRequests - NumNonStreamingLockRequests; ++i)
	{
		const uint32 SortedIndex = NumLockRequests + NumStreamingNonLockRequests + i;
		CopyRequestToSorted(SortedIndex);
		checkfSlow((SortedKeys[SortedIndex].GetPriorityKey().Streaming == 0) && !SortedFlags[WriteIndex - 1].bLocked, TEXT("If this asserts, then the sorting is invalid : the code assumes locked requests are the ones at the beginning of the sorted list"));
	}

	check(NewNumLoadRequests == WriteIndex);
	LoadRequests = SortedLoadRequests;
	LoadRequestGroupMask = SortedGroupMask;
	LoadRequestFlags = SortedFlags;

	// Remap LoadRequest indices for all the mapping requests
	// Can discard any mapping request that refers to a LoadRequest that's no longer being performed this frame
	uint32 NewNumMappingRequests = 0u;
	for (uint32 i = 0u; i < NumMappingRequests; ++i)
	{
		FMappingRequest Request = GetMappingRequest(i);
		checkSlow(Request.LoadRequestIndex < NumLoadRequests);
		const uint16 SortedLoadIndex = LoadIndexToSortedLoadIndex[Request.LoadRequestIndex];
		if (SortedLoadIndex != 0xffff)
		{
			check(SortedLoadIndex < NewNumLoadRequests);
			Request.LoadRequestIndex = SortedLoadIndex;
			MappingRequests[NewNumMappingRequests++] = Request;
		}
	}

	NumLoadRequests = NewNumLoadRequests;
	NumNonStreamingLoadRequests = NewNumNonStreamingRequests;
	check(!bUseCombinedLimit || NumLoadRequests == NumNonStreamingLoadRequests);
	NumMappingRequests = NewNumMappingRequests;
}

