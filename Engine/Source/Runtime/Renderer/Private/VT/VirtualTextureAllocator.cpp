// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureAllocator.h"
#include "AllocatedVirtualTexture.h"
#include "VirtualTexturing.h"
#include "Modules/ModuleManager.h"
#include "IImageWrapperModule.h"
#include "Misc/Paths.h"

#if WITH_EDITOR
#include "HAL/FileManager.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#endif // WITH_EDITOR

FVirtualTextureAllocator::FVirtualTextureAllocator(uint32 Dimensions)
	: vDimensions(Dimensions)
	, AllocatedWidth(0u)
	, AllocatedHeight(0u)
	, NumAllocations(0u)
	, NumAllocatedPages(0u)
{
}

void FVirtualTextureAllocator::Initialize(uint32 MaxSize)
{
	const uint32 vLogSize = FMath::CeilLogTwo(MaxSize);
	check(vLogSize <= VIRTUALTEXTURE_LOG2_MAX_PAGETABLE_SIZE);
	check(NumAllocations == 0);

	AddressBlocks.Reset(1);
	SortedAddresses.Reset(1);
	SortedIndices.Reset(1);
	FreeList.Reset(vLogSize + 1);
	PartiallyFreeList.Reset(vLogSize + 1);

	// Start with one empty block
	FAddressBlock DefaultBlock(vLogSize);
	DefaultBlock.State = EBlockState::FreeList;
	AddressBlocks.Add(DefaultBlock);

	SortedAddresses.Add(0u);
	SortedIndices.Add(0u);

	// Init free list
	FreeList.AddUninitialized(vLogSize + 1);
	PartiallyFreeList.AddUninitialized(vLogSize + 1);
	for (uint8 i = 0; i < vLogSize; i++)
	{
		FreeList[i] = 0xffff;
		FMemory::Memset(&PartiallyFreeList[i], 0xff, sizeof(FPartiallyFreeMip));
	}
	FreeList[vLogSize] = 0;
	FMemory::Memset(&PartiallyFreeList[vLogSize], 0xff, sizeof(FPartiallyFreeMip));

	// Init global free list
	GlobalFreeList = 0xffff;

	RootIndex = 0;
}

void FVirtualTextureAllocator::LinkFreeList(uint16& InOutListHead, EBlockState State, uint16 Index)
{
	FAddressBlock& AddressBlock = AddressBlocks[Index];
	check(AddressBlock.State == EBlockState::None);
	check(AddressBlock.NextFree == 0xffff);
	check(AddressBlock.PrevFree == 0xffff);

	// Only the PartiallyFreeList free list is allowed to have children
	check(State == EBlockState::PartiallyFreeList || AddressBlock.FirstChild == 0xffff);

	AddressBlock.State = State;
	AddressBlock.NextFree = InOutListHead;
	if (AddressBlock.NextFree != 0xffff)
	{
		AddressBlocks[AddressBlock.NextFree].PrevFree = Index;
	}
	InOutListHead = Index;
}

void FVirtualTextureAllocator::UnlinkFreeList(uint16& InOutListHead, EBlockState State, uint16 Index)
{
	FAddressBlock& AddressBlock = AddressBlocks[Index];
	check(AddressBlock.State == State);
	const uint32 PrevFreeIndex = AddressBlock.PrevFree;
	const uint32 NextFreeIndex = AddressBlock.NextFree;
	if (PrevFreeIndex != 0xffff)
	{
		AddressBlocks[PrevFreeIndex].NextFree = NextFreeIndex;
		AddressBlock.PrevFree = 0xffff;
	}
	if (NextFreeIndex != 0xffff)
	{
		AddressBlocks[NextFreeIndex].PrevFree = PrevFreeIndex;
		AddressBlock.NextFree = 0xffff;
	}
	if (InOutListHead == Index)
	{
		InOutListHead = NextFreeIndex;
	}
	AddressBlock.State = EBlockState::None;
}

int32 FVirtualTextureAllocator::AcquireBlock()
{
	int32 Index = GlobalFreeList;
	if (Index == 0xffff)
	{
		Index = AddressBlocks.AddUninitialized();
		check(Index <= 0xffff); // make sure we fit in 16bit index
	}
	else
	{
		UnlinkFreeList(GlobalFreeList, EBlockState::GlobalFreeList, Index);
	}

	// Debug fill memory to invalid value
	FAddressBlock& AddressBlock = AddressBlocks[Index];
	FMemory::Memset(AddressBlock, 0xCC);

	return Index;
}

uint32 FVirtualTextureAllocator::FindAddressBlock(uint32 vAddress) const
{
	uint32 Min = 0;
	uint32 Max = SortedAddresses.Num();

	// Binary search for lower bound
	while (Min != Max)
	{
		const uint32 Mid = Min + (Max - Min) / 2;
		const uint32 Key = SortedAddresses[Mid];

		if (vAddress < Key)
			Min = Mid + 1;
		else
			Max = Mid;
	}

	return Min;
}

FAllocatedVirtualTexture* FVirtualTextureAllocator::Find(uint32 vAddress, uint32& OutLocal_vAddress) const
{
	const uint32 SortedIndex = FindAddressBlock(vAddress);

	const uint16 Index = SortedIndices[SortedIndex];
	const FAddressBlock& AddressBlock = AddressBlocks[Index];
	check(SortedAddresses[SortedIndex] == AddressBlock.vAddress);

	FAllocatedVirtualTexture* AllocatedVT = nullptr;
	const uint32 BlockSize = 1 << (vDimensions * AddressBlock.vLogSize);
	if (vAddress >= AddressBlock.vAddress &&
		vAddress < AddressBlock.vAddress + BlockSize)
	{
		AllocatedVT = AddressBlock.VT;
		if (AllocatedVT)
		{
			OutLocal_vAddress = vAddress - AllocatedVT->GetVirtualAddress();
		}
		// TODO mip bias
	}

	return AllocatedVT;
}

bool FVirtualTextureAllocator::TryAlloc(uint32 InLogSize)
{
	for (int i = InLogSize; i < FreeList.Num(); i++)
	{
		uint16 FreeIndex = FreeList[i];
		if (FreeIndex != 0xffff)
		{
			return true;
		}
	}
	return false;
}

void FVirtualTextureAllocator::SubdivideBlock(uint32 ParentIndex)
{
	const uint32 NumChildren = (1 << vDimensions);

	const uint32 vParentLogSize = AddressBlocks[ParentIndex].vLogSize;
	check(vParentLogSize > 0u);
	const uint32 vChildLogSize = vParentLogSize - 1u;

	// Only free blocks can be subdivided, move to the partially free list
	check(AddressBlocks[ParentIndex].FirstChild == 0xffff);
	UnlinkFreeList(FreeList[vParentLogSize], EBlockState::FreeList, ParentIndex);
	AddressBlocks[ParentIndex].FreeMip = 0;
	LinkFreeList(PartiallyFreeList[vParentLogSize].Mips[0], EBlockState::PartiallyFreeList, ParentIndex);

	const uint32 vAddress = AddressBlocks[ParentIndex].vAddress;
	const int32 SortedIndex = FindAddressBlock(vAddress);
	check(vAddress == SortedAddresses[SortedIndex]);

	// Make room for newly added
	SortedAddresses.InsertUninitialized(SortedIndex, NumChildren - 1u);
	SortedIndices.InsertUninitialized(SortedIndex, NumChildren - 1u);
	check(SortedAddresses.Num() == SortedIndices.Num());

	uint16 FirstSiblingIndex = 0xffff;
	uint16 PrevChildIndex = 0xffff;
	for (uint32 Sibling = 0; Sibling < NumChildren; Sibling++)
	{
		const int32 ChildBlockIndex = AcquireBlock();
		const uint32 vChildAddress = vAddress + (Sibling << (vDimensions * vChildLogSize));

		const int32 SortedIndexOffset = NumChildren - 1 - Sibling;
		SortedAddresses[SortedIndex + SortedIndexOffset] = vChildAddress;
		SortedIndices[SortedIndex + SortedIndexOffset] = ChildBlockIndex;

		if (Sibling == 0u)
		{
			FirstSiblingIndex = ChildBlockIndex;
			AddressBlocks[ParentIndex].FirstChild = ChildBlockIndex;
		}
		else
		{
			AddressBlocks[PrevChildIndex].NextSibling = ChildBlockIndex;
		}
	
		FAddressBlock ChildBlock(vChildLogSize);
		ChildBlock.vAddress = vChildAddress;
		ChildBlock.Parent = ParentIndex;
		ChildBlock.FirstSibling = FirstSiblingIndex;
		ChildBlock.NextSibling = 0xffff;
		AddressBlocks[ChildBlockIndex] = ChildBlock;

		// New child blocks start out on the free list
		LinkFreeList(FreeList[vChildLogSize], EBlockState::FreeList, ChildBlockIndex);

		PrevChildIndex = ChildBlockIndex;
	}
}

void FVirtualTextureAllocator::FreeMipUpdateParents(uint16 ParentIndex)
{
	for (uint32 ParentDepth = 0; ParentDepth < PartiallyFreeMipDepth; ParentDepth++)
	{
		if (ParentIndex == 0xffff)
		{
			break;
		}

		FAddressBlock& ParentBlock = AddressBlocks[ParentIndex];
		uint8 OldFreeMip = ParentBlock.FreeMip;
		uint8 NewFreeMip = ComputeFreeMip(ParentIndex);

		if (NewFreeMip != OldFreeMip)
		{
			UnlinkFreeList(PartiallyFreeList[ParentBlock.vLogSize].Mips[OldFreeMip], EBlockState::PartiallyFreeList, ParentIndex);
			ParentBlock.FreeMip = NewFreeMip;
			LinkFreeList(PartiallyFreeList[ParentBlock.vLogSize].Mips[NewFreeMip], EBlockState::PartiallyFreeList, ParentIndex);
		}

		ParentIndex = ParentBlock.Parent;
	}
}

void FVirtualTextureAllocator::MarkBlockAllocated(uint32 Index, uint32 vAllocatedTileX0, uint32 vAllocatedTileY0, FAllocatedVirtualTexture* VT)
{
	FAddressBlock* AllocBlock = &AddressBlocks[Index];
	check(AllocBlock->State != EBlockState::None);
	check(AllocBlock->State != EBlockState::GlobalFreeList);

	const uint32 vLogSize = AllocBlock->vLogSize;

	// check to see if block is in the correct position
	const uint32 vAllocatedTileX1 = vAllocatedTileX0 + VT->GetWidthInTiles();
	const uint32 vAllocatedTileY1 = vAllocatedTileY0 + VT->GetHeightInTiles();
	const uint32 BlockSize = (1u << vLogSize);
	const uint32 vBlockAddress = AllocBlock->vAddress;
	const uint32 vBlockTileX0 = FMath::ReverseMortonCode2(vBlockAddress);
	const uint32 vBlockTileY0 = FMath::ReverseMortonCode2(vBlockAddress >> 1);
	const uint32 vBlockTileX1 = vBlockTileX0 + BlockSize;
	const uint32 vBlockTileY1 = vBlockTileY0 + BlockSize;

	if (vAllocatedTileX1 > vBlockTileX0 &&
		vAllocatedTileX0 < vBlockTileX1 &&
		vAllocatedTileY1 > vBlockTileY0 &&
		vAllocatedTileY0 < vBlockTileY1)
	{
		// Block overlaps the VT we're trying to allocate
		if (vBlockTileX0 >= vAllocatedTileX0 &&
			vBlockTileX1 <= vAllocatedTileX1 &&
			vBlockTileY0 >= vAllocatedTileY0 &&
			vBlockTileY1 <= vAllocatedTileY1)
		{
			// Block is entirely contained within the VT we're trying to allocate

			// In this case, block must be completely free (or else there's an error somewhere else)
			check(AllocBlock->FirstChild == 0xffff);
			UnlinkFreeList(FreeList[vLogSize], EBlockState::FreeList, Index);
			
			++NumAllocations;
			NumAllocatedPages += 1u << (vDimensions * vLogSize);

			// Add to hash table
			uint16 Key = reinterpret_cast<UPTRINT>(VT) / 16;
			HashTable.Add(Key, Index);

			AllocBlock->VT = VT;
			AllocBlock->State = EBlockState::AllocatedTexture;

			FreeMipUpdateParents(AllocBlock->Parent);
		}
		else
		{
			// Block intersects the VT
			if (AllocBlock->State == EBlockState::FreeList)
			{
				// If block is completely free, need to subdivide further
				SubdivideBlock(Index);
			}
			// otherwise block is already subdivided
			AllocBlock = nullptr; // list will be potentially reallocated
			check(AddressBlocks[Index].State == EBlockState::PartiallyFreeList);
	
			uint32 NumChildren = 0u;
			uint16 ChildIndex = AddressBlocks[Index].FirstChild;
			check(ChildIndex == AddressBlocks[ChildIndex].FirstSibling);
			while (ChildIndex != 0xffff)
			{
				check(AddressBlocks[ChildIndex].Parent == Index);

				MarkBlockAllocated(ChildIndex, vAllocatedTileX0, vAllocatedTileY0, VT);

				ChildIndex = AddressBlocks[ChildIndex].NextSibling;
				NumChildren++;
			}
			check(NumChildren == (1u << vDimensions));
		}
	}
}

bool FVirtualTextureAllocator::TestAllocation(uint32 Index, uint32 vAllocatedTileX0, uint32 vAllocatedTileY0, uint32 vAllocatedTileX1, uint32 vAllocatedTileY1) const
{
	const FAddressBlock& AllocBlock = AddressBlocks[Index];
	const uint32 vLogSize = AllocBlock.vLogSize;
	const uint32 BlockSize = (1u << vLogSize);

	const uint32 vBlockAddress = AllocBlock.vAddress;
	const uint32 vBlockTileX0 = FMath::ReverseMortonCode2(vBlockAddress);
	const uint32 vBlockTileY0 = FMath::ReverseMortonCode2(vBlockAddress >> 1);
	const uint32 vBlockTileX1 = vBlockTileX0 + BlockSize;
	const uint32 vBlockTileY1 = vBlockTileY0 + BlockSize;

	if (vAllocatedTileX1 > vBlockTileX0 &&
		vAllocatedTileX0 < vBlockTileX1 &&
		vAllocatedTileY1 > vBlockTileY0 &&
		vAllocatedTileY0 < vBlockTileY1)
	{
		// Block overlaps the VT we're trying to allocate
		if (AllocBlock.State == EBlockState::AllocatedTexture)
		{
			return false;
		}
		else
		{
			check(AllocBlock.State == EBlockState::PartiallyFreeList);
			if (vBlockTileX0 >= vAllocatedTileX0 &&
				vBlockTileX1 <= vAllocatedTileX1 &&
				vBlockTileY0 >= vAllocatedTileY0 &&
				vBlockTileY1 <= vAllocatedTileY1)
			{
				// If block is fully contained within the check region, don't need to search children, we are guaranteed to find an intersection
				return false;
			}

			uint16 ChildIndex = AddressBlocks[Index].FirstChild;
			check(ChildIndex == AddressBlocks[ChildIndex].FirstSibling);
			while (ChildIndex != 0xffff)
			{
				const FAddressBlock& ChildBlock = AddressBlocks[ChildIndex];
				check(ChildBlock.Parent == Index);
				if (ChildBlock.State != EBlockState::FreeList &&
					!TestAllocation(ChildIndex, vAllocatedTileX0, vAllocatedTileY0, vAllocatedTileX1, vAllocatedTileY1))
				{
					return false;
				}
				ChildIndex = ChildBlock.NextSibling;
			}
		}
	}

	return true;
}

uint32 FVirtualTextureAllocator::Alloc(FAllocatedVirtualTexture* VT)
{
	const uint32 WidthInTiles = VT->GetWidthInTiles();
	const uint32 HeightInTiles = VT->GetHeightInTiles();
	const uint32 MinSize = FMath::Min(WidthInTiles, HeightInTiles);
	const uint32 MaxSize = FMath::Max(WidthInTiles, HeightInTiles);
	const int32 vLogMinSize = FMath::CeilLogTwo(MinSize);
	const int32 vLogMaxSize = FMath::CeilLogTwo(MaxSize);

	// Tile must be aligned to match the max level of the VT, otherwise tiles at lower mip levels may intersect neighboring regions
	const uint32 MaxLevel = VT->GetMaxLevel();
	const uint32 vAddressAlignment = 1u << (vDimensions * MaxLevel);

	if (vLogMaxSize >= FreeList.Num())
	{
		// VT is larger than the entire page table
		return ~0u;
	}

	uint16 AllocIndex = 0xffff;
	uint32 vAddress = ~0u;

	// See if we have any completely free blocks big enough
	// Here we search all free blocks, including ones that are too large (large blocks will still be subdivided to fit)
	for (int32 vLogSize = vLogMaxSize; vLogSize < FreeList.Num(); ++vLogSize)
	{
		// Could avoid this loop if FreeList was kept sorted by vAddress
		uint16 FreeIndex = FreeList[vLogSize];
		while (FreeIndex != 0xffff)
		{
			const FAddressBlock& AllocBlock = AddressBlocks[FreeIndex];
			check(AllocBlock.State == EBlockState::FreeList);
			if (AllocBlock.vAddress < vAddress)
			{
				AllocIndex = FreeIndex;
				vAddress = AllocBlock.vAddress;
			}
			FreeIndex = AllocBlock.NextFree;
		}
	}

	// Look for a partially allocated block that has room for this allocation.  Only need to check partially allocated blocks
	// of the correct size, which could allocate a sub-block of MaxLevel alignment.  Larger partially allocated blocks will
	// contain a child block that's completely free, that will already be discovered by the above search.
	check((uint32)vLogMaxSize >= MaxLevel);
	uint32 FreeMipCount = FMath::Min(3u, vLogMaxSize - MaxLevel);

	for (uint32 FreeMipIndex = 0; FreeMipIndex <= FreeMipCount; FreeMipIndex++)
	{
		uint16 FreeIndex = PartiallyFreeList[vLogMaxSize].Mips[FreeMipIndex];
		while (FreeIndex != 0xffff)
		{
			FAddressBlock& AllocBlock = AddressBlocks[FreeIndex];

			// TODO:  consider making "ComputeFreeMip" test a checkSlow once system is proven reliable
			check((AllocBlock.FreeMip == FreeMipIndex) && (AllocBlock.FreeMip == ComputeFreeMip(FreeIndex)));

			if (AllocBlock.vAddress < vAddress)
			{
				check(AllocBlock.State == EBlockState::PartiallyFreeList);
				const uint32 BlockSize = 1u << vLogMaxSize;
				uint32 vCheckAddress = AllocBlock.vAddress;
				const uint32 vBlockTileX0 = FMath::ReverseMortonCode2(vCheckAddress);
				const uint32 vBlockTileY0 = FMath::ReverseMortonCode2(vCheckAddress >> 1);
				const uint32 vBlockTileX1 = vBlockTileX0 + BlockSize;
				const uint32 vBlockTileY1 = vBlockTileY0 + BlockSize;

				// Search all valid positions within the block (in ascending morton order), looking for a fit for the texture we're trying to allocate
				// Step size is driven by our alignment requirements
				// Seems like there is probably a more clever way to accomplish this, if perf becomes an issue
				while (true)
				{
					const uint32 vTileX0 = FMath::ReverseMortonCode2(vCheckAddress);
					const uint32 vTileY0 = FMath::ReverseMortonCode2(vCheckAddress >> 1);
					const uint32 vTileX1 = vTileX0 + WidthInTiles;
					const uint32 vTileY1 = vTileY0 + HeightInTiles;
					if (vTileY1 > vBlockTileY1)
					{
						break;
					}

					if (vTileX1 <= vBlockTileX1)
					{
						if (TestAllocation(FreeIndex, vTileX0, vTileY0, vTileX1, vTileY1))
						{
							// here AllocIndex won't point to exactly the correct block yet, but we don't want to subdivide yet, until we're sure this is the best fit
							// MarkBlockAllocated will properly subdivide the initial block as needed
							AllocIndex = FreeIndex;
							vAddress = vCheckAddress;
							break;
						}
					}

					vCheckAddress += vAddressAlignment;
				}
			}
			FreeIndex = AllocBlock.NextFree;
		}
	}

	if (AllocIndex != 0xffff)
	{
		check(vAddress != ~0u);
		FAddressBlock& AllocBlock = AddressBlocks[AllocIndex];
		const uint32 vTileX = FMath::ReverseMortonCode2(vAddress);
		const uint32 vTileY = FMath::ReverseMortonCode2(vAddress >> 1);

		MarkBlockAllocated(AllocIndex, vTileX, vTileY, VT);

		check(AddressBlocks[AllocIndex].State != EBlockState::FreeList);

		// Make sure we allocate enough space in the backing texture so all the mip levels fit
		const uint32 SizeAlign = 1u << MaxLevel;
		const uint32 AlignedWidthInTiles = Align(WidthInTiles, SizeAlign);
		const uint32 AlignedHeightInTiles = Align(HeightInTiles, SizeAlign);

		AllocatedWidth = FMath::Max(AllocatedWidth, vTileX + AlignedWidthInTiles);
		AllocatedHeight = FMath::Max(AllocatedHeight, vTileY + AlignedHeightInTiles);
	}

	return vAddress;
}

void FVirtualTextureAllocator::Free(FAllocatedVirtualTexture* VT)
{
	// Find block index
	uint16 Key = reinterpret_cast<UPTRINT>(VT) / 16;
	uint32 Index = HashTable.First(Key);
	while (HashTable.IsValid(Index))
	{
		const uint32 NextIndex = HashTable.Next(Index);
		FAddressBlock& AddressBlock = AddressBlocks[Index];
		if (AddressBlock.VT == VT)
		{
			check(AddressBlock.State == EBlockState::AllocatedTexture);
			check(AddressBlock.FirstChild == 0xffff); // texture allocation should be leaf
			AddressBlock.State = EBlockState::None;
			AddressBlock.VT = nullptr;

			// TODO (perf):  This is slightly wasteful for textures composed of multiple blocks,
			// as it will update the same parents more than once.  Many textures only have one
			// block, so perhaps this isn't a large overhead.
			FreeMipUpdateParents(AddressBlock.Parent);

			check(NumAllocations > 0u);
			--NumAllocations;

			const uint32 NumPagesForBlock = 1u << (vDimensions * AddressBlock.vLogSize);
			check(NumAllocatedPages >= NumPagesForBlock);
			NumAllocatedPages -= NumPagesForBlock;

			// Add block to free list
			// This handles merging free siblings
			FreeAddressBlock(Index, true);

			// Remove the index from the hash table as it may be reused later
			HashTable.Remove(Key, Index);
		}
		Index = NextIndex;
	}
}

void FVirtualTextureAllocator::FreeAddressBlock(uint32 Index, bool bTopLevelBlock)
{
	FAddressBlock& AddressBlock = AddressBlocks[Index];
	if (bTopLevelBlock)
	{
		// Block was freed directly, should already be removed froms lists
		check(AddressBlock.State == EBlockState::None);
	}
	else
	{
		// Block was freed by consolidating children
		UnlinkFreeList(PartiallyFreeList[AddressBlock.vLogSize].Mips[AddressBlock.FreeMip], EBlockState::PartiallyFreeList, Index);
	}

	check(AddressBlock.VT == nullptr);
	check(AddressBlock.NextFree == 0xffff);
	check(AddressBlock.PrevFree == 0xffff);

	// If we got here, the block's children have already been consolidated/removed
	AddressBlock.FirstChild = 0xffff;

	// If all siblings are free then we can merge them
	uint32 SiblingIndex = AddressBlock.FirstSibling;
	bool bConsolidateSiblings = SiblingIndex != 0xffff;
	while (bConsolidateSiblings && SiblingIndex != 0xffff)
	{
		const FAddressBlock& SiblingBlock = AddressBlocks[SiblingIndex];
		if (SiblingIndex != Index)
		{
			check(SiblingBlock.State != EBlockState::None);
			check(SiblingBlock.State != EBlockState::GlobalFreeList);
			bConsolidateSiblings &= (SiblingBlock.State == EBlockState::FreeList);
		}
		SiblingIndex = SiblingBlock.NextSibling;
	}

	if (!bConsolidateSiblings)
	{
		// Simply place this block on the free list
		LinkFreeList(FreeList[AddressBlock.vLogSize], EBlockState::FreeList, Index);
	}
	else
	{
		// Remove all of this block's siblings from free list and add to global free list
		uint32 FreeIndex = AddressBlock.FirstSibling;
		while (FreeIndex != 0xffff)
		{
			FAddressBlock& FreeBlock = AddressBlocks[FreeIndex];

			if (FreeIndex != Index)
			{
				// All our siblings must be free (we checked above to get into this case)
				UnlinkFreeList(FreeList[AddressBlock.vLogSize], EBlockState::FreeList, FreeIndex);
			}

			LinkFreeList(GlobalFreeList, EBlockState::GlobalFreeList, FreeIndex);

			FreeIndex = FreeBlock.NextSibling;
		}

		check(AddressBlock.State == EBlockState::GlobalFreeList);

		// Remove this block and its siblings from the sorted lists
		// We can assume that the sibling blocks are sequential in the sorted list since they are free and so have no children
		// FirstSibling will be the last in the range of siblings in the sorted lists 
		const uint32 SortedIndexRangeEnd = FindAddressBlock(AddressBlocks[AddressBlock.FirstSibling].vAddress);
		check(SortedAddresses[SortedIndexRangeEnd] == AddressBlocks[AddressBlock.FirstSibling].vAddress);
		const uint32 NumSiblings = 1 << vDimensions;
		check(SortedIndexRangeEnd + 1 >= NumSiblings);
		const uint32 SortedIndexRangeStart = SortedIndexRangeEnd + 1 - NumSiblings;

		// Remove all but one siblings because...
		SortedAddresses.RemoveAt(SortedIndexRangeStart, NumSiblings - 1, EAllowShrinking::No);
		SortedIndices.RemoveAt(SortedIndexRangeStart, NumSiblings - 1, EAllowShrinking::No);
		// ... we replace first sibling with parent
		SortedIndices[SortedIndexRangeStart] = AddressBlock.Parent;
		check(SortedAddresses[SortedIndexRangeStart] == AddressBlocks[AddressBlock.Parent].vAddress);

		// Add parent block to free list (and possibly consolidate)
		FreeAddressBlock(AddressBlock.Parent, false);
	}
}

void FVirtualTextureAllocator::RecurseComputeFreeMip(uint16 BlockIndex, uint32 Depth, uint64& IoBlockMap) const
{
	const FAddressBlock& Block = AddressBlocks[BlockIndex];

	// Add children first...
	if (Depth < 3)
	{
		uint32 ChildIndex = Block.FirstChild;
		while (ChildIndex != 0xffff)
		{
			RecurseComputeFreeMip(ChildIndex, Depth + 1, IoBlockMap);
			ChildIndex = AddressBlocks[ChildIndex].NextSibling;
		}
	}

	if (Block.State == EBlockState::AllocatedTexture)
	{
		// Bit mask of pixels covered by a block of the given log2 size.  Think of each byte
		// as a row of 8 single bit pixels, moving left in bits representing increasing X, and bytes
		// increasing Y, similar to how a linear 2D array of texels is usually arranged.  Here's a
		// diagram of pixels that are covered by increasing powers of 2, and the corresponding bytes,
		// to show where the entries in the table come from:
		//
		//								1x1     2x2     4x4     8x8
		//        1 2 4 4 8 8 8 8		0x01    0x03    0x0f    0xff
		//        2 2 4 4 8 8 8 8		0x00    0x03    0x0f    0xff
		//        4 4 4 4 8 8 8 8		0x00    0x00    0x0f    0xff
		//        4 4 4 4 8 8 8 8		0x00    0x00    0x0f    0xff
		//        8 8 8 8 8 8 8 8		0x00    0x00    0x00    0xff
		//        8 8 8 8 8 8 8 8		0x00    0x00    0x00    0xff
		//        8 8 8 8 8 8 8 8		0x00    0x00    0x00    0xff
		//        8 8 8 8 8 8 8 8		0x00    0x00    0x00    0xff
		//
		static const uint64 BlockMaskByDepth[4] =
		{
			0xffffffffffffffffull,		// 8x8 block
			0x000000000f0f0f0full,		// 4x4 block
			0x0000000000000303ull,		// 2x2 block
			0x0000000000000001ull,		// 1x1 block
		};

		// Absolute address
		uint32 X = FMath::ReverseMortonCode2(Block.vAddress);
		uint32 Y = FMath::ReverseMortonCode2(Block.vAddress >> 1);

		// Parent block relative address.  Let's say the original parent block was 32 by 32 -- these
		// bit mask operations would mask out the bottom 5 bits of the address for any given block,
		// producing offsets in the range [0..32).  For the original parent, Depth will be zero, and
		// Block.vLogSize would be 5, so the mask generated would be (1 << (5 + 0)) - 1 == 0x1f.  For
		// the next child, Block.vLogSize would be 4, and depth 1, generating the same 0x1f mask, and
		// so on.
		uint32 RelativeX = X & ((1 << (Block.vLogSize + Depth)) - 1);
		uint32 RelativeY = Y & ((1 << (Block.vLogSize + Depth)) - 1);

		// Mapping address (8x8).  For our 32x32 case, we want to map our [0..32) range to [0..8), which
		// requires us to divide by 32 and multiply by 8.  In bit shifts, that means a right shift by
		// Block.vLogSize + Depth == 5, and a left shift by 3.  If the original tile is smaller than 8x8,
		// the shift could go negative, so we need to handle that case.
		int32 MapShift = (Block.vLogSize + Depth - 3);
		uint32 MapX;
		uint32 MapY;
		if (MapShift >= 0)
		{
			MapX = RelativeX >> MapShift;
			MapY = RelativeY >> MapShift;
		}
		else
		{
			MapX = RelativeX << (-MapShift);
			MapY = RelativeY << (-MapShift);
		}

		// Set bits in our bitmap
		uint32 BitIndex = MapX + MapY * 8;
		IoBlockMap |= BlockMaskByDepth[Depth] << BitIndex;
	}
}

uint32 FVirtualTextureAllocator::ComputeFreeMip(uint16 BlockIndex) const
{
	// First we need to generate a map of the block, where data is allocated.  This is an
	// 8x8 pixel map in bits in a single 64-bit word.
	// 
	// TODO (perf):  It might be interesting to actually store this map in the block itself,
	// since we've spent the time computing it, as you could use it to early out in
	// "TestAllocation".
	uint64 BlockMap = 0;
	RecurseComputeFreeMip(BlockIndex, 0, BlockMap);

	// Mapping that specifies pixels that need to be covered by child blocks to block any allocation at the
	// given alignment.  If the corner pixel at a given resolution alone is covered, we can't allocate,
	// because aligned block addresses always start at pixel corners.  But if not, there is potential to
	// squeeze an allocation in there (at least a 1x1 if nothing else).  For finer granularity alignments,
	// we need to check multiple pixel corners to cover all allocation offsets.  Allocation offsets
	// correspond to steps by the "vAddressAlignment" variable in FVirtualTextureAllocator::Alloc.  Here's
	// a diagram of the pixels we are looking at, and the corresponding bytes.  X increases as you go left
	// in bits, and Y increases as you go left in bytes:
	//
	//								1x1     2x2     4x4     8x8
	//        8 1 2 1 4 1 2 1		0xff    0x33    0x11    0x01
	//        1 1 1 1 1 1 1 1		0xff    0x00    0x00    0x00
	//        2 1 2 1 2 1 2 1		0xff    0x33    0x00    0x00
	//        1 1 1 1 1 1 1 1		0xff    0x00    0x00    0x00
	//        4 1 2 1 4 1 2 1		0xff    0x33    0x11    0x00
	//        1 1 1 1 1 1 1 1		0xff    0x00    0x00    0x00
	//        2 1 2 1 2 1 2 1		0xff    0x33    0x00    0x00
	//        1 1 1 1 1 1 1 1		0xff    0x00    0x00    0x00
	//
	static const uint64 BlockOverlapByDepth[4] =
	{
		0x0000000000000001ull,		// 8x8 block
		0x0000001100000011ull,		// 4x4 block
		0x0033003300330033ull,		// 2x2 block
		0xffffffffffffffffull,		// 1x1 block (unused by code, just here for reference)
	};

	uint32 FreeMip;
	for (FreeMip = 0; FreeMip < 3; FreeMip++)
	{
		// Are all the corner pixels at this alignment resolution covered?  If not,
		// break out of the loop.
		if ((BlockOverlapByDepth[FreeMip] & BlockMap) != BlockOverlapByDepth[FreeMip])
		{
			break;
		}
	}

	return FreeMip;
}

void FVirtualTextureAllocator::DumpToConsole(bool verbose)
{
	for (int32 BlockID = SortedIndices.Num() - 1; BlockID >= 0; BlockID--)
	{
		FAddressBlock& Block = AddressBlocks[SortedIndices[BlockID]];
		uint32 X = FMath::ReverseMortonCode2(Block.vAddress);
		uint32 Y = FMath::ReverseMortonCode2(Block.vAddress >> 1);
		uint32 Size = 1 << Block.vLogSize;

		UE_LOGF(LogVirtualTexturing, Display, "Block: vAddress %i,%i, size: %ix%i (tiles),  ", X, Y, Size, Size);
		if (Block.VT != nullptr)
		{
			if (verbose)
			{
				UE_LOGF(LogVirtualTexturing, Display, "%p", Block.VT);
			}
			Block.VT->DumpToConsole(verbose);
		}
		else
		{
			if (verbose)
			{
				UE_LOGF(LogVirtualTexturing, Display, "NULL VT");
			}
		}
	}
}

#if WITH_EDITOR
void FVirtualTextureAllocator::FillDebugImage(uint32 Index, uint32* ImageData, TMap<FAllocatedVirtualTexture*, uint32>& ColorMap) const
{
	const FAddressBlock& AddressBlock = AddressBlocks[Index];
	if (AddressBlock.State == EBlockState::AllocatedTexture || AddressBlock.State == EBlockState::FreeList)
	{
		const uint32 vTileX = FMath::ReverseMortonCode2(AddressBlock.vAddress);
		const uint32 vTileY = FMath::ReverseMortonCode2(AddressBlock.vAddress >> 1);
		const uint32 BlockSize = (1u << AddressBlock.vLogSize);

		if (vTileX + BlockSize <= AllocatedWidth && vTileY + BlockSize <= AllocatedHeight)
		{
			uint32 Color;
			uint32 BorderColor;
			if (AddressBlock.State == EBlockState::FreeList)
			{
				// Free blocks are black, with grey border
				Color = FColor::Black.ToPackedABGR();
				BorderColor = FColor(100u, 100u, 100u).ToPackedABGR();
			}
			else
			{
				// Allocated blocks have white border, random color for each AllocatedVT
				uint32* FoundColor = ColorMap.Find(AddressBlock.VT);
				if (!FoundColor)
				{
					FoundColor = &ColorMap.Add(AddressBlock.VT, FColor::MakeRandomColor().ToPackedABGR());
				}
				Color = *FoundColor;
				BorderColor = FColor::White.ToPackedABGR();
			}

			// Add top/bottom borders
			for (uint32 X = 0u; X < BlockSize; ++X)
			{
				const uint32 ImageY0 = vTileY;
				const uint32 ImageY1 = vTileY + BlockSize - 1u;
				const uint32 ImageX = vTileX + X;
				ImageData[ImageY0 * AllocatedWidth + ImageX] = BorderColor;
				ImageData[ImageY1 * AllocatedWidth + ImageX] = BorderColor;
			}

			for (uint32 Y = 1u; Y < BlockSize - 1u; ++Y)
			{
				const uint32 ImageY = vTileY + Y;

				// Add left/right borders
				ImageData[ImageY * AllocatedWidth + vTileX] = BorderColor;
				ImageData[ImageY * AllocatedWidth + vTileX + BlockSize - 1u] = BorderColor;

				for (uint32 X = 1u; X < BlockSize - 1u; ++X)
				{
					const uint32 ImageX = vTileX + X;
					ImageData[ImageY * AllocatedWidth + ImageX] = Color;
				}
			}
		}
		else
		{
			// If block is outside allocated size, it must be free
			check(AddressBlock.State == EBlockState::FreeList);
		}
	}
	else if (AddressBlock.State == EBlockState::PartiallyFreeList)
	{
		uint32 ChildIndex = AddressBlock.FirstChild;
		while (ChildIndex != 0xffff)
		{
			FillDebugImage(ChildIndex, ImageData, ColorMap);
			ChildIndex = AddressBlocks[ChildIndex].NextSibling;
		}
	}
	else
	{
		// Blocks of this state should not be in the tree
		checkf(false, TEXT("Invalid block state %d"), (int32)AddressBlock.State);
	}
}

void FVirtualTextureAllocator::SaveDebugImage(const TCHAR* ImageName) const
{
	const uint32 EmptyColor = FColor(255, 0, 255).ToPackedABGR();
	TArray<uint32> ImageData;
	ImageData.Init(EmptyColor, AllocatedWidth * AllocatedHeight);
	TMap<FAllocatedVirtualTexture*, uint32> ColorMap;
	FillDebugImage(RootIndex, ImageData.GetData(), ColorMap);

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);
	ImageWrapper->SetRaw(ImageData.GetData(), ImageData.Num() * 4, AllocatedWidth, AllocatedHeight, ERGBFormat::RGBA, 8);

	// Compress and write image
	IFileManager& FileManager = IFileManager::Get();
	const FString BasePath = FPaths::ProjectUserDir();
	const FString ImagePath = BasePath / ImageName;
	FArchive* Ar = FileManager.CreateFileWriter(*ImagePath);
	if (Ar)
	{
		const TArray64<uint8>& CompressedData = ImageWrapper->GetCompressed();
		Ar->Serialize((void*)CompressedData.GetData(), CompressedData.Num());
		delete Ar;
	}
}
#endif // WITH_EDITOR
