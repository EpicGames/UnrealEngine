// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/HashTable.h"
#include "VirtualTextureShared.h"

class FAllocatedVirtualTexture;

/** Allocates virtual memory address space. */
class FVirtualTextureAllocator
{
public:
	RENDERER_API explicit FVirtualTextureAllocator(uint32 Dimensions);
	~FVirtualTextureAllocator() = default;

	/**
	 * Initialise the allocator
	 */
	RENDERER_API void Initialize(uint32 MaxSize);

	uint32 GetAllocatedWidth() const { return AllocatedWidth; }
	uint32 GetAllocatedHeight() const { return AllocatedHeight; }

	/**
	 * Translate a virtual page address in the address space to a local page address within a virtual texture.
	 * @return nullptr If there is no virtual texture allocated at this address.
	 */
	RENDERER_API FAllocatedVirtualTexture* Find(uint32 vAddress, uint32& OutLocal_vAddress) const;
	inline FAllocatedVirtualTexture* Find(uint32 vAddress) const { uint32 UnusedLocal_vAddress = 0u; return Find(vAddress, UnusedLocal_vAddress); }

	/**
	 * Allocate address space for the virtual texture.
	 * @return (~0) if no space left, the virtual page address if successfully allocated.
	 */
	RENDERER_API uint32 Alloc(FAllocatedVirtualTexture* VT);

	/**
	 * Test if an allocation of the given size will succeed.
	 * @return false if there isn't enough space left.
	 */
	RENDERER_API bool TryAlloc(uint32 InSize);

	/**
	 * Free the virtual texture.
	 */
	RENDERER_API void Free(FAllocatedVirtualTexture* VT);

	/** Get current number of allocations. */
	inline uint32 GetNumAllocations() const { return NumAllocations; }

	/** Get current number of allocated pages. */
	inline uint32 GetNumAllocatedPages() const { return NumAllocatedPages; }

	/** Output debugging information to the console. */
	RENDERER_API void DumpToConsole(bool verbose);

#if WITH_EDITOR
	RENDERER_API void SaveDebugImage(const TCHAR* ImageName) const;
#endif

private:
	enum class EBlockState : uint8
	{
		None,
		GlobalFreeList,
		FreeList,
		PartiallyFreeList,
		AllocatedTexture,
	};

	struct FTestRegion
	{
		uint32 BaseIndex;
		uint32 vTileX0;
		uint32 vTileY0;
		uint32 vTileX1;
		uint32 vTileY1;
	};

	RENDERER_API void LinkFreeList(uint16& InOutListHead, EBlockState State, uint16 Index);
	RENDERER_API void UnlinkFreeList(uint16& InOutListHead, EBlockState State, uint16 Index);

	RENDERER_API int32 AcquireBlock();
	RENDERER_API void FreeAddressBlock(uint32 Index, bool bTopLevelBlock);
	RENDERER_API uint32 FindAddressBlock(uint32 vAddress) const;

	RENDERER_API void SubdivideBlock(uint32 ParentIndex);

	RENDERER_API void MarkBlockAllocated(uint32 Index, uint32 vAllocatedTileX, uint32 vAllocatedTileY, FAllocatedVirtualTexture* VT);

	RENDERER_API bool TestAllocation(uint32 Index, uint32 vTileX0, uint32 vTileY0, uint32 vTileX1, uint32 vTileY1) const;

	RENDERER_API void RecurseComputeFreeMip(uint16 BlockIndex, uint32 Depth, uint64& IoBlockMap) const;
	RENDERER_API uint32 ComputeFreeMip(uint16 BlockIndex) const;

	RENDERER_API void FreeMipUpdateParents(uint16 ParentIndex);

#if WITH_EDITOR
	RENDERER_API void FillDebugImage(uint32 Index, uint32* ImageData, TMap<FAllocatedVirtualTexture*, uint32>& ColorMap) const;
#endif

	struct FAddressBlock
	{
		FAllocatedVirtualTexture*	VT;
		uint32						vAddress : 24;
		uint32						vLogSize : 4;
		uint32						MipBias : 4;
		uint16						Parent;
		uint16						FirstChild;
		uint16						FirstSibling;
		uint16						NextSibling;
		uint16						NextFree;
		uint16						PrevFree;
		EBlockState                 State;
		uint8						FreeMip;

		FAddressBlock()
		{}

		FAddressBlock(uint8 LogSize)
			: VT(nullptr)
			, vAddress(0)
			, vLogSize(LogSize)
			, MipBias(0)
			, Parent(0xffff)
			, FirstChild(0xffff)
			, FirstSibling(0xffff)
			, NextSibling(0xffff)
			, NextFree(0xffff)
			, PrevFree(0xffff)
			, State(EBlockState::None)
			, FreeMip(0)
		{}

		FAddressBlock(const FAddressBlock& Block, uint32 Offset, uint32 Dimensions)
			: VT(nullptr)
			, vAddress(Block.vAddress + (Offset << (Dimensions * Block.vLogSize)))
			, vLogSize(Block.vLogSize)
			, MipBias(0)
			, Parent(Block.Parent)
			, FirstChild(0xffff)
			, FirstSibling(Block.FirstSibling)
			, NextSibling(0xffff)
			, NextFree(0xffff)
			, PrevFree(0xffff)
			, State(EBlockState::None)
			, FreeMip(0)
		{}
	};

	// We separate items in the partially free list by the alignment of sub-allocation that can
	// potentially fit, based on what's already allocated.  A level of zero means a block could
	// be allocated at 1x1 tile alignment, one means 2x2, etc.  It caps at level 3 (8x8) to
	// keep the cost of updating this information manageable.  Capping it means we don't need
	// to recurse too far into children or update too many parents when a block's state changes.
	static const uint32 PartiallyFreeMipDepth = 4;

	struct FPartiallyFreeMip
	{
		uint16	Mips[PartiallyFreeMipDepth];
	};

	const uint32				vDimensions;
	uint32						AllocatedWidth;
	uint32						AllocatedHeight;

	TArray< FAddressBlock >		AddressBlocks;
	TArray< uint16 >			FreeList;
	TArray< FPartiallyFreeMip >	PartiallyFreeList;
	uint16						GlobalFreeList;
	TArray< uint32 >			SortedAddresses;
	TArray< uint16 >			SortedIndices;
	FHashTable					HashTable;
	uint16						RootIndex;

	uint32						NumAllocations;
	uint32						NumAllocatedPages;
};
