// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTextureFeedback.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "VirtualTexturing.h"

int32 GVirtualTextureFeedbackLatency = 3;
static FAutoConsoleVariableRef CVarVirtualTextureFeedbackLatency(
	TEXT("r.vt.FeedbackLatency"),
	GVirtualTextureFeedbackLatency,
	TEXT("How much latency to allow in the GPU feedback pipeline before we start mapping multiple buffers to catch up."),
	ECVF_RenderThreadSafe);

DECLARE_DWORD_COUNTER_STAT(TEXT("Num Feedback Pending"), STAT_VirtualTexture_PendingFeedback, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Feedback Writes"), STAT_VirtualTexture_WriteFeedback, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Feedback Reads"), STAT_VirtualTexture_ReadFeedback, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Feedback Lost Buffers"), STAT_VirtualTexture_LostFeedback, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Feedback Lost Pages"), STAT_VirtualTexture_ReadFeedbackLostPage, STATGROUP_VirtualTexturing);

/** Container for GPU fences. */
class FFeedbackGPUFencePool
{
public:
	TArray<FGPUFenceRHIRef> Fences;

	FFeedbackGPUFencePool(int32 NumFences)
	{
		Fences.AddDefaulted(NumFences);
	}

	void InitRHI(FRHICommandListBase& RHICmdList)
	{
	}

	void ReleaseRHI()
	{
		for (int i = 0; i < Fences.Num(); ++i)
		{
			Fences[i].SafeRelease();
		}
	}

	void Allocate(FRHICommandList& RHICmdList, int32 Index)
	{
		if (!Fences[Index])
		{
			Fences[Index] = RHICreateGPUFence(FName());
		}
		Fences[Index]->Clear();
	}

	void Write(FRHICommandList& RHICmdList, int32 Index)
	{
		RHICmdList.WriteGPUFence(Fences[Index]);
	}

	bool Poll(FRHICommandList& RHICmdList, int32 Index)
	{
		return Fences[Index]->Poll(RHICmdList.GetGPUMask());
	}

	FGPUFenceRHIRef GetMapFence(int32 Index)
	{
		return Fences[Index];
	}

	void Release(int32 Index)
	{
		Fences[Index].SafeRelease();
	}
};


FVirtualTextureFeedback::FVirtualTextureFeedback()
	: NumPending(0)
	, WriteIndex(0)
	, ReadIndex(0)
{
	Fences = new FFeedbackGPUFencePool(MaxTransfers);
}

FVirtualTextureFeedback::~FVirtualTextureFeedback()
{
	delete Fences;
}

void FVirtualTextureFeedback::InitRHI(FRHICommandListBase& RHICmdList)
{
	for (int32 Index = 0; Index < MaxTransfers; ++Index)
	{
		FeedbackItems[Index].StagingBuffer = RHICreateStagingBuffer();
	}

	Fences->InitRHI(RHICmdList);
}

void FVirtualTextureFeedback::ReleaseRHI()
{
	for (int32 Index = 0; Index < MaxTransfers; ++Index)
	{
		FeedbackItems[Index].StagingBuffer.SafeRelease();
	}

	Fences->ReleaseRHI();
}

void FVirtualTextureFeedback::TransferGPUToCPU(FRHICommandList& RHICmdList, FBufferRHIRef const& Buffer, FVirtualTextureFeedbackBufferDesc const& Desc)
{
	// Validate that we don't have an empty buffer.
	if (!ensure(Desc.BufferSize > 0))
	{
		return;
	}

	INC_DWORD_STAT(STAT_VirtualTexture_WriteFeedback);

	if (NumPending >= MaxTransfers)
	{
		// If we have too many pending transfers, start throwing away the oldest in the ring buffer.
		// We will need to allocate a new fence, since the previous fence will still be set on the old CopyToResolveTarget command (which we will now ignore/discard).
		INC_DWORD_STAT(STAT_VirtualTexture_LostFeedback);

		Fences->Release(ReadIndex);
		NumPending --;
		ReadIndex = (ReadIndex + 1) % MaxTransfers;
	}

	FFeedbackItem& FeedbackItem = FeedbackItems[WriteIndex];
	FeedbackItem.Desc = Desc;

	// We only need to transfer 1 copy of the data, so restrict mask to the first active GPU.
	FeedbackItem.GPUMask = FRHIGPUMask::FromIndex(RHICmdList.GetGPUMask().GetFirstIndex());
	SCOPED_GPU_MASK(RHICmdList, FeedbackItem.GPUMask);

	const uint32 FeedbackStride = Desc.bPageAndCount ? 2 : 1;
	RHICmdList.CopyToStagingBuffer(Buffer, FeedbackItem.StagingBuffer, 0, Desc.BufferSize * FeedbackStride * sizeof(uint32));

	Fences->Allocate(RHICmdList, WriteIndex);
	Fences->Write(RHICmdList, WriteIndex);

	// Increment the ring buffer write position.
	WriteIndex = (WriteIndex + 1) % MaxTransfers;
	++NumPending;
}

BEGIN_SHADER_PARAMETER_STRUCT(FVirtualTextureFeedbackCopyParameters, )
	RDG_BUFFER_ACCESS(Input, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

void FVirtualTextureFeedback::TransferGPUToCPU(FRDGBuilder& GraphBuilder, FRDGBuffer* Buffer, FVirtualTextureFeedbackBufferDesc const& Desc)
{
	FVirtualTextureFeedbackCopyParameters* Parameters = GraphBuilder.AllocParameters<FVirtualTextureFeedbackCopyParameters>();
	Parameters->Input = Buffer;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("VirtualTextureFeedbackCopy"),
		Parameters,
		ERDGPassFlags::Readback,
		[this, Buffer, Desc](FRHICommandListImmediate& InRHICmdList)
	{
		TransferGPUToCPU(InRHICmdList, Buffer->GetRHI(), Desc);
	});
}

bool FVirtualTextureFeedback::CanMap(FRHICommandListImmediate& RHICmdList)
{
	if (NumPending > 0u)
	{
		SCOPED_GPU_MASK(RHICmdList, FeedbackItems[ReadIndex].GPUMask);
		return Fences->Poll(RHICmdList, ReadIndex);
	}
	else
	{
		return false;
	}
}

static void FeedbackCopyAndInterleave(FUintPoint* RESTRICT InDest, uint32 const* RESTRICT InSource, int32 InElementCount, bool bIsPreInterleaved)
{
	if (bIsPreInterleaved)
	{
		FMemory::Memcpy(InDest, InSource, InElementCount * sizeof(FUintPoint));
	}
	else
	{
		// Legacy path for readback buffers without interleaved counts is slower. But we pay the cost now when filling the buffer rather than when parsing the buffer.
		for (int32 Index = 0; Index < InElementCount; ++Index)
		{
			*(InDest++) = FUintPoint(*(InSource++), 1);
		}
	}
}

FVirtualTextureFeedback::FMapResult FVirtualTextureFeedback::Map(FRHICommandListImmediate& RHICmdList, int32 MaxTransfersToMap)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_VirtualTextureFeedback_Map);

	FVirtualTextureFeedback::FMapResult MapResult;

	// Calculate number and size of available results.
	int32 NumResults = 0;
	uint32 TotalReadSize = 0;
	for (int32 ResultIndex = 0; ResultIndex < MaxTransfersToMap && ResultIndex < NumPending; ++ResultIndex)
	{
		const int32 FeedbackIndex = (ReadIndex + ResultIndex) % MaxTransfers;
		FVirtualTextureFeedbackBufferDesc const& FeedbackItemDesc = FeedbackItems[FeedbackIndex].Desc;

		SCOPED_GPU_MASK(RHICmdList, FeedbackItems[FeedbackIndex].GPUMask);
		if (!Fences->Poll(RHICmdList, FeedbackIndex))
		{
			break;
		}

		NumResults ++;
		TotalReadSize += FeedbackItemDesc.BufferSize;
	}

	// Fetch the valid results.
	if (NumResults > 0)
	{
		// Get a FMapResources object to store anything that will need cleaning up on Unmap()
		MapResult.MapHandle = FreeMapResources.Num() ? FreeMapResources.Pop() : MapResources.AddDefaulted();

		if (NumResults == 1 && FeedbackItems[ReadIndex].Desc.bPageAndCount)
		{
			// If there is only one target and it is already interleaved page/count pairs, then fast path is to return the locked buffer.
			const int32 FeedbackIndex = ReadIndex;
			FVirtualTextureFeedbackBufferDesc const& FeedbackItemDesc = FeedbackItems[FeedbackIndex].Desc;
			FRHIGPUMask GPUMask = FeedbackItems[FeedbackIndex].GPUMask;
			FStagingBufferRHIRef StagingBuffer = FeedbackItems[FeedbackIndex].StagingBuffer;
			
			SCOPED_GPU_MASK(RHICmdList, GPUMask);
			MapResult.Data = (FUintPoint*)RHICmdList.LockStagingBuffer(StagingBuffer, Fences->GetMapFence(FeedbackIndex), 0, FeedbackItemDesc.BufferSize * sizeof(FUintPoint));
			MapResult.Size = FeedbackItemDesc.BufferSize;

			if (FeedbackItemDesc.bSizeInHeader)
			{
				const uint32 BufferWriteCount = MapResult.Data->X;
				MapResult.Data += 1;
				MapResult.Size = FMath::Min(BufferWriteCount, FeedbackItemDesc.BufferSize - 1);

				INC_DWORD_STAT_BY(STAT_VirtualTexture_ReadFeedbackLostPage, BufferWriteCount - MapResult.Size);
			}

			// Store index so that we can unlock staging buffer when we call Unmap().
			MapResources[MapResult.MapHandle].FeedbackItemToUnlockIndex = FeedbackIndex;
		}
		else
		{
			// Concatenate the results to a single buffer (stored in the MapResources) and return that.
			MapResources[MapResult.MapHandle].ResultData.SetNumUninitialized(TotalReadSize, EAllowShrinking::No);
			MapResult.Data = MapResources[MapResult.MapHandle].ResultData.GetData();
			MapResult.Size = 0;

			for (int32 ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
			{
				const int32 FeedbackIndex = (ReadIndex + ResultIndex) % MaxTransfers;
				FVirtualTextureFeedbackBufferDesc const& FeedbackItemDesc = FeedbackItems[FeedbackIndex].Desc;
				const int32 FeedbackStride = FeedbackItemDesc.bPageAndCount ? 2 : 1;
				FRHIGPUMask GPUMask = FeedbackItems[FeedbackIndex].GPUMask;
				FStagingBufferRHIRef StagingBuffer = FeedbackItems[FeedbackIndex].StagingBuffer;

				SCOPED_GPU_MASK(RHICmdList, GPUMask);
				uint32 const* Data = (uint32*)RHICmdList.LockStagingBuffer(StagingBuffer, Fences->GetMapFence(FeedbackIndex), 0, FeedbackItemDesc.BufferSize * sizeof(uint32) * FeedbackStride);

				if (!FeedbackItemDesc.bSizeInHeader)
				{
					FeedbackCopyAndInterleave(MapResult.Data + MapResult.Size, Data, FeedbackItemDesc.BufferSize, FeedbackItemDesc.bPageAndCount);
					MapResult.Size += FeedbackItemDesc.BufferSize;
				}
				else
				{
					const uint32 BufferWriteCount = *Data;
					Data += FeedbackStride;
					const int32 BufferSize = FMath::Min(BufferWriteCount, FeedbackItemDesc.BufferSize - 1);
					FeedbackCopyAndInterleave(MapResult.Data + MapResult.Size, Data, BufferSize, FeedbackItemDesc.bPageAndCount);
					MapResult.Size += BufferSize;

					INC_DWORD_STAT_BY(STAT_VirtualTexture_ReadFeedbackLostPage, BufferWriteCount - BufferSize);
				}

				RHICmdList.UnlockStagingBuffer(StagingBuffer);
			}
		}

		INC_DWORD_STAT_BY(STAT_VirtualTexture_ReadFeedback, NumResults);

		check(MapResult.Size <= TotalReadSize)

		// Increment the ring buffer read position.
		NumPending -= NumResults;
		ReadIndex = (ReadIndex + NumResults) % MaxTransfers;
	}

	return MapResult;
}

FVirtualTextureFeedback::FMapResult FVirtualTextureFeedback::Map(FRHICommandListImmediate& RHICmdList)
{
	// Note that this stat for pending could vary over the frame, particularly if we Map() more than once.
	SET_DWORD_STAT(STAT_VirtualTexture_PendingFeedback, NumPending);

	// Allow some slack in the pipeline before we start mapping more than one buffer.
	// Otherwise we can get into an oscillating pattern of mapping 2 buffers, then 0 buffers, then 2 again etc.
	const uint32 MaxTransfersToMap = NumPending < GVirtualTextureFeedbackLatency ? 1 : MaxTransfers;

	return Map(RHICmdList, MaxTransfersToMap);
}

void FVirtualTextureFeedback::Unmap(FRHICommandListImmediate& RHICmdList, int32 MapHandle)
{
	if (MapHandle >= 0)
	{
		FMapResources& Resources = MapResources[MapHandle];

		// Do any required buffer Unlock.
		if (Resources.FeedbackItemToUnlockIndex >= 0)
		{
			SCOPED_GPU_MASK(RHICmdList, FeedbackItems[Resources.FeedbackItemToUnlockIndex].GPUMask);
			RHICmdList.UnlockStagingBuffer(FeedbackItems[Resources.FeedbackItemToUnlockIndex].StagingBuffer);
			Resources.FeedbackItemToUnlockIndex = -1;
		}

		// Reset any allocated data buffer.
		Resources.ResultData.Reset();

		// Return to free list.
		FreeMapResources.Add(MapHandle);
	}
}

TGlobalResource< FVirtualTextureFeedback > GVirtualTextureFeedback;
