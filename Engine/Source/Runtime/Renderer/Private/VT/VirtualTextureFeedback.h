// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "VT/VirtualTextureFeedbackBuffer.h"

/** 
 * Class that handles the read back of feedback buffers from the GPU.
 * Multiple buffers can be transferred per frame using TransferGPUToCPU().
 * Use Map()/Unmap() to return the data. It will only return data that is ready to fetch without stalling the GPU.
 * All calls are expected to be from the single render thread only.
 */
class FVirtualTextureFeedback : public FRenderResource
{
public:
	FVirtualTextureFeedback();
	~FVirtualTextureFeedback();

	/** Commit a RHIBuffer to be transferred for later CPU analysis. */
	void TransferGPUToCPU(FRHICommandList& RHICmdList, FBufferRHIRef const& Buffer, FVirtualTextureFeedbackBufferDesc const& Desc);
	void TransferGPUToCPU(FRDGBuilder& GraphBuilder, class FRDGBuffer* Buffer, FVirtualTextureFeedbackBufferDesc const& Desc);

	/** Returns true if there are any pending transfer results that are ready so that we can call Map(). */
	bool CanMap(FRHICommandListImmediate& RHICmdList);

	/** Structure returned by Map() containing the feedback data. */
	struct FMapResult
	{
		// Feedback Data is .X=PageId, .Y=PageCount
		FUintPoint* RESTRICT Data = nullptr;
		uint32 Size = 0;
		int32 MapHandle = -1;
	};

	/** Fetch all pending results into a flat output buffer for analysis. */
	FMapResult Map(FRHICommandListImmediate& RHICmdList);
	/** Fetch up to MaxTransfersToMap pending results into a flat output buffer for analysis. */
	FMapResult Map(FRHICommandListImmediate& RHICmdList, int32 MaxTransfersToMap);
	
	/** Always call Unmap() after finishing processing of the returned data. This releases any resources allocated for the Map(). */
	void Unmap(FRHICommandListImmediate& RHICmdList, int32 MapHandle);

protected:
	//~ Begin FRenderResource Interface
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
	//~ End FRenderResource Interface

private:
	/** Description of a pending feedback transfer. */
	struct FFeedbackItem
	{
		FVirtualTextureFeedbackBufferDesc Desc;
		FRHIGPUMask GPUMask;
		FStagingBufferRHIRef StagingBuffer;
	};

	/** The maximum number of pending feedback transfers that can be held before we start dropping them. */
	static const uint32 MaxTransfers = 8u;
	/** Pending feedback transfers are stored as a ring buffer. */
	FFeedbackItem FeedbackItems[MaxTransfers];

	/** GPU fence pool. Contains a fence array that is kept in sync with the FeedbackItems ring buffer. Fences are used to know when a transfer is ready to Map() without stalling. */
	class FFeedbackGPUFencePool* Fences;

	int32 NumPending;
	uint32 WriteIndex;
	uint32 ReadIndex;

	/** Structure describing resources associated with a Map() that will need freeing on UnMap() */
	struct FMapResources
	{
		int32 FeedbackItemToUnlockIndex = -1;
		TArray<FUintPoint> ResultData;
	};

	/** Array of FMapResources. Will always be size 1 unless we need to Map() multiple individual buffers at once. */
	TArray<FMapResources> MapResources;
	/** Free indices in the MapResources array. */
	TArray<int32> FreeMapResources;
};

/** Global object for handling the read back of Virtual Texture Feedback. */
extern TGlobalResource< FVirtualTextureFeedback > GVirtualTextureFeedback;
