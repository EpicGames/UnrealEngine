// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTextureFeedbackBuffer.h"

#include "VT/VirtualTextureFeedback.h"

void FVirtualTextureFeedbackBufferDesc::Init(int32 InBufferSize)
{
	BufferSize = InBufferSize;
	bSizeInHeader = false;
	bPageAndCount = false;
}


void FVirtualTextureFeedbackBufferDesc::Init2D(FIntPoint InBufferSize)
{
	BufferSize = InBufferSize.X * InBufferSize.Y;
	bSizeInHeader = false;
	bPageAndCount = false;
}

void FVirtualTextureFeedbackBufferDesc::Init2D(FIntPoint InUnscaledBufferSize, TArrayView<FIntRect> const& InUnscaledViewRects, int32 InBufferScale)
{
	const int32 BufferScale = FMath::Max(InBufferScale, 1);
	const FIntPoint ScaledBufferSize = FIntPoint::DivideAndRoundUp(InUnscaledBufferSize, InBufferScale);
	BufferSize = ScaledBufferSize.X * ScaledBufferSize.Y;
	bSizeInHeader = false;
	bPageAndCount = false;
}

void SubmitVirtualTextureFeedbackBuffer(FRHICommandList& RHICmdList, FBufferRHIRef const& InBuffer, FVirtualTextureFeedbackBufferDesc const& InDesc)
{
	GVirtualTextureFeedback.TransferGPUToCPU(RHICmdList, InBuffer, InDesc);
}

void SubmitVirtualTextureFeedbackBuffer(class FRDGBuilder& GraphBuilder, FRDGBuffer* InBuffer, FVirtualTextureFeedbackBufferDesc const& InDesc)
{
	GVirtualTextureFeedback.TransferGPUToCPU(GraphBuilder, InBuffer, InDesc);
}
