// Copyright Epic Games, Inc. All Rights Reserved.

#include "VT/VirtualTextureFeedbackResource.h"

#include "GlobalRenderResources.h"
#include "GPUFeedbackCompaction.h"
#include "RenderGraphUtils.h"
#include "RenderResource.h"
#include "SceneView.h"
#include "VT/VirtualTextureFeedbackBuffer.h"
#include "VT/VirtualTextureScalability.h"
#include "VT/VirtualTextureSystem.h"

DECLARE_GPU_STAT(VirtualTextureUpdate);

int32 GVirtualTextureFeedbackDefaultBufferSize = 4 * 1024;
static FAutoConsoleVariableRef CVarVirtualTextureFeedbackDefaultBufferSize(
	TEXT("r.vt.FeedbackDefaultBufferSize"),
	GVirtualTextureFeedbackDefaultBufferSize,
	TEXT("Virtual texture feedback buffer size for cases where we don't calculate it based on screen size."),
	ECVF_RenderThreadSafe);

int32 GVirtualTextureFeedbackOverdrawFactor = 2;
static FAutoConsoleVariableRef CVarVirtualTextureFeedbackOverdrawFactor(
	TEXT("r.vt.FeedbackOverdrawFactor"),
	GVirtualTextureFeedbackOverdrawFactor,
	TEXT("A multiplicative factor to apply to virtual texture feedback buffer sizes to account for ")
	TEXT("multiple layers of virtual texture feedback from Decal/Transparency/PostFX etc."),
	ECVF_RenderThreadSafe);

int32 GVirtualTextureFeedbackCompactionFactor = 16;
static FAutoConsoleVariableRef CVarVirtualTextureFeedbackCompactionFactor(
	TEXT("r.vt.FeedbackCompactionFactor"),
	GVirtualTextureFeedbackCompactionFactor,
	TEXT("A division factor to apply to the size of the virtual texture feedback compaction buffer ")
	TEXT("to account for compaction of duplicate page ids."),
	ECVF_RenderThreadSafe);

/** GPU feedback buffer for tracking virtual texture requests from the GPU. */
class FVirtualTextureFeedbackBufferResource : public FRenderResource
{
public:
	/** Allocates and prepares a new feedback buffer for write access. */
	void Begin(FRDGBuilder& GraphBuilder, uint32 InFeedbackBufferSize, uint32 InExtendedDebugBufferSize, ERHIFeatureLevel::Type InFeatureLevel);
	/** Compacts the feedback buffer on the GPU and copies the compacted buffer for readback. */
	void End(FRDGBuilder& GraphBuilder);

	/** Resolve and return any extended debug information that is currently stored at the end of the feedback buffer. */
	FRDGBuffer* ResolveExtendedDebugBuffer(FRDGBuilder& GraphBuilder);

	/** Get the size of the feedback buffer from the last call to Begin(). */
	uint32 GetBufferSize() const;
	/** Get the added size for the extended debug area of the feedback buffer from the last call to Begin(). */
	uint32 GetExtendedDebugBufferSize() const;
	/** Get the UAV of the feedback buffer. */
	FRHIUnorderedAccessView* GetUAV() const;

	//~ Begin FRenderResource Interface
	virtual void ReleaseRHI() override;
	//~ End FRenderResource Interface

private:
	int32 BeginEndScopeCount = 0;
	uint32 FeedbackBufferSize = 0;
	uint32 ExtendedDebugBufferSize = 0;
	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::Num;
	TRefCountPtr<FRDGPooledBuffer> PooledBuffer;
	FRHIUnorderedAccessView* UAV = nullptr;
};

/** Single global feedback buffer resource used when calling the BeginFeedback()/EndFeedback free functions. */
static TGlobalResource<FVirtualTextureFeedbackBufferResource> GVirtualTextureFeedbackBufferResource;

static uint32 GetVirtualTextureCompactedFeedbackBufferSize(uint32 InSourceBufferSize);

void FVirtualTextureFeedbackBufferResource::Begin(FRDGBuilder& GraphBuilder, uint32 InFeedbackBufferSize, uint32 InExtendedDebugBufferSize, ERHIFeatureLevel::Type InFeatureLevel)
{
	// NOTE: Transitions and allocations are handled manually right now, because the VT feedback UAV is used by
	// the view uniform buffer, which is not an RDG uniform buffer. If it can be factored out into its own RDG
	// uniform buffer (or put on the pass uniform buffers), then the resource can be fully converted to RDG.

	if (++BeginEndScopeCount != 1)
	{
		// Where overlapping scopes occur we will assume that it is OK to use the UAV from the outer scope.
		// This remains correct, but not ideal, if the inner scope requests a bigger buffer size then the outer scope.
		// In that case the feedback shader still respects the smaller size when writing, but we are more likely to fill the buffer and lose feedback.
		return;
	}

	FeedbackBufferSize = InFeedbackBufferSize;
	ExtendedDebugBufferSize = InExtendedDebugBufferSize;
	FeatureLevel = InFeatureLevel;

	FRDGBufferDesc BufferDesc(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FeedbackBufferSize + InExtendedDebugBufferSize));
	BufferDesc.Usage |= BUF_SourceCopy;

	AllocatePooledBuffer(BufferDesc, PooledBuffer, TEXT("VirtualTexture_FeedbackBuffer"));

	FRDGBufferUAVDesc UAVDesc{};
	UAV = PooledBuffer->GetOrCreateUAV(GraphBuilder.RHICmdList, UAVDesc);

	AddPass(GraphBuilder, RDG_EVENT_NAME("VirtualTextureClear"), [UAV = UAV](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		// Clear virtual texture feedback to default value
		RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
		RHICmdList.ClearUAVUint(UAV, FUintVector4(0u, 0u, 0u, 0u));
		RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::UAVCompute, ERHIAccess::UAVMask));
		RHICmdList.BeginUAVOverlap(UAV);
	});
}

void FVirtualTextureFeedbackBufferResource::End(FRDGBuilder& GraphBuilder)
{
	if (--BeginEndScopeCount != 0)
	{
		if (!ensure(BeginEndScopeCount > 0))
		{ 
			// Calling End() more often than Begin() should never happen. Recover by resetting the count.
			BeginEndScopeCount = 0;
		}
		return;
	}

	// VirtualTextureFeedback would be a more descriptive stat name, but VirtualTextureUpdate was used historically and some profile tools may depend on that.
	RDG_EVENT_SCOPE_STAT(GraphBuilder, VirtualTextureUpdate, "VirtualTextureUpdate");

	AddPass(GraphBuilder, RDG_EVENT_NAME("VirtualTextureFeedbackTransition"), [UAV = UAV](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.EndUAVOverlap(UAV);
		RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::UAVMask, ERHIAccess::SRVCompute));
	});

	// If there are no allocated VTs then we can assume that no GPU feedback was generated.
	// So early out here to avoid paying any GPU cost compacting the empty feedback buffer.
	if (!FVirtualTextureSystem::Get().HasAllocatedVirtualTextures())
	{
		return;
	}

	FRDGBufferRef FeedbackBuffer = GraphBuilder.RegisterExternalBuffer(PooledBuffer, ERDGBufferFlags::SkipTracking);
	FRDGBufferSRVRef FeedbackBufferSRV = GraphBuilder.CreateSRV(FeedbackBuffer);

	// We will compact feedback before queuing for readback.
	// The stride will be 2 to account for interleaved pairs of page ids and page counts.
	const uint32 CompactedFeedbackStride = 2;
	const uint32 CompactedFeedbackBufferSize = GetVirtualTextureCompactedFeedbackBufferSize(FeedbackBufferSize);
	const uint32 HashTableSize = 2 * CompactedFeedbackBufferSize;
	const uint32 HashTableIndexWrapMask = HashTableSize - 1;

	FRDGBufferDesc CompactedFeedbackBufferDesc(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32) * CompactedFeedbackStride, CompactedFeedbackBufferSize));
	CompactedFeedbackBufferDesc.Usage |= BUF_SourceCopy;
	FRDGBufferRef CompactedFeedbackBuffer = GraphBuilder.CreateBuffer(CompactedFeedbackBufferDesc, TEXT("VirtualTexture.CompactedFeedback"));

	// Need to clear this buffer, as first element will be used as an allocator.
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(CompactedFeedbackBuffer, PF_R32_UINT), 0);

	FRDGBufferDesc HashTableBufferDesc(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), HashTableSize));
	FRDGBufferRef HashTableKeyBuffer = GraphBuilder.CreateBuffer(HashTableBufferDesc, TEXT("VirtualTexture.HashTableKeys"));
	FRDGBufferRef HashTableElementIndexBuffer = GraphBuilder.CreateBuffer(HashTableBufferDesc, TEXT("VirtualTexture.HashTableElementIndices"));
	FRDGBufferRef HashTableElementCountBuffer = GraphBuilder.CreateBuffer(HashTableBufferDesc, TEXT("VirtualTexture.HashTableElementCounts"));

	// Hash table depends on empty slots to be 0.
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HashTableKeyBuffer, PF_R32_UINT), 0);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HashTableElementCountBuffer, PF_R32_UINT), 0);

	FRDGBufferRef BuildHashTableIndirectArgBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("VirtualTexture.BuildHashTableIndirectArgs"));

	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);

	// Set indirect dispatch arguments for hash table building.
	{
		FBuildFeedbackHashTableIndirectArgsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildFeedbackHashTableIndirectArgsCS::FParameters>();

		PassParameters->RWBuildHashTableIndirectArgs = GraphBuilder.CreateUAV(BuildHashTableIndirectArgBuffer, PF_R32_UINT);

		PassParameters->FeedbackBufferAllocator = FeedbackBufferSRV;
		PassParameters->FeedbackBuffer = FeedbackBufferSRV;
		PassParameters->FeedbackBufferSize = FeedbackBufferSize;

		FBuildFeedbackHashTableIndirectArgsCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FBuildFeedbackHashTableIndirectArgsCS::FFeedbackBufferStride>(1);

		auto ComputeShader = GlobalShaderMap->GetShader<FBuildFeedbackHashTableIndirectArgsCS>(PermutationVector);
		const FIntVector GroupSize = FIntVector(1, 1, 1);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Hash table indirect arguments"),
			ComputeShader,
			PassParameters,
			GroupSize);
	}

	// Build hash table of feedback elements.
	{
		FBuildFeedbackHashTableCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildFeedbackHashTableCS::FParameters>();

		PassParameters->BuildHashTableIndirectArgs = BuildHashTableIndirectArgBuffer;

		PassParameters->RWHashTableKeys = GraphBuilder.CreateUAV(HashTableKeyBuffer);
		PassParameters->RWHashTableElementIndices = GraphBuilder.CreateUAV(HashTableElementIndexBuffer);
		PassParameters->RWHashTableElementCounts = GraphBuilder.CreateUAV(HashTableElementCountBuffer);
		PassParameters->HashTableSize = HashTableSize;
		PassParameters->HashTableIndexWrapMask = HashTableIndexWrapMask;

		PassParameters->FeedbackBufferAllocator = FeedbackBufferSRV;
		PassParameters->FeedbackBuffer = FeedbackBufferSRV;
		PassParameters->FeedbackBufferSize = FeedbackBufferSize;

		FBuildFeedbackHashTableCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FBuildFeedbackHashTableCS::FFeedbackBufferStride>(1);

		auto ComputeShader = GlobalShaderMap->GetShader<FBuildFeedbackHashTableCS>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Build feedback hash table"),
			ComputeShader,
			PassParameters,
			BuildHashTableIndirectArgBuffer,
			0);
	}

	// Compact hash table into an array of unique feedback elements.
	{
		FCompactFeedbackHashTableCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompactFeedbackHashTableCS::FParameters>();

		PassParameters->RWCompactedFeedbackBuffer = GraphBuilder.CreateUAV(CompactedFeedbackBuffer);
		PassParameters->CompactedFeedbackBufferSize = CompactedFeedbackBufferSize;
		PassParameters->CompactedFeedbackCountShiftBits = 0;

		PassParameters->HashTableElementIndices = GraphBuilder.CreateSRV(HashTableElementIndexBuffer, PF_R32_UINT);
		PassParameters->HashTableElementCounts = GraphBuilder.CreateSRV(HashTableElementCountBuffer, PF_R32_UINT);
		PassParameters->HashTableSize = HashTableSize;
		PassParameters->HashTableIndexWrapMask = HashTableIndexWrapMask;

		PassParameters->FeedbackBufferAllocator = FeedbackBufferSRV;
		PassParameters->FeedbackBuffer = FeedbackBufferSRV;
		PassParameters->FeedbackBufferSize = FeedbackBufferSize;

		FCompactFeedbackHashTableCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FCompactFeedbackHashTableCS::FFeedbackBufferStride>(1);

		auto ComputeShader = GlobalShaderMap->GetShader<FCompactFeedbackHashTableCS>(PermutationVector);
		const FIntVector GroupSize = FComputeShaderUtils::GetGroupCount(HashTableSize, FCompactFeedbackHashTableCS::GetGroupSize());

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Compact feedback hash table"),
			ComputeShader,
			PassParameters,
			GroupSize);
	}

	// Compaction writes size in the buffer header and interleaves page ids and page counts.
	FVirtualTextureFeedbackBufferDesc CompactedDesc;
	CompactedDesc.BufferSize = CompactedFeedbackBufferSize;
	CompactedDesc.bSizeInHeader = true;
	CompactedDesc.bPageAndCount = true;

	SubmitVirtualTextureFeedbackBuffer(GraphBuilder, CompactedFeedbackBuffer, CompactedDesc);
}

FRDGBuffer* FVirtualTextureFeedbackBufferResource::ResolveExtendedDebugBuffer(FRDGBuilder& GraphBuilder)
{
	if (BeginEndScopeCount <= 0 || ExtendedDebugBufferSize == 0)
	{
		return nullptr;
	}

	// Transition for reading.
	AddPass(GraphBuilder, RDG_EVENT_NAME("VirtualTextureFeedbackTransitionBeforeExtract"), [UAV = UAV](FRHICommandList& RHICmdList)
	{
		RHICmdList.EndUAVOverlap(UAV);
		RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::UAVMask, ERHIAccess::CopySrc));
	});

	// Copy the extended debug payload.
	FRDGBufferRef FeedbackBuffer = GraphBuilder.RegisterExternalBuffer(PooledBuffer, ERDGBufferFlags::SkipTracking);

	FRDGBufferDesc DebugBufferCopyDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), ExtendedDebugBufferSize);
	FRDGBufferRef DebugBufferCopy = GraphBuilder.CreateBuffer(DebugBufferCopyDesc, TEXT("VirtualTexture.DebugBufferCopy"));

	AddCopyBufferPass(GraphBuilder, DebugBufferCopy, 0, FeedbackBuffer, FeedbackBufferSize * sizeof(uint32), ExtendedDebugBufferSize * sizeof(uint32));

	// Transition feedback to writing in case we have any subsequent feedback passes (that we won't have captured debug info for).
	AddPass(GraphBuilder, RDG_EVENT_NAME("VirtualTextureFeedbackTransitionAfterExtract"), [UAV = UAV](FRHICommandList& RHICmdList)
	{
		RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::CopySrc, ERHIAccess::UAVMask));
		RHICmdList.BeginUAVOverlap(UAV);
	});

	return DebugBufferCopy;
}

uint32 FVirtualTextureFeedbackBufferResource::GetBufferSize() const
{
	return BeginEndScopeCount > 0 ? FeedbackBufferSize : 0;
}

uint32 FVirtualTextureFeedbackBufferResource::GetExtendedDebugBufferSize() const
{
	return BeginEndScopeCount > 0 ? ExtendedDebugBufferSize : 0;
}

FRHIUnorderedAccessView* FVirtualTextureFeedbackBufferResource::GetUAV() const
{
	return UAV != nullptr && BeginEndScopeCount > 0 ? UAV : GEmptyStructuredBufferWithUAV->UnorderedAccessViewRHI.GetReference();
}

void FVirtualTextureFeedbackBufferResource::ReleaseRHI()
{
	PooledBuffer = nullptr;
	UAV = nullptr;
}

/**
 * Get the feedback buffer size in dwords based on view size.
 * Takes into account the feedback tile size and overdraw factors.
 */
static uint32 GetVirtualTextureFeedbackBufferSize(FIntPoint InViewportSize, uint32 InVirtualTextureFeedbackTileSize)
{
	// Tile size must be power or 2.
	check(InVirtualTextureFeedbackTileSize == 1 << FMath::FloorLog2(InVirtualTextureFeedbackTileSize));

	FIntPoint ScaledExtent = FIntPoint::DivideAndRoundUp(InViewportSize, InVirtualTextureFeedbackTileSize);
	return ScaledExtent.X * ScaledExtent.Y * FMath::Max(GVirtualTextureFeedbackOverdrawFactor, 1);
}

/** Get compacted feedback buffer size in dwords based on the original feedback buffer size. */
static uint32 GetVirtualTextureCompactedFeedbackBufferSize(uint32 InSourceBufferSize)
{
	// Could possibly do this dynamically according to some tracked recent high watermark?
	const uint32 CompactedBufferSize = InSourceBufferSize / FMath::Max(GVirtualTextureFeedbackCompactionFactor, 1);
	// Size needs to be power of two for hash table wrapping.
	const uint32 AlignedCompactedBufferSize = FMath::RoundUpToPowerOfTwo(FMath::Clamp(CompactedBufferSize, 16u, 16u * 1024u));
	return AlignedCompactedBufferSize;
}

/** Apply alignment rules to the feedback tile size. */
static uint32 AlignVirtualTextureFeedbackTileSize(uint32 InTileSize)
{
	// Round to nearest power of two to ensure that shader maths is efficient and sampling sequence logic is simple.
	return FMath::RoundUpToPowerOfTwo(FMath::Max(InTileSize, 1u));
}

/** Get jittered pixel index within feedback tile. */
static uint32 SampleVirtualTextureFeedbackSequence(uint32 InFrameIndex, uint32 InVirtualTextureFeedbackTileSize)
{
	// Tile size must be power or 2.
	check(InVirtualTextureFeedbackTileSize == 1 << FMath::FloorLog2(InVirtualTextureFeedbackTileSize));

	const uint32 TileSize = InVirtualTextureFeedbackTileSize;
	const uint32 TileSizeLog2 = FMath::CeilLogTwo(TileSize);
	const uint32 SequenceSize = FMath::Square(TileSize);
	const uint32 PixelIndex = InFrameIndex % SequenceSize;
	const uint32 PixelAddress = ReverseBits(PixelIndex) >> (32U - 2 * TileSizeLog2);
	const uint32 X = FMath::ReverseMortonCode2(PixelAddress);
	const uint32 Y = FMath::ReverseMortonCode2(PixelAddress >> 1);
	const uint32 PixelSequenceIndex = X + Y * TileSize;
	return PixelSequenceIndex;
}

namespace VirtualTexture {

void GetFeedbackShaderParams(int32 InFrameIndex, int32 InVirtualTextureFeedbackTileSize, FFeedbackShaderParams& OutParams)
{
	OutParams.BufferUAV = GVirtualTextureFeedbackBufferResource.GetUAV();
	OutParams.BufferSize = GVirtualTextureFeedbackBufferResource.GetBufferSize();
	OutParams.ExtendedDebugBufferSize = GVirtualTextureFeedbackBufferResource.GetExtendedDebugBufferSize();

	// Round to nearest power of two to ensure that shader maths is efficient and sampling sequence logic is simple.
	InVirtualTextureFeedbackTileSize = AlignVirtualTextureFeedbackTileSize(InVirtualTextureFeedbackTileSize);

	OutParams.TileShift = FMath::FloorLog2(InVirtualTextureFeedbackTileSize);
	OutParams.TileMask = InVirtualTextureFeedbackTileSize - 1;
	// Use some low(ish) discrepancy sequence to run over every pixel in the virtual texture feedback tile.
	OutParams.TileJitterOffset = SampleVirtualTextureFeedbackSequence(InFrameIndex, InVirtualTextureFeedbackTileSize);
	// SampleOffset is used to cycle through all VT samples in a material. It just needs to monotonically increase.
	OutParams.SampleOffset = InFrameIndex;
}

void GetFeedbackShaderParams(FFeedbackShaderParams& OutParams)
{
	const uint32 VirtualTextureFeedbackTileSize = AlignVirtualTextureFeedbackTileSize(VirtualTextureScalability::GetVirtualTextureFeedbackFactor());
	VirtualTexture::GetFeedbackShaderParams(GFrameNumberRenderThread, VirtualTextureFeedbackTileSize, OutParams);
}

void UpdateViewUniformShaderParameters(FFeedbackShaderParams const& InParams, FViewUniformShaderParameters& ViewUniformShaderParameters)
{
	ViewUniformShaderParameters.VTFeedbackBuffer = InParams.BufferUAV;
	ViewUniformShaderParameters.VirtualTextureFeedbackBufferSize = InParams.BufferSize;
	ViewUniformShaderParameters.VirtualTextureFeedbackShift = InParams.TileShift;
	ViewUniformShaderParameters.VirtualTextureFeedbackMask = InParams.TileMask;
	ViewUniformShaderParameters.VirtualTextureFeedbackJitterOffset = InParams.TileJitterOffset;
	ViewUniformShaderParameters.VirtualTextureFeedbackSampleOffset = InParams.SampleOffset;
	ViewUniformShaderParameters.VirtualTextureExtendedDebugBufferSize = InParams.ExtendedDebugBufferSize;
}

void BeginFeedback(FRDGBuilder& GraphBuilder, uint32 InBufferSize, ERHIFeatureLevel::Type InFeatureLevel)
{
	const uint32 BufferSize = InBufferSize > 0 ? InBufferSize : GVirtualTextureFeedbackDefaultBufferSize;
	const ERHIFeatureLevel::Type FeatureLevel = InFeatureLevel < ERHIFeatureLevel::Num ? InFeatureLevel : GMaxRHIFeatureLevel;
	GVirtualTextureFeedbackBufferResource.Begin(GraphBuilder, BufferSize, 0, FeatureLevel);
}

void BeginFeedback(FRDGBuilder& GraphBuilder, FIntPoint InViewportSize, uint32 InVirtualTextureFeedbackTileSize, bool bInExtendFeedbackForDebug, ERHIFeatureLevel::Type InFeatureLevel)
{
	const FIntPoint ViewportSize = FIntPoint(FMath::Max(InViewportSize.X, 1), FMath::Max(InViewportSize.Y, 1));
	const uint32 TileSize = InVirtualTextureFeedbackTileSize > 0 ? InVirtualTextureFeedbackTileSize : VirtualTextureScalability::GetVirtualTextureFeedbackFactor();
	const uint32 AlignedTileSize = AlignVirtualTextureFeedbackTileSize(TileSize);
	const uint32 BufferSize = GetVirtualTextureFeedbackBufferSize(ViewportSize, AlignedTileSize);
	const uint32 ExtendedDebugBufferSize = bInExtendFeedbackForDebug ? ViewportSize.X * ViewportSize.Y : 0;
	GVirtualTextureFeedbackBufferResource.Begin(GraphBuilder, BufferSize, ExtendedDebugBufferSize, InFeatureLevel);
}

void EndFeedback(FRDGBuilder& GraphBuilder)
{
	GVirtualTextureFeedbackBufferResource.End(GraphBuilder);
}

FRDGBuffer* ResolveExtendedDebugBuffer(FRDGBuilder& GraphBuilder)
{
	return GVirtualTextureFeedbackBufferResource.ResolveExtendedDebugBuffer(GraphBuilder);
}

} // namespace VirtualTexture
