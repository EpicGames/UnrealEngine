// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTexturePhysicalSpace.h"

#include "BatchedElements.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "RenderTargetPool.h"
#include "VisualizeTexture.h"
#include "VT/VirtualTexturePoolConfig.h"
#include "VT/VirtualTextureScalability.h"
#include "VT/VirtualTextureSystem.h"
#include "RHIUtilities.h"
#include "RenderGraphUtils.h"

DECLARE_MEMORY_STAT_POOL(TEXT("Total Physical Memory"), STAT_TotalPhysicalMemory, STATGROUP_VirtualTextureMemory, FPlatformMemory::MCR_GPU);

CSV_DECLARE_CATEGORY_EXTERN(VirtualTexturing);

static TAutoConsoleVariable<float> CVarVTResidencyMaxMipMapBias(
	TEXT("r.VT.Residency.MaxMipMapBias"),
	4,
	TEXT("Maximum mip bias to apply to prevent Virtual Texture pool residency over-subscription.\n")
	TEXT("Default 4"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarVTResidencyUpperBound(
	TEXT("r.VT.Residency.UpperBound"),
	0.95f,
	TEXT("Virtual Texture pool residency above which we increase mip bias.\n")
	TEXT("Default 0.95"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarVTResidencyLowerBound(
	TEXT("r.VT.Residency.LowerBound"),
	0.95f,
	TEXT("Virtual Texture pool residency below which we decrease mip bias.\n")
	TEXT("Default 0.95"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarVTResidencyLockedUpperBound(
	TEXT("r.VT.Residency.LockedUpperBound"),
	0.65f,
	TEXT("Virtual Texture pool locked page residency above which we kill any mip bias.\n")
	TEXT("That's because locked pages are never affected by the mip bias setting. So it is unlikely that we can get the pool within budget.\n")
	TEXT("Default 0.65"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarVTResidencyAdjustmentRate(
	TEXT("r.VT.Residency.AdjustmentRate"),
	0.2f,
	TEXT("Rate at which we adjust mip bias due to Virtual Texture pool residency.\n")
	TEXT("Default 0.2"),
	ECVF_RenderThreadSafe);


FVirtualTexturePhysicalSpace::FVirtualTexturePhysicalSpace(uint16 InID, const FVTPhysicalSpaceDescription& InDesc, FVTPhysicalSpaceDescriptionExt& InDescExt)
	: Description(InDesc)
	, DescriptionExt(InDescExt)
	, ID(InID)
	, NumRefs(0u)
	, NumResourceRefs(0u)
	, ResidencyMipMapBias(0.0f)
	, LastFrameOversubscribed(0)
#if !UE_BUILD_SHIPPING
	, VisibleHistory(HistorySize)
	, LockedHistory(HistorySize)
	, MipMapBiasHistory(HistorySize)
	, HistoryIndex(0)
#endif
{
	Pool.Initialize(GetNumTiles());

	// Initialize this resource FeatureLevel, so it gets re-created on FeatureLevel changes
	SetFeatureLevel(GMaxRHIFeatureLevel);

	// Store string for logging.
	for (uint32 Layer = 0u; Layer < Description.NumLayers; ++Layer)
	{
		FormatString += GPixelFormats[Description.Format[Layer]].Name;
		
		// sRGB flag is only relevant on platforms that do not support texture views
		if (!GRHISupportsTextureViews)
		{
			FormatString += (Description.bHasLayerSrgbView[Layer] ? TEXT(" (sRGB)") : TEXT(" (Linear)"));
		}

		if (Layer + 1u < Description.NumLayers)
		{
			FormatString += TEXT(", ");
		}
	}

#if !UE_BUILD_SHIPPING
	// Store string for resorce debug names.
	for (uint32 Layer = 0; Layer < Description.NumLayers; ++Layer)
	{
		if (Description.NumLayers > 1)
		{
			PooledRenderTargetDebugNames[Layer] = FString::Printf(TEXT("VirtualTexture_Physical (%s) %d/%d"), *FormatString, Layer + 1, Description.NumLayers);
		}
		else
		{
			PooledRenderTargetDebugNames[Layer] = FString::Printf(TEXT("VirtualTexture_Physical (%s)"), *FormatString);
		}
	}
#endif
}

FVirtualTexturePhysicalSpace::~FVirtualTexturePhysicalSpace()
{
}

static EPixelFormat GetUnorderedAccessViewFormat(EPixelFormat InFormat)
{
	// Use alias formats for compressed textures on APIs where that is possible
	// This allows us to compress runtime data directly to the physical texture
	if (IsAnyBlockCompressedPixelFormat(InFormat))
	{
		return GRHISupportsUAVFormatAliasing ? GetBlockCompressedFormatUAVAliasFormat(InFormat) : PF_Unknown;
	}

	if (InFormat == PF_B5G5R5A1_UNORM || InFormat == PF_R5G6B5_UNORM)
	{
		return PF_Unknown;
	}

	return InFormat;
}

EPixelFormat RemapVirtualTexturePhysicalSpaceFormat(EPixelFormat InFormat)
{
	if (InFormat == PF_B8G8R8A8 && IsOpenGLPlatform(GMaxRHIShaderPlatform) && IsMobilePlatform(GMaxRHIShaderPlatform))
	{
		// FIXME: Mobile/Android OpenGL can't copy data between swizzled formats
		// Always use RGBA format for both VT intermediate render targets and VT physical texture
		// This will also make uncompressed streaming VT to have a R and B channel swapped 
		return PF_R8G8B8A8;
	}

	return InFormat;
}

void FVirtualTexturePhysicalSpace::InitRHI(FRHICommandListBase& RHICmdList)
{
	for (int32 Layer = 0; Layer < Description.NumLayers; ++Layer)
	{
		const EPixelFormat FormatSRV = RemapVirtualTexturePhysicalSpaceFormat(Description.Format[Layer]);
		const EPixelFormat FormatUAV = GetUnorderedAccessViewFormat(FormatSRV);
		const bool bCreateUAV = (FormatUAV != PF_Unknown);
		
		// Not all RHIs support sRGB views/aliasing. On those platforms create texture in an expected storage format 
		const bool bDefaultToSRGB = Description.bHasLayerSrgbView[Layer];
		// Not all formats support sRGB.
		const bool bFormatSupportsSRGB = FormatSRV != EPixelFormat::PF_R5G6B5_UNORM && FormatSRV != EPixelFormat::PF_B5G5R5A1_UNORM && FormatSRV != EPixelFormat::PF_G16;
		ETextureCreateFlags VT_SRGB = (bDefaultToSRGB && bFormatSupportsSRGB) ? TexCreate_SRGB : TexCreate_None;

		// Allocate physical texture from the render target pool
		const uint32 TextureSize = GetTextureSize();
		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			FIntPoint(TextureSize, TextureSize),
			FormatSRV,
			FClearValueBinding::None,
			VT_SRGB,
			bCreateUAV ? TexCreate_ShaderResource | TexCreate_UAV : TexCreate_ShaderResource,
			false);

		if (bCreateUAV)
		{
			Desc.UAVFormat = FormatUAV;
		}

		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, PooledRenderTarget[Layer], TEXT("VirtualTexture_Physical"));
		FRHITexture* TextureRHI = PooledRenderTarget[Layer]->GetRHI();

		// Create sRGB and non-sRGB shader resource views into the physical texture
		TextureSRV[Layer] = RHICmdList.CreateShaderResourceView(
			TextureRHI,
			FRHIViewDesc::CreateTextureSRV()
				.SetDimensionFromTexture(TextureRHI)
				.SetFormat(FormatSRV)
				.SetDisableSRGB(true));

		TextureSRV_SRGB[Layer] = RHICmdList.CreateShaderResourceView(
			TextureRHI,
			FRHIViewDesc::CreateTextureSRV()
				.SetDimensionFromTexture(TextureRHI)
				.SetFormat(FormatSRV)
				.SetDisableSRGB(false));
	}

	INC_MEMORY_STAT_BY(STAT_TotalPhysicalMemory, GetSizeInBytes());
}

void FVirtualTexturePhysicalSpace::ReleaseRHI()
{
	DEC_MEMORY_STAT_BY(STAT_TotalPhysicalMemory, GetSizeInBytes());

	for (int32 Layer = 0; Layer < Description.NumLayers; ++Layer)
	{
		GRenderTargetPool.FreeUnusedResource(PooledRenderTarget[Layer]);
		TextureSRV[Layer].SafeRelease();
		TextureSRV_SRGB[Layer].SafeRelease();
	}
}

void FVirtualTexturePhysicalSpace::FinalizeTextures(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue)
{
	for (int32 Layer = 0; Layer < Description.NumLayers; ++Layer)
	{
		// It's only necessary to enable external access mode on textures modified by RDG this frame.
		if (FRDGTexture* Texture = GraphBuilder.FindExternalTexture(PooledRenderTarget[Layer]))
		{
			ExternalAccessQueue.Add(Texture, ERHIAccess::SRVMask, ERHIPipeline::All);
		}
	}
}

uint32 FVirtualTexturePhysicalSpace::GetTileSizeInBytes() const
{
	SIZE_T TileSizeBytes = 0;
	for (int32 Layer = 0; Layer < Description.NumLayers; ++Layer)
	{
		TileSizeBytes += CalculateImageBytes(Description.TileSize, Description.TileSize, 0, Description.Format[Layer]);
	}
	return TileSizeBytes;

}

uint32 FVirtualTexturePhysicalSpace::GetSizeInBytes() const
{
	SIZE_T TextureSizeBytes = 0;
	const uint32 TextureSize = GetTextureSize();
	for (int32 Layer = 0; Layer < Description.NumLayers; ++Layer)
	{
		TextureSizeBytes += CalculateImageBytes(TextureSize, TextureSize, 0, Description.Format[Layer]);
	}
	return TextureSizeBytes;
}

void FVirtualTexturePhysicalSpace::UpdateResidencyTracking(uint32 Frame)
{
	float LockedUpperBound = CVarVTResidencyLockedUpperBound.GetValueOnRenderThread();
	float LowerBound = CVarVTResidencyLowerBound.GetValueOnRenderThread();
	float UpperBound = CVarVTResidencyUpperBound.GetValueOnRenderThread();
	float AdjustmentRate = CVarVTResidencyAdjustmentRate.GetValueOnRenderThread();
	float MaxMipMapBias = CVarVTResidencyMaxMipMapBias.GetValueOnRenderThread();

	const uint32 NumPages = Pool.GetNumPages();
	const uint32 NumLockedPages = Pool.GetNumLockedPages();
	const float LockedPageResidency = (float)NumLockedPages / (float)NumPages;

	const uint32 PageFreeThreshold = FMath::Max(VirtualTextureScalability::GetPageFreeThreshold(), 0u);
	const uint32 FrameMinusThreshold = Frame > PageFreeThreshold ? Frame - PageFreeThreshold : 0;
	const uint32 NumVisiblePages = Pool.GetNumVisiblePages(FrameMinusThreshold);
	const float VisiblePageResidency = (float)NumVisiblePages / (float)NumPages;
	
	if (ResidencyMipMapBias > 0.f && VisiblePageResidency < LowerBound)
	{
		ResidencyMipMapBias -= AdjustmentRate * (LowerBound - VisiblePageResidency);
	}
	else if (VisiblePageResidency > UpperBound)
	{
		ResidencyMipMapBias += AdjustmentRate * (VisiblePageResidency - UpperBound);
	}

	if (ResidencyMipMapBias > 0.f)
	{
		LastFrameOversubscribed = Frame;
	}

	ResidencyMipMapBias = FMath::Clamp(ResidencyMipMapBias, 0.f, MaxMipMapBias);

	if (!DescriptionExt.bEnableResidencyMipMapBias || LockedPageResidency > LockedUpperBound)
	{
		ResidencyMipMapBias = 0.f;
	}

#if !UE_BUILD_SHIPPING
	LockedHistory[HistoryIndex+1] = LockedPageResidency;
	VisibleHistory[HistoryIndex+1] = VisiblePageResidency;
	MipMapBiasHistory[HistoryIndex+1] = ResidencyMipMapBias / MaxMipMapBias;
	HistoryIndex++;
#endif
}

void FVirtualTexturePhysicalSpace::DrawResidencyGraph(FCanvas* Canvas, FBox2D CanvasPosition, bool bDrawKey)
{
	// Note that this is called on game thread and reads the history values written on the render thread.
	// But it should be safe and any race condition will only lead to a slightly incorrect graph.

#if !UE_BUILD_SHIPPING
	const int32 BorderSize = 10;

	const FLinearColor BackgroundColor(0.0f, 0.0f, 0.0f, 0.7f);
	const FLinearColor GraphBorderColor(0.1f, 0.1f, 0.1f);
	const FLinearColor GraphResidencyColor(0.8f, 0.1f, 0.1f);
	const FLinearColor GraphLockedPageColor(0.8f, 0.8f, 0.1f);
	const FLinearColor GraphMipMapBiasColor(0.1f, 0.8f, 0.1f);

	FCanvasTileItem BackgroundTile(CanvasPosition.Min, CanvasPosition.GetSize(), BackgroundColor);
	BackgroundTile.BlendMode = SE_BLEND_AlphaBlend;
	Canvas->DrawItem(BackgroundTile);

	const FString Title = FString::Printf(TEXT("%s (%dPages, %dMB)"), *FormatString, Pool.GetNumPages(), GetSizeInBytes() / (1024 * 1024));
	Canvas->DrawShadowedString(CanvasPosition.Min.X + BorderSize, CanvasPosition.Min.Y, *Title, GEngine->GetSmallFont(), FLinearColor::White);

	if (bDrawKey)
	{
		Canvas->DrawShadowedString(CanvasPosition.Min.X, CanvasPosition.Max.Y + 10, TEXT("Page Residency"), GEngine->GetSmallFont(), GraphResidencyColor);
		Canvas->DrawShadowedString(CanvasPosition.Min.X + 100, CanvasPosition.Max.Y + 10, TEXT("MipMap Bias"), GEngine->GetSmallFont(), GraphMipMapBiasColor);
		Canvas->DrawShadowedString(CanvasPosition.Min.X + 180, CanvasPosition.Max.Y + 10, TEXT("LockedPage Residency"), GEngine->GetSmallFont(), GraphLockedPageColor);
	}

	CanvasPosition.Min += FVector2D(BorderSize, BorderSize);
	CanvasPosition.Max -= FVector2D(BorderSize, BorderSize);

	const uint32 NumSamples = FMath::Min3<uint32>((uint32)CanvasPosition.GetSize().X, HistorySize, HistoryIndex);

	FHitProxyId HitProxyId = Canvas->GetHitProxyId();
	FBatchedElements* BatchedElements = Canvas->GetBatchedElements(FCanvas::ET_Line);
	BatchedElements->AddReserveLines(2 + 3 * NumSamples);

	BatchedElements->AddLine(
		FVector(CanvasPosition.Min.X - 1.0f, CanvasPosition.Max.Y, 0.0f),
		FVector(CanvasPosition.Min.X - 1.0f, CanvasPosition.Min.Y - 1.0f, 0.0f),
		GraphBorderColor,
		HitProxyId);

	BatchedElements->AddLine(
		FVector(CanvasPosition.Min.X, CanvasPosition.Max.Y - 1.0f, 0.0f),
		FVector(CanvasPosition.Max.X, CanvasPosition.Max.Y - 1.0f, 0.0f),
		GraphBorderColor,
		HitProxyId);

	for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
	{
		float Visible0 = VisibleHistory[HistoryIndex - NumSamples + SampleIndex];
		float Visible1 = VisibleHistory[HistoryIndex - NumSamples + SampleIndex + 1];

		BatchedElements->AddLine(
			FVector(CanvasPosition.Min.X + SampleIndex, CanvasPosition.Max.Y - Visible0 * CanvasPosition.GetSize().Y, 0.0f),
			FVector(CanvasPosition.Min.X + SampleIndex + 1, CanvasPosition.Max.Y - Visible1 * CanvasPosition.GetSize().Y, 0.0f),
			GraphResidencyColor,
			HitProxyId);

		float Locked0 = LockedHistory[HistoryIndex - NumSamples + SampleIndex];
		float Locked1 = LockedHistory[HistoryIndex - NumSamples + SampleIndex + 1];

		BatchedElements->AddLine(
			FVector(CanvasPosition.Min.X + SampleIndex, CanvasPosition.Max.Y - Locked0 * CanvasPosition.GetSize().Y, 0.0f),
			FVector(CanvasPosition.Min.X + SampleIndex + 1, CanvasPosition.Max.Y - Locked1 * CanvasPosition.GetSize().Y, 0.0f),
			GraphLockedPageColor,
			HitProxyId);

		float MipMapBias0 = MipMapBiasHistory[HistoryIndex - NumSamples + SampleIndex];
		float MipMapBias1 = MipMapBiasHistory[HistoryIndex - NumSamples + SampleIndex + 1];

		BatchedElements->AddLine(
			FVector(CanvasPosition.Min.X + SampleIndex, CanvasPosition.Max.Y - MipMapBias0 * CanvasPosition.GetSize().Y, 0.0f),
			FVector(CanvasPosition.Min.X + SampleIndex + 1, CanvasPosition.Max.Y - MipMapBias1 * CanvasPosition.GetSize().Y, 0.0f),
			GraphMipMapBiasColor,
			HitProxyId);
	}
#endif // UE_BUILD_SHIPPING
}

void FVirtualTexturePhysicalSpace::UpdateCsvStats() const
{
#if CSV_PROFILER_STATS && !UE_BUILD_SHIPPING
	FCsvProfiler* Profiler = FCsvProfiler::Get();
	if (Profiler->IsCapturing_Renderthread())
	{
		const FString LockedTitle = FString::Printf(TEXT("%s_%d/LockedPages"), *FormatString, GetID());
		Profiler->RecordCustomStat(*LockedTitle, CSV_CATEGORY_INDEX(VirtualTexturing), LockedHistory[HistoryIndex], ECsvCustomStatOp::Set);
		const FString VisbileTitle = FString::Printf(TEXT("%s_%d/VisiblePages"), *FormatString, GetID());
		Profiler->RecordCustomStat(*VisbileTitle, CSV_CATEGORY_INDEX(VirtualTexturing), VisibleHistory[HistoryIndex], ECsvCustomStatOp::Set);
		const FString MipBiasTitle = FString::Printf(TEXT("%s_%d/MipBias"), *FormatString, GetID());
		Profiler->RecordCustomStat(*MipBiasTitle, CSV_CATEGORY_INDEX(VirtualTexturing), MipMapBiasHistory[HistoryIndex], ECsvCustomStatOp::Set);
	}
#endif
}
