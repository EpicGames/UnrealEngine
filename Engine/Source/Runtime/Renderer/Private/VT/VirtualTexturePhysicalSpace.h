// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Containers/CircularBuffer.h"
#include "RendererInterface.h"
#include "TexturePagePool.h"
#include "VirtualTextureShared.h"
#include "VirtualTexturing.h"
#include "RenderGraphFwd.h"
#include "RHIFwd.h"
#include "RenderResource.h"
#include "RenderUtils.h"

class FRDGExternalAccessQueue;

/** Description that determines a unique physical space. This is filled out by a producer. */
struct FVTPhysicalSpaceDescription
{
	uint32 TileSize;
	uint8 Dimensions;
	uint8 NumLayers;
	TEnumAsByte<EPixelFormat> Format[VIRTUALTEXTURE_SPACE_MAXLAYERS];
	bool bHasLayerSrgbView[VIRTUALTEXTURE_SPACE_MAXLAYERS];
	bool bCanSplit;
};

inline bool operator==(const FVTPhysicalSpaceDescription& Lhs, const FVTPhysicalSpaceDescription& Rhs)
{
	if (Lhs.TileSize != Rhs.TileSize ||
		Lhs.NumLayers != Rhs.NumLayers || 
		Lhs.Dimensions != Rhs.Dimensions ||
		Lhs.bCanSplit != Rhs.bCanSplit)
	{
		return false;
	}
	for (int32 Layer = 0; Layer < Lhs.NumLayers; ++Layer)
	{
		if (Lhs.Format[Layer] != Rhs.Format[Layer])
		{
			return false;
		}

		if (Lhs.bHasLayerSrgbView[Layer] != Rhs.bHasLayerSrgbView[Layer])
		{
			return false;
		}

	}
	return true;
}
inline bool operator!=(const FVTPhysicalSpaceDescription& Lhs, const FVTPhysicalSpaceDescription& Rhs)
{
	return !operator==(Lhs, Rhs);
}

inline uint32 GetTypeHash(const FVTPhysicalSpaceDescription& Desc)
{
	uint32 Hash = Desc.TileSize;
	Hash = HashCombine(Hash, GetTypeHash(Desc.Dimensions));
	Hash = HashCombine(Hash, GetTypeHash(Desc.NumLayers));
	Hash = HashCombine(Hash, GetTypeHash(Desc.bCanSplit));
	for (int32 Layer = 0; Layer < Desc.NumLayers; ++Layer)
	{
		Hash = HashCombine(Hash, GetTypeHash(Desc.Format[Layer]));
		Hash = HashCombine(Hash, GetTypeHash(Desc.bHasLayerSrgbView[Layer]));
	}
	return Hash;
}

/** Additional part of the description for a unique physical space that is derived from the virtual texture pool config. */
struct FVTPhysicalSpaceDescriptionExt
{
	int32 TileWidthHeight = 0;
	int32 PoolCount = 1;
	bool bEnableResidencyMipMapBias = false;
	int32 ResidencyMipMapBiasGroup = 0;
};

inline bool operator==(const FVTPhysicalSpaceDescriptionExt& Lhs, const FVTPhysicalSpaceDescriptionExt& Rhs)
{
	return Lhs.TileWidthHeight == Rhs.TileWidthHeight && Lhs.PoolCount == Rhs.PoolCount && 
		Lhs.bEnableResidencyMipMapBias == Rhs.bEnableResidencyMipMapBias && Lhs.ResidencyMipMapBiasGroup == Rhs.ResidencyMipMapBiasGroup;
}

/** Virtual texture physical space that contains the physical texture and the page pool that tracks residency. */
class FVirtualTexturePhysicalSpace final : public FRenderResource
{
public:
	FVirtualTexturePhysicalSpace(uint16 InID, const FVTPhysicalSpaceDescription& InDesc, FVTPhysicalSpaceDescriptionExt& InDescExt);
	virtual ~FVirtualTexturePhysicalSpace();

	inline const FVTPhysicalSpaceDescription& GetDescription() const { return Description; }
	inline const FVTPhysicalSpaceDescriptionExt& GetDescriptionExt() const { return DescriptionExt; }
	inline const FString& GetFormatString() const { return FormatString; }
	inline EPixelFormat GetFormat(int32 Layer) const { return Description.Format[Layer]; }
	inline uint16 GetID() const { return ID; }
	inline uint32 GetSizeInTiles() const { return DescriptionExt.TileWidthHeight; }
	inline uint32 GetNumTiles() const { return GetSizeInTiles() * GetSizeInTiles(); }
	inline uint32 GetTextureSize() const { return GetSizeInTiles() * Description.TileSize; }
	inline FIntVector GetPhysicalLocation(uint16 pAddress) const { return FIntVector(pAddress % GetSizeInTiles(), pAddress / GetSizeInTiles(), 0); }

	// 16bit page tables allocate 6bits to address TileX/Y, so can only address tiles from 0-63
	inline bool DoesSupport16BitPageTable() const { return GetSizeInTiles() <= 64u; }

	uint32 GetTileSizeInBytes() const;
	uint32 GetSizeInBytes() const;

	inline const FTexturePagePool& GetPagePool() const { return Pool; }
	inline FTexturePagePool& GetPagePool() { return Pool; }

	// FRenderResource interface
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	// RefCount for TRefCountPtr management of physical space.
	// Both producers and allocated VT keep references to keep a physical space alive.
	inline uint32 AddRef() { return ++NumRefs; }
	inline uint32 Release() { check(NumRefs > 0u); return --NumRefs; }
	inline uint32 GetRefCount() const { return NumRefs; }

	// RefCount for tracking allocated VT references only.
	// We only allocate underlying physical texture when referenced from an allocated VT.
	inline uint32 AddResourceRef() { return ++NumResourceRefs; }
	inline uint32 ReleaseResourceRef() { check(NumResourceRefs > 0u); return --NumResourceRefs; }
	inline uint32 GetResourceRefCount() const { return NumResourceRefs; }

	inline TRefCountPtr<IPooledRenderTarget> GetPhysicalTexture(int32 Layer) const
	{
		return PooledRenderTarget[Layer];
	}

	inline FRHIShaderResourceView* GetPhysicalTextureSRV(int32 Layer, bool bSRGB) const
	{
		return bSRGB ? TextureSRV_SRGB[Layer] : TextureSRV[Layer];
	}

	/** Gather the necessary physical texture transitions back to SRVMask after the producer Finalize() point in the frame. */
	void FinalizeTextures(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue);

	/** Update internal tracking of residency. This is used to update stats and to calculate a mip bias to keep within the pool budget. */
	void UpdateResidencyTracking(uint32 Frame);
	/** Get dynamic mip bias used to keep within residency budget. */
	inline float GetResidencyMipMapBias() const { return ResidencyMipMapBias; }
	/** Get frame at which residency tracking last saw over-subscription. */
	inline uint32 GetLastFrameOversubscribed() const { return LastFrameOversubscribed; }
	/** Draw residency graph on screen. */
	void DrawResidencyGraph(class FCanvas* Canvas, FBox2D CanvasPosition, bool bDrawKey);
	/** Write residency stats to CSV profiler */
	void UpdateCsvStats() const;

private:
	FVTPhysicalSpaceDescription Description;
	FVTPhysicalSpaceDescriptionExt DescriptionExt;
	FString FormatString;
	FTexturePagePool Pool;
	TRefCountPtr<IPooledRenderTarget> PooledRenderTarget[VIRTUALTEXTURE_SPACE_MAXLAYERS];
	FShaderResourceViewRHIRef TextureSRV[VIRTUALTEXTURE_SPACE_MAXLAYERS];
	FShaderResourceViewRHIRef TextureSRV_SRGB[VIRTUALTEXTURE_SPACE_MAXLAYERS];
	FString PooledRenderTargetDebugNames[VIRTUALTEXTURE_SPACE_MAXLAYERS];

	uint16 ID;
	uint32 NumRefs;
	uint32 NumResourceRefs;
	
	float ResidencyMipMapBias;
	uint32 LastFrameOversubscribed;

#if !UE_BUILD_SHIPPING
	static const int32 HistorySize = 512;
	TCircularBuffer<float> VisibleHistory;
	TCircularBuffer<float> LockedHistory;
	TCircularBuffer<float> MipMapBiasHistory;
	uint32 HistoryIndex;
#endif
};
