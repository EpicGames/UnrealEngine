// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureProducer.h"

#include "VT/VirtualTexturePhysicalSpace.h"
#include "VT/VirtualTextureSystem.h"

static TAutoConsoleVariable<int32> CVarVTEnableNotifyCompleted(
	TEXT("r.VT.EnableNotifyCompleted"),
	0,
	TEXT("Enable calls to OnRequestsCompleted().\n")
	TEXT("Note that support for OnRequestsCompleted() callbacks is deprecated in 5.8 and will be removed in a later version."),
	ECVF_RenderThreadSafe
);

FVirtualTextureProducer::~FVirtualTextureProducer()
{
}

void FVirtualTextureProducer::Release(FVirtualTextureSystem* System, const FVirtualTextureProducerHandle& HandleToSelf)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureProducer::Release);

	if (Description.bPersistentHighestMip)
	{
		System->ForceUnlockAllTiles(HandleToSelf, this);
	}

	for (uint32 LayerIndex = 0u; LayerIndex < Description.NumPhysicalGroups; ++LayerIndex)
	{
		FVirtualTexturePhysicalSpace* Space = PhysicalGroups[LayerIndex].PhysicalSpace;
		Space->GetPagePool().EvictPages(System, HandleToSelf);
		PhysicalGroups[LayerIndex].PhysicalSpace.SafeRelease();
	}

	PhysicalGroups.Reset();

	FGraphEventArray ProducePageTasks;
	VirtualTexture->GatherProducePageDataTasks(HandleToSelf, ProducePageTasks);
	if (ProducePageTasks.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureProducer::Release_Wait);
		FTaskGraphInterface::Get().WaitUntilTasksComplete(ProducePageTasks, ENamedThreads::GetRenderThread_Local());
	}

	delete VirtualTexture;
	VirtualTexture = nullptr;
	
	Description = FVTProducerDescription();
}

FVirtualTextureProducerCollection::FVirtualTextureProducerCollection() : NumPendingCallbacks(0u)
{
	Producers.AddDefaulted(1);
	Producers[0].Magic = 1u; // make sure FVirtualTextureProducerHandle(0) will not resolve to the dummy producer entry

	Callbacks.AddDefaulted(CallbackList_Count);
	for (uint32 CallbackIndex = 0u; CallbackIndex < CallbackList_Count; ++CallbackIndex)
	{
		FCallbackEntry& Callback = Callbacks[CallbackIndex];
		Callback.NextIndex = Callback.PrevIndex = CallbackIndex;
	}
}

FVirtualTextureProducerHandle FVirtualTextureProducerCollection::RegisterProducer(FRHICommandListBase& RHICmdList, FVirtualTextureSystem* System, const FVTProducerDescription& InDesc, IVirtualTexture* InProducer)
{
	const uint32 ProducerWidth = InDesc.BlockWidthInTiles * InDesc.WidthInBlocks * InDesc.TileSize;
	const uint32 ProducerHeight = InDesc.BlockHeightInTiles * InDesc.HeightInBlocks * InDesc.TileSize;
	check(ProducerWidth > 0u);
	check(ProducerHeight > 0u);
	check(InDesc.MaxLevel <= FMath::CeilLogTwo(FMath::Max(ProducerWidth, ProducerHeight)));
	check(InDesc.NumTextureLayers <= VIRTUALTEXTURE_SPACE_MAXLAYERS);
	check(InDesc.NumPhysicalGroups <= InDesc.NumTextureLayers);
	check(InProducer);

	const uint32 Index = AcquireEntry();
	FProducerEntry& Entry = Producers[Index];
	Entry.Producer.Description = InDesc;
	Entry.Producer.VirtualTexture = InProducer;
	Entry.DestroyedCallbacksIndex = AcquireCallback();

	FVTPhysicalSpaceDescription PhysicalSpaceDesc;
	PhysicalSpaceDesc.Dimensions = InDesc.Dimensions;
	PhysicalSpaceDesc.TileSize = InDesc.TileSize + InDesc.TileBorderSize * 2u;
	PhysicalSpaceDesc.bCanSplit = !InDesc.bRequiresSinglePhysicalPool;

	Entry.Producer.PhysicalGroups.SetNum(InDesc.NumPhysicalGroups);
	for (uint32 PhysicalGroupIndex = 0u; PhysicalGroupIndex < InDesc.NumPhysicalGroups; ++PhysicalGroupIndex)
	{
		PhysicalSpaceDesc.NumLayers = 0;
		for (uint32 LayerIndex = 0u; LayerIndex < InDesc.NumTextureLayers; ++LayerIndex)
		{
			if (InDesc.PhysicalGroupIndex[LayerIndex] == PhysicalGroupIndex)
			{
				PhysicalSpaceDesc.Format[PhysicalSpaceDesc.NumLayers] = InDesc.LayerFormat[LayerIndex];
				// sRGB on/off flag is ignored on platforms that has support for texture aliasing
				PhysicalSpaceDesc.bHasLayerSrgbView[PhysicalSpaceDesc.NumLayers] = (GRHISupportsTextureViews ? true : InDesc.bIsLayerSRGB[LayerIndex]);
				PhysicalSpaceDesc.NumLayers++;
			}
		}
		check(PhysicalSpaceDesc.NumLayers > 0);
		Entry.Producer.PhysicalGroups[PhysicalGroupIndex].PhysicalSpace = System->AcquirePhysicalSpace(RHICmdList, PhysicalSpaceDesc);
	}

	if (InDesc.bTrackSamplingHistory)
	{
		Entry.LastFramePerMip.SetNumZeroed(InDesc.MaxLevel + 1);
	}

	const FVirtualTextureProducerHandle Handle(Index, Entry.Magic);
	return Handle;
}

void FVirtualTextureProducerCollection::UpdateSamplingHistory(uint32 PackedProducerHandle, uint8 MipLevel, uint32 Frame)
{
	const FVirtualTextureProducerHandle Handle(PackedProducerHandle);
	if (FProducerEntry* Entry = GetEntry(Handle))
	{
		if (MipLevel < (uint8)Entry->LastFramePerMip.Num())
		{
			Entry->LastFramePerMip[MipLevel] = Frame;
		}
	}
}

TConstArrayView<uint32> FVirtualTextureProducerCollection::GetLastFramePerMip(const FVirtualTextureProducerHandle& Handle) const
{
	if (FProducerEntry const* Entry = GetEntry(Handle))
	{
		if (!Entry->LastFramePerMip.IsEmpty())
		{
			return Entry->LastFramePerMip;
		}
	}
	return {};
}

void FVirtualTextureProducerCollection::ReleaseProducer(FVirtualTextureSystem* System, const FVirtualTextureProducerHandle& Handle)
{
	if (FProducerEntry* Entry = GetEntry(Handle))
	{
		uint32 CallbackIndex = Callbacks[Entry->DestroyedCallbacksIndex].NextIndex;
		while (CallbackIndex != Entry->DestroyedCallbacksIndex)
		{
			FCallbackEntry& Callback = Callbacks[CallbackIndex];
			const uint32 NextIndex = Callback.NextIndex;
			check(Callback.OwnerHandle == Handle);

			check(!Callback.bPending);
			Callback.bPending = true;

			// Move callback to pending list
			RemoveCallbackFromList(CallbackIndex);
			AddCallbackToList(CallbackList_Pending, CallbackIndex);
			++NumPendingCallbacks;
			
			CallbackIndex = NextIndex;
		}

		ReleaseCallback(Entry->DestroyedCallbacksIndex);
		Entry->DestroyedCallbacksIndex = 0u;

		Entry->Magic = (Entry->Magic + 1u) & 1023u;
		Entry->Producer.Release(System, Handle);
		ReleaseEntry(Handle.Index);
	}
}

bool FVirtualTextureProducerCollection::TryReleaseProducer(FVirtualTextureSystem* System, const FVirtualTextureProducerHandle& Handle)
{
	if (FProducerEntry* Entry = GetEntry(Handle))
	{
		FGraphEventArray ProducePageTasks;
		Entry->Producer.GetVirtualTexture()->GatherProducePageDataTasks(Handle, ProducePageTasks);

		if (ProducePageTasks.Num() != 0)
		{
			// Don't release with tasks remaining.
			return false;
		}
	}

	ReleaseProducer(System, Handle);
	return true;
}

void FVirtualTextureProducerCollection::CallPendingCallbacks()
{
	uint32 CallbackIndex = Callbacks[CallbackList_Pending].NextIndex;
	uint32 NumCallbacksChecked = 0u;
	while (CallbackIndex != CallbackList_Pending)
	{
		FCallbackEntry& Callback = Callbacks[CallbackIndex];
		check(Callback.bPending);
	
		// Make a copy, then release the callback entry before calling the callback function
		// (The destroyed callback may try to remove this or other callbacks, so need to make sure state is valid before calling)
		const FCallbackEntry CallbackCopy(Callback);
		if (Callback.Baton)
		{
			TArray<uint32>& BatonCallbacks = CallbacksMap.FindChecked(Callback.Baton);
			BatonCallbacks.Remove(CallbackIndex);
			if (BatonCallbacks.IsEmpty())
			{
				CallbacksMap.Remove(Callback.Baton);
			}
		}
		Callback.DestroyedFunction = nullptr;
		Callback.Baton = nullptr;
		Callback.OwnerHandle = FVirtualTextureProducerHandle();
		Callback.PackedFlags = 0u;
		ReleaseCallback(CallbackIndex);

		// Possible that this callback may have been removed from the list by a previous pending callback
		// In this case, the function pointer will be set to nullptr
		if (CallbackCopy.DestroyedFunction)
		{
			check(CallbackCopy.OwnerHandle.PackedValue != 0u);
			CallbackCopy.DestroyedFunction(CallbackCopy.OwnerHandle, CallbackCopy.Baton);
		}
		CallbackIndex = CallbackCopy.NextIndex;
		++NumCallbacksChecked;
	}

	// Extra check to detect list corruption
	check(NumCallbacksChecked == NumPendingCallbacks);
	NumPendingCallbacks = 0u;

}

bool FVirtualTextureProducerCollection::HasPendingCallbacks() const
{
	return NumPendingCallbacks != 0;
}

void FVirtualTextureProducerCollection::AddDestroyedCallback(const FVirtualTextureProducerHandle& Handle, FVTProducerDestroyedFunction* Function, void* Baton)
{
	check(Function);
	check(Baton);

	FProducerEntry* Entry = GetEntry(Handle);
	if (Entry)
	{
		const uint32 CallbackIndex = AcquireCallback();
		AddCallbackToList(Entry->DestroyedCallbacksIndex, CallbackIndex);
		FCallbackEntry& Callback = Callbacks[CallbackIndex];
		Callback.DestroyedFunction = Function;
		Callback.Baton = Baton;
		Callback.OwnerHandle = Handle;
		Callback.PackedFlags = 0u;
		CallbacksMap.FindOrAdd(Baton).Add(CallbackIndex);
	}
}

uint32 FVirtualTextureProducerCollection::RemoveAllCallbacks(const void* Baton)
{
	check(Baton);

	uint32 NumRemoved = 0u;
	TArray<uint32>* CallbackIndices = CallbacksMap.Find(Baton);
	if (CallbackIndices != nullptr)
	{
		for (int32 CallbackIndex : *CallbackIndices)
		{
			FCallbackEntry& Callback = Callbacks[CallbackIndex];
			check(Callback.Baton == Baton);
			check(Callback.DestroyedFunction);
			Callback.DestroyedFunction = nullptr;
			Callback.Baton = nullptr;
			Callback.OwnerHandle = FVirtualTextureProducerHandle();

			// If callback is already pending, we can't move it back to free list, or we risk corrupting the pending list while it's being iterated
			// Setting DestroyedFunction to nullptr here will ensure callback is no longer invoked, and it will be moved to free list later when it's removed from pending list
			if (!Callback.bPending)
			{
				Callback.PackedFlags = 0u;
				ReleaseCallback(CallbackIndex);
			}
			++NumRemoved;
		}
		CallbacksMap.Remove(Baton);
	}
	return NumRemoved;
}

void FVirtualTextureProducerCollection::NotifyRequestsCompleted()
{
	if (CVarVTEnableNotifyCompleted.GetValueOnAnyThread() != 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::NotifyRequestsComplete);

		for (FProducerEntry& Entry : Producers)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (Entry.Producer.Description.bNotifyCompleted)
			{
				Entry.Producer.GetVirtualTexture()->OnRequestsCompleted();
			}
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

FVirtualTextureProducer* FVirtualTextureProducerCollection::FindProducer(const FVirtualTextureProducerHandle& Handle)
{
	FProducerEntry* Entry = GetEntry(Handle);
	return Entry ? &Entry->Producer : nullptr;
}

FVirtualTextureProducer& FVirtualTextureProducerCollection::GetProducer(const FVirtualTextureProducerHandle& Handle)
{
	const uint32 Index = Handle.Index;
	check(Index < (uint32)Producers.Num());
	FProducerEntry& Entry = Producers[Index];
	check(Entry.Magic == Handle.Magic);
	return Entry.Producer;
}

FVirtualTextureProducerCollection::FProducerEntry const* FVirtualTextureProducerCollection::GetEntry(const FVirtualTextureProducerHandle& Handle) const
{
	const uint32 Index = Handle.Index;
	if (Index < (uint32)Producers.Num())
	{
		FProducerEntry const& Entry = Producers[Index];
		if (Entry.Magic == Handle.Magic)
		{
			return &Entry;
		}
	}
	return nullptr;
}

FVirtualTextureProducerCollection::FProducerEntry* FVirtualTextureProducerCollection::GetEntry(const FVirtualTextureProducerHandle& Handle)
{
	const uint32 Index = Handle.Index;
	if (Index < (uint32)Producers.Num())
	{
		FProducerEntry& Entry = Producers[Index];
		if (Entry.Magic == Handle.Magic)
		{
			return &Entry;
		}
	}
	return nullptr;
}
