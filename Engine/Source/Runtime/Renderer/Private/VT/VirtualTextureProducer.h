// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTextureShared.h"
#include "RendererInterface.h"
#include "VirtualTexturing.h"
#include "VirtualTexturePhysicalSpace.h"

class FVirtualTextureSystem;
class FVirtualTexturePhysicalSpace;

class FVirtualTextureProducer
{
public:
	FVirtualTextureProducer() : VirtualTexture(nullptr) { }

	void Release(FVirtualTextureSystem* System, const FVirtualTextureProducerHandle& HandleToSelf);

	inline const FVTProducerDescription& GetDescription() const { return Description; }
	inline IVirtualTexture* GetVirtualTexture() const { return VirtualTexture; }
	inline const FName& GetName() const { return Description.Name; }
	inline uint32 GetWidthInTiles() const { return Description.BlockWidthInTiles * Description.WidthInBlocks; }
	inline uint32 GetHeightInTiles() const { return Description.BlockHeightInTiles * Description.HeightInBlocks; }
	inline uint32 GetDepthInTiles() const { return Description.DepthInTiles; }
	inline uint32 GetMaxLevel() const { return Description.MaxLevel; }

	inline uint32 GetNumTextureLayers() const { return Description.NumTextureLayers; }
	inline EPixelFormat GetLayerFormat(uint32 LayerIndex) const { check(LayerIndex < Description.NumTextureLayers); return Description.LayerFormat[LayerIndex]; }
	inline uint32 GetPhysicalGroupIndexForTextureLayer(uint32 LayerIndex) const { check(LayerIndex < Description.NumTextureLayers); return Description.PhysicalGroupIndex[LayerIndex]; }

	inline uint32 GetNumPhysicalGroups() const { return Description.NumPhysicalGroups; }
	inline FVirtualTexturePhysicalSpace* GetPhysicalSpaceForPhysicalGroup(uint32 GroupIndex) const { check(GroupIndex < Description.NumPhysicalGroups); return PhysicalGroups[GroupIndex].PhysicalSpace; }

	inline void SetLockedMipCount(uint8 InLockedMipCount) { LockedMipCount = InLockedMipCount; }
	inline uint8 GetLockedMipCount() const { return LockedMipCount; }

private:
	friend class FVirtualTextureProducerCollection;
	~FVirtualTextureProducer();

	FVTProducerDescription Description;
	
	// Number of mips that we have requested to keep locked.
	uint8 LockedMipCount = 0;

	// A physical group is a physical space which multiple texture layers map to
	// Texture layers that share a physical group will always be sampled via the same page table channel (they will have the same physical UV coordinates)
	// More than one physical group can share the same FVirtualTexturePhysicalSpace
	struct FPhysicalGroupDesc
	{
		TRefCountPtr<FVirtualTexturePhysicalSpace> PhysicalSpace;
	};
	TArray<FPhysicalGroupDesc> PhysicalGroups;

	// The virtual texture provider implementation
	IVirtualTexture* VirtualTexture;
};

class FVirtualTextureProducerCollection
{
public:
	FVirtualTextureProducerCollection();
	FVirtualTextureProducerHandle RegisterProducer(FRHICommandListBase& RHICmdList, FVirtualTextureSystem* System, const FVTProducerDescription& InDesc, IVirtualTexture* InProducer);
	void ReleaseProducer(FVirtualTextureSystem* System, const FVirtualTextureProducerHandle& Handle);
	bool TryReleaseProducer(FVirtualTextureSystem* System, const FVirtualTextureProducerHandle& Handle);
	
	/**
	 * Record that a page at MipLevel was sampled for the producer identified by PackedProducerHandle.
	 * Only updates producers that were registered with bTrackSamplingHistory=true.
	 */
	void UpdateSamplingHistory(uint32 PackedProducerHandle, uint8 MipLevel, uint32 Frame);

	/**
	 * Returns the LastFramePerMip array for the given producer, or an empty view if not tracking.
	 * Entry [i] is the last frame mip i was sampled; 0 means never sampled.
	 */
	TConstArrayView<uint32> GetLastFramePerMip(const FVirtualTextureProducerHandle& Handle) const;
	
	void AddDestroyedCallback(const FVirtualTextureProducerHandle& Handle, FVTProducerDestroyedFunction* Function, void* Baton);
	uint32 RemoveAllCallbacks(const void* Baton);
	void CallPendingCallbacks();
	bool HasPendingCallbacks() const;

	/** 
	 * Notify producers that all requests have been completed. 
	 * Only notifies when r.VT.EnableNotifyCompleted=1 and for producers with bNotifyCompleted=true.
	 */
	void NotifyRequestsCompleted();

	/**
	 * Gets the producer associated with the given handle, or nullptr if handle is invalid
	 * Returned pointer is only valid until the next call to RegisterProducer, so should not be stored beyond scope of a function
	 */
	FVirtualTextureProducer* FindProducer(const FVirtualTextureProducerHandle& Handle);

	/** Like FindProducer, but fails check/crashes if handle is not valid.  Returns a reference, since never needs to return nullptr */
	FVirtualTextureProducer& GetProducer(const FVirtualTextureProducerHandle& Handle);

private:
	struct FProducerEntry
	{
		FVirtualTextureProducer Producer;
		uint32 DestroyedCallbacksIndex = 0u;
		uint32 NextIndex = 0u;
		uint32 PrevIndex = 0u;
		uint16 Magic = 0u;

		/** Per-mip last-sampled frame. Empty when bTrackSamplingHistory was not set at registration; otherwise sized to MaxLevel+1. */
		TArray<uint32> LastFramePerMip;
	};

	struct FCallbackEntry
	{
		FVTProducerDestroyedFunction* DestroyedFunction = nullptr;
		void* Baton = nullptr;
		FVirtualTextureProducerHandle OwnerHandle;
		uint32 NextIndex = 0u;
		uint32 PrevIndex = 0u;
		union
		{
			uint32 PackedFlags = 0u;
			struct
			{
				uint32 bPending : 1;
				uint32 Pad : 31;
			};
		};
	};

	enum ECallbackListType
	{
		CallbackList_Free,
		CallbackList_Pending,
		CallbackList_Count,
	};

	FProducerEntry const* GetEntry(const FVirtualTextureProducerHandle& Handle) const;
	FProducerEntry* GetEntry(const FVirtualTextureProducerHandle& Handle);

	void RemoveEntryFromList(uint32 Index)
	{
		FProducerEntry& Entry = Producers[Index];
		Producers[Entry.PrevIndex].NextIndex = Entry.NextIndex;
		Producers[Entry.NextIndex].PrevIndex = Entry.PrevIndex;
		Entry.NextIndex = Entry.PrevIndex = Index;
	}

	void AddEntryToList(uint32 HeadIndex, uint32 Index)
	{
		FProducerEntry& Head = Producers[HeadIndex];
		FProducerEntry& Entry = Producers[Index];
		check(Index > 0u); // make sure we're not trying to add a list head to another list

		// make sure we're not currently in any list
		check(Entry.NextIndex == Index);
		check(Entry.PrevIndex == Index);

		Entry.NextIndex = HeadIndex;
		Entry.PrevIndex = Head.PrevIndex;
		Producers[Head.PrevIndex].NextIndex = Index;
		Head.PrevIndex = Index;
	}

	uint32 AcquireEntry()
	{
		FProducerEntry& FreeHead = Producers[0u];
		uint32 Index = FreeHead.NextIndex;
		if (Index != 0u)
		{
			RemoveEntryFromList(Index);
			return Index;
		}

		Index = Producers.AddDefaulted();
		FProducerEntry& Entry = Producers[Index];
		Entry.NextIndex = Entry.PrevIndex = Index;
		return Index;
	}

	void ReleaseEntry(uint32 Index)
	{
		RemoveEntryFromList(Index);
		AddEntryToList(0u, Index);
	}

	void RemoveCallbackFromList(uint32 Index)
	{
		FCallbackEntry& Entry = Callbacks[Index];
		Callbacks[Entry.PrevIndex].NextIndex = Entry.NextIndex;
		Callbacks[Entry.NextIndex].PrevIndex = Entry.PrevIndex;
		Entry.NextIndex = Entry.PrevIndex = Index;
	}

	void AddCallbackToList(uint32 HeadIndex, uint32 Index)
	{
		FCallbackEntry& Head = Callbacks[HeadIndex];
		FCallbackEntry& Entry = Callbacks[Index];
		check(Index >= CallbackList_Count); // make sure we're not trying to add a list head to another list

		// make sure we're not currently in any list
		check(Entry.NextIndex == Index);
		check(Entry.PrevIndex == Index);

		Entry.NextIndex = HeadIndex;
		Entry.PrevIndex = Head.PrevIndex;
		Callbacks[Head.PrevIndex].NextIndex = Index;
		Head.PrevIndex = Index;
	}

	uint32 AcquireCallback()
	{
		FCallbackEntry& FreeHead = Callbacks[CallbackList_Free];
		uint32 Index = FreeHead.NextIndex;
		if (Index != CallbackList_Free)
		{
			RemoveCallbackFromList(Index);
			return Index;
		}

		Index = Callbacks.AddDefaulted();
		FCallbackEntry& Entry = Callbacks[Index];
		Entry.NextIndex = Entry.PrevIndex = Index;
		return Index;
	}

	void ReleaseCallback(uint32 Index)
	{
		RemoveCallbackFromList(Index);
		AddCallbackToList(CallbackList_Free, Index);
	}

	TArray<FProducerEntry> Producers;
	TArray<FCallbackEntry> Callbacks;
	TMap<void*, TArray<uint32>> CallbacksMap;
	uint32 NumPendingCallbacks;
};
