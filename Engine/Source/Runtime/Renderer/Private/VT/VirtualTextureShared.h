// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class EVTProducerPriority : uint8;
enum class EVTInvalidatePriority : uint8;

union FPhysicalTileLocation
{
	FPhysicalTileLocation() {}
	FPhysicalTileLocation(const FIntVector& InVec)
		: TileX(InVec.X)
		, TileY(InVec.Y)
	{
		checkSlow(InVec.X >= 0 && InVec.X <= 255);
		checkSlow(InVec.Y >= 0 && InVec.Y <= 255);
	}

	uint16 Packed;
	struct 
	{
		uint8 TileX;
		uint8 TileY;
	};
};

struct FPageTableUpdate
{
	uint32					vAddress;
	FPhysicalTileLocation	pTileLocation;
	uint8					vLevel;
	uint8					vLogSize;

	FPageTableUpdate() {}
	FPageTableUpdate(const FPageTableUpdate& Other) = default;
	FPageTableUpdate& operator=(const FPageTableUpdate& Other) = default;

	FPageTableUpdate(const FPageTableUpdate& Update, uint32 Offset, uint8 vDimensions)
		: vAddress(Update.vAddress + (Offset << (vDimensions * Update.vLogSize)))
		, pTileLocation(Update.pTileLocation)
		, vLevel(Update.vLevel)
		, vLogSize(Update.vLogSize)
	{}

	inline void Check(uint8 vDimensions)
	{
		const uint32 LowBitMask = (1u << (vDimensions * vLogSize)) - 1;
		check((vAddress & LowBitMask) == 0);
		//checkSlow(vLogSize <= vLevel);
	}
};

// Little utility struct that allows to quickly sort different VT tile-related containers with different policies (size is limited to 64 bits, including the N bits-index, where N = NumBitsForIndex)
//  PriorityKeyType the priority value on the remaining bits (64 - NumBitsForIndex)
//  The bigger the priority value, the more important the tile
template <typename PriorityKeyType, uint8 NumBitsForIndex = 16>
union TVTTilePriorityAndIndex final
{
	TVTTilePriorityAndIndex() = default;

	TVTTilePriorityAndIndex(uint64 InIndex, PriorityKeyType&& InPriorityKey)
		// Shift by NumBitsForIndex the priority key so it occupies the most significant bits and thus defines the sorting :
		: SortablePackedValue(InIndex | (InPriorityKey.PackedValue << NumBitsForIndex))
	{
		checkfSlow(InIndex < (1ull << NumBitsForIndex), TEXT("The index and priority are packed onto a single uint64. Only NumBitsForIndex bits are allowed for the index"));
		checkfSlow((InPriorityKey.PackedValue & ~(~0ull >> NumBitsForIndex)) == 0, TEXT("In TVTTilePriorityAndIndex, the priority key is merged with the index into a single uint64. Thus the first N bits of the priority key will be ignored."));
	}

	template<typename... TArgs>
	TVTTilePriorityAndIndex(uint64 InIndex, TArgs&&... InArgs)
		: TVTTilePriorityAndIndex(InIndex, PriorityKeyType(std::forward<TArgs>(InArgs) ...))
	{}

	// sort from largest to smallest
	inline bool operator<(const TVTTilePriorityAndIndex<PriorityKeyType, NumBitsForIndex>& InOther) const { return SortablePackedValue > InOther.SortablePackedValue; }

	PriorityKeyType GetPriorityKey() { return PriorityKeyType(SortablePackedValue >> NumBitsForIndex); }

	uint64 SortablePackedValue = 0;
	// The index are the first N (least significant) bits, so that they don't affect sorting
	uint64 Index : NumBitsForIndex;

	static_assert(sizeof(PriorityKeyType) == sizeof(uint64), "Unexpected size for TPriorityAndIndex. This struct should be kept as small as possible for the sorting to remain efficient");
};

// Sorting key for VT requests
struct FVTRequestPriority final
{
	FVTRequestPriority() = default;

	FVTRequestPriority(bool bInLocked, bool bInStreaming, EVTProducerPriority InProducerPriority, EVTInvalidatePriority InInvalidatePriority, uint32 InPagePriority)
		: PagePriority(InPagePriority)
		, InvalidatePriority(static_cast<uint8>(InInvalidatePriority))
		, ProducerPriority(static_cast<uint8>(InProducerPriority))
		, Streaming((uint64)(bInStreaming ? 1 : 0))
		, Locked((uint64)(bInLocked ? 1 : 0))
		, Pad(0)
	{
		checkfSlow(static_cast<uint64>(InInvalidatePriority) <= (1 << 1), TEXT("EVTInvalidatePriority should be packable on 1 bit"));
		checkfSlow(static_cast<uint64>(InProducerPriority) <= (1 << 3), TEXT("EVTProducerPriority should be packable on 3 bits"));
	}

	FVTRequestPriority(uint64 InPackedValue)
		: PackedValue(InPackedValue)
	{}

	union
	{
		uint64 PackedValue = 0;
		struct
		{
			// Important note : The order of these members is important : it defines the sort order (last member first)
			uint64 PagePriority : 32; // Page priority depends on the number of requests and the mip level (higher mips come first)
			uint64 InvalidatePriority : 1; // Manually-prioritized pages get processed before others
			uint64 ProducerPriority : 3; // Sort by producer priority first
			uint64 Streaming : 1; // Streaming pages get processed before others : important note : this needs to remain the second most significant bit, since FUniqueRequestList::SortRequests assumes streaming pages come next
			uint64 Locked : 1; // Locked pages get processed before others : important note : this needs to remain the most significant bit, since FUniqueRequestList::SortRequests assumes locked pages come first 
			uint64 Pad : 26;
		};
	};
};

// Sorting key for plain FVirtualTextureLocalTile
struct FVTLocalTilePriority final
{
	FVTLocalTilePriority() = default;

	FVTLocalTilePriority(EVTProducerPriority InProducerPriority, EVTInvalidatePriority InInvalidatePriority, uint8 InMipLevel)
		: MipLevel(InMipLevel)
		, InvalidatePriority(static_cast<uint8>(InInvalidatePriority))
		, ProducerPriority(static_cast<uint8>(InProducerPriority))
		, Pad(0)
	{
		checkfSlow(static_cast<uint64>(InProducerPriority) <= (1 << 3), TEXT("EVTProducerPriority should be packable on 3 bits"));
		checkfSlow(static_cast<uint64>(InMipLevel) <= (1 << 4), TEXT("Mip level should be packable on 4 bits"));
	}

	FVTLocalTilePriority(uint64 InPackedValue)
		: PackedValue(InPackedValue)
	{}

	union
	{
		uint64 PackedValue = 0;
		struct
		{
			// Important note : The order of these members is important : it defines the sort order (last member first)
			uint64 MipLevel : 4; // Page priority depends on the number of requests and the mip level (higher mips come first)
			uint64 InvalidatePriority : 1; // Prioritized pages get processed before others
			uint64 ProducerPriority : 3; // Sort by producer priority first
			uint64 Pad : 56;
		};
	};
};

using FVTRequestPriorityAndIndex = TVTTilePriorityAndIndex<FVTRequestPriority>;
using FVTLocalTilePriorityAndIndex = TVTTilePriorityAndIndex<FVTLocalTilePriority>;
