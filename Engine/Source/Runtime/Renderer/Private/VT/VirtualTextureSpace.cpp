// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureSpace.h"
#include "VirtualTexturePhysicalSpace.h"
#include "VirtualTextureSystem.h"
#include "SpriteIndexBuffer.h"
#include "PostProcess/SceneFilterRendering.h"
#include "RenderTargetPool.h"
#include "VisualizeTexture.h"
#include "CommonRenderResources.h"
#include "GlobalShader.h"
#include "GlobalRenderResources.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"
#include "HAL/IConsoleManager.h"
#include "SceneUtils.h"
#include "RenderGraph.h"

#include "VT/AllocatedVirtualTexture.h"

DEFINE_LOG_CATEGORY_STATIC(LogVirtualTextureSpace, Log, All);

static TAutoConsoleVariable<int32> CVarVTRefreshEntirePageTable(
	TEXT("r.VT.RefreshEntirePageTable"),
	0,
	TEXT("Refreshes the entire page table texture every frame"),
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarVTMaskedPageTableUpdates(
	TEXT("r.VT.MaskedPageTableUpdates"),
	1,
	TEXT("Masks the page table update quads to reduce pixel fill costs"),
	ECVF_RenderThreadSafe
	);

// Align value of VT PageTable texture dimension, reduce frame to frame variation of the render target size 
static int32 GVTTexturePageTableSizeAlign = 256;
static FAutoConsoleVariableRef CVarVTTexturePageTableSizeAlign(
	TEXT("r.VT.TexturePageTableSizeAlign"),
	GVTTexturePageTableSizeAlign,
	TEXT("Align value the size of the Virtual Texture PageTable texture width and height. (default: 256 bytes)"),
	ECVF_RenderThreadSafe);

static EPixelFormat GetFormatForNumLayers(uint32 NumLayers, EVTPageTableFormat Format)
{
	const bool bUse16Bits = (Format == EVTPageTableFormat::UInt16);
	switch (NumLayers)
	{
	case 1u: return bUse16Bits ? PF_R16_UINT : PF_R32_UINT;
	case 2u: return bUse16Bits ? PF_R16G16_UINT : PF_R32G32_UINT;
	case 3u:
	case 4u: return bUse16Bits ? PF_R16G16B16A16_UINT : PF_R32G32B32A32_UINT;
	default: checkNoEntry(); return PF_Unknown;
	}
}

FVirtualTextureSpace::FVirtualTextureSpace(FVirtualTextureSystem* InSystem, uint8 InID, const FVTSpaceDescription& InDesc, uint32 InSizeNeeded)
	: Description(InDesc)
	, Allocator(InDesc.Dimensions)
	, ID(InID)
	, bNeedToAllocatePageTable(true)
	, bNeedToAllocatePageTableIndirection(InDesc.IndirectionTextureSize > 0)
{
	// Initialize page map with large enough capacity to handle largest possible physical texture
	const uint32 PhysicalTileSize = InDesc.TileSize + InDesc.TileBorderSize * 2u;
	const uint32 MaxSizeInTiles = GetMax2DTextureDimension() / PhysicalTileSize;
	const uint32 MaxNumTiles = MaxSizeInTiles * MaxSizeInTiles;
	for (uint32 LayerIndex = 0u; LayerIndex < InDesc.NumPageTableLayers; ++LayerIndex)
	{
		PhysicalPageMap[LayerIndex].Initialize(MaxNumTiles, LayerIndex, InDesc.Dimensions);
	}

	uint32 NumLayersToAllocate = InDesc.NumPageTableLayers;
	uint32 PageTableIndex = 0u;
	FMemory::Memzero(TexturePixelFormat);
	while (NumLayersToAllocate > 0u)
	{
		const uint32 NumLayersForTexture = FMath::Min(NumLayersToAllocate, LayersPerPageTableTexture);
		const EPixelFormat PixelFormat = GetFormatForNumLayers(NumLayersForTexture, InDesc.PageTableFormat);
		TexturePixelFormat[PageTableIndex] = PixelFormat;
		NumLayersToAllocate -= NumLayersForTexture;
		++PageTableIndex;
	}

#if !UE_BUILD_SHIPPING
	const uint32 NumPageTableTextures = GetNumPageTableTextures();
	check (NumPageTableTextures == PageTableIndex);
	for (uint32 TextureIndex = 0; TextureIndex < NumPageTableTextures; ++TextureIndex)
	{
		if (PageTableIndex > 1u)
		{
			PageTableDebugNames[TextureIndex] = FString::Printf(TEXT("VirtualTexture_PageTable (%s) %d/%d"), GPixelFormats[TexturePixelFormat[TextureIndex]].Name, TextureIndex + 1, NumPageTableTextures);
		}
		else
		{
			PageTableDebugNames[TextureIndex] = FString::Printf(TEXT("VirtualTexture_PageTable (%s)"), GPixelFormats[TexturePixelFormat[TextureIndex]].Name);
		}
	}
#endif

	Allocator.Initialize(Description.MaxSpaceSize);

}

FVirtualTextureSpace::~FVirtualTextureSpace()
{
}

uint32 FVirtualTextureSpace::AllocateVirtualTexture(FAllocatedVirtualTexture* VirtualTexture)
{
	const uint32 vAddress = Allocator.Alloc(VirtualTexture);
	
	// After allocation, check if we need to reallocate the page table texture.
	const FUintPoint RequiredPageTableSize = GetRequiredPageTableAllocationSize(); 
	if (RequiredPageTableSize.X > CachedPageTableWidth || RequiredPageTableSize.Y > CachedPageTableHeight)
	{
		bNeedToAllocatePageTable = true;
	}

	return vAddress;
}

void FVirtualTextureSpace::FreeVirtualTexture(FAllocatedVirtualTexture* VirtualTexture)
{
	Allocator.Free(VirtualTexture);
}

void FVirtualTextureSpace::InitRHI(FRHICommandListBase& RHICmdList)
{
	for (uint32 TextureIndex = 0u; TextureIndex < GetNumPageTableTextures(); ++TextureIndex)
	{
		FTextureEntry& TextureEntry = PageTable[TextureIndex];
		TextureEntry.TextureReferenceRHI = RHICmdList.CreateTextureReference(GBlackUintTexture->TextureRHI);
	}
	PageTableIndirection.TextureReferenceRHI = RHICmdList.CreateTextureReference(GBlackUintTexture->TextureRHI);
}

void FVirtualTextureSpace::ReleaseRHI()
{
	for (uint32 i = 0u; i < TextureCapacity; ++i)
	{
		FTextureEntry& TextureEntry = PageTable[i];
		TextureEntry.TextureReferenceRHI.SafeRelease();
		GRenderTargetPool.FreeUnusedResource(TextureEntry.RenderTarget);
	}

	PageTableIndirection.TextureReferenceRHI.SafeRelease();
	GRenderTargetPool.FreeUnusedResource(PageTableIndirection.RenderTarget);
}

FUintPoint FVirtualTextureSpace::GetRequiredPageTableAllocationSize() const
{
	// Private spaces should allocate the full page table texture up front.

	const uint32 Width =  FMath::Max(  Description.bPrivateSpace ? Description.MaxSpaceSize : Allocator.GetAllocatedWidth(), VIRTUALTEXTURE_MIN_PAGETABLE_SIZE );
	const uint32 Height = FMath::Max(  Description.bPrivateSpace ? Description.MaxSpaceSize : Allocator.GetAllocatedHeight(), VIRTUALTEXTURE_MIN_PAGETABLE_SIZE) ;

	const uint32 AlignValue = (Description.bPrivateSpace) ? VIRTUALTEXTURE_MIN_PAGETABLE_SIZE : FMath::RoundUpToPowerOfTwo( FMath::Clamp(GVTTexturePageTableSizeAlign, 1, GRHIGlobals.MaxTextureDimensions) );

	const uint32 WidthAligned	= Align(Width, AlignValue);
	const uint32 HeightAligned	= Align(Height, AlignValue);
	return FUintPoint(WidthAligned, HeightAligned);
}

uint32 FVirtualTextureSpace::GetSizeInBytes() const
{
	const FUintPoint RequiredPageTableSize = GetRequiredPageTableAllocationSize();
	const uint32 NumPageTableLevels = FMath::FloorLog2(FMath::Max(RequiredPageTableSize.X, RequiredPageTableSize.Y)) + 1u;

	uint32 TotalSize = 0u;
	for (uint32 TextureIndex = 0u; TextureIndex < GetNumPageTableTextures(); ++TextureIndex)
	{
		const SIZE_T TextureSize = CalcTextureSize(RequiredPageTableSize.X, RequiredPageTableSize.Y, TexturePixelFormat[TextureIndex], NumPageTableLevels);
		TotalSize += TextureSize;
	}
	
	TotalSize += CalculateImageBytes(Description.IndirectionTextureSize, Description.IndirectionTextureSize, 0, PF_R32_UINT);

	return TotalSize;
}

void FVirtualTextureSpace::QueueUpdate(uint8 Layer, uint8 vLogSize, uint32 vAddress, uint8 vLevel, const FPhysicalTileLocation& pTileLocation)
{
	FPageTableUpdate Update;
	Update.vAddress = vAddress;
	Update.pTileLocation = pTileLocation;
	Update.vLevel = vLevel;
	Update.vLogSize = vLogSize;
	Update.Check( Description.Dimensions );
	PageTableUpdates[Layer].Add( Update );
}


TGlobalResource< FSpriteIndexBuffer<8> > GQuadIndexBuffer;

class FPageTableUpdateVS : public FGlobalShader
{
public:
	SHADER_USE_PARAMETER_STRUCT(FPageTableUpdateVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint4>, UpdateBuffer)
		SHADER_PARAMETER(FUintVector2, PageTableSize)
		SHADER_PARAMETER(uint32, FirstUpdate)
		SHADER_PARAMETER(uint32, NumUpdates)
	END_SHADER_PARAMETER_STRUCT()
};

template<bool Use16Bits>
class TPageTableUpdateVS : public FPageTableUpdateVS
{
	DECLARE_GLOBAL_SHADER(TPageTableUpdateVS);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment( Parameters, OutEnvironment );
		OutEnvironment.SetDefine(TEXT("USE_16BIT"), Use16Bits);
	}
};

template<EPixelFormat TargetFormat>
class TPageTableUpdatePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(TPageTableUpdatePS);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment( Parameters, OutEnvironment );
		OutEnvironment.SetRenderTargetOutputFormat(0u, TargetFormat);
	}
};

IMPLEMENT_SHADER_TYPE(template<>, TPageTableUpdateVS<false>, TEXT("/Engine/Private/PageTableUpdate.usf"), TEXT("PageTableUpdateVS"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(template<>, TPageTableUpdateVS<true>, TEXT("/Engine/Private/PageTableUpdate.usf"), TEXT("PageTableUpdateVS"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(template<>, TPageTableUpdatePS<PF_R16_UINT>, TEXT("/Engine/Private/PageTableUpdate.usf"), TEXT("PageTableUpdatePS_1"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, TPageTableUpdatePS<PF_R16G16_UINT>, TEXT("/Engine/Private/PageTableUpdate.usf"), TEXT("PageTableUpdatePS_2"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, TPageTableUpdatePS<PF_R16G16B16A16_UINT>, TEXT("/Engine/Private/PageTableUpdate.usf"), TEXT("PageTableUpdatePS_4"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, TPageTableUpdatePS<PF_R32_UINT>, TEXT("/Engine/Private/PageTableUpdate.usf"), TEXT("PageTableUpdatePS_1"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, TPageTableUpdatePS<PF_R32G32_UINT>, TEXT("/Engine/Private/PageTableUpdate.usf"), TEXT("PageTableUpdatePS_2"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, TPageTableUpdatePS<PF_R32G32B32A32_UINT>, TEXT("/Engine/Private/PageTableUpdate.usf"), TEXT("PageTableUpdatePS_4"), SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FPageTableUpdateParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FPageTableUpdateVS::FParameters, VS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FVirtualTextureSpace::QueueUpdateEntirePageTable()
{
	bForceEntireUpdate = true;
}

void FVirtualTextureSpace::AllocateTextures(FRDGBuilder& GraphBuilder)
{
	if (bNeedToAllocatePageTable)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

		const FUintPoint RequiredPageTableSize = GetRequiredPageTableAllocationSize();
		CachedPageTableWidth = RequiredPageTableSize.X;
		CachedPageTableHeight = RequiredPageTableSize.Y;
		CachedNumPageTableLevels = FMath::FloorLog2(FMath::Max(CachedPageTableWidth, CachedPageTableHeight)) + 1u;

		for (uint32 TextureIndex = 0u; TextureIndex < GetNumPageTableTextures(); ++TextureIndex)
		{
			// Page Table
			FTextureEntry& TextureEntry = PageTable[TextureIndex];
			const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				FIntPoint(CachedPageTableWidth, CachedPageTableHeight),
				TexturePixelFormat[TextureIndex],
				FClearValueBinding::None,
				TexCreate_RenderTargetable | TexCreate_ShaderResource,
				CachedNumPageTableLevels);

			FRDGTextureRef DstTexture = GraphBuilder.CreateTexture(Desc, TEXT("VirtualTexture_PageTable"));

			if (TextureEntry.RenderTarget)
			{
				FRDGTextureRef SrcTexture = GraphBuilder.RegisterExternalTexture(TextureEntry.RenderTarget);
				const FRDGTextureDesc& SrcDesc = SrcTexture->Desc;

				// Copy previously allocated page table to new texture
				FRHICopyTextureInfo CopyInfo;
				CopyInfo.Size.X = FMath::Min(Desc.Extent.X, SrcDesc.Extent.X);
				CopyInfo.Size.Y = FMath::Min(Desc.Extent.Y, SrcDesc.Extent.Y);
				CopyInfo.Size.Z = 1;
				CopyInfo.NumMips = FMath::Min(Desc.NumMips, SrcDesc.NumMips);

				GraphBuilder.UseInternalAccessMode(SrcTexture);
				AddCopyTexturePass(GraphBuilder, SrcTexture, DstTexture, CopyInfo);
			}

			TextureEntry.RenderTarget = GraphBuilder.ConvertToExternalTexture(DstTexture);
			GraphBuilder.UseExternalAccessMode(DstTexture, ERHIAccess::SRVMask, ERHIPipeline::All);

			GraphBuilder.RHICmdList.UpdateTextureReference(TextureEntry.TextureReferenceRHI, TextureEntry.RenderTarget->GetRHI());
		}

		bNeedToAllocatePageTable = false;
	}

	if (bNeedToAllocatePageTableIndirection)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

		if (Description.IndirectionTextureSize > 0)
		{
			const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				FIntPoint(Description.IndirectionTextureSize, Description.IndirectionTextureSize),
				PF_R32_UINT,
				FClearValueBinding::None,
				TexCreate_UAV | TexCreate_ShaderResource);

			FRDGTextureRef PageTableIndirectionTexture = GraphBuilder.CreateTexture(Desc, TEXT("VirtualTexture_PageTableAdaptiveIndirection"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PageTableIndirectionTexture), FUintVector4(ForceInitToZero));
			PageTableIndirection.RenderTarget = GraphBuilder.ConvertToExternalTexture(PageTableIndirectionTexture);
			GraphBuilder.UseExternalAccessMode(PageTableIndirectionTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
			GraphBuilder.RHICmdList.UpdateTextureReference(PageTableIndirection.TextureReferenceRHI, PageTableIndirection.RenderTarget->GetRHI());
		}

		bNeedToAllocatePageTableIndirection = false;
	}
}


bool FVirtualTextureSpace::BeginUpdates(FVirtualTextureSystem* System, FRDGBuilder& GraphBuilder, FUpdateContext& OutContext)
{
	static TArray<FPageTableUpdate> ExpandedUpdates[VIRTUALTEXTURE_SPACE_MAXLAYERS][16];

	if (bNeedToAllocatePageTable)
	{
		// Defer updates until next frame if page table texture needs to be re-allocated
		// We can't update the page table texture at this point in frame, as RHIUpdateTextureReference can't be called during RHIBegin/EndScene
		// Note that the virtual texture system doesn't account for page table updates being deferred. So this can potentially lead to sampling invalid page table addresses.
		// This could cause a glitch if we sample a VT on the first frame it is allocated. That's usually not the case (we usually sample some time after loading).
		// But it can be the case for Adaptive Virtual Texture which does a lot of dynamic page table allocation during the texture life.
		// However Adaptive Virtual Texture is OK because it always sets bPrivateSpace which gives fixed allocation of the actual page table texture.
		return false;
	}

	for (uint32 LayerIndex = 0u; LayerIndex < Description.NumPageTableLayers; ++LayerIndex)
	{
		FTexturePageMap& PageMap = PhysicalPageMap[LayerIndex];
		if (bForceEntireUpdate || CVarVTRefreshEntirePageTable.GetValueOnRenderThread())
		{
			PageMap.RefreshEntirePageTable(System, ExpandedUpdates[LayerIndex]);
		}
		else
		{
			for (const FPageTableUpdate& Update : PageTableUpdates[LayerIndex])
			{
				if (CVarVTMaskedPageTableUpdates.GetValueOnRenderThread())
				{
					PageMap.ExpandPageTableUpdateMasked(System, Update, ExpandedUpdates[LayerIndex]);
				}
				else
				{
					PageMap.ExpandPageTableUpdatePainters(System, Update, ExpandedUpdates[LayerIndex]);
				}
			}
		}
		PageTableUpdates[LayerIndex].Reset();
	}
	bForceEntireUpdate = false;

	uint32 TotalNumUpdates = 0;
	uint32 TextureUpdateMask = 0;
	for (uint32 LayerIndex = 0u; LayerIndex < Description.NumPageTableLayers; ++LayerIndex)
	{
		const uint32 TextureIndex = LayerIndex / LayersPerPageTableTexture;
		for (uint32 Mip = 0; Mip < CachedNumPageTableLevels; Mip++)
		{
			const int32 NumUpdates = ExpandedUpdates[LayerIndex][Mip].Num();
			TotalNumUpdates += NumUpdates;
			TextureUpdateMask |= NumUpdates > 0 ? (1 << TextureIndex) : 0;
		}
	}

	if (TotalNumUpdates == 0u)
	{
		return false;
	}

	void* StagingMemory = GraphBuilder.Alloc(TotalNumUpdates * sizeof(FPageTableUpdate));
	uint8* WritePtr = (uint8*)StagingMemory;

	for (uint32 TextureIndex = 0; TextureIndex < GetNumPageTableTextures(); ++TextureIndex)
	{
		const uint32 MinLayerIndex = TextureIndex * LayersPerPageTableTexture;
		const uint32 MaxLayerIndex = FMath::Min(MinLayerIndex + LayersPerPageTableTexture, (uint32)Description.NumPageTableLayers);

		for (uint32 Mip = 0; Mip < CachedNumPageTableLevels; Mip++)
		{
			for (uint32 LayerIndex = MinLayerIndex; LayerIndex < MaxLayerIndex; ++LayerIndex)
			{
				const uint32 NumUpdates = ExpandedUpdates[LayerIndex][Mip].Num();
				if (NumUpdates)
				{
					size_t UploadSize = NumUpdates * sizeof(FPageTableUpdate);
					FMemory::Memcpy(WritePtr, ExpandedUpdates[LayerIndex][Mip].GetData(), UploadSize);
					ExpandedUpdates[LayerIndex][Mip].Reset();
					WritePtr += UploadSize;
				}
				OutContext.NumUpdates[LayerIndex][Mip] = NumUpdates;
			}
		}
	}

	FRDGBufferRef UpdateBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(FPageTableUpdate), TotalNumUpdates), TEXT("VirtualTexture_PageTableUpdateBuffer"));
	GraphBuilder.QueueBufferUpload(UpdateBuffer, StagingMemory, TotalNumUpdates * sizeof(FPageTableUpdate), ERDGInitialDataFlags::NoCopy);

	OutContext.UpdateBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(UpdateBuffer, PF_R16G16B16A16_UINT));

	FRDGTextureAccessArray BatchBarrierTextures;
	for (uint32 TextureIndex = 0; TextureIndex < GetNumPageTableTextures(); ++TextureIndex)
	{
		if ((TextureUpdateMask & (1 << TextureIndex)) != 0)
		{
			// ForceImmediateFirstBarrier so that RTV transitions for the page table textures aren't hoisted
			// into RenderFinalize() or earlier where they will be incorrect for virtual texture sampling.
			FTextureEntry& PageTableEntry = PageTable[TextureIndex];
			FRDGTextureRef PageTableTexture = GraphBuilder.RegisterExternalTexture(PageTableEntry.RenderTarget, ERDGTextureFlags::ForceImmediateFirstBarrier);
			GraphBuilder.UseInternalAccessMode(PageTableTexture);

			OutContext.Textures[TextureIndex] = PageTableTexture;
			BatchBarrierTextures.Add(FRDGTextureAccess(PageTableTexture, ERHIAccess::RTV));
		}
	}

	// Add barrier pass to move all SRV->RTV transition barriers before all the render passes.
	AddBarrierPass(GraphBuilder, ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass | ERDGPassFlags::NeverMerge, BatchBarrierTextures);

	OutContext.Space = this;
	return true;
}

void FVirtualTextureSpace::ApplyUpdates(FRDGBuilder& GraphBuilder, FUpdateContext const& Context)
{
	auto ShaderMap = GetGlobalShaderMap(GetFeatureLevel());
	TShaderRef<FPageTableUpdateVS> VertexShader;
	if (Description.PageTableFormat == EVTPageTableFormat::UInt16)
	{
		VertexShader = ShaderMap->GetShader< TPageTableUpdateVS<true> >();
	}
	else
	{
		VertexShader = ShaderMap->GetShader< TPageTableUpdateVS<false> >();
	}
	check(VertexShader.IsValid());

	uint32 FirstUpdate = 0;

	// Loop over mips for each texture.
	// Inner loop is over texture channels setting the channel mask state.
	for (uint32 TextureIndex = 0; TextureIndex < GetNumPageTableTextures(); ++TextureIndex)
	{
		FRDGTextureRef PageTableTexture = Context.Textures[TextureIndex];
		if (PageTableTexture == nullptr)
		{
			continue;
		}

		TShaderRef<FGlobalShader> PixelShader;
		switch (TexturePixelFormat[TextureIndex])
		{
		case PF_R16_UINT: PixelShader = ShaderMap->GetShader< TPageTableUpdatePS<PF_R16_UINT> >(); break;
		case PF_R16G16_UINT: PixelShader = ShaderMap->GetShader< TPageTableUpdatePS<PF_R16G16_UINT> >(); break;
		case PF_R16G16B16A16_UINT: PixelShader = ShaderMap->GetShader< TPageTableUpdatePS<PF_R16G16B16A16_UINT> >(); break;
		case PF_R32_UINT: PixelShader = ShaderMap->GetShader< TPageTableUpdatePS<PF_R32_UINT> >(); break;
		case PF_R32G32_UINT: PixelShader = ShaderMap->GetShader< TPageTableUpdatePS<PF_R32G32_UINT> >(); break;
		case PF_R32G32B32A32_UINT: PixelShader = ShaderMap->GetShader< TPageTableUpdatePS<PF_R32G32B32A32_UINT> >(); break;
		default: checkNoEntry(); break;
		}
		check(PixelShader.IsValid());

		const uint32 MinLayerIndex = TextureIndex * LayersPerPageTableTexture;
		const uint32 MaxLayerIndex = FMath::Min(MinLayerIndex + LayersPerPageTableTexture, (uint32)Description.NumPageTableLayers);
		
		uint32 MipWidth = CachedPageTableWidth;
		uint32 MipHeight = CachedPageTableHeight;
	
		for (uint32 Mip = 0; Mip < CachedNumPageTableLevels; Mip++)
		{
			for (uint32 LayerIndex = MinLayerIndex; LayerIndex < MaxLayerIndex; ++LayerIndex)
			{
				const uint32 NumUpdates = Context.NumUpdates[LayerIndex][Mip];
				if (NumUpdates == 0)
				{
					continue;
				}

				const uint32 LayerInTexture = LayerIndex - MinLayerIndex;

				// Use color write mask to update the proper page table entry for this layer
				FRHIBlendState* BlendStateRHI = nullptr;
				switch (LayerInTexture)
				{
				case 0u: BlendStateRHI = TStaticBlendState<CW_RED>::GetRHI(); break;
				case 1u: BlendStateRHI = TStaticBlendState<CW_GREEN>::GetRHI(); break;
				case 2u: BlendStateRHI = TStaticBlendState<CW_BLUE>::GetRHI(); break;
				case 3u: BlendStateRHI = TStaticBlendState<CW_ALPHA>::GetRHI(); break;
				default: check(false); break;
				}

				auto* PassParameters = GraphBuilder.AllocParameters<FPageTableUpdateParameters>();
				PassParameters->VS.UpdateBuffer = Context.UpdateBufferSRV;
				PassParameters->VS.PageTableSize = FUintVector2(CachedPageTableWidth, CachedPageTableHeight);
				PassParameters->VS.FirstUpdate = FirstUpdate;
				PassParameters->VS.NumUpdates = NumUpdates;
				PassParameters->RenderTargets[0] = FRenderTargetBinding(PageTableTexture, ERenderTargetLoadAction::ELoad, Mip);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("PageTableUpdate (Id: %d, Tex %d, Mip: %d)", ID, TextureIndex, Mip),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, VertexShader, PixelShader, BlendStateRHI, NumUpdates, MipWidth, MipHeight](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					RHICmdList.SetViewport(0, 0, 0.0f, MipWidth, MipHeight, 1.0f);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

					GraphicsPSOInit.BlendState = BlendStateRHI;
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;

					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);

					// needs to be the same on shader side (faster on NVIDIA and AMD)
					uint32 QuadsPerInstance = 8;

					RHICmdList.SetStreamSource(0, NULL, 0);
					RHICmdList.DrawIndexedPrimitive(GQuadIndexBuffer.IndexBufferRHI, 0, 0, 32, 0, 2 * QuadsPerInstance, FMath::DivideAndRoundUp(NumUpdates, QuadsPerInstance));
				});

				FirstUpdate += NumUpdates;
			}

			MipWidth = FMath::Max(MipWidth / 2u, 1u);
			MipHeight = FMath::Max(MipHeight / 2u, 1u);
		}
	}
}

void FVirtualTextureSpace::EndUpdates(FRDGBuilder& GraphBuilder, FUpdateContext const& Context, FRDGExternalAccessQueue& ExternalAccessQueue)
{
	// Add barrier pass to keep all RTV->SRV transition after all the render passes.
	// Marking the access as RTV here stops the RDG barrier optimizer from moving the barriers earlier where they serialize the render passes.
	FRDGTextureAccessArray BatchBarrierTextures;
	for (FRDGTextureRef Texture : Context.Textures)
	{
		if (Texture != nullptr)
		{
			BatchBarrierTextures.Add(FRDGTextureAccess(Texture, ERHIAccess::RTV));
		}
	}
	AddBarrierPass(GraphBuilder, ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass | ERDGPassFlags::NeverMerge, BatchBarrierTextures);

	// Return textures to be externally managed.
	for (uint32 TextureIndex = 0; TextureIndex < TextureCapacity; ++TextureIndex)
	{
		if (FRDGTextureRef Texture = Context.Textures[TextureIndex])
		{
			FTextureEntry& PageTableEntry = PageTable[TextureIndex];
			PageTableEntry.RenderTarget = GraphBuilder.ConvertToExternalTexture(Texture);
			ExternalAccessQueue.Add(Texture, ERHIAccess::SRVMask, ERHIPipeline::All);
		}
	}
}

void FVirtualTextureSpace::DumpToConsole(bool verbose)
{
	UE_LOGF(LogConsoleResponse, Display, "-= Space ID %i =-", ID);
	Allocator.DumpToConsole(verbose);
}

#if WITH_EDITOR
void FVirtualTextureSpace::SaveAllocatorDebugImage() const
{
	const FString ImageName = FString::Printf(TEXT("Space%dAllocator.png"), ID);
	Allocator.SaveDebugImage(*ImageName);
}
#endif
