// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualTextureShared.h"
#include "VirtualTexturePhysicalSpace.h"
#include "TexturePageMap.h"
#include "VirtualTextureAllocator.h"
#include "RendererInterface.h"
#include "VirtualTexturing.h"

class FAllocatedVirtualTexture;
class FVirtualTextureSystem;

struct FVTSpaceDescription
{
	uint32 TileSize = 0u;
	uint32 TileBorderSize = 0u;
	uint8 Dimensions = 0u;
	EVTPageTableFormat PageTableFormat = EVTPageTableFormat::UInt16;
	uint8 NumPageTableLayers = 0u;
	uint8 bPrivateSpace = false;
	uint8 bContinuousUpdate = false;
	uint32 MaxSpaceSize = VIRTUALTEXTURE_MAX_PAGETABLE_SIZE;
	uint32 IndirectionTextureSize = 0u;
};

inline bool operator==(const FVTSpaceDescription& Lhs, const FVTSpaceDescription& Rhs)
{
	return Lhs.Dimensions == Rhs.Dimensions &&
		Lhs.TileSize == Rhs.TileSize &&
		Lhs.TileBorderSize == Rhs.TileBorderSize &&
		Lhs.NumPageTableLayers == Rhs.NumPageTableLayers &&
		Lhs.PageTableFormat == Rhs.PageTableFormat &&
		Lhs.bPrivateSpace == Rhs.bPrivateSpace &&
		Lhs.bContinuousUpdate == Rhs.bContinuousUpdate &&
		Lhs.MaxSpaceSize == Rhs.MaxSpaceSize &&
		Lhs.IndirectionTextureSize == Rhs.IndirectionTextureSize;
}
inline bool operator!=(const FVTSpaceDescription& Lhs, const FVTSpaceDescription& Rhs)
{
	return !operator==(Lhs, Rhs);
}

// Virtual memory address space mapped by a page table texture
class FVirtualTextureSpace final : public FRenderResource
{
public:
	static const uint32 LayersPerPageTableTexture = IAllocatedVirtualTexture::LayersPerPageTableTexture;
	static const uint32 TextureCapacity = (VIRTUALTEXTURE_SPACE_MAXLAYERS + LayersPerPageTableTexture - 1u) / LayersPerPageTableTexture;

	FVirtualTextureSpace(FVirtualTextureSystem* InSystem, uint8 InID, const FVTSpaceDescription& InDesc, uint32 InSizeNeeded);
	virtual ~FVirtualTextureSpace();

	inline const FVTSpaceDescription& GetDescription() const { return Description; }
	inline uint32 GetPageTableWidth() const { return CachedPageTableWidth; }
	inline uint32 GetPageTableHeight() const { return CachedPageTableHeight; }
	inline uint8 GetDimensions() const { return Description.Dimensions; }
	inline EVTPageTableFormat GetPageTableFormat() const { return Description.PageTableFormat; }
	inline uint32 GetNumPageTableLayers() const { return Description.NumPageTableLayers; }
	inline uint32 GetNumPageTableTextures() const { return (Description.NumPageTableLayers + LayersPerPageTableTexture - 1u) / LayersPerPageTableTexture; }
	inline uint8 GetID() const { return ID; }

	inline uint32 GetNumPageTableLevels() const { return CachedNumPageTableLevels; }
	inline FVirtualTextureAllocator& GetAllocator() { return Allocator; }
	inline const FVirtualTextureAllocator& GetAllocator() const { return Allocator; }
	inline FTexturePageMap& GetPageMapForPageTableLayer(uint32 PageTableLayerIndex) { check(PageTableLayerIndex < Description.NumPageTableLayers); return PhysicalPageMap[PageTableLayerIndex]; }
	inline const FTexturePageMap& GetPageMapForPageTableLayer(uint32 PageTableLayerIndex) const { check(PageTableLayerIndex < Description.NumPageTableLayers); return PhysicalPageMap[PageTableLayerIndex]; }

	// FRenderResource interface
	virtual void		InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void		ReleaseRHI() override;

	inline uint32 AddRef() { return ++NumRefs; }
	inline uint32 Release() { check(NumRefs > 0u); return --NumRefs; }
	inline uint32 GetRefCount() const { return NumRefs; }

	inline void SetReleasedFrame(uint32 InReleasedFrame) { ReleasedFrame = InReleasedFrame; }
	inline uint32 GetReleasedFrame() const { return ReleasedFrame; }

	uint32 GetSizeInBytes() const;

	uint32 AllocateVirtualTexture(FAllocatedVirtualTexture* VirtualTexture);

	void FreeVirtualTexture(FAllocatedVirtualTexture* VirtualTexture);

	FRHITextureReference* GetPageTableTexture(uint32 PageTableIndex) const
	{
		check(PageTableIndex < GetNumPageTableTextures());
		return PageTable[PageTableIndex].TextureReferenceRHI.GetReference();
	}

	FRHITextureReference* GetPageTableIndirectionTexture() const
	{
		return PageTableIndirection.TextureReferenceRHI.GetReference();
	}

	TRefCountPtr<IPooledRenderTarget> GetPageTableIndirectionRenderTarget() const
	{
		return PageTableIndirection.RenderTarget;
	}

	void QueueUpdate(uint8 Layer, uint8 vLogSize, uint32 vAddress, uint8 vLevel, const FPhysicalTileLocation& pTileLocation);
	void AllocateTextures(FRDGBuilder& GraphBuilder);
	void QueueUpdateEntirePageTable();

	/** Update context. For efficient generation of GPU barriers we want to call each update phase (Begin/Apply/End) on all spaces before moving to the next phase. */
	struct FUpdateContext
	{
		FVirtualTextureSpace* Space = nullptr;
		FRDGBufferSRVRef UpdateBufferSRV = nullptr;
		FRDGTextureRef Textures[TextureCapacity] = { nullptr };
		int32 NumUpdates[VIRTUALTEXTURE_SPACE_MAXLAYERS][16] = {{ 0 }};
	};
	
	bool BeginUpdates(FVirtualTextureSystem* System, FRDGBuilder& GraphBuilder, FUpdateContext& OutContext);
	void ApplyUpdates(FRDGBuilder& GraphBuilder, FUpdateContext const& Context);
	void EndUpdates(FRDGBuilder& GraphBuilder, FUpdateContext const& Context, FRDGExternalAccessQueue& ExternalAccessQueue);

	void DumpToConsole(bool verbose);

#if WITH_EDITOR
	void SaveAllocatorDebugImage() const;
#endif // WITH_EDITOR

private:
	FUintPoint GetRequiredPageTableAllocationSize() const;


	struct FTextureEntry
	{
		TRefCountPtr<IPooledRenderTarget> RenderTarget;
		FTextureReferenceRHIRef TextureReferenceRHI;
	};

	FVTSpaceDescription Description;
	
	FVirtualTextureAllocator Allocator;
	FTexturePageMap PhysicalPageMap[VIRTUALTEXTURE_SPACE_MAXLAYERS];

	FTextureEntry PageTable[TextureCapacity];
	TEnumAsByte<EPixelFormat> TexturePixelFormat[TextureCapacity];
	FString PageTableDebugNames[TextureCapacity];

	FTextureEntry PageTableIndirection;

	TArray<FPageTableUpdate> PageTableUpdates[VIRTUALTEXTURE_SPACE_MAXLAYERS];

	uint32 NumRefs = 0;
	uint32 ReleasedFrame = 0;

	uint8 ID = 0;
	uint32 CachedPageTableWidth = 0;
	uint32 CachedPageTableHeight = 0;
	uint32 CachedNumPageTableLevels = 0;
	bool bNeedToAllocatePageTable = false;
	bool bNeedToAllocatePageTableIndirection = false;
	bool bForceEntireUpdate = false;
};
