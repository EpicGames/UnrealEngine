// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualTextureSystem.h"

#include "AllocatedVirtualTexture.h"
#include "Engine/Canvas.h"
#include "Engine/Texture2D.h"
#include "Debug/DebugDrawService.h"
#include "HAL/IConsoleManager.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "PostProcess/SceneRenderTargets.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "RendererOnScreenNotification.h"
#include "ScenePrivate.h"
#include "SceneRenderBuilder.h"
#include "SceneUtils.h"
#include "Stats/Stats.h"
#include "VirtualTexturing.h"
#include "VirtualTextureEnum.h"
#include "MaterialCache/MaterialCacheVirtualTextureAllocator.h"
#include "VT/AdaptiveVirtualTexture.h"
#include "VT/TexturePagePool.h"
#include "VT/UniquePageList.h"
#include "VT/UniqueRequestList.h"
#include "VT/VirtualTextureFeedback.h"
#include "VT/VirtualTexturePhysicalSpace.h"
#include "VT/VirtualTexturePoolConfig.h"
#include "VT/VirtualTextureScalability.h"
#include "VT/VirtualTextureSpace.h"
#include "VirtualTextureDefinitions.h"

#define LOCTEXT_NAMESPACE "VirtualTexture"

CSV_DEFINE_CATEGORY(VirtualTexturing, (!UE_BUILD_SHIPPING));

DECLARE_CYCLE_STAT(TEXT("VirtualTextureSystem Update"), STAT_VirtualTextureSystem_Update, STATGROUP_VirtualTexturing);

DECLARE_CYCLE_STAT(TEXT("Gather Requests"), STAT_ProcessRequests_Gather, STATGROUP_VirtualTexturing);
DECLARE_CYCLE_STAT(TEXT("Sort Requests"), STAT_ProcessRequests_Sort, STATGROUP_VirtualTexturing);
DECLARE_CYCLE_STAT(TEXT("Submit Requests"), STAT_ProcessRequests_Submit, STATGROUP_VirtualTexturing);
DECLARE_CYCLE_STAT(TEXT("Map Requests"), STAT_ProcessRequests_Map, STATGROUP_VirtualTexturing);
DECLARE_CYCLE_STAT(TEXT("Map New VTs"), STAT_ProcessRequests_MapNew, STATGROUP_VirtualTexturing);
DECLARE_CYCLE_STAT(TEXT("Finalize Requests"), STAT_ProcessRequests_Finalize, STATGROUP_VirtualTexturing);
DECLARE_CYCLE_STAT(TEXT("Wait Tasks"), STAT_ProcessRequests_WaitTasks, STATGROUP_VirtualTexturing);

DECLARE_CYCLE_STAT(TEXT("Queue Adaptive Requests"), STAT_ProcessRequests_QueueAdaptiveRequests, STATGROUP_VirtualTexturing);
DECLARE_CYCLE_STAT(TEXT("Finalize Adaptive Requests"), STAT_ProcessRequests_UpdateAdaptiveAllocations, STATGROUP_VirtualTexturing);

DECLARE_CYCLE_STAT(TEXT("Feedback Map"), STAT_FeedbackMap, STATGROUP_VirtualTexturing);
DECLARE_CYCLE_STAT(TEXT("Page Table Updates"), STAT_PageTableUpdates, STATGROUP_VirtualTexturing);
DECLARE_CYCLE_STAT(TEXT("Flush Cache"), STAT_FlushCache, STATGROUP_VirtualTexturing);
DECLARE_CYCLE_STAT(TEXT("Update Residency Tracking"), STAT_ResidencyTracking, STATGROUP_VirtualTexturing);

DECLARE_DWORD_COUNTER_STAT(TEXT("Num page visible"), STAT_NumPageVisible, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num page visible resident"), STAT_NumPageVisibleResident, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num page visible not resident"), STAT_NumPageVisibleNotResident, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num page prefetch"), STAT_NumPagePrefetch, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num page update"), STAT_NumPageUpdate, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num mapped page update"), STAT_NumMappedPageUpdate, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num continuous page update"), STAT_NumContinuousPageUpdate, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num page produced"), STAT_NumPagesProduced, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num page pending"), STAT_NumPagesPending, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num page throttled"), STAT_NumPagesThrottled, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num page saturated"), STAT_NumPagesSaturated, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num page invalid"), STAT_NumPagesInvalid, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num page fail allocate"), STAT_NumPagesFailAllocate, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num stacks requested"), STAT_NumStacksRequested, STATGROUP_VirtualTexturing);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num stacks produced"), STAT_NumStacksProduced, STATGROUP_VirtualTexturing);

DECLARE_DWORD_COUNTER_STAT(TEXT("Num flush caches"), STAT_NumFlushCache, STATGROUP_VirtualTexturing);

DECLARE_MEMORY_STAT_POOL(TEXT("Total Pagetable Memory"), STAT_TotalPagetableMemory, STATGROUP_VirtualTextureMemory, FPlatformMemory::MCR_GPU);

DECLARE_GPU_STAT(VirtualTexture);
DECLARE_GPU_STAT(VirtualTextureAllocate);

static TAutoConsoleVariable<int32> CVarVTEnableFeedback(
	TEXT("r.VT.EnableFeedback"),
	1,
	TEXT("Enable processing of the GPU generated feedback buffer."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarVTEnableFeedbackProduce(
	TEXT("r.VT.EnableFeedback.Produce"),
	1,
	TEXT("Enable page loading triggered by GPU virtual texture feedback."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarVTEnablePlayback(
	TEXT("r.VT.EnablePlayback"),
	1,
	TEXT("Enable playback of recorded feedback requests."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<float> CVarVTPlaybackMipBias(
	TEXT("r.VT.PlaybackMipBias"),
	0,
	TEXT("Mip bias to apply during playback of recorded feedback requests.\n")
	TEXT("This is in addition to the automatic bias applied for the difference between capture and playback resolutions."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarVTForceContinuousUpdate(
	TEXT("r.VT.ForceContinuousUpdate"),
	0,
	TEXT("Force continuous update on all virtual textures."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarVTSyncProduceLockedTiles(
	TEXT("r.VT.SyncProduceLockedTiles"),
	0,
	TEXT("Should we sync loading when producing locked tiles"),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarVTProduceLockedTilesOnFlush(
	TEXT("r.VT.ProduceLockedTilesOnFlush"),
	1,
	TEXT("Should locked tiles be (re)produced when flushing the cache"),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarVTSpaceReleaseFrames(
	TEXT("r.VT.SpaceReleaseFrames"),
	150,
	TEXT("Number of frames to wait before releasing a virtual texture space. < 0 for no release."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarVTResidencyShow(
	TEXT("r.VT.Residency.Show"),
	0,
	TEXT("Show on screen HUD for virtual texture physical pool residency"),
	ECVF_Default
);
static TAutoConsoleVariable<int32> CVarVTResidencyNotify(
	TEXT("r.VT.Residency.Notify"),
	0,
	TEXT("Show on screen notifications for virtual texture physical pool residency"),
	ECVF_Default
);
static TAutoConsoleVariable<int32> CVarVTCsvStats(
	TEXT("r.VT.CsvStats"),
	1,
	TEXT("Send virtual texturing stats to CSV profiler\n")
	TEXT("0=off, 1=on, 2=verbose"),
	ECVF_Default
);
static TAutoConsoleVariable<int32> CVarVTAsyncPageRequestTask(
	TEXT("r.VT.AsyncPageRequestTask"),
	1,
	TEXT("Performs VT page requests on an async task."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<float> CVarVTAsyncPageRequestTaskEarlyOut(
	TEXT("r.VT.AsyncPageRequestTask.EarlyOut"),
	0.0f,
	TEXT("Allow early out of non essential parts of async task instead of waiting for full completion.")
	TEXT("0 (default) of less disables. A value above 0 enables and is the minimum time in milliseconds before we allow the early out."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarVTWaitBeforeUpdate(
	TEXT("r.VT.WaitBeforeUpdate"),
	0,
	TEXT("Wait on the SceneRenderBuilder before kicking VT async update."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarVTSortTileRequestsByPriority(
	TEXT("r.VT.SortTileRequestsByPriority"),
	1,
	TEXT("Sorts the list of tile requests by priority before submitting. This allows priority requests to get processed before others in case of throttling"),
	ECVF_RenderThreadSafe
);

static bool ShouldSyncProduceLockedTiles()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		// Always force loading of locked root pages in editor to be sure that any build steps give best quality.
		return true;
	}
#endif
	// If we return false then we may render with a VT before the root pages are mapped.
	// When that happens the shader switches to using the single color fallback value instead of sampling the VT.
	// When the root page is finally mapped we will return to normal high quality VT sampling.
	// todo[VT]: Make root pages always resident so that we never need to load sync the root pages.
	return CVarVTSyncProduceLockedTiles.GetValueOnRenderThread() != 0;
}

static float GetMinSubmitAsyncEarlyOutTime()
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		return 0.0f;
	}
#endif
	return FMath::Max(0.0f, CVarVTAsyncPageRequestTaskEarlyOut.GetValueOnRenderThread()) * 0.001f;
}

FVirtualTextureUpdateSettings::FVirtualTextureUpdateSettings()
{
	bEnableFeedback = CVarVTEnableFeedback.GetValueOnRenderThread() != 0;
	bEnableFeedbackProduce = CVarVTEnableFeedbackProduce.GetValueOnRenderThread() != 0;
	bEnablePlayback = CVarVTEnablePlayback.GetValueOnRenderThread() != 0;
	bForceContinuousUpdate = CVarVTForceContinuousUpdate.GetValueOnRenderThread() != 0;
	bSyncProduceAllTiles = false;
	bSyncProduceLockedTiles = ShouldSyncProduceLockedTiles();
	bEnableAsyncTasks = CVarVTAsyncPageRequestTask.GetValueOnRenderThread() != 0;
	MaxRVTPageUploads = VirtualTextureScalability::GetMaxUploadsPerFrame();
	MaxSVTPageUploads = VirtualTextureScalability::GetMaxUploadsPerFrameForStreamingVT();
	MaxPagesProduced = VirtualTextureScalability::GetMaxPagesProducedPerFrame();
	MaxContinuousUpdates = VirtualTextureScalability::GetMaxContinuousUpdatesPerFrame();
	MaxGatherTime = VirtualTextureScalability::GetMaxGatherTimePerFrame();
	MaxSubmitTime = VirtualTextureScalability::GetMaxSubmitTimePerFrame();
	MinSubmitAsyncEarlyOutTime = GetMinSubmitAsyncEarlyOutTime();
}

FVirtualTextureUpdateSettings::FVirtualTextureUpdateSettings(FViewFamilyInfo const& InViewFamily)
	: FVirtualTextureUpdateSettings()
{
	if (InViewFamily.bOverrideVirtualTextureThrottle)
	{
		EnableThrottling(false);
	}
	PlaybackViewSize = InViewFamily.SceneTexturesConfig.Extent;
}

static FORCEINLINE uint32 EncodePage(uint32 ID, uint32 vLevel, uint32 vTileX, uint32 vTileY)
{
	const uint32 vLevelOffset = vLevel + UE::HLSL::VirtualTextureFeedbackBias;

	// Mirrors VirtualTextureCommon.ush, see TextureLoadVirtualPageTableInternal and related
	uint32 Page;
	Page = vTileX << 0;
	Page |= vTileY << 12;
	Page |= vLevelOffset << 24;
	Page |= ID << 28;
	return Page;
}

struct FFeedbackAnalysisParameters
{
	const FUintPoint* FeedbackBuffer = nullptr;
	FUniquePageList* UniquePageList = nullptr;
	uint32 FeedbackSize = 0u;
};

struct FAddRequestedTilesParameters
{
	uint32 ViewLevelBias = 0u;
	const uint32* RequestBuffer = nullptr;
	uint32 NumRequests = 0u;
	FUniquePageList* UniquePageList = nullptr;
};

struct FGatherRequestsParameters
{
	const FUniquePageList* UniquePageList = nullptr;
	FUniqueRequestList* RequestList = nullptr;
	uint32 PageStartIndex = 0u;
	uint32 NumPages = 0u;
	uint32 FrameRequested;
	bool bForceContinuousUpdate = false;
	bool bEnableLoadRequests = false;
	float TimeBudget = 0.0f;
};

static FVirtualTextureSystem* GVirtualTextureSystem = nullptr;

void FVirtualTextureSystem::Initialize()
{
	if (!GVirtualTextureSystem)
	{
		GVirtualTextureSystem = new FVirtualTextureSystem();
	}
}

void FVirtualTextureSystem::Shutdown()
{
	if (GVirtualTextureSystem)
	{
		delete GVirtualTextureSystem;
		GVirtualTextureSystem = nullptr;
	}
}

FVirtualTextureSystem& FVirtualTextureSystem::Get()
{
	check(GVirtualTextureSystem);
	return *GVirtualTextureSystem;
}

FVirtualTextureSystem::FVirtualTextureSystem()
	: Frame(1024u) // Need to start on a high enough value that we'll be able to allocate pages
	, bFlushCaches(false)
	, FlushCachesCommand(TEXT("r.VT.Flush"), TEXT("Flush all the physical caches in the VT system."),
		FConsoleCommandDelegate::CreateRaw(this, &FVirtualTextureSystem::FlushCachesFromConsole))
	, DumpCommand(TEXT("r.VT.Dump"), TEXT("Dump a whole lot of info on the VT system state."),
		FConsoleCommandDelegate::CreateRaw(this, &FVirtualTextureSystem::DumpFromConsole))
	, ListPhysicalPools(TEXT("r.VT.ListPhysicalPools"), TEXT("Dump a whole lot of info on the VT system state."),
		FConsoleCommandDelegate::CreateRaw(this, &FVirtualTextureSystem::ListPhysicalPoolsFromConsole))
	, DumpPoolUsageCommand(TEXT("r.VT.DumpPoolUsage"), TEXT("Dump detailed info about VT pool usage."),
		FConsoleCommandDelegate::CreateRaw(this, &FVirtualTextureSystem::DumpPoolUsageFromConsole))
#if WITH_EDITOR
	, SaveAllocatorImages(TEXT("r.VT.SaveAllocatorImages"), TEXT("Save images showing allocator usage."),
		FConsoleCommandDelegate::CreateRaw(this, &FVirtualTextureSystem::SaveAllocatorImagesFromConsole))
	, PageRequestRecordHandle(~0ull)
#endif
{
#if !UE_BUILD_SHIPPING
	OnScreenMessageDelegateHandle = FRendererOnScreenNotification::Get().AddLambda([this](FCoreDelegates::FSeverityMessageMap& OutMessages) { GetOnScreenMessages(OutMessages); });
	DrawResidencyHudDelegateHandle = UDebugDrawService::Register(TEXT("VirtualTextureResidency"), FDebugDrawDelegate::CreateRaw(this, &FVirtualTextureSystem::DrawResidencyHud));
#endif
}

FVirtualTextureSystem::~FVirtualTextureSystem()
{
#if !UE_BUILD_SHIPPING
	FRendererOnScreenNotification::Get().Remove(OnScreenMessageDelegateHandle);
	UDebugDrawService::Unregister(DrawResidencyHudDelegateHandle);
#endif

	DestroyPendingVirtualTextures(true);

	check(AllocatedVTs.Num() == 0);
	check(PersistentVTMap.Num() == 0);

	for (int32 SpaceID = 0; SpaceID < Spaces.Num(); ++SpaceID)
	{
		FVirtualTextureSpace* Space = Spaces[SpaceID].Get();
		if (Space)
		{
			check(Space->GetRefCount() == 0u);
			DEC_MEMORY_STAT_BY(STAT_TotalPagetableMemory, Space->GetSizeInBytes());
			BeginReleaseResource(Space);
		}
	}
	for(int32 i = 0; i < PhysicalSpaces.Num(); ++i)
	{
		FVirtualTexturePhysicalSpace* PhysicalSpace = PhysicalSpaces[i];
		if (PhysicalSpace)
		{
			check(PhysicalSpace->GetRefCount() == 0u);
			BeginReleaseResource(PhysicalSpace);
		}
	}
}

void FVirtualTextureSystem::FlushCachesFromConsole()
{
	FlushCache();
}

void FVirtualTextureSystem::FlushCache()
{
	// We defer the actual flush to the render thread in the Update function
	bFlushCaches = true;
}

void FVirtualTextureSystem::FlushCache(FVirtualTextureProducerHandle const& ProducerHandle, int32 SpaceID, FIntRect const& TextureRegion, uint32 MaxLevelToEvict, uint32 MaxAgeToKeepMapped, EVTInvalidatePriority InvalidatePriority)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);

	SCOPE_CYCLE_COUNTER(STAT_FlushCache);
	INC_DWORD_STAT_BY(STAT_NumFlushCache, 1);

	FVirtualTextureProducer const* Producer = Producers.FindProducer(ProducerHandle);
	if (Producer != nullptr)
	{
		FVTProducerDescription const& ProducerDescription = Producer->GetDescription();

		TArray<FVirtualTexturePhysicalSpace*> PhysicalSpacesForProducer;
		for (uint32 i = 0; i < Producer->GetNumPhysicalGroups(); ++i)
		{
			PhysicalSpacesForProducer.AddUnique(Producer->GetPhysicalSpaceForPhysicalGroup(i));
		}

		check(TransientCollectedTilesToProduce.Num() == 0);

		const uint32 MinFrameToKeepMapped = Frame > MaxAgeToKeepMapped ? Frame - MaxAgeToKeepMapped : 0;

		// If this is an Adaptive VT we need to collect all of the associated Producers to flush.
		TArray<FAdaptiveVirtualTexture::FProducerInfo> ProducerInfos;
		if (FAdaptiveVirtualTexture* AdaptiveVT = GetAdaptiveVirtualTexture(SpaceID))
		{
			AdaptiveVT->GetProducers(TextureRegion, MaxLevelToEvict, ProducerInfos);
		}

		for (int32 i = 0; i < PhysicalSpacesForProducer.Num(); ++i)
		{
			FTexturePagePool& Pool = PhysicalSpacesForProducer[i]->GetPagePool();
		
			if (ProducerInfos.Num())
			{
				// Adaptive VT flushes.
				for (FAdaptiveVirtualTexture::FProducerInfo& Info : ProducerInfos)
				{
					Pool.EvictPages(this, Info.ProducerHandle, ProducerDescription, Info.RemappedTextureRegion, Info.RemappedMaxLevel, MinFrameToKeepMapped, InvalidatePriority, TransientCollectedTilesToProduce);
				}
			}
			else
			{
				// Regular flush.
				Pool.EvictPages(this, ProducerHandle, ProducerDescription, TextureRegion, MaxLevelToEvict, MinFrameToKeepMapped, InvalidatePriority, TransientCollectedTilesToProduce);
			}
		}

		for (const FVirtualTextureLocalTileRequest& TileRequest : TransientCollectedTilesToProduce)
		{
			AddOrMergeTileRequest(TileRequest, MappedTilesToProduce);
		}

		// Don't resize to allow this container to grow as needed (avoid allocations when collecting)
		TransientCollectedTilesToProduce.Reset();
	}
}

void FVirtualTextureSystem::DumpFromConsole()
{
	bool verbose = false;
	for (int ID = 0; ID < Spaces.Num(); ID++)
	{
		FVirtualTextureSpace* Space = Spaces[ID].Get();
		if (Space)
		{
			Space->DumpToConsole(verbose);
		}
	}
}

void FVirtualTextureSystem::ListPhysicalPoolsFromConsole()
{
	uint64 TotalPhysicalMemory = 0u;
	uint64 TotalLockedMemory = 0u;
	for(int32 i = 0; i < PhysicalSpaces.Num(); ++i)
	{
		if (PhysicalSpaces[i] && PhysicalSpaces[i]->IsInitialized())
		{
			const FVirtualTexturePhysicalSpace& PhysicalSpace = *PhysicalSpaces[i];
			const FVTPhysicalSpaceDescription& Desc = PhysicalSpace.GetDescription();
			const FTexturePagePool& PagePool = PhysicalSpace.GetPagePool();
			const uint32 TotalSizeInBytes = PhysicalSpace.GetSizeInBytes();
			const uint32 NumTiles = PhysicalSpace.GetNumTiles();

			UE_LOGF(LogVirtualTexturing, Display, "PhysicalPool: [%i] %ls (%ix%i):", i, *PhysicalSpace.GetFormatString(), Desc.TileSize, Desc.TileSize);
			
			const uint32 AllocatedTiles = PagePool.GetNumAllocatedPages();
			const uint32 AllocatedMemoryInBytes = AllocatedTiles * (TotalSizeInBytes / NumTiles);
			const float AllocatedMemory = (float)AllocatedMemoryInBytes / 1024.0f / 1024.0f;

			const uint32 LockedTiles = PagePool.GetNumLockedPages();
			const uint32 LockedMemoryInBytes = LockedTiles * (TotalSizeInBytes / NumTiles);
			const float LockedMemory = (float)LockedMemoryInBytes / 1024.0f / 1024.0f;

			UE_LOGF(LogVirtualTexturing, Display, "  SizeInMegabyte= %f", (float)TotalSizeInBytes / 1024.0f / 1024.0f);
			UE_LOGF(LogVirtualTexturing, Display, "  Dimensions= %ix%i", PhysicalSpace.GetTextureSize(), PhysicalSpace.GetTextureSize());
			UE_LOGF(LogVirtualTexturing, Display, "  Tiles= %i", NumTiles);
			UE_LOGF(LogVirtualTexturing, Display, "  Tiles Allocated= %i (%fMB)", AllocatedTiles, AllocatedMemory);
			UE_LOGF(LogVirtualTexturing, Display, "  Tiles Locked= %i (%fMB)", LockedTiles, LockedMemory);
			UE_LOGF(LogVirtualTexturing, Display, "  Tiles Mapped= %i", PagePool.GetNumMappedPages());

			TotalPhysicalMemory += TotalSizeInBytes;
			TotalLockedMemory += LockedMemoryInBytes;
		}
	}

	uint64 TotalPageTableMemory = 0u;
	for (int32 ID = 0; ID < Spaces.Num(); ID++)
	{
		const FVirtualTextureSpace* Space = Spaces[ID].Get();
		if (Space == nullptr)
		{
			continue;
		}

		const FVTSpaceDescription& Desc = Space->GetDescription();
		const FVirtualTextureAllocator& Allocator = Space->GetAllocator();
		const uint32 PageTableWidth = Space->GetPageTableWidth();
		const uint32 PageTableHeight = Space->GetPageTableHeight();
		const uint32 TotalSizeInBytes = Space->GetSizeInBytes();
		const uint32 NumAllocatedPages = Allocator.GetNumAllocatedPages();
		const uint32 NumTotalPages = PageTableWidth * PageTableHeight;
		const double AllocatedRatio = (double)NumAllocatedPages / NumTotalPages;

		const uint32 PhysicalTileSize = Desc.TileSize + Desc.TileBorderSize * 2u;
		const TCHAR* FormatName = nullptr;
		switch (Desc.PageTableFormat)
		{
		case EVTPageTableFormat::UInt16: FormatName = TEXT("UInt16"); break;
		case EVTPageTableFormat::UInt32: FormatName = TEXT("UInt32"); break;
		default: checkNoEntry(); break;
		}

		UE_LOGF(LogVirtualTexturing, Display, "Pool: [%i] %ls (%ix%i) x %i:", ID, FormatName, PhysicalTileSize, PhysicalTileSize, Desc.NumPageTableLayers);
		UE_LOGF(LogVirtualTexturing, Display, "  PageTableSize= %ix%i", PageTableWidth, PageTableHeight);
		UE_LOGF(LogVirtualTexturing, Display, "  Allocations= %i, %i%% (%fMB)",
			Allocator.GetNumAllocations(),
			(int)(AllocatedRatio * 100.0),
			(float)(AllocatedRatio * TotalSizeInBytes / 1024.0 / 1024.0));

		TotalPageTableMemory += TotalSizeInBytes;
	}

	UE_LOGF(LogVirtualTexturing, Display, "TotalPageTableMemory: %fMB", (double)TotalPageTableMemory / 1024.0 / 1024.0);
	UE_LOGF(LogVirtualTexturing, Display, "TotalPhysicalMemory: %fMB", (double)TotalPhysicalMemory / 1024.0 / 1024.0);
	UE_LOGF(LogVirtualTexturing, Display, "TotalLockedMemory: %fMB", (double)TotalLockedMemory / 1024.0 / 1024.0);
}

void FVirtualTextureSystem::DumpPoolUsageFromConsole()
{
	for (int32 i = 0; i < PhysicalSpaces.Num(); ++i)
	{
		if (PhysicalSpaces[i])
		{
			const FVirtualTexturePhysicalSpace& PhysicalSpace = *PhysicalSpaces[i];
			const FVTPhysicalSpaceDescription& Desc = PhysicalSpace.GetDescription();
			const FTexturePagePool& PagePool = PhysicalSpace.GetPagePool();

			UE_LOGF(LogVirtualTexturing, Display, "PhysicalPool: [%i] %ls (%ix%i):", i, *PhysicalSpace.GetFormatString(), Desc.TileSize, Desc.TileSize);

			TMap<uint32, FUintVector2> ProducerCountMap;
			PagePool.CollectProducerCounts(ProducerCountMap);

			TSet<TPair<uint32, FUintVector2>> SortedProducerCounts;
			for (TPair<uint32, FUintVector2> ProducerCount : ProducerCountMap)
			{
				// Filter out producers that only have locked pages mapped.
				// Keep all producers for non compressed formats (assuming that we want to avoid these as much as possible).
				// In future we can add other filters here (count, age etc), or we can dump more info and process off line.
				if (ProducerCount.Value.X > 1 || GPixelFormats[PhysicalSpace.GetFormat(0)].BlockSizeX == 1)
				{
					SortedProducerCounts.Add(ProducerCount);
				}
			}
			SortedProducerCounts.Sort([](TPair<uint32, FUintVector2> const& LHS, TPair<uint32, FUintVector2> const& RHS) { return LHS.Value.X > RHS.Value.X; });

			for (TPair<uint32, FUintVector2> ProducerCount : SortedProducerCounts)
			{
				const uint32 PackedProducerHandle = ProducerCount.Key;
				const uint32 Count = ProducerCount.Value.X;
				const uint32 LockedCount = ProducerCount.Value.Y;

				FVirtualTextureProducer* Producer = Producers.FindProducer(FVirtualTextureProducerHandle(PackedProducerHandle));
				if (Producer != nullptr)
				{
					UE_LOGF(LogVirtualTexturing, Display, "   %ls %d (%d)", *Producer->GetName().ToString(), Count, LockedCount);
				}
			}
		}
	}
}

#if WITH_EDITOR
void FVirtualTextureSystem::SaveAllocatorImagesFromConsole()
{
	for (int32 ID = 0; ID < Spaces.Num(); ID++)
	{
		const FVirtualTextureSpace* Space = Spaces[ID].Get();
		if (Space)
		{
			Space->SaveAllocatorDebugImage();
		}
	}
}
#endif // WITH_EDITOR

IAllocatedVirtualTexture* FVirtualTextureSystem::AllocateVirtualTexture(FRHICommandListBase& RHICmdList, const FAllocatedVTDescription& Desc)
{
	check(Desc.NumTextureLayers <= VIRTUALTEXTURE_SPACE_MAXLAYERS);

	UE::TScopeLock Lock(Mutex);

	// Check to see if we already have an allocated VT that matches this description
	// This can happen often as multiple material instances will share the same textures
	FAllocatedVirtualTexture*& AllocatedVT = AllocatedVTs.FindOrAdd(Desc);
	if (AllocatedVT)
	{
		const int32 PrevNumRefs = AllocatedVT->NumRefs++;
		check(PrevNumRefs >= 0);
		if (PrevNumRefs == 0)
		{
			// Bringing a VT 'back to life', remove it from the pending delete list
			verify(PendingDeleteAllocatedVTs.RemoveSwap(AllocatedVT, EAllowShrinking::No) == 1);
		}

		return AllocatedVT;
	}

	uint32 BlockWidthInTiles = 0u;
	uint32 BlockHeightInTiles = 0u;
	uint32 WidthInBlocks = 0u;
	uint32 HeightInBlocks = 0u;
	uint32 DepthInTiles = 0u;
	FVirtualTextureProducer* ProducerForLayer[VIRTUALTEXTURE_SPACE_MAXLAYERS] = { nullptr };
	bool bAnyLayerProducerWantsPersistentHighestMip = false;
	for (uint32 LayerIndex = 0u; LayerIndex < Desc.NumTextureLayers; ++LayerIndex)
	{
		FVirtualTextureProducer* Producer = Producers.FindProducer(Desc.ProducerHandle[LayerIndex]);
		ProducerForLayer[LayerIndex] = Producer;
		if (Producer)
		{
			const FVTProducerDescription& ProducerDesc = Producer->GetDescription();
			BlockWidthInTiles = FMath::Max(BlockWidthInTiles, ProducerDesc.BlockWidthInTiles);
			BlockHeightInTiles = FMath::Max(BlockHeightInTiles, ProducerDesc.BlockHeightInTiles);
			WidthInBlocks = FMath::Max<uint32>(WidthInBlocks, ProducerDesc.WidthInBlocks);
			HeightInBlocks = FMath::Max<uint32>(HeightInBlocks, ProducerDesc.HeightInBlocks);
			DepthInTiles = FMath::Max(DepthInTiles, ProducerDesc.DepthInTiles);

			uint32 ProducerLayerIndex = Desc.ProducerLayerIndex[LayerIndex];
			uint32 ProducerPhysicalGroup = Producer->GetPhysicalGroupIndexForTextureLayer(ProducerLayerIndex);
			FVirtualTexturePhysicalSpace* PhysicalSpace = Producer->GetPhysicalSpaceForPhysicalGroup(ProducerPhysicalGroup);
			bAnyLayerProducerWantsPersistentHighestMip |= Producer->GetDescription().bPersistentHighestMip;
		}
	}

	check(BlockWidthInTiles > 0u);
	check(BlockHeightInTiles > 0u);
	check(DepthInTiles > 0u);
	check(WidthInBlocks > 0u);
	check(HeightInBlocks > 0u);

	// Sum the total number of physical groups from all producers
	uint32 NumPhysicalGroups = 0;
	if (Desc.bShareDuplicateLayers)
	{
		TArray<FVirtualTextureProducer*> UniqueProducers;
		for (uint32 LayerIndex = 0u; LayerIndex < Desc.NumTextureLayers; ++LayerIndex)
		{
			if (ProducerForLayer[LayerIndex] != nullptr)
			{
				UniqueProducers.AddUnique(ProducerForLayer[LayerIndex]);
			}
		}
		for (int32 ProducerIndex = 0u; ProducerIndex < UniqueProducers.Num(); ++ProducerIndex)
		{
			NumPhysicalGroups += UniqueProducers[ProducerIndex]->GetNumPhysicalGroups();
		}
	}
	else
	{
		NumPhysicalGroups = Desc.NumTextureLayers;
	}

	AllocatedVT = new FAllocatedVirtualTexture(RHICmdList, this, Frame, Desc, ProducerForLayer, BlockWidthInTiles, BlockHeightInTiles, WidthInBlocks, HeightInBlocks, DepthInTiles);
	AllocatedVT->NumRefs = 1;
	if (bAnyLayerProducerWantsPersistentHighestMip)
	{
		AllocatedVTsToMap.Add(AllocatedVT);
		AllocatedVT->bIsWaitingToMap = true;
	}

	// Add to deterministic map that should apply across runs.
	// Note that this may overwrite a duplicate old mapping whenever a new AllocatedVT is recreated after its producers have been recreated.
	// In contrast AllocatedVTs maintains the duplicate entries temporarily until the older entry is deleted.
	PersistentVTMap.Add(TPair<uint32, IAllocatedVirtualTexture*>(AllocatedVT->GetPersistentHash(), AllocatedVT));
	
	return AllocatedVT;
}

void FVirtualTextureSystem::DestroyVirtualTexture(IAllocatedVirtualTexture* AllocatedVT)
{
	UE::TScopeLock Lock(Mutex);
	const int32 NewNumRefs = --AllocatedVT->NumRefs;
	check(NewNumRefs >= 0);
	if (NewNumRefs == 0)
	{
		AllocatedVT->FrameDeleted = Frame;
		PendingDeleteAllocatedVTs.Add(AllocatedVT);
	}
}

void FVirtualTextureSystem::DestroyPendingVirtualTextures(bool bForceDestroyAll)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::DestroyPendingVirtualTextures);

	TArray<IAllocatedVirtualTexture*> AllocatedVTsToDelete;
	{
		if (bForceDestroyAll)
		{
			AllocatedVTsToDelete = MoveTemp(PendingDeleteAllocatedVTs);
			PendingDeleteAllocatedVTs.Reset();
		}
		else
		{
			const int32 MaxDeleteBudget = VirtualTextureScalability::GetMaxAllocatedVTReleasedPerFrame();
			const uint32 CurrentFrame = Frame;
			int32 Index = 0;
			while (Index < PendingDeleteAllocatedVTs.Num())
			{
				IAllocatedVirtualTexture* AllocatedVT = PendingDeleteAllocatedVTs[Index];
				const FAllocatedVTDescription& Desc = AllocatedVT->GetDescription();
				check(AllocatedVT->NumRefs == 0);

				// If the AllocatedVT is using a private space release it immediately, we don't want to hold references to these private spaces any longer then needed.
				const bool bForceDelete = Desc.bPrivateSpace;
				// Keep deleted VTs around for a few frames, in case they are reused.
				const bool bCanDeleteForAge = CurrentFrame >= AllocatedVT->FrameDeleted + 60u;
				// Time slice deletion unless we can make it cheaper.
				const bool bCanDeleteForBudget = MaxDeleteBudget <= 0 || AllocatedVTsToDelete.Num() < MaxDeleteBudget;
				if (bForceDelete || (bCanDeleteForAge && bCanDeleteForBudget))
				{
					AllocatedVTsToDelete.Add(AllocatedVT);
					PendingDeleteAllocatedVTs.RemoveAtSwap(Index, EAllowShrinking::No);
				}
				else
				{
					Index++;
				}
			}
		}
	}

	for (IAllocatedVirtualTexture* AllocatedVT : AllocatedVTsToDelete)
	{
		// shouldn't be more than 1 instance of this in the list
		verify(AllocatedVTsToMap.Remove(AllocatedVT) <= 1);
		verify(AllocatedVTs.Remove(AllocatedVT->GetDescription()) == 1);
		
		// persistent entry might have already been reallocated
		IAllocatedVirtualTexture** Found = PersistentVTMap.Find(AllocatedVT->GetPersistentHash());
		if (Found != nullptr && *Found == AllocatedVT)
		{
			PersistentVTMap.Remove(AllocatedVT->GetPersistentHash());
		}
		
		AllocatedVT->Destroy(this);
		delete AllocatedVT;
	}
}

IAdaptiveVirtualTexture* FVirtualTextureSystem::AllocateAdaptiveVirtualTexture(FRHICommandListBase& RHICmdList, const FAdaptiveVTDescription& AdaptiveVTDesc, const FAllocatedVTDescription& AllocatedVTDesc)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);
	FAdaptiveVirtualTexture* AdaptiveVT = new FAdaptiveVirtualTexture(AdaptiveVTDesc, AllocatedVTDesc);
	AdaptiveVT->Init(RHICmdList, this);
	check(AdaptiveVTs.IsValidIndex(AdaptiveVT->GetSpaceID()));
	check(AdaptiveVTs[AdaptiveVT->GetSpaceID()] == nullptr);
	AdaptiveVTs[AdaptiveVT->GetSpaceID()] = AdaptiveVT;
	return AdaptiveVT;
}

void FVirtualTextureSystem::DestroyAdaptiveVirtualTexture(IAdaptiveVirtualTexture* AdaptiveVT)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);
	check(AdaptiveVTs.IsValidIndex(AdaptiveVT->GetSpaceID()));
	check(AdaptiveVTs[AdaptiveVT->GetSpaceID()] == AdaptiveVT);
	AdaptiveVTs[AdaptiveVT->GetSpaceID()] = nullptr;
	AdaptiveVT->Destroy(this);
}

FVirtualTextureProducerHandle FVirtualTextureSystem::RegisterProducer(FRHICommandListBase& RHICmdList, const FVTProducerDescription& InDesc, IVirtualTexture* InProducer)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);
	return Producers.RegisterProducer(RHICmdList, this, InDesc, InProducer);
}

void FVirtualTextureSystem::ReleaseProducer(const FVirtualTextureProducerHandle& Handle)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);
	Producers.ReleaseProducer(this, Handle);
}

bool FVirtualTextureSystem::TryReleaseProducer(const FVirtualTextureProducerHandle& Handle)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);
	return Producers.TryReleaseProducer(this, Handle);
}

void FVirtualTextureSystem::AddProducerDestroyedCallback(const FVirtualTextureProducerHandle& Handle, FVTProducerDestroyedFunction* Function, void* Baton)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);
	Producers.AddDestroyedCallback(Handle, Function, Baton);
}

uint32 FVirtualTextureSystem::RemoveAllProducerDestroyedCallbacks(const void* Baton)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);
	return Producers.RemoveAllCallbacks(Baton);
}

FVirtualTextureProducer* FVirtualTextureSystem::FindProducer(const FVirtualTextureProducerHandle& Handle)
{
	return Producers.FindProducer(Handle);
}

FVirtualTextureSpace* FVirtualTextureSystem::AcquireSpace(FRHICommandListBase& RHICmdList, const FVTSpaceDescription& InDesc, uint8 InForceSpaceID, FAllocatedVirtualTexture* AllocatedVT)
{
	check(!bUpdating);
	LLM_SCOPE(ELLMTag::VirtualTextureSystem);

	uint32 NumFailedAllocations = 0;

	// If InDesc requests a private space, don't reuse any existing spaces (unless it is a forced space)
	if (!InDesc.bPrivateSpace || InForceSpaceID != 0xff)
	{
		for (int32 SpaceIndex = 0; SpaceIndex < Spaces.Num(); ++SpaceIndex)
		{
			if (SpaceIndex == InForceSpaceID || InForceSpaceID == 0xff)
			{
				FVirtualTextureSpace* Space = Spaces[SpaceIndex].Get();
				if (Space != nullptr && Space->GetDescription() == InDesc)
				{
					const int32 PagetableMemory = Space->GetSizeInBytes();
					const uint32 vAddress = Space->AllocateVirtualTexture(AllocatedVT);
					if (vAddress != ~0u)
					{
						const int32 NewPagetableMemory = Space->GetSizeInBytes();
						INC_MEMORY_STAT_BY(STAT_TotalPagetableMemory, NewPagetableMemory - PagetableMemory);

						AllocatedVT->AssignVirtualAddress(vAddress);
						Space->AddRef();
						return Space;
					}
					else
					{
						++NumFailedAllocations;
					}
				}
			}
		}
	}
	
	if (InForceSpaceID != 0xff)
	{
		// ForceSpaceID is used by adaptive virtual textures. 
		// If we get here we failed to allocate in an adaptive page table. 
		// This should be unlikely/impossible since adaptive virtual textures test with a TryAlloc() before really allocating.
		// If you do hit this somehow, try reducing r.VT.AVT.MaxPageResidency to fix.
		// We allocate a new space to avoid crashing, but warn the user that things won't work well from this point on.
		UE_LOGF(LogVirtualTexturing, Warning, "Failed to allocate in requested Virtual Texture Space %d. Expect to see visual corruption.", InForceSpaceID);
	}

	// Find an empty space slot.
	int32 FreeSpaceIndex = INDEX_NONE;

	// We don't support feedback on more than 16 spaces, because we only have 4 bits available in the 32bit feedback requests.
	// If we are at the limit of spaces that support feedback then we prepare for a retry below.
	const int32 TryCount = NumAllocatedSpaces >= VIRTUALTEXTURE_MAX_FEEDBACK_SPACES ? 2 : 1;

	for (int32 TryIndex = 0; FreeSpaceIndex == INDEX_NONE && TryIndex < TryCount; ++TryIndex)
	{
		if (TryIndex == 1)
		{
			// We failed to find a space on the first iteration of the loop.
			// Flush all pending destroys now which may free up a space.
			// This is a potentially expensive operation, which is why we only do it if necessary.
			DestroyPendingVirtualTextures(true);
		}

		// Use an empty space slot if it exists.
		for (int32 SpaceIndex = 0; FreeSpaceIndex == INDEX_NONE && SpaceIndex < Spaces.Num(); ++SpaceIndex)
		{
			if (!Spaces[SpaceIndex].IsValid())
			{
				FreeSpaceIndex = SpaceIndex;
			}
		}

		// If no empty space slot was found then try to recycle a space. 
		// A space may be available because we don't always release spaces immediately so as to avoid unnecessary page table recreation.
		for (int32 SpaceIndex = 0; FreeSpaceIndex == INDEX_NONE && SpaceIndex < Spaces.Num(); ++SpaceIndex)
		{
			FVirtualTextureSpace* Space = Spaces[SpaceIndex].Get();
			if (Space != nullptr && Space->GetRefCount() == 0)
			{
				DEC_MEMORY_STAT_BY(STAT_TotalPagetableMemory, Space->GetSizeInBytes());
				Space->ReleaseResource();
				NumAllocatedSpaces--;

				FreeSpaceIndex = SpaceIndex;
			}
		}
	}

	// Nothing found so we need to create a new slot.
	if (FreeSpaceIndex == INDEX_NONE)
	{
		// Allocate a new Space slot.
		FreeSpaceIndex = Spaces.AddDefaulted();
		// Expect an AdaptiveVT slot for each space.
		AdaptiveVTs.Add(nullptr);
		check (Spaces.Num() == AdaptiveVTs.Num());
	}

	if (FreeSpaceIndex >= VIRTUALTEXTURE_MAX_FEEDBACK_SPACES)
	{
		// We still create a new space but warn the user.
		// If you hit this warning: To reduce the number of spaces created, try using r.VT.PageTableMode=0 or modifying RVTs to not use private spaces.
		UE_LOGF(LogVirtualTexturing, Warning, "Allocating a Virtual Texture space with ID %d. Feedback will not work for this space. A dump of the current pool state follows.", FreeSpaceIndex);
		ListPhysicalPoolsFromConsole();
	}

	const uint32 InitialPageTableSize = InDesc.bPrivateSpace ? InDesc.MaxSpaceSize : FMath::Max(AllocatedVT->GetWidthInTiles(), AllocatedVT->GetHeightInTiles());
	FVirtualTextureSpace* Space = new FVirtualTextureSpace(this, FreeSpaceIndex, InDesc, InitialPageTableSize);
	Spaces[FreeSpaceIndex].Reset(Space);

	NumAllocatedSpaces++;
	INC_MEMORY_STAT_BY(STAT_TotalPagetableMemory, Space->GetSizeInBytes());
	Space->InitResource(RHICmdList);

	const uint32 vAddress = Space->AllocateVirtualTexture(AllocatedVT);
	AllocatedVT->AssignVirtualAddress(vAddress);

	Space->AddRef();
	return Space;
}

void FVirtualTextureSystem::ReleaseSpace(FVirtualTextureSpace* Space)
{
	check(!bUpdating);
	const uint32 NumRefs = Space->Release();
	if (NumRefs == 0u)
	{
		// Private spaces can't be recycled and so are destroyed immediately when ref count reaches 0.
		// Other spaces wait on a timer to avoid unnecessary recreation during load phases etc.
		const int32 ReleaseFrames = CVarVTSpaceReleaseFrames.GetValueOnRenderThread();

		if (Space->GetDescription().bPrivateSpace || ReleaseFrames == 0)
		{
			// We only get here on the render thread, so we can call ReleaseResource() directly and then delete the pointer immediately
			DEC_MEMORY_STAT_BY(STAT_TotalPagetableMemory, Space->GetSizeInBytes());
			Space->ReleaseResource();
			Spaces[Space->GetID()].Reset();
			NumAllocatedSpaces--;
		}
		else
		{
			Space->SetReleasedFrame(Frame);
		}
	}
}

/** Get the extra physical space description that depends on the virtual texture pool config. */
void GetPhysicalSpaceExtraDescription(FVTPhysicalSpaceDescription const& InDesc, FVTPhysicalSpaceDescriptionExt& OutDescExt)
{
	// Find matching config from pool settings.
	FVirtualTextureSpacePoolConfig Config;
	VirtualTexturePool::FindPoolConfig(InDesc.Format, InDesc.NumLayers, InDesc.TileSize, Config);
	int32 SizeInMegabyte = Config.SizeInMegabyte;

	// Adjust found config for scaling.
	const float Scale = Config.bAllowSizeScale ? VirtualTexturePool::GetPoolSizeScale() : 1.f;
	SizeInMegabyte = (int32)(Scale * (float)SizeInMegabyte);
	if (Scale < 1.f && Config.MinScaledSizeInMegabyte > 0)
	{
		SizeInMegabyte = FMath::Max(SizeInMegabyte, Config.MinScaledSizeInMegabyte);
	}
	if (Scale > 1.f && Config.MaxScaledSizeInMegabyte > 0)
	{
		SizeInMegabyte = FMath::Min(SizeInMegabyte, Config.MaxScaledSizeInMegabyte);
	}
	const int32 PoolSizeInBytes = SizeInMegabyte * 1024u * 1024u;

	// Get size of a single tile.
	const FPixelFormatInfo& FormatInfo = GPixelFormats[InDesc.Format[0]];
	check(InDesc.TileSize % FormatInfo.BlockSizeX == 0);
	check(InDesc.TileSize % FormatInfo.BlockSizeY == 0);
	SIZE_T TileSizeBytes = 0;
	for (int32 Layer = 0; Layer < InDesc.NumLayers; ++Layer)
	{
		TileSizeBytes += CalculateImageBytes(InDesc.TileSize, InDesc.TileSize, 0, InDesc.Format[Layer]);
	}

	// Calculate final size in tiles.
	// Loop to find matching pool count if necessary.
	int32 TileWidthHeight = 0;
	int32 PoolCount = 1;
	while (1)
	{
		const uint32 NumTiles = FMath::Max((uint32)(PoolSizeInBytes / (PoolCount * TileSizeBytes)), 1u);
		
		TileWidthHeight = FMath::FloorToInt(FMath::Sqrt((float)NumTiles));

		// If we allow splitting one pool confing into multiple physical pools by size then keep incrementing pool count until we fit.
		const int32 SplitPhysicalPoolSize = VirtualTexturePool::GetSplitPhysicalPoolSize();
		if (!InDesc.bCanSplit || SplitPhysicalPoolSize <= 0 || TileWidthHeight <= SplitPhysicalPoolSize)
		{
			break;
		}

		PoolCount++;
	}

	// Need to clamp for maximum texture size.
	if (TileWidthHeight * InDesc.TileSize > GetMax2DTextureDimension())
	{
		// A good option to support extremely large caches would be to allow additional slices in an array here for caches...
		// Just try to use the maximum texture size for now
		TileWidthHeight = FMath::DivideAndRoundDown(GetMax2DTextureDimension(), InDesc.TileSize);
	}

	// Need to clamp for maximum tile count in a pool. 
	// Page table encoding limits this to 1<<16. But FTexturePagePool::FreeHeap being 16bit and needing an overflow bit reduces us to 1<<15.
	const int32 MaxTiles = 1 << 15;
	const int32 MaxTilesSqrt = 181; //sqrt(1<<15)
	TileWidthHeight = FMath::Min(TileWidthHeight, MaxTilesSqrt);

	OutDescExt.TileWidthHeight = TileWidthHeight;
	OutDescExt.PoolCount = PoolCount;
	OutDescExt.bEnableResidencyMipMapBias = Config.bEnableResidencyMipMapBias;
	OutDescExt.ResidencyMipMapBiasGroup = FMath::Clamp(Config.ResidencyMipMapBiasGroup, 0, 3);
}

/** Cached version of GetPhysicalSpaceExtraDescription() to avoid regularly repeating the heavy work in that function. */
void GetPhysicalSpaceExtraDescription_Cached(FVTPhysicalSpaceDescription const& InDesc, FVTPhysicalSpaceDescriptionExt& OutDescExt)
{
	static TMap<FVTPhysicalSpaceDescription, FVTPhysicalSpaceDescriptionExt> Map;

	// Invalidate the cache if any config settings change.
	uint32 PhysicalPoolSettingsHash = VirtualTexturePool::GetConfigHash();
	static uint32 LastPhysicalPoolSettingsHash = PhysicalPoolSettingsHash;
	if (LastPhysicalPoolSettingsHash != PhysicalPoolSettingsHash)
	{
		LastPhysicalPoolSettingsHash = PhysicalPoolSettingsHash;
		Map.Reset();
	}

	FVTPhysicalSpaceDescriptionExt* InitDescriptionPtr = Map.Find(InDesc);
	if (InitDescriptionPtr == nullptr)
	{
		GetPhysicalSpaceExtraDescription(InDesc, OutDescExt);
		Map.Add(InDesc, OutDescExt);
	}
	else
	{
		OutDescExt = *InitDescriptionPtr;
	}
}

FVirtualTexturePhysicalSpace* FVirtualTextureSystem::AcquirePhysicalSpace(FRHICommandListBase& RHICmdList, const FVTPhysicalSpaceDescription& InDesc)
{
	LLM_SCOPE(ELLMTag::VirtualTextureSystem);

	// Get extra setup information from the virtual pool configs.
	FVTPhysicalSpaceDescriptionExt DescExt;
	GetPhysicalSpaceExtraDescription_Cached(InDesc, DescExt);

	// Find matching pools.
	// We support multiple matching pools to allow for 16bit page table memory optimization.
	TArray<int32, TInlineAllocator<8>> Matching;
	for (int32 i = 0; i < PhysicalSpaces.Num(); ++i)
	{
		FVirtualTexturePhysicalSpace* PhysicalSpace = PhysicalSpaces[i];
		if (PhysicalSpace && PhysicalSpace->GetDescription() == InDesc && PhysicalSpace->GetDescriptionExt() == DescExt)
		{
			Matching.Add(i);
		}
	}

	if (DescExt.PoolCount <= Matching.Num())
	{
		// Randomly select from any pools that exist.
		int32 RandomIndex = FMath::RandHelper(Matching.Num());
		return PhysicalSpaces[Matching[RandomIndex]];
	}
	
	// Not reached maximum matching pool count yet so create a new pool.
	uint32 ID = PhysicalSpaces.Num();
	check(ID <= 0x0fff);

	for (int32 i = 0; i < PhysicalSpaces.Num(); ++i)
	{
		if (!PhysicalSpaces[i])
		{
			ID = i;
			break;
		}
	}

	if (ID == PhysicalSpaces.Num())
	{
		PhysicalSpaces.AddZeroed();
	}

	FVirtualTexturePhysicalSpace* PhysicalSpace = new FVirtualTexturePhysicalSpace(ID, InDesc, DescExt);
	PhysicalSpaces[ID] = PhysicalSpace;

	return PhysicalSpace;
}

void FVirtualTextureSystem::ReleasePendingSpaces(bool bForceReleaseAll)
{
	for (int32 Id = 0; Id < PhysicalSpaces.Num(); ++Id)
	{
		// Physical space is immediately released when ref count hits 0.
		FVirtualTexturePhysicalSpace* PhysicalSpace = PhysicalSpaces[Id];
		if (PhysicalSpace != nullptr && PhysicalSpace->GetRefCount() == 0u)
		{
			const FTexturePagePool& PagePool = PhysicalSpace->GetPagePool();
			check(PagePool.GetNumMappedPages() == 0u);
			check(PagePool.GetNumLockedPages() == 0u);

			PhysicalSpace->ReleaseResource();
			delete PhysicalSpace;
			PhysicalSpaces[Id] = nullptr;
		}
	}

	for (int32 Id = 0; Id < Spaces.Num(); ++Id)
	{
		// Virtual space is released when ref count hits 0 and after some delay to avoid unnecessary recreation.
		FVirtualTextureSpace* Space = Spaces[Id].Get();
		if (Space != nullptr && Space->GetRefCount() == 0u)
		{
			const int32 ReleaseFrames = CVarVTSpaceReleaseFrames.GetValueOnRenderThread();
			if (bForceReleaseAll || (ReleaseFrames >= 0 && Space->GetReleasedFrame() + ReleaseFrames <= Frame))
			{
				DEC_MEMORY_STAT_BY(STAT_TotalPagetableMemory, Spaces[Id]->GetSizeInBytes());
				Space->ReleaseResource();
				Spaces[Id].Reset();
				NumAllocatedSpaces--;
			}
		}
	}
}

void FVirtualTextureSystem::LockTile(const FVirtualTextureLocalTile& Tile)
{
	check(!bUpdating);

	if (TileLocks.Lock(Tile))
	{
		checkSlow(!TilesToLock.Contains(Tile));
		TilesToLock.Add(Tile);
	}
}

static void UnlockTileInternal(const FVirtualTextureProducerHandle& ProducerHandle, const FVirtualTextureProducer* Producer, const FVirtualTextureLocalTile& Tile, uint32 Frame)
{
	for (uint32 ProducerPhysicalGroupIndex = 0u; ProducerPhysicalGroupIndex < Producer->GetNumPhysicalGroups(); ++ProducerPhysicalGroupIndex)
	{
		FVirtualTexturePhysicalSpace* PhysicalSpace = Producer->GetPhysicalSpaceForPhysicalGroup(ProducerPhysicalGroupIndex);
		FTexturePagePool& PagePool = PhysicalSpace->GetPagePool();
		const uint32 pAddress = PagePool.FindPageAddress(ProducerHandle, ProducerPhysicalGroupIndex, Tile.Local_vAddress, Tile.Local_vLevel);
		if (pAddress != ~0u)
		{
			PagePool.Unlock(Frame, pAddress);
		}
	}
}

void FVirtualTextureSystem::UnlockTile(const FVirtualTextureLocalTile& Tile, const FVirtualTextureProducer* Producer)
{
	check(!bUpdating);

	if (TileLocks.Unlock(Tile))
	{
		// Tile is no longer locked
		const int32 NumTilesRemoved = TilesToLock.Remove(Tile);
		check(NumTilesRemoved <= 1);
		// If tile was still in the 'TilesToLock' list, that means it was never actually locked, so we don't need to do the unlock here
		if (NumTilesRemoved == 0)
		{
			UnlockTileInternal(Tile.GetProducerHandle(), Producer, Tile, Frame);
		}
	}
}

void FVirtualTextureSystem::ForceUnlockAllTiles(const FVirtualTextureProducerHandle& ProducerHandle, const FVirtualTextureProducer* Producer)
{
	check(!bUpdating);

	TArray<FVirtualTextureLocalTile> TilesToUnlock;
	TileLocks.ForceUnlockAll(ProducerHandle, TilesToUnlock);

	for (const FVirtualTextureLocalTile& Tile : TilesToUnlock)
	{
		const int32 NumTilesRemoved = TilesToLock.Remove(Tile);
		check(NumTilesRemoved <= 1);
		if (NumTilesRemoved == 0)
		{
			UnlockTileInternal(ProducerHandle, Producer, Tile, Frame);
		}
	}
}

static float ComputeMipLevel(const IAllocatedVirtualTexture* AllocatedVT, const FVector2D& InScreenSpaceSize)
{
	const uint32 TextureWidth = AllocatedVT->GetWidthInPixels();
	const uint32 TextureHeight = AllocatedVT->GetHeightInPixels();
	const FVector2D dfdx(TextureWidth / InScreenSpaceSize.X, 0.0f);
	const FVector2D dfdy(0.0f, TextureHeight / InScreenSpaceSize.Y);
	const float ppx = FVector2D::DotProduct(dfdx, dfdx);
	const float ppy = FVector2D::DotProduct(dfdy, dfdy);
	return 0.5f * FMath::Log2(FMath::Max(ppx, ppy));
}

void FVirtualTextureSystem::RequestTiles(const FVector2D& InScreenSpaceSize, int32 InMipLevel)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);

	for (const auto& Pair : AllocatedVTs)
{
		RequestTilesInternal(Pair.Value, InScreenSpaceSize, InMipLevel);
	}
}

void FVirtualTextureSystem::RequestTiles(const FMaterialRenderProxy* InMaterialRenderProxy, const FVector2D& InScreenSpaceSize, ERHIFeatureLevel::Type InFeatureLevel)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);

	for (IAllocatedVirtualTexture* AllocatedVT : InMaterialRenderProxy->UniformExpressionCache[InFeatureLevel].AllocatedVTs)
	{
		if (AllocatedVT != nullptr)
		{
			RequestTilesInternal(AllocatedVT, InScreenSpaceSize, INDEX_NONE);
		}
	}
}

void FVirtualTextureSystem::RequestTilesInternal(const IAllocatedVirtualTexture* InAllocatedVT, const FVector2D& InScreenSpaceSize, int32 InMipLevel)
{
	if (InMipLevel < 0)
	{
		const uint32 vMaxLevel = InAllocatedVT->GetMaxLevel();
		const float vLevel = ComputeMipLevel(InAllocatedVT, InScreenSpaceSize);
		const int32 vMipLevelDown = FMath::Clamp((int32)FMath::FloorToInt(vLevel), 0, (int32)vMaxLevel);

		RequestTilesInternal(InAllocatedVT, vMipLevelDown);
		if (vMipLevelDown + 1u <= vMaxLevel)
		{
			// Need to fetch 2 levels to support trilinear filtering
			RequestTilesInternal(InAllocatedVT, vMipLevelDown + 1u);
		}
	}
	else
	{
		RequestTilesInternal(InAllocatedVT, InMipLevel);
	}
}

void FVirtualTextureSystem::RequestTiles(IAllocatedVirtualTexture* AllocatedVT, const FVector2D& InScreenSpaceSize, const FVector2D& InViewportPosition, const FVector2D& InViewportSize, const FVector2D& InUV0, const FVector2D& InUV1, int32 InMipLevel)
{
	UE::TScopeLock Lock(Mutex);
	if (InMipLevel >= 0)
	{
		if (FAdaptiveVirtualTexture* AdaptiveVT = GetAdaptiveVirtualTexture(AllocatedVT->GetSpaceID()))
		{
			// Adaptive virtual textures are a special case where the "root" AllocatedVT is one of many AllocatedVTs.
			// We get the full set of AllocatedVTs before requesting tiles.
			TArray<FAdaptiveVirtualTexture::FAllocatedInfo> Infos;
			TArray<uint32> AdaptiveAllocationRequests;
			AdaptiveVT->GetAllocatedVirtualTextures(FBox2D(InUV0, InUV1), InMipLevel, Infos, AdaptiveAllocationRequests);
			for (FAdaptiveVirtualTexture::FAllocatedInfo const& Info : Infos)
			{
				RequestTilesForRegionInternal(Info.AllocatedVirtualTexture, InScreenSpaceSize, InViewportPosition, InViewportSize, Info.RemappedUV.Min, Info.RemappedUV.Max, Info.RemappedLevel);
			}
			
			// If we found any adaptive AllocatedVTs that need reallocation to reach the requested mip level then queue those requests immediately.
			AdaptiveVT->QueuePackedAllocationRequests(AdaptiveAllocationRequests, Frame);
		}
		else
		{
			RequestTilesForRegionInternal(AllocatedVT, InScreenSpaceSize, InViewportPosition, InViewportSize, InUV0, InUV1, InMipLevel);
		}
	}
	else
	{
		const uint32 vMaxLevel = AllocatedVT->GetMaxLevel();
		const float vLevel = ComputeMipLevel(AllocatedVT, InScreenSpaceSize); // TODO: ComputeMipLevel() is incorrect if not using the whole UV range
		const int32 vMipLevelDown = FMath::Clamp((int32)FMath::FloorToInt(vLevel), 0, (int32)vMaxLevel);

		RequestTilesForRegionInternal(AllocatedVT, InScreenSpaceSize, InViewportPosition, InViewportSize, InUV0, InUV1, vMipLevelDown);
		if (vMipLevelDown + 1u <= vMaxLevel)
		{
			// Need to fetch 2 levels to support trilinear filtering
			RequestTilesForRegionInternal(AllocatedVT, InScreenSpaceSize, InViewportPosition, InViewportSize, InUV0, InUV1, vMipLevelDown + 1u);
		}
	}
}

void FVirtualTextureSystem::LoadPendingTiles(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);

	TArray<uint32> PackedTiles;
	if (RequestedPackedTiles.Num() > 0)
	{
		PackedTiles = MoveTemp(RequestedPackedTiles);
		RequestedPackedTiles.Reset();
	}

	if (PackedTiles.Num() > 0)
	{
		FVirtualTextureUpdateSettings Settings;
		Settings.EnableThrottling(false);
		
		FConcurrentLinearBulkObjectAllocator Allocator;

		FUniquePageList* UniquePageList = Allocator.Create<FUniquePageList>();
		UniquePageList->Initialize();
		for (uint32 Tile : PackedTiles)
		{
			UniquePageList->Add(Tile, 0xffff);
		}

		FUniqueRequestList* RequestList = Allocator.Create<FUniqueRequestList>(Allocator);
		RequestList->Initialize();

		GatherLockedTileRequests(RequestList);
		GatherRequests(RequestList, UniquePageList, Frame, Allocator, Settings);
		// No need to sort requests, since we're submitting all of them here (no throttling)
		AllocateResources(GraphBuilder);
		SubmitRequests(GraphBuilder.RHICmdList, FeatureLevel, Allocator, Settings, RequestList);
		FinalizeRequests(GraphBuilder, nullptr);

		// Swap any failed tile locks to be picked up by next BeginUpdate()
		TilesToLock = MoveTemp(TilesToLockForNextFrame);
	}
}

void FVirtualTextureSystem::RequestTilesInternal(const IAllocatedVirtualTexture* AllocatedVT, int32 InMipLevel)
{
	// Don't generate tile requests for spaces that don't support it.
	if (AllocatedVT->GetSpaceID() >= VIRTUALTEXTURE_MAX_FEEDBACK_SPACES)
	{
		return;
	}

	const int32 MipWidthInTiles = FMath::Max<int32>(AllocatedVT->GetWidthInTiles() >> InMipLevel, 1);
	const int32 MipHeightInTiles = FMath::Max<int32>(AllocatedVT->GetHeightInTiles() >> InMipLevel, 1);
	const uint32 vBaseTileX = AllocatedVT->GetVirtualPageX() >> InMipLevel;
	const uint32 vBaseTileY = AllocatedVT->GetVirtualPageY() >> InMipLevel;

	for (int32 TilePositionY = 0; TilePositionY < MipHeightInTiles; TilePositionY++)
	{
		const uint32 vGlobalTileY = vBaseTileY + TilePositionY;
		for (int32 TilePositionX = 0; TilePositionX < MipWidthInTiles; TilePositionX++)
		{
			const uint32 vGlobalTileX = vBaseTileX + TilePositionX;
			const uint32 EncodedTile = EncodePage(AllocatedVT->GetSpaceID(), InMipLevel, vGlobalTileX, vGlobalTileY);
			RequestedPackedTiles.Add(EncodedTile);
		}
	}
}

void FVirtualTextureSystem::SetMipLevelToLock(FVirtualTextureProducerHandle ProducerHandle, int32 InMipLevel)
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);

	FVirtualTextureProducer& Producer = Producers.GetProducer(ProducerHandle);
	const uint32 TileCountX = Producer.GetWidthInTiles();
	const uint32 TileCountY = Producer.GetHeightInTiles();
	const uint32 NumLevels = Producer.GetMaxLevel() + 1;
	const uint32 NumNonStreamingLevels = FMath::FloorLog2(Producer.GetDescription().TileSize);
	const uint32 NumLockMips = NumLevels - FMath::Min(NumLevels, InMipLevel + NumNonStreamingLevels);

	// Compare with current setting on the producer and Lock or Unlock tiles to reach the new setting.
	const uint32 NumLockMipsPrevious = Producer.GetLockedMipCount();
	Producer.SetLockedMipCount(NumLockMips);

	const bool bLock = NumLockMips > NumLockMipsPrevious;
	const uint32 FirstMip = NumNonStreamingLevels + (bLock ? NumLockMipsPrevious : NumLockMips);
	const uint32 LastMip = NumNonStreamingLevels + (bLock ? NumLockMips : NumLockMipsPrevious);

	for (uint32 MipIndex = FirstMip; MipIndex < LastMip; ++MipIndex)
	{
		const uint32 vLevel = NumLevels - MipIndex - 1;
		const uint32 MipWidthInTiles = FMath::DivideAndRoundUp(TileCountX, 1u << vLevel);
		const uint32 MipHeightInTiles = FMath::DivideAndRoundUp(TileCountY, 1u << vLevel);
		for (uint32 PosY = 0; PosY < MipHeightInTiles; PosY++)
		{
			for (uint32 PosX = 0; PosX < MipWidthInTiles; PosX++)
			{
				const uint32 Address = FMath::MortonCode2(PosX) | (FMath::MortonCode2(PosY) << 1);
				const FVirtualTextureLocalTile Tile(ProducerHandle, Address, vLevel);
				if (bLock)
				{
					LockTile(Tile);
				}
				else
				{
					UnlockTile(Tile, &Producer);
				}
			}
		}
	}
}

static int32 WrapTilePosition(int32 Position, int32 Size)
{
	const int32 Result = Position % Size;
	return (Result >= 0) ? Result : Result + Size;
}

void FVirtualTextureSystem::RequestTilesForRegionInternal(const IAllocatedVirtualTexture* AllocatedVT, const FVector2D& InScreenSpaceSize, const FVector2D& InViewportPosition, const FVector2D& InViewportSize, const FVector2D& InUV0, const FVector2D& InUV1, int32 InMipLevel)
{
	// Don't generate tile requests for spaces that don't support it.
	if (AllocatedVT->GetSpaceID() >= VIRTUALTEXTURE_MAX_FEEDBACK_SPACES)
	{
		return;
	}

	// Screen size must be a least a pixel
	FVector2D ScreenSize = FVector2D::Max(InScreenSpaceSize, FVector2D::One());

	// TopLeft vs BottomRight - In viewport space
	FVector2D TextureTopLeftViewportSpace = InViewportPosition;
	FVector2D TextureBottomRightViewportSpace = InViewportPosition + ScreenSize;

	// TopLeft vs BottomRight - Clamped to viewport
	FVector2D TextureTopLeftViewportSpaceClamped = FVector2D::Clamp(TextureTopLeftViewportSpace, FVector2D::Zero(), InViewportSize);
	FVector2D TextureBottomRightViewportSpaceClamped = FVector2D::Clamp(TextureBottomRightViewportSpace, FVector2D::Zero(), InViewportSize);

	// Range of initial screen size for TopLeft & BottomRight
	// For example, if 10% of the image is outside each side of the viewport, LerpTopLeft & LerpBottomRight would be (0.1, 0.1) & (0.9, 0.9), respectively
	FVector2D LerpTopLeft = FVector2D::Clamp((TextureTopLeftViewportSpaceClamped - TextureTopLeftViewportSpace) / ScreenSize, FVector2D::Zero(), FVector2D::One());
	FVector2D LerpBottomRight = FVector2D::Clamp((TextureBottomRightViewportSpaceClamped - TextureTopLeftViewportSpace) / ScreenSize, FVector2D::Zero(), FVector2D::One());
	
	const int32 WidthInBlocks = AllocatedVT->GetWidthInBlocks();
	const int32 HeightInBlocks = AllocatedVT->GetHeightInBlocks();

	// Map coordinates to UV space
	const float PositionU0 = FMath::Lerp(InUV0.X, InUV1.X, LerpTopLeft.X) / WidthInBlocks;
	const float PositionV0 = FMath::Lerp(InUV0.Y, InUV1.Y, LerpTopLeft.Y) / HeightInBlocks;
	const float PositionU1 = FMath::Lerp(InUV0.X, InUV1.X, LerpBottomRight.X) / WidthInBlocks;
	const float PositionV1 = FMath::Lerp(InUV0.Y, InUV1.Y, LerpBottomRight.Y) / HeightInBlocks;

	// Map UVs to tile coordinates
	const int32 MipWidthInTiles = FMath::Max<int32>(AllocatedVT->GetWidthInTiles() >> InMipLevel, 1);
	const int32 MipHeightInTiles = FMath::Max<int32>(AllocatedVT->GetHeightInTiles() >> InMipLevel, 1);
	const int32 TilePositionX0 = FMath::FloorToInt(FMath::Min(PositionU0, PositionU1) * MipWidthInTiles);
	const int32 TilePositionY0 = FMath::FloorToInt(FMath::Min(PositionV0, PositionV1) * MipHeightInTiles);
	const int32 TilePositionX1 = FMath::CeilToInt(FMath::Max(PositionU0, PositionU1) * MipWidthInTiles);
	const int32 TilePositionY1 = FMath::CeilToInt(FMath::Max(PositionV0, PositionV1) * MipHeightInTiles);

	// RequestedPackedTiles stores packed tiles with vPosition shifted relative to current mip level
	const uint32 vBaseTileX = AllocatedVT->GetVirtualPageX() >> InMipLevel;
	const uint32 vBaseTileY = AllocatedVT->GetVirtualPageY() >> InMipLevel;

	for (int32 TilePositionY = TilePositionY0; TilePositionY < TilePositionY1; TilePositionY++)
	{
		const uint32 vGlobalTileY = vBaseTileY + WrapTilePosition(TilePositionY, MipHeightInTiles);
		for (int32 TilePositionX = TilePositionX0; TilePositionX < TilePositionX1; TilePositionX++)
		{
			const uint32 vGlobalTileX = vBaseTileX + WrapTilePosition(TilePositionX, MipWidthInTiles);
			const uint32 EncodedTile = EncodePage(AllocatedVT->GetSpaceID(), InMipLevel, vGlobalTileX, vGlobalTileY);
			RequestedPackedTiles.Add(EncodedTile);
		}
	}
}

void FVirtualTextureSystem::FeedbackAnalysisTask(const FFeedbackAnalysisParameters& Parameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::FeedbackAnalysisTask);

	FUniquePageList* RESTRICT RequestedPageList = Parameters.UniquePageList;
	const FUintPoint* RESTRICT Buffer = Parameters.FeedbackBuffer;
	const uint32 BufferSize = Parameters.FeedbackSize;

	// Combine simple runs of identical requests
	uint32 LastPixel = 0xffffffff;
	uint32 LastCount = 0;

	for (uint32 Index = 0; Index < BufferSize; Index++)
	{
		const uint32 Pixel = Buffer[Index].X;
		const uint32 Count = Buffer[Index].Y;
		if (Pixel == LastPixel)
		{
			LastCount += Count;
			continue;
		}

		if (LastPixel != 0xffffffff)
		{
			RequestedPageList->Add(LastPixel, LastCount);
		}

		LastPixel = Pixel;
		LastCount = Count;
	}

	if (LastPixel != 0xffffffff)
	{
		RequestedPageList->Add(LastPixel, LastCount);
	}
}

void FVirtualTextureSystem::GatherRequests(FUniqueRequestList* MergedRequestList, const FUniquePageList* UniquePageList, uint32 FrameRequested, FConcurrentLinearBulkObjectAllocator& Allocator, FVirtualTextureUpdateSettings const& Settings)
{
	FGatherRequestsParameters Parameters;
	Parameters.UniquePageList = UniquePageList;
	Parameters.RequestList = MergedRequestList;
	Parameters.PageStartIndex = 0;
	Parameters.NumPages = UniquePageList->GetNum();
	Parameters.FrameRequested = FrameRequested;
	Parameters.bForceContinuousUpdate = Settings.bForceContinuousUpdate;
	Parameters.bEnableLoadRequests = Settings.bEnableFeedbackProduce;
	Parameters.TimeBudget = Settings.MaxGatherTime;

	GatherRequestsTask(Parameters);
}

void FVirtualTextureSystem::AddPageUpdate(uint32 PhysicalSpaceID, uint16 pAddress, uint8 vLevelOffset, uint32 CurrentFrame)
{
	FVirtualTexturePhysicalSpace* RESTRICT PhysicalSpace = GetPhysicalSpace(PhysicalSpaceID);
	FTexturePagePool& RESTRICT PagePool = PhysicalSpace->GetPagePool();
	PagePool.UpdateUsage(CurrentFrame, pAddress);
	
	int8 vMagOffset = FMath::Min<int8>(0, static_cast<int8>(vLevelOffset) - UE::HLSL::VirtualTextureFeedbackBias);
	int8 vLevelMag  = FMath::Max(0, PagePool.GetLocalLevelForAddress(pAddress) + vMagOffset);
	
	Producers.UpdateSamplingHistory(PagePool.GetPackedProducerHandleForAddress(pAddress), vLevelMag, CurrentFrame);
}

uint8 FVirtualTextureSystem::GetFinestSampledMipInLastNFrames(const FAllocatedVirtualTexture* AllocatedVT, uint32 FrameWindow) const
{
	const uint32 CurrentFrame = Frame;
	uint8 FinestMip = 0xFF;
	for (uint32 ProducerIndex = 0u; ProducerIndex < AllocatedVT->GetNumUniqueProducers(); ++ProducerIndex)
	{
		const TConstArrayView<uint32> LastFramePerMip = Producers.GetLastFramePerMip(AllocatedVT->GetUniqueProducerHandle(ProducerIndex));
		if (LastFramePerMip.IsEmpty())
		{
			continue;
		}
		const uint8 MipBias = AllocatedVT->GetUniqueProducerMipBias(ProducerIndex);
		for (int32 ProducerMip = 0; ProducerMip < LastFramePerMip.Num(); ++ProducerMip)
		{
			if (LastFramePerMip[ProducerMip] != 0u && (CurrentFrame - LastFramePerMip[ProducerMip]) <= FrameWindow)
			{
				const uint8 AllocatedMip = static_cast<uint8>(ProducerMip) + MipBias;
				FinestMip = FMath::Min(FinestMip, AllocatedMip);
				break;
			}
		}
	}
	return FinestMip;
}

void FVirtualTextureSystem::GatherRequestsTask(const FGatherRequestsParameters& Parameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::GatherRequestsTask);
	SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_Gather);

	const FUniquePageList* RESTRICT UniquePageList = Parameters.UniquePageList;
	FUniqueRequestList* RESTRICT RequestList = Parameters.RequestList;
	const uint32 PageStartIndex = Parameters.PageStartIndex;
	const uint32 PageEndIndex = Parameters.PageStartIndex + Parameters.NumPages;
	const bool bForceContinuousUpdate = Parameters.bForceContinuousUpdate;
	const double TimeSliceEnd = Parameters.TimeBudget > 0 ? FPlatformTime::Seconds() + Parameters.TimeBudget : 0;

	uint32 NumRequestsPages = 0u;
	uint32 NumResidentPages = 0u;
	uint32 NumNonResidentPages = 0u;
	uint32 NumPrefetchPages = 0u;

	for (uint32 i = PageStartIndex; (i < PageEndIndex) && (TimeSliceEnd == 0 || FPlatformTime::Seconds() < TimeSliceEnd); ++i)
	{
		const uint32 PageEncoded = UniquePageList->GetPage(i);
		const uint32 PageCount = UniquePageList->GetCount(i);

		// Decode page
		const uint32 ID = (PageEncoded >> 28);
		const FVirtualTextureSpace* RESTRICT Space = GetSpace(ID);
		if (Space == nullptr)
		{
			continue;
		}

		const uint32 vLevelPlusFixedOffset = ((PageEncoded >> 24) & 0x0f);
		const uint32 vLevel = FMath::Max<uint32>(vLevelPlusFixedOffset, UE::HLSL::VirtualTextureFeedbackBias) - UE::HLSL::VirtualTextureFeedbackBias;
		
		// vPageX/Y passed from shader are relative to the given vLevel, we shift them up so be relative to level0
		const uint32 vPageX = (PageEncoded & 0xfff) << vLevel;
		const uint32 vPageY = ((PageEncoded >> 12) & 0xfff) << vLevel;

		const uint32 vAddress = FMath::MortonCode2(vPageX) | (FMath::MortonCode2(vPageY) << 1);
	
		const FAdaptiveVirtualTexture* RESTRICT AdaptiveVT = GetAdaptiveVirtualTexture(ID);
		if (AdaptiveVT != nullptr && vLevelPlusFixedOffset <= UE::HLSL::VirtualTextureFeedbackBias)
		{
			// If vLevelPlusFixedOffset <  VirtualTextureFeedbackBias this is a request for higher resolution adaptive VT.
			// If vLevelPlusFixedOffset == VirtualTextureFeedbackBias then this indicates the current resolution is correct, and so we mark to keep residency.
			// (In future we could consider 2, or even higher, as being enough to keep current residency to reduce adaptive reallocation).
			uint32 AdaptiveAllocationRequest = AdaptiveVT->GetPackedAllocationRequest(vAddress, vLevelPlusFixedOffset, Frame);
			if (AdaptiveAllocationRequest != 0)
			{
				// This is a valid request to pass on to the adaptive VT.
				RequestList->AddAdaptiveAllocationRequest(AdaptiveAllocationRequest);
			}
			if (vLevelPlusFixedOffset < UE::HLSL::VirtualTextureFeedbackBias)
			{
				// The feedback is for a page that won't exist until the adaptive VT is reallocated.
				// So we can skip further processing here.
				continue;
			}
		}

		uint32 PageTableLayersToLoad[VIRTUALTEXTURE_SPACE_MAXLAYERS] = { 0 };
		uint32 NumPageTableLayersToLoad = 0u;
		{
			const FTexturePage VirtualPage(vLevel, vAddress);
			const uint16 VirtualPageHash = MurmurFinalize32(VirtualPage.Packed);
			for (uint32 PageTableLayerIndex = 0u; PageTableLayerIndex < Space->GetNumPageTableLayers(); ++PageTableLayerIndex)
			{
				const FTexturePageMap& RESTRICT PageMap = Space->GetPageMapForPageTableLayer(PageTableLayerIndex);

				++NumRequestsPages;
				const FPhysicalSpaceIDAndAddress PhysicalSpaceIDAndAddress = PageMap.FindPagePhysicalSpaceIDAndAddress(VirtualPage, VirtualPageHash);
				if (PhysicalSpaceIDAndAddress.Packed != ~0u)
				{
#if DO_GUARD_SLOW
					const FVirtualTexturePhysicalSpace* RESTRICT PhysicalSpace = GetPhysicalSpace(PhysicalSpaceIDAndAddress.PhysicalSpaceID);
					checkSlow(PhysicalSpaceIDAndAddress.pAddress < PhysicalSpace->GetNumTiles());
#endif // DO_GUARD_SLOW

					// Page is already resident, just need to update LRU free list
					AddPageUpdate(PhysicalSpaceIDAndAddress.PhysicalSpaceID, PhysicalSpaceIDAndAddress.pAddress, vLevelPlusFixedOffset, Frame);

					// If continuous update flag is set then add this to pages which can be potentially updated if we have spare upload bandwidth
					if (bForceContinuousUpdate || Space->GetDescription().bContinuousUpdate)
					{
						FTexturePagePool& RESTRICT PagePool = GetPhysicalSpace(PhysicalSpaceIDAndAddress.PhysicalSpaceID)->GetPagePool();
						const FVirtualTextureLocalTile LocalTile = PagePool.GetLocalTileFromPhysicalAddress(PhysicalSpaceIDAndAddress.pAddress);
						//todo[vt]: the FVirtualTextureLocalTileRequest produced here doesn't have a ProducerPriority set. Technically it would be possible to retrieve it from the producer 
						// for correctness, but this would require extra indirection so need to profile first
						RequestList->AddContinuousUpdateRequest(FVirtualTextureLocalTileRequest(LocalTile, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal));
					}

					++NumResidentPages;
				}
				else
				{
					// Page not resident, store for later processing
					PageTableLayersToLoad[NumPageTableLayersToLoad++] = PageTableLayerIndex;
				}
			}
		}

		if (NumPageTableLayersToLoad == 0u)
		{
			// All pages are resident and properly mapped, we're done
			// This is the fast path, as most frames should generally have the majority of tiles already mapped
			continue;
		}

		// Need to resolve AllocatedVT in order to determine which pages to load
		const FAllocatedVirtualTexture* RESTRICT AllocatedVT = Space->GetAllocator().Find(vAddress);
		if (!AllocatedVT)
		{
			UE_LOGF(LogVirtualTexturing, Verbose, "Space %i, vAddr %i@%i is not allocated to any AllocatedVT but was still requested.", ID, vAddress, vLevel);
			continue;
		}

		if (AllocatedVT->GetFrameAllocated() > Parameters.FrameRequested)
		{
			// If the VT was allocated after the frame that generated this feedback, it's no longer valid
			continue;
		}

		if (AllocatedVT->bIsWaitingToMap)
		{
			// If the VT is still waiting to map locked pages, don't map other pages first
			continue;
		}

		const uint32 AllocatedAdaptiveBias = AllocatedVT->GetAllocatedAdaptiveBias();
		const uint32 vDimensions = AllocatedVT->GetDimensions();
		const uint32 MaxLevel = AllocatedVT->GetMaxLevel();

		check(AllocatedVT->GetNumPageTableLayers() == Space->GetNumPageTableLayers());
		if (vLevel > MaxLevel)
		{
			// Requested level is outside the given allocated VT
			// This can happen for requests made by expanding mips, since we don't know the current allocated VT in that context
			check(NumPageTableLayersToLoad == Space->GetNumPageTableLayers()); // no pages from this request should have been resident
			check(NumRequestsPages >= Space->GetNumPageTableLayers()); // don't want to track these requests, since it turns out they're not valid
			NumRequestsPages -= Space->GetNumPageTableLayers();
			continue;
		}

		// Build producer local layer masks from physical layers that we need to load
		uint8 ProducerGroupMaskToLoad[VIRTUALTEXTURE_SPACE_MAXLAYERS] = { 0u };
		uint8 ProducerTextureLayerMaskToLoad[VIRTUALTEXTURE_SPACE_MAXLAYERS] = { 0u };

		const uint32 NumUniqueProducers = AllocatedVT->GetNumUniqueProducers();

		for (uint32 LoadPageTableLayerIndex = 0u; LoadPageTableLayerIndex < NumPageTableLayersToLoad; ++LoadPageTableLayerIndex)
		{
			const uint32 PageTableLayerIndex = PageTableLayersToLoad[LoadPageTableLayerIndex];
			const uint32 ProducerIndex = AllocatedVT->GetProducerIndexForPageTableLayer(PageTableLayerIndex);
			check(ProducerIndex < NumUniqueProducers);
			
			const uint32 ProducerTextureLayerMask = AllocatedVT->GetProducerTextureLayerMaskForPageTableLayer(PageTableLayerIndex);
			ProducerTextureLayerMaskToLoad[ProducerIndex] |= ProducerTextureLayerMask;
			
			const uint32 ProducerPhysicalGroupIndex = AllocatedVT->GetProducerPhysicalGroupIndexForPageTableLayer(PageTableLayerIndex);
			ProducerGroupMaskToLoad[ProducerIndex] |= 1 << ProducerPhysicalGroupIndex;
		}

		const uint32 AllocatedPageX = AllocatedVT->GetVirtualPageX();
		const uint32 AllocatedPageY = AllocatedVT->GetVirtualPageY();

		check(vAddress >= AllocatedVT->GetVirtualAddress());
		check(vPageX >= AllocatedPageX);
		check(vPageY >= AllocatedPageY);

		for (uint32 ProducerIndex = 0u; ProducerIndex < NumUniqueProducers; ++ProducerIndex)
		{
			uint8 GroupMaskToLoad = ProducerGroupMaskToLoad[ProducerIndex];
			if (GroupMaskToLoad == 0u)
			{
				continue;
			}

			const FVirtualTextureProducerHandle ProducerHandle = AllocatedVT->GetUniqueProducerHandle(ProducerIndex);
			const FVirtualTextureProducer* RESTRICT Producer = Producers.FindProducer(ProducerHandle);
			if (!Producer)
			{
				continue;
			}

			const uint32 ProducerMipBias = AllocatedVT->GetUniqueProducerMipBias(ProducerIndex);
			const uint32 ProducerMaxLevel = Producer->GetMaxLevel();
			const EVTProducerPriority ProducerPriority = Producer->GetDescription().Priority;

			// Adjust from adaptive space to producer space
			const uint32 vLevel_Producer = vLevel + AllocatedAdaptiveBias;
			uint32 Mapping_vLevel = FMath::Max(vLevel_Producer, ProducerMipBias);

			// Local_vLevel is the level within the producer that we want to allocate/map
			// here we subtract ProducerMipBias, which effectively matches more detailed mips of lower resolution producers with less detailed mips of higher resolution producers
			uint32 Local_vLevel = Mapping_vLevel - ProducerMipBias;

			// We still resolve the X/Y coordinates in the adaptive space.
			uint32 Mapping_AllocLevel = Mapping_vLevel - AllocatedAdaptiveBias;
			const uint32 Local_vPageX = (vPageX - AllocatedPageX) >> Mapping_AllocLevel;
			const uint32 Local_vPageY = (vPageY - AllocatedPageY) >> Mapping_AllocLevel;
			uint32 Local_vAddress = FMath::MortonCode2(Local_vPageX) | (FMath::MortonCode2(Local_vPageY) << 1);

			const uint32 LocalMipBias = Producer->GetVirtualTexture()->GetLocalMipBias(Local_vLevel, Local_vAddress);
			if (LocalMipBias > 0u)
			{
				Local_vLevel += LocalMipBias;
				Mapping_vLevel += LocalMipBias;
				Mapping_AllocLevel += LocalMipBias;
				Local_vAddress >>= (LocalMipBias * vDimensions);
			}

			uint8 ProducerPhysicalGroupMaskToPrefetchForLevel[16] = { 0u };
			uint32 MaxPrefetchLocal_vLevel = Local_vLevel;

			// Iterate local layers that we found unmapped
			for (uint32 ProducerGroupIndex = 0u; ProducerGroupIndex < Producer->GetNumPhysicalGroups(); ++ProducerGroupIndex)
			{
				if ((GroupMaskToLoad & (1u << ProducerGroupIndex)) == 0u)
				{
					continue;
				}

				const FVirtualTexturePhysicalSpace* RESTRICT PhysicalSpace = Producer->GetPhysicalSpaceForPhysicalGroup(ProducerGroupIndex);
				const FTexturePagePool& RESTRICT PagePool = PhysicalSpace->GetPagePool();

				// Find the highest resolution tile that's currently loaded
				const uint32 Allocated_pAddress = PagePool.FindNearestPageAddress(ProducerHandle, ProducerGroupIndex, Local_vAddress, Local_vLevel, ProducerMaxLevel);

				bool bRequestedPageWasResident = false;
				uint32 AllocatedLocal_vLevel = ProducerMaxLevel;
				if (Allocated_pAddress != ~0u)
				{
					AllocatedLocal_vLevel = PagePool.GetLocalLevelForAddress(Allocated_pAddress);
					check(AllocatedLocal_vLevel >= Local_vLevel);

					const uint32 AllocatedMapping_vLevel = AllocatedLocal_vLevel + ProducerMipBias;
					
					// AllocatedMapping_vLevel is in full producer space.
					// For new adaptive virtual textures we need to adjust to the allocated virtual texture space to handle page map operations.
					uint32 Allocated_vLevel = FMath::Clamp((int32)AllocatedMapping_vLevel - (int32)AllocatedAdaptiveBias, 0, (int32)MaxLevel);

					if (AllocatedLocal_vLevel == Local_vLevel)
					{
						// page at the requested level was already resident, no longer need to load
						bRequestedPageWasResident = true;
						// We can map the already resident page at the original requested vLevel
						// This may be different from Allocated_vLevel when various biases are involved
						// Without this, we'll never see anything mapped to the original requested level
						Allocated_vLevel = vLevel;
						GroupMaskToLoad &= ~(1u << ProducerGroupIndex);
						++NumResidentPages;
					}

					ensure(Allocated_vLevel <= MaxLevel);
					const uint32 Allocated_vAddress = vAddress & (0xffffffff << (Allocated_vLevel * vDimensions));

					AddPageUpdate(PhysicalSpace->GetID(), Allocated_pAddress, vLevelPlusFixedOffset, Frame);

					const FPhysicalSpaceIDAndAddress PhysicalSpaceIDAndAddress(PhysicalSpace->GetID(), Allocated_pAddress);
					uint32 NumMappedPages = 0u;
					for (uint32 LoadLayerIndex = 0u; LoadLayerIndex < NumPageTableLayersToLoad; ++LoadLayerIndex)
					{
						const uint32 PageTableLayerIndex = PageTableLayersToLoad[LoadLayerIndex];
						if (AllocatedVT->GetProducerPhysicalGroupIndexForPageTableLayer(PageTableLayerIndex) == ProducerGroupIndex &&
							AllocatedVT->GetProducerIndexForPageTableLayer(PageTableLayerIndex) == ProducerIndex)
						{
							// if we found a lower resolution tile than was requested, it may have already been mapped, check for that first
							const FTexturePageMap& PageMap = Space->GetPageMapForPageTableLayer(PageTableLayerIndex);
							const FPhysicalSpaceIDAndAddress PrevPhysicalSpaceIDAndAddress = PageMap.FindPagePhysicalSpaceIDAndAddress(Allocated_vLevel, Allocated_vAddress);

							// either it wasn't mapped, or it's mapped to the current physical address...
							// otherwise that means that the same local tile is mapped to two separate physical addresses, which is an error
							ensure(PrevPhysicalSpaceIDAndAddress.Packed == ~0u || PrevPhysicalSpaceIDAndAddress.Packed == 0u || PrevPhysicalSpaceIDAndAddress == PhysicalSpaceIDAndAddress);

							if (PrevPhysicalSpaceIDAndAddress.Packed == ~0u)
							{
								// map the page now if it wasn't already mapped
								RequestList->AddDirectMappingRequest(Space->GetID(), PhysicalSpace->GetID(), PageTableLayerIndex, MaxLevel, Allocated_vAddress, Allocated_vLevel, AllocatedMapping_vLevel, Allocated_pAddress);
							}
							++NumMappedPages;
						}
					}
					check(NumMappedPages > 0u);
				}

				if (!bRequestedPageWasResident)
				{
					// page not resident...see if we want to prefetch a page with resolution incrementally larger than what's currently resident
					// this means we'll ultimately load more data, but these lower resolution pages should load much faster than the requested high resolution page
					// this should make popping less noticeable
					uint32 PrefetchLocal_vLevel = AllocatedLocal_vLevel - FMath::Min(2u, AllocatedLocal_vLevel);
					// Take adaptive bias into account when capping to MaxLevel.
					PrefetchLocal_vLevel = FMath::Min<uint32>(PrefetchLocal_vLevel, MaxLevel + AllocatedAdaptiveBias - ProducerMipBias);
					if (PrefetchLocal_vLevel > Local_vLevel)
					{
						ProducerPhysicalGroupMaskToPrefetchForLevel[PrefetchLocal_vLevel] |= (1u << ProducerGroupIndex);
						MaxPrefetchLocal_vLevel = FMath::Max(MaxPrefetchLocal_vLevel, PrefetchLocal_vLevel);
						++NumPrefetchPages;
					}
					++NumNonResidentPages;
				}
			}

			// Check to see if we have any levels to prefetch
			for (uint32 PrefetchLocal_vLevel = Local_vLevel + 1u; PrefetchLocal_vLevel <= MaxPrefetchLocal_vLevel; ++PrefetchLocal_vLevel)
			{
				uint32 ProducerPhysicalGroupMaskToPrefetch = ProducerPhysicalGroupMaskToPrefetchForLevel[PrefetchLocal_vLevel];
				if (ProducerPhysicalGroupMaskToPrefetch != 0u)
				{
					const uint32 PrefetchLocal_vAddress = Local_vAddress >> ((PrefetchLocal_vLevel - Local_vLevel) * vDimensions);

					// If we want to prefetch any layers for a given level, need to ensure that we request all the layers that aren't currently loaded
					// This is required since the VT producer interface needs to be able to write data for all layers if desired, so we need to make sure that all layers are allocated
					for (uint32 ProducerPhysicalGroupIndex = 0u; ProducerPhysicalGroupIndex < Producer->GetNumPhysicalGroups(); ++ProducerPhysicalGroupIndex)
					{
						if ((ProducerPhysicalGroupMaskToPrefetch & (1u << ProducerPhysicalGroupIndex)) == 0u)
						{
							const FVirtualTexturePhysicalSpace* RESTRICT PhysicalSpace = Producer->GetPhysicalSpaceForPhysicalGroup(ProducerPhysicalGroupIndex);
							const FTexturePagePool& RESTRICT PagePool = PhysicalSpace->GetPagePool();
							const uint32 pAddress = PagePool.FindPageAddress(ProducerHandle, ProducerPhysicalGroupIndex, PrefetchLocal_vAddress, PrefetchLocal_vLevel);
							if (pAddress == ~0u)
							{
								ProducerPhysicalGroupMaskToPrefetch |= (1u << ProducerPhysicalGroupIndex);
								++NumPrefetchPages;
							}
							else
							{
								// Need to mark the page as recently used, otherwise it may be evicted later this frame
								AddPageUpdate(PhysicalSpace->GetID(), pAddress, vLevelPlusFixedOffset, Frame);
							}
						}
					}

					if (Parameters.bEnableLoadRequests)
					{
						const bool bStreamingRequest = Producer->GetVirtualTexture()->IsPageStreamed(PrefetchLocal_vLevel, PrefetchLocal_vAddress);
						const FVirtualTextureLocalTile Tile(ProducerHandle, PrefetchLocal_vAddress, PrefetchLocal_vLevel);
						const uint16 LoadRequestIndex = RequestList->AddLoadRequest(
							FVirtualTextureLocalTileRequest(Tile, ProducerPriority, EVTInvalidatePriority::Normal),
							ProducerPhysicalGroupMaskToPrefetch, PageCount, bStreamingRequest);
						if (LoadRequestIndex != 0xffff)
						{
							const uint32 PrefetchMapping_vLevel_Producer = PrefetchLocal_vLevel + ProducerMipBias;
							const uint32 PrefetchMapping_vLevel = PrefetchMapping_vLevel_Producer - AllocatedAdaptiveBias;
							ensure(PrefetchMapping_vLevel <= MaxLevel);
							const uint32 Prefetch_vAddress = vAddress & (0xffffffff << (PrefetchMapping_vLevel * vDimensions));
							for (uint32 LoadLayerIndex = 0u; LoadLayerIndex < NumPageTableLayersToLoad; ++LoadLayerIndex)
							{
								const uint32 LayerIndex = PageTableLayersToLoad[LoadLayerIndex];
								if (AllocatedVT->GetProducerIndexForPageTableLayer(LayerIndex) == ProducerIndex)
								{
									const uint32 ProducerPhysicalGroupIndex = AllocatedVT->GetProducerPhysicalGroupIndexForPageTableLayer(LayerIndex);
									if (ProducerPhysicalGroupMaskToPrefetch & (1u << ProducerPhysicalGroupIndex))
									{
										RequestList->AddMappingRequest(
											LoadRequestIndex, ProducerPhysicalGroupIndex, ID, LayerIndex, MaxLevel, 
											Prefetch_vAddress, PrefetchMapping_vLevel, PrefetchMapping_vLevel_Producer
										);
									}
								}
							}
						}
					}
				}
			}

			if (GroupMaskToLoad != 0u && Parameters.bEnableLoadRequests)
			{
				const bool bStreamingRequest = Producer->GetVirtualTexture()->IsPageStreamed(Local_vLevel, Local_vAddress);
				const FVirtualTextureLocalTile Tile(ProducerHandle, Local_vAddress, Local_vLevel);
				const uint16 LoadRequestIndex = RequestList->AddLoadRequest(
					FVirtualTextureLocalTileRequest(Tile, ProducerPriority, EVTInvalidatePriority::Normal),
					GroupMaskToLoad, PageCount, bStreamingRequest);
				if (LoadRequestIndex != 0xffff)
				{
					for (uint32 LoadLayerIndex = 0u; LoadLayerIndex < NumPageTableLayersToLoad; ++LoadLayerIndex)
					{
						const uint32 LayerIndex = PageTableLayersToLoad[LoadLayerIndex];
						if (AllocatedVT->GetProducerIndexForPageTableLayer(LayerIndex) == ProducerIndex)
						{
							const uint32 ProducerPhysicalGroupIndex = AllocatedVT->GetProducerPhysicalGroupIndexForPageTableLayer(LayerIndex);
							if (GroupMaskToLoad & (1u << ProducerPhysicalGroupIndex))
							{
								RequestList->AddMappingRequest(
									LoadRequestIndex, ProducerPhysicalGroupIndex, ID, LayerIndex, MaxLevel,
									vAddress, vLevel, Mapping_vLevel
								);
							}
						}
					}
				}
			}
		}
	}

	INC_DWORD_STAT_BY(STAT_NumPageVisible, NumRequestsPages);
	INC_DWORD_STAT_BY(STAT_NumPageVisibleResident, NumResidentPages);
	INC_DWORD_STAT_BY(STAT_NumPageVisibleNotResident, NumNonResidentPages);
	INC_DWORD_STAT_BY(STAT_NumPagePrefetch, NumPrefetchPages);
}

void FVirtualTextureSystem::GetContinuousUpdatesToProduce(FUniqueRequestList const* RequestList, int32 MaxTilesToProduce, int32 MaxContinuousUpdates)
{
	const int32 NumContinuousUpdateRequests = (int32)RequestList->GetNumContinuousUpdateRequests();
	
	// Negative maximum continous updates allows for uncapped requests
	if (MaxContinuousUpdates < 0)
	{
		for (int32 i = 0; i < NumContinuousUpdateRequests; ++i)
		{
			AddOrMergeTileRequest(RequestList->GetContinuousUpdateRequest(i), ContinuousUpdateTilesToProduce);
		}
	}
	else
	{
		const int32 MaxContinousUpdates = FMath::Min(MaxContinuousUpdates, NumContinuousUpdateRequests);

		int32 NumContinuousUpdates = 0;
		while (NumContinuousUpdates < MaxContinousUpdates && ContinuousUpdateTilesToProduce.Num() < MaxTilesToProduce)
		{
			// Note it's possible that we add a duplicate value to the TSet here, and so MappedTilesToProduce doesn't grow.
			// But ending up with fewer continuous updates then the maximum is OK.
			int32 RandomIndex = FMath::Rand() % NumContinuousUpdateRequests;
			AddOrMergeTileRequest(RequestList->GetContinuousUpdateRequest(RandomIndex), ContinuousUpdateTilesToProduce);
			NumContinuousUpdates++;
		}
	}
}

void FVirtualTextureSystem::UpdateResidencyTracking() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::UpdateResidencyTracking);
	SCOPE_CYCLE_COUNTER(STAT_ResidencyTracking);

	for (int32 i = 0; i < PhysicalSpaces.Num(); ++i)
	{
		FVirtualTexturePhysicalSpace* PhysicalSpace = PhysicalSpaces[i];
		if (PhysicalSpace)
		{
			PhysicalSpace->UpdateResidencyTracking(Frame);
		}
	}
}

void FVirtualTextureSystem::GrowPhysicalPools() const
{
	if (!VirtualTexturePool::GetPoolAutoGrow())
	{
		return;
	}

	TArray<FVirtualTextureSpacePoolConfig> Configs;
	for (int32 i = 0; i < PhysicalSpaces.Num(); ++i)
	{
		FVirtualTexturePhysicalSpace* PhysicalSpace = PhysicalSpaces[i];
		if (PhysicalSpace && PhysicalSpace->GetLastFrameOversubscribed() == Frame)
		{
			FVirtualTextureSpacePoolConfig& Config = Configs.AddDefaulted_GetRef();
			const FVTPhysicalSpaceDescription& Desc = PhysicalSpace->GetDescription();
			Config.Formats.Append(Desc.Format, Desc.NumLayers);
			Config.MaxTileSize = Config.MinTileSize = Desc.TileSize;

			// Increase pool by 1 tile or 4MB, whichever is greater.
			const int32 TileCount = PhysicalSpace->GetSizeInTiles();
			const int32 TileSizeInBytes = PhysicalSpace->GetTileSizeInBytes();
			const int32 PhysicalPoolCount = PhysicalSpace->GetDescriptionExt().PoolCount;
			const int32 CurrentSizeInBytes = TileCount * TileCount * TileSizeInBytes * PhysicalPoolCount;
			const int32 NextSizeInBytes = (TileCount + 1) * (TileCount + 1) * TileSizeInBytes * PhysicalPoolCount;
			const int32 MinIncreaseInBytes = 4 * 1024 * 1024;
			const int32 ClampedNextSizeInBytes = FMath::Max(NextSizeInBytes, CurrentSizeInBytes + MinIncreaseInBytes);
			Config.SizeInMegabyte = FMath::DivideAndRoundUp(ClampedNextSizeInBytes, 1024 * 1024);
		}
	}

	if (Configs.Num())
	{
		VirtualTexturePool::AddOrModifyTransientPoolConfigs_RenderThread(Configs);
	}
}

TArrayView<FVTLocalTilePriorityAndIndex> FVirtualTextureSystem::SortLocalTileRequests(FConcurrentLinearBulkObjectAllocator& Allocator, const TConstArrayView<FVirtualTextureLocalTileRequest>& InLocalTileRequests)
{
	TArrayView<FVTLocalTilePriorityAndIndex> Result;
	if (InLocalTileRequests.IsEmpty())
	{
		return Result;
	}

	const bool bSortByPriority = CVarVTSortTileRequestsByPriority.GetValueOnRenderThread() != 0;
	FVTLocalTilePriorityAndIndex* SortedKeys = Allocator.CreateArray<FVTLocalTilePriorityAndIndex>(InLocalTileRequests.Num());
	const uint16 NumTiles = IntCastChecked<uint16>(InLocalTileRequests.Num());
	FVirtualTextureProducerHandle CurrentProducerHandle;
	FVirtualTextureProducer const* CurrentProducer = nullptr;
	int32 CurrentKeyIndex = 0;
	for (uint16 TileIndex = 0; TileIndex < NumTiles; ++TileIndex)
	{
		const FVirtualTextureLocalTileRequest& TileRequest = InLocalTileRequests[TileIndex];
		const FVirtualTextureLocalTile Tile = TileRequest.GetTile();
		const FVirtualTextureProducerHandle ProducerHandle = Tile.GetProducerHandle();
		// No need to fetch the producer if we share the same handle as the last tile (which is very likely to happen because we sort by producer): 
		if (CurrentProducerHandle.IsNull() || (ProducerHandle != CurrentProducerHandle))
		{
			CurrentProducerHandle = ProducerHandle;
			CurrentProducer = Producers.FindProducer(ProducerHandle);
		}

		// If we didn't process the tile last frame and deferred processing and if the producer got removed in the meantime we end up with a nullptr producer (just throw away all the requests from this one)
		if (CurrentProducer != nullptr)
		{
			SortedKeys[CurrentKeyIndex++] = FVTLocalTilePriorityAndIndex(
				TileIndex, 
				bSortByPriority ? TileRequest.GetProducerPriority() : static_cast<EVTProducerPriority>(0),
				bSortByPriority ? TileRequest.GetInvalidatePriority() : static_cast<EVTInvalidatePriority>(0),
				/*InMipLevel = */Tile.Local_vLevel);
		}
	}

	if (CurrentKeyIndex > 0)
	{
		Result = MakeArrayView(SortedKeys, CurrentKeyIndex);

		if (bSortByPriority)
		{
			// Sort by decreasing priority : 
			Algo::Sort(Result);
		}
	}
	return Result;
}

int32 FVirtualTextureSystem::SubmitRequestsFromLocalTileRequests(FConcurrentLinearBulkObjectAllocator& Allocator, FRHICommandList& RHICmdList, TSet<FVirtualTextureLocalTileRequest>& OutDeferredTileRequests, const TSet<FVirtualTextureLocalTileRequest>& LocalTileRequests, EVTProducePageFlags Flags, ERHIFeatureLevel::Type FeatureLevel, uint32 MaxRequestsToProduce)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::SubmitRequestsFromLocalTileList);
	LLM_SCOPE(ELLMTag::VirtualTextureSystem);

	uint32 NumPagesProduced = 0;

	const TArray<FVirtualTextureLocalTileRequest> LocalTileRequestsArray = LocalTileRequests.Array();
	TArrayView<FVTLocalTilePriorityAndIndex> SortedKeys = SortLocalTileRequests(Allocator, LocalTileRequestsArray);
	FVirtualTextureProducerHandle CurrentProducerHandle;
	FVirtualTextureProducer const* CurrentProducer = nullptr;
	for (const FVTLocalTilePriorityAndIndex& SortedTileKeyIndex : SortedKeys)
	{
		const FVirtualTextureLocalTileRequest& TileRequest = LocalTileRequestsArray[SortedTileKeyIndex.Index];
		const FVirtualTextureLocalTile Tile = TileRequest.GetTile();
		const FVirtualTextureProducerHandle ProducerHandle = Tile.GetProducerHandle();
		// No need to fetch the producer if we share the same handle as the last tile (which is very likely to happen because we sort by producer): 
		if (CurrentProducerHandle.IsNull() || (ProducerHandle != CurrentProducerHandle))
		{
			CurrentProducer = Producers.FindProducer(ProducerHandle);
		}

		// Fill targets for each layer
		// Each producer can have multiple physical layers
		// If the phys layer is mapped then we get the textures it owns and map them into the producer local slots and set the flags
		uint32 LayerMask = 0;
		FVTProduceTargetLayer ProduceTarget[VIRTUALTEXTURE_SPACE_MAXLAYERS];
		for (uint32 ProducerPhysicalGroupIndex = 0u; ProducerPhysicalGroupIndex < CurrentProducer->GetNumPhysicalGroups(); ++ProducerPhysicalGroupIndex)
		{
			FVirtualTexturePhysicalSpace* RESTRICT PhysicalSpace = CurrentProducer->GetPhysicalSpaceForPhysicalGroup(ProducerPhysicalGroupIndex);
			FTexturePagePool& RESTRICT PagePool = PhysicalSpace->GetPagePool();
			const uint32 pAddress = PagePool.FindPageAddress(ProducerHandle, ProducerPhysicalGroupIndex, Tile.Local_vAddress, Tile.Local_vLevel);
			if (pAddress != ~0u)
			{
				int32 PhysicalLocalTextureIndex = 0;
				for (uint32 ProducerLayerIndex = 0u; ProducerLayerIndex < CurrentProducer->GetNumTextureLayers(); ++ProducerLayerIndex)
				{
					if (CurrentProducer->GetPhysicalGroupIndexForTextureLayer(ProducerLayerIndex) == ProducerPhysicalGroupIndex)
					{
						ProduceTarget[ProducerLayerIndex].PooledRenderTarget = PhysicalSpace->GetPhysicalTexture(PhysicalLocalTextureIndex);
						ProduceTarget[ProducerLayerIndex].pPageLocation = PhysicalSpace->GetPhysicalLocation(pAddress);
						LayerMask |= 1 << ProducerLayerIndex;
						PhysicalLocalTextureIndex++;
					}
				}
			}
		}

		if (LayerMask == 0)
		{
			// If we don't have anything mapped then we can ignore (since we only want to refresh existing mapped data)
			continue;
		}

		if (MaxRequestsToProduce > 0 && NumPagesProduced >= MaxRequestsToProduce)
		{
			// Keep the request for the next frame?
			AddOrMergeTileRequest(TileRequest, OutDeferredTileRequests);
			continue;
		}

		FVTRequestPageResult RequestPageResult = CurrentProducer->GetVirtualTexture()->RequestPageData(
			RHICmdList, ProducerHandle, LayerMask, Tile.Local_vLevel, Tile.Local_vAddress, EVTRequestPagePriority::High);

		if (RequestPageResult.Status != EVTRequestPageStatus::Available)
		{
			// Keep the request for the next frame?
			AddOrMergeTileRequest(TileRequest, OutDeferredTileRequests);
			continue;
		}

		IVirtualTextureFinalizer* VTFinalizer = CurrentProducer->GetVirtualTexture()->ProducePageData(
			RHICmdList, FeatureLevel,
			Flags,
			ProducerHandle, LayerMask, Tile.Local_vLevel, Tile.Local_vAddress,
			RequestPageResult.Handle,
			ProduceTarget);

		if (VTFinalizer != nullptr)
		{
			// Add the finalizer here but note that we don't call Finalize until SubmitRequests()
			Finalizers.AddUnique(VTFinalizer);
		}

		NumPagesProduced++;
	}

	return NumPagesProduced;
}

void FVirtualTextureSystem::SubmitThrottledRequests(FRHICommandList& RHICmdList, FVirtualTextureUpdater* Updater, EUpdatePhase UpdatePhase)
{
	const FVirtualTextureUpdateSettings& Settings   = Updater->Settings;
	const ERHIFeatureLevel::Type  FeatureLevel      = Updater->FeatureLevel;
	FConcurrentLinearBulkObjectAllocator& Allocator = Updater->Allocator;
	FUniqueRequestList* MergedRequestList           = Updater->MergedRequestList;
	const bool bSortByPriority = CVarVTSortTileRequestsByPriority.GetValueOnRenderThread() != 0;

	if (MergedRequestList->GetNumAdaptiveAllocationRequests() > 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_QueueAdaptiveRequests);
		FAdaptiveVirtualTexture::QueuePackedAllocationRequests(this, MakeConstArrayView(&MergedRequestList->GetAdaptiveAllocationRequest(0), MergedRequestList->GetNumAdaptiveAllocationRequests()), Frame);
	}

	// Deal with dirty MappedTilesToProduce pages first.
	if (UpdatePhase == EUpdatePhase::Begin)
	{
		// Only take runtime generated page budget into account since we expect that only runtime generated pages should have been dirtied.
		int32 NumPagesProduced = SubmitRequestsFromLocalTileRequests(Allocator, RHICmdList, TransientCollectedTilesToProduce, MappedTilesToProduce, EVTProducePageFlags::None, FeatureLevel, Updater->PageUploadBudgetRVT);
		Updater->PageUploadBudgetRVT = FMath::Max(Updater->PageUploadBudgetRVT - NumPagesProduced, 0);

		INC_DWORD_STAT_BY(STAT_NumMappedPageUpdate, MappedTilesToProduce.Num());
		MappedTilesToProduce.Reset();
		MappedTilesToProduce.Append(TransientCollectedTilesToProduce);
		TransientCollectedTilesToProduce.Reset();
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_Sort);
		
		// Throttle the total number of requests during sort according to the remaining budget.
		// Note that we use distinct budgets for streaming and runtime generated pages, since they have difference performance characteristics.
		// We may want a higher budget for streaming pages so that I/O can be initiated as early as possible.
		
		// SVT budget of 0 is a special value that enables the old behavior where all pages are limited by a single budget.
		const bool bUseCombinedLimit = Settings.MaxSVTPageUploads == 0;
		
		MergedRequestList->SortRequests(Producers, Allocator, Updater->PageUploadBudgetRVT, Updater->PageUploadBudgetSVT, bUseCombinedLimit, bSortByPriority);
		
		// Subtract sorted request count from the remaining budgets.
		const int32 NumRequestsRVT = (int32)MergedRequestList->GetNumNonStreamingLoadRequests();
		const int32 NumRequestsSVT = (int32)MergedRequestList->GetNumLoadRequests() - (int32)MergedRequestList->GetNumNonStreamingLoadRequests();
		check(NumRequestsSVT >= 0);
		Updater->PageUploadBudgetRVT = FMath::Max(Updater->PageUploadBudgetRVT - NumRequestsRVT, 0);
		Updater->PageUploadBudgetSVT = FMath::Max(Updater->PageUploadBudgetSVT - NumRequestsSVT, 0);
	}

	// If we have any remaining page budget then use it to add continuous updates.
	if (UpdatePhase == EUpdatePhase::End)
	{
		// Don't take streaming page budget into account since they async load and likely won't be produced this frame.
		// Also we expect only runtime generated pages to need continous updates (since continuous updates are there to handle stale pages coming from rendering before ready etc).
		GetContinuousUpdatesToProduce(MergedRequestList, Updater->PageUploadBudgetRVT, Settings.MaxContinuousUpdates);
		int32 NumPagesProduced = SubmitRequestsFromLocalTileRequests(Allocator, RHICmdList, TransientCollectedTilesToProduce, ContinuousUpdateTilesToProduce, EVTProducePageFlags::ContinuousUpdate, FeatureLevel, 0);
		Updater->PageUploadBudgetRVT = FMath::Max(Updater->PageUploadBudgetRVT - NumPagesProduced, 0);

		INC_DWORD_STAT_BY(STAT_NumContinuousPageUpdate, ContinuousUpdateTilesToProduce.Num());
		ContinuousUpdateTilesToProduce.Reset();
		TransientCollectedTilesToProduce.Reset();
	}

	// Submit the merged, sorted and throttled page load requests.
	SubmitRequests(RHICmdList, FeatureLevel, Allocator, Settings, MergedRequestList);
}

// Global flag used to request early out of work inside SubmitRequests.
volatile bool GAsyncTaskEarlyOut = false;

void FVirtualTextureSystem::SubmitRequests(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FConcurrentLinearBulkObjectAllocator& Allocator, FVirtualTextureUpdateSettings const& Settings, FUniqueRequestList* RequestList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::SubmitRequests);
	LLM_SCOPE(ELLMTag::VirtualTextureSystem);

	// Allocate space to hold the physical address we allocate for each page load (1 page per layer per request)
	uint32* RequestPhysicalAddress = Allocator.MallocAndMemsetArray<uint32>(RequestList->GetNumLoadRequests() * VIRTUALTEXTURE_SPACE_MAXLAYERS, 0xFF);

	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_Submit);

		struct FProducePageDataPrepareTask
		{
			IVirtualTexture* VirtualTexture;
			EVTProducePageFlags Flags;
			FVirtualTextureProducerHandle ProducerHandle;
			uint8 LayerMask;
			uint8 vLevel;
			uint32 vAddress;
			uint64 RequestHandle;
			FVTProduceTargetLayer ProduceTarget[VIRTUALTEXTURE_SPACE_MAXLAYERS];
		};

		TArray<FProducePageDataPrepareTask> PrepareTasks;
		PrepareTasks.Reserve(RequestList->GetNumLoadRequests());

		bool bWaitForProducers = false;

		const uint32 MaxPagesProduced = Settings.MaxPagesProduced;
		const uint32 PageFreeThreshold = VirtualTextureScalability::GetPageFreeThreshold();
		const double StartTime = FPlatformTime::Seconds();
		const double TimeSliceEarlyOut = Settings.MinSubmitAsyncEarlyOutTime > 0 ? StartTime + Settings.MinSubmitAsyncEarlyOutTime : 0;
		const double TimeSliceEnd = Settings.MaxSubmitTime > 0 ? StartTime + Settings.MaxSubmitTime : 0;
		
		uint32 NumPagesPending = 0u;
		uint32 NumPagesThrottled = 0u;
		uint32 NumPagesInvalid = 0u;
		uint32 NumPagesSaturated = 0u;
		uint32 NumPagesProduced = 0u;
		uint32 NumPagesFailAllocate = 0u;
		uint32 NumStacksProduced = 0u;

		for (uint32 RequestIndex = 0u; RequestIndex < RequestList->GetNumLoadRequests(); ++RequestIndex)
		{
			const bool bLockTile = RequestList->IsLocked(RequestIndex);
			const bool bForceProduceTile = Settings.bSyncProduceAllTiles || (bLockTile && Settings.bSyncProduceLockedTiles);

			// Always process locked tiles but skip work if we have run past our time slice or have been asked to early out.
			const double CurrentTime = FPlatformTime::Seconds();
			if (!bLockTile && ((GAsyncTaskEarlyOut && TimeSliceEarlyOut > 0 && CurrentTime > TimeSliceEarlyOut) || (TimeSliceEnd > 0 && CurrentTime > TimeSliceEnd)))
			{
				NumPagesThrottled++;
				continue;
			}
			
			const FVirtualTextureLocalTile TileToLoad = RequestList->GetLoadRequest(RequestIndex).GetTile();
			const FVirtualTextureProducerHandle ProducerHandle = TileToLoad.GetProducerHandle();
			const FVirtualTextureProducer& Producer = Producers.GetProducer(ProducerHandle);

			const uint32 ProducerPhysicalGroupMask = RequestList->GetGroupMask(RequestIndex);
			uint32 ProducerTextureLayerMask = 0;
			for (uint32 ProducerLayerIndex = 0; ProducerLayerIndex < Producer.GetNumTextureLayers(); ++ProducerLayerIndex)
			{
				if (ProducerPhysicalGroupMask & (1 << Producer.GetPhysicalGroupIndexForTextureLayer(ProducerLayerIndex)))
				{
					ProducerTextureLayerMask |= (1 << ProducerLayerIndex);
				}
			}

			const EVTRequestPagePriority Priority = bLockTile || Settings.bSyncProduceAllTiles ? EVTRequestPagePriority::High : EVTRequestPagePriority::Normal;
			FVTRequestPageResult RequestPageResult = Producer.GetVirtualTexture()->RequestPageData(RHICmdList, ProducerHandle, ProducerTextureLayerMask, TileToLoad.Local_vLevel, TileToLoad.Local_vAddress, Priority);
			if (RequestPageResult.Status == EVTRequestPageStatus::Pending)
			{
				if (bForceProduceTile)
				{
					// If we're forcing production of this tile, we're OK producing data now (and possibly waiting) as long as data is pending
					RequestPageResult.Status = EVTRequestPageStatus::Available;
					bWaitForProducers = true;
				}
				else
				{
					NumPagesPending++;
				}
			}

			if (RequestPageResult.Status == EVTRequestPageStatus::Available && !bForceProduceTile && NumPagesProduced >= MaxPagesProduced)
			{
				// Don't produce non-locked pages yet, if we're over our limit
				RequestPageResult.Status = EVTRequestPageStatus::Pending;
				NumPagesThrottled++;
			}

			bool bTileLoaded = false;
			bool bTileInvalid = false;
			if (RequestPageResult.Status == EVTRequestPageStatus::Invalid)
			{
				bTileInvalid = true;
				NumPagesInvalid++;
				UE_LOGF(LogVirtualTexturing, Verbose, "vAddr %i@%i is not a valid request for AllocatedVT but is still requested.", TileToLoad.Local_vAddress, TileToLoad.Local_vLevel);
			}
			else if (RequestPageResult.Status == EVTRequestPageStatus::Saturated)
			{
				NumPagesSaturated++;
			}
			else if (RequestPageResult.Status == EVTRequestPageStatus::Available)
			{
				FVTProduceTargetLayer ProduceTarget[VIRTUALTEXTURE_SPACE_MAXLAYERS];
				uint32 Allocate_pAddress[VIRTUALTEXTURE_SPACE_MAXLAYERS];
				FMemory::Memset(Allocate_pAddress, 0xff);

				// try to allocate a page for each layer we need to load
				bool bProduceTargetValid = true;
				for (uint32 ProducerPhysicalGroupIndex = 0u; ProducerPhysicalGroupIndex < Producer.GetNumPhysicalGroups(); ++ProducerPhysicalGroupIndex)
				{
					// If mask isn't set, we must already have a physical tile allocated for this layer, don't need to allocate another one
					if (ProducerPhysicalGroupMask & (1u << ProducerPhysicalGroupIndex))
					{
						FVirtualTexturePhysicalSpace* RESTRICT PhysicalSpace = Producer.GetPhysicalSpaceForPhysicalGroup(ProducerPhysicalGroupIndex);
						FTexturePagePool& RESTRICT PagePool = PhysicalSpace->GetPagePool();
						if (PagePool.AnyFreeAvailable(Frame, bLockTile ? 0 : PageFreeThreshold))
						{
							const uint32 pAddress = PagePool.Alloc(this, Frame, ProducerHandle, ProducerPhysicalGroupIndex, TileToLoad.Local_vAddress, TileToLoad.Local_vLevel, bLockTile);
							check(pAddress != ~0u);

							int32 PhysicalLocalTextureIndex = 0;
							for (uint32 ProducerLayerIndex = 0u; ProducerLayerIndex < Producer.GetNumTextureLayers(); ++ProducerLayerIndex)
							{
								if (Producer.GetPhysicalGroupIndexForTextureLayer(ProducerLayerIndex) == ProducerPhysicalGroupIndex)
								{
									ProduceTarget[ProducerLayerIndex].PooledRenderTarget = PhysicalSpace->GetPhysicalTexture(PhysicalLocalTextureIndex);
									ProduceTarget[ProducerLayerIndex].pPageLocation = PhysicalSpace->GetPhysicalLocation(pAddress);
									PhysicalLocalTextureIndex++;

									Allocate_pAddress[ProducerPhysicalGroupIndex] = pAddress;
								}
							}

							++NumPagesProduced;
						}
						else
						{
							bProduceTargetValid = false;
							NumPagesFailAllocate++;
							break;
						}
					}
				}

				if (bProduceTargetValid)
				{
					// Successfully allocated required pages, now we can make the request
					for (uint32 ProducerPhysicalGroupIndex = 0u; ProducerPhysicalGroupIndex < Producer.GetNumPhysicalGroups(); ++ProducerPhysicalGroupIndex)
					{
						if (ProducerPhysicalGroupMask & (1u << ProducerPhysicalGroupIndex))
						{
							// Associate the addresses we allocated with this request, so they can be mapped if required
							const uint32 pAddress = Allocate_pAddress[ProducerPhysicalGroupIndex];
							check(pAddress != ~0u);
							RequestPhysicalAddress[RequestIndex * VIRTUALTEXTURE_SPACE_MAXLAYERS + ProducerPhysicalGroupIndex] = pAddress;
						}
						else
						{
							// Fill in pAddress for layers that are already resident
							const FVirtualTexturePhysicalSpace* RESTRICT PhysicalSpace = Producer.GetPhysicalSpaceForPhysicalGroup(ProducerPhysicalGroupIndex);
							const FTexturePagePool& RESTRICT PagePool = PhysicalSpace->GetPagePool();
							const uint32 pAddress = PagePool.FindPageAddress(ProducerHandle, ProducerPhysicalGroupIndex, TileToLoad.Local_vAddress, TileToLoad.Local_vLevel);
							checkf(pAddress != ~0u,
								TEXT("%s missing tile: LayerMask: %X, Layer %d, vAddress %06X, vLevel %d"),
								*Producer.GetName().ToString(), ProducerPhysicalGroupMask, ProducerPhysicalGroupIndex, TileToLoad.Local_vAddress, TileToLoad.Local_vLevel);
							
							int32 PhysicalLocalTextureIndex = 0;
							for (uint32 ProducerLayerIndex = 0u; ProducerLayerIndex < Producer.GetNumTextureLayers(); ++ProducerLayerIndex)
							{
								if (Producer.GetPhysicalGroupIndexForTextureLayer(ProducerLayerIndex) == ProducerPhysicalGroupIndex)
								{
									ProduceTarget[ProducerLayerIndex].PooledRenderTarget = PhysicalSpace->GetPhysicalTexture(PhysicalLocalTextureIndex);
									ProduceTarget[ProducerLayerIndex].pPageLocation = PhysicalSpace->GetPhysicalLocation(pAddress);
									PhysicalLocalTextureIndex++;
								}
							}
						}
					}

					{
						FProducePageDataPrepareTask& Task = PrepareTasks.AddDefaulted_GetRef();
						Task.VirtualTexture = Producer.GetVirtualTexture();
						Task.Flags = EVTProducePageFlags::None;
						Task.ProducerHandle = ProducerHandle;
						Task.LayerMask = ProducerTextureLayerMask;
						Task.vLevel = TileToLoad.Local_vLevel;
						Task.vAddress = TileToLoad.Local_vAddress;
						Task.RequestHandle = RequestPageResult.Handle;
						FMemory::Memcpy(Task.ProduceTarget, ProduceTarget, sizeof(ProduceTarget));
					}

					bTileLoaded = true;
					++NumStacksProduced;
				}
				else
				{
					// Failed to allocate required physical pages for the tile, free any pages we did manage to allocate
					for (uint32 ProducerPhysicalGroupIndex = 0u; ProducerPhysicalGroupIndex < Producer.GetNumPhysicalGroups(); ++ProducerPhysicalGroupIndex)
					{
						const uint32 pAddress = Allocate_pAddress[ProducerPhysicalGroupIndex];
						if (pAddress != ~0u)
						{
							FVirtualTexturePhysicalSpace* RESTRICT PhysicalSpace = Producer.GetPhysicalSpaceForPhysicalGroup(ProducerPhysicalGroupIndex);
							FTexturePagePool& RESTRICT PagePool = PhysicalSpace->GetPagePool();
							PagePool.Free(this, pAddress);
						}
					}
				}
			}

			if (bLockTile && !bTileLoaded && !bTileInvalid)
			{
				// Want to lock this tile, but didn't manage to load it this frame, add it back to the list to try the lock again next frame
				TilesToLockForNextFrame.Add(TileToLoad);
			}
		}

		if (PrepareTasks.Num())
		{
			static bool bWaitForTasks = true;
			if (bWaitForProducers && bWaitForTasks)
			{
				// Wait for all producers here instead of inside each individual call to ProducePageData()
				FGraphEventArray ProducePageTasks;
				ProducePageTasks.Reserve(PrepareTasks.Num());

				for (FProducePageDataPrepareTask& Task : PrepareTasks)
				{
					if (Task.RequestHandle != 0)
					{
						Task.VirtualTexture->GatherProducePageDataTasks(Task.RequestHandle, ProducePageTasks);
					}
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::ProcessRequests_Wait);
					QUICK_SCOPE_CYCLE_COUNTER(ProcessRequests_Wait);
					FTaskGraphInterface::Get().WaitUntilTasksComplete(ProducePageTasks);
				}
			}

			for (FProducePageDataPrepareTask& Task : PrepareTasks)
			{
				IVirtualTextureFinalizer* VTFinalizer = Task.VirtualTexture->ProducePageData(RHICmdList, FeatureLevel,
					Task.Flags,
					Task.ProducerHandle, Task.LayerMask, Task.vLevel, Task.vAddress,
					Task.RequestHandle,
					Task.ProduceTarget);

				if (VTFinalizer)
				{
					Finalizers.AddUnique(VTFinalizer); // we expect the number of unique finalizers to be very limited. if this changes, we might have to do something better then gathering them every update
				}
			}
		}

		INC_DWORD_STAT_BY(STAT_NumStacksRequested, RequestList->GetNumLoadRequests());
		INC_DWORD_STAT_BY(STAT_NumStacksProduced, NumStacksProduced);
		INC_DWORD_STAT_BY(STAT_NumPagesProduced, NumPagesProduced);
		INC_DWORD_STAT_BY(STAT_NumPagesPending, NumPagesPending);
		INC_DWORD_STAT_BY(STAT_NumPagesThrottled, NumPagesThrottled);
		INC_DWORD_STAT_BY(STAT_NumPagesSaturated, NumPagesSaturated);
		INC_DWORD_STAT_BY(STAT_NumPagesInvalid, NumPagesInvalid);
		INC_DWORD_STAT_BY(STAT_NumPagesFailAllocate, NumPagesFailAllocate);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_Map);

		// Update page mappings that were directly requested
		for (uint32 RequestIndex = 0u; RequestIndex < RequestList->GetNumDirectMappingRequests(); ++RequestIndex)
		{
			const FDirectMappingRequest MappingRequest = RequestList->GetDirectMappingRequest(RequestIndex);
			FVirtualTextureSpace* Space = GetSpace(MappingRequest.SpaceID);
			FVirtualTexturePhysicalSpace* PhysicalSpace = GetPhysicalSpace(MappingRequest.PhysicalSpaceID);

			PhysicalSpace->GetPagePool().MapPage(Space, PhysicalSpace, MappingRequest.PageTableLayerIndex, MappingRequest.MaxLevel, MappingRequest.vLevel, MappingRequest.vAddress, MappingRequest.Local_vLevel, MappingRequest.pAddress);
		}

		// Update page mappings for any requested page that completed allocation this frame
		for (uint32 RequestIndex = 0u; RequestIndex < RequestList->GetNumMappingRequests(); ++RequestIndex)
		{
			const FMappingRequest MappingRequest = RequestList->GetMappingRequest(RequestIndex);
			const uint32 pAddress = RequestPhysicalAddress[MappingRequest.LoadRequestIndex * VIRTUALTEXTURE_SPACE_MAXLAYERS + MappingRequest.ProducerPhysicalGroupIndex];
			if (pAddress != ~0u)
			{
				const FVirtualTextureLocalTile TileToLoad = RequestList->GetLoadRequest(MappingRequest.LoadRequestIndex).GetTile();
				const FVirtualTextureProducerHandle ProducerHandle = TileToLoad.GetProducerHandle();
				FVirtualTextureProducer& Producer = Producers.GetProducer(ProducerHandle);
				FVirtualTexturePhysicalSpace* PhysicalSpace = Producer.GetPhysicalSpaceForPhysicalGroup(MappingRequest.ProducerPhysicalGroupIndex);
				FVirtualTextureSpace* Space = GetSpace(MappingRequest.SpaceID);
				check(RequestList->GetGroupMask(MappingRequest.LoadRequestIndex) & (1u << MappingRequest.ProducerPhysicalGroupIndex));

				PhysicalSpace->GetPagePool().MapPage(Space, PhysicalSpace, MappingRequest.PageTableLayerIndex, MappingRequest.MaxLevel, MappingRequest.vLevel, MappingRequest.vAddress, MappingRequest.Local_vLevel, pAddress);
			}
		}
	}

	// Map any resident tiles to newly allocated VTs
	if(AllocatedVTsToMap.Num() > 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_MapNew);

		uint32 Index = 0u;
		while (Index < (uint32)AllocatedVTsToMap.Num())
		{
			IAllocatedVirtualTexture* AllocatedVT = AllocatedVTsToMap[Index];
			if (AllocatedVT->TryMapLockedTiles(this))
			{
				AllocatedVTsToMap.RemoveAtSwap(Index, EAllowShrinking::No);
				AllocatedVT->bIsWaitingToMap = false;
			}
			else
			{
				Index++;
			}
		}

		AllocatedVTsToMap.Shrink();
	}
}

void FVirtualTextureSystem::FinalizeRequests(FRDGBuilder& GraphBuilder, ISceneRenderer* SceneRenderer)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, VirtualTextureSystem_Update);
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::FinalizeRequests);

	FRDGExternalAccessQueue ExternalAccessQueue;

	// Finalize requests
	if (Finalizers.Num())
	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_Finalize);
		RDG_EVENT_SCOPE_STAT(GraphBuilder, VirtualTexture, "VirtualTextureFinalizeRequests");
		TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::Finalize);

		for (IVirtualTextureFinalizer* VTFinalizer : Finalizers)
		{
			VTFinalizer->RenderFinalize(GraphBuilder, SceneRenderer);
		}

		for (IVirtualTextureFinalizer* VTFinalizer : Finalizers)
		{
			VTFinalizer->Finalize(GraphBuilder);
		}
		
		Finalizers.Reset();

		// Transition any touched physical textures
		for (FVirtualTexturePhysicalSpace* PhysicalSpace : PhysicalSpaces)
		{
			if (PhysicalSpace != nullptr)
			{
				PhysicalSpace->FinalizeTextures(GraphBuilder, ExternalAccessQueue);
			}
		}
	}

	// Update page tables
	{
		SCOPE_CYCLE_COUNTER(STAT_PageTableUpdates);
		RDG_EVENT_SCOPE_STAT(GraphBuilder, VirtualTexture, "VirtualTexturePageTableUpdates");
		TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::ApplyUpdates);
		RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

		TArray<FVirtualTextureSpace::FUpdateContext, TInlineAllocator<16>> UpdateContexts;
		for (TUniquePtr<FVirtualTextureSpace>& Space : Spaces)
		{
			FVirtualTextureSpace::FUpdateContext Context;
			if (Space.IsValid() && Space->BeginUpdates(this, GraphBuilder, Context))
			{
				UpdateContexts.Add(Context);
			}
		}
		for (FVirtualTextureSpace::FUpdateContext const& Context : UpdateContexts)
		{
			Context.Space->ApplyUpdates(GraphBuilder, Context);
		}
		for (FVirtualTextureSpace::FUpdateContext const& Context : UpdateContexts)
		{
			Context.Space->EndUpdates(GraphBuilder, Context, ExternalAccessQueue);
		}

		for (TUniquePtr<FVirtualTextureSpace>& Space : Spaces)
		{
			if (Space.IsValid())
			{
				if (FAdaptiveVirtualTexture* AdaptiveVT = GetAdaptiveVirtualTexture(Space->GetID()))
				{
					AdaptiveVT->ApplyPageTableUpdates(this, GraphBuilder, ExternalAccessQueue);
				}
			}
		}
	}

	ExternalAccessQueue.Submit(GraphBuilder);

	Producers.NotifyRequestsCompleted();

	Frame++;
}

void FVirtualTextureSystem::AllocateResources(FRDGBuilder& GraphBuilder)
{
	LLM_SCOPE(ELLMTag::VirtualTextureSystem);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, VirtualTextureAllocate, "VirtualTextureAllocate");
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, VirtualTextureSystem_Update);

	for (int32 ID = 0; ID < Spaces.Num(); ID++)
	{
		if (Spaces[ID])
		{
			Spaces[ID]->AllocateTextures(GraphBuilder);
		}
	}
}

void FVirtualTextureSystem::GatherFeedbackRequests(FConcurrentLinearBulkObjectAllocator& Allocator, const FVirtualTextureUpdateSettings& Settings, const FVirtualTextureFeedback::FMapResult& FeedbackResult, FUniqueRequestList* MergedRequestList)
{
	FUniquePageList* MergedUniquePageList = Allocator.Create<FUniquePageList>();
	MergedUniquePageList->Initialize();

	if (Settings.bEnableFeedback)
	{
		FFeedbackAnalysisParameters Parameters;
		Parameters.FeedbackBuffer = FeedbackResult.Data;
		Parameters.UniquePageList = MergedUniquePageList;
		Parameters.FeedbackSize = FeedbackResult.Size;

		FeedbackAnalysisTask(Parameters);
	}

#if WITH_EDITOR
	// If we're are recording page requests, then copy off pages to the recording buffer.
	if (PageRequestRecordHandle != ~0ull)
	{
		RecordPageRequests(MergedUniquePageList, PageRequestRecordBuffer);
	}
#endif

	// Add any page requests from recording playback.
	if (PageRequestPlaybackBuffer.Num() > 0)
	{
		if (Settings.bEnablePlayback)
		{
			float LevelBias = CVarVTPlaybackMipBias.GetValueOnRenderThread();
			LevelBias += UTexture2D::GetGlobalMipMapLODBias();
			if (Settings.PlaybackViewSize.X > 0 && Settings.PlaybackViewSize.Y > 0)
			{
				LevelBias += FMath::Log2(FMath::Max3((float)PageRequestCaptureResolution.X / (float)Settings.PlaybackViewSize.X, (float)PageRequestCaptureResolution.Y / (float)Settings.PlaybackViewSize.Y, 1.f));
			}

			FAddRequestedTilesParameters Parameters;
			Parameters.ViewLevelBias = FMath::RoundToInt(FMath::Max(LevelBias, 0.f));
			Parameters.RequestBuffer = PageRequestPlaybackBuffer.GetData();
			Parameters.NumRequests = PageRequestPlaybackBuffer.Num();
			Parameters.UniquePageList = Allocator.Create<FUniquePageList>();
			Parameters.UniquePageList->Initialize();

			AddRequestedTilesTask(Parameters);
			MergedUniquePageList->MergePages(Parameters.UniquePageList);
		}

		PageRequestPlaybackBuffer.Reset(0);
		PageRequestCaptureResolution = FIntPoint::ZeroValue;
	}

	// If we are time slicing GatherRequests then sort the pages to ensure that we only skip lower priority work.
	if (Settings.MaxGatherTime > 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_Sort);
		MergedUniquePageList->SortPages();
	}

	// Pages from feedback buffer were generated several frames ago, so they may no longer be valid for newly allocated VTs
	static uint32 PendingFrameDelay = 3u;
	if (Frame >= PendingFrameDelay)
	{
		GatherRequests(MergedRequestList, MergedUniquePageList, Frame - PendingFrameDelay, Allocator, Settings);
	}
}

void FVirtualTextureSystem::GatherLockedTileRequests(FUniqueRequestList* MergedRequestList)
{
	for (const FVirtualTextureLocalTile& Tile : TilesToLock)
	{
		const FVirtualTextureProducerHandle ProducerHandle = Tile.GetProducerHandle();
		const FVirtualTextureProducer* Producer = Producers.FindProducer(ProducerHandle);
		checkSlow(TileLocks.IsLocked(Tile));
		if (Producer)
		{
			uint8 ProducerLayerMaskToLoad = 0u;
			for (uint32 ProducerLayerIndex = 0u; ProducerLayerIndex < Producer->GetNumTextureLayers(); ++ProducerLayerIndex)
			{
				uint32 GroupIndex = Producer->GetPhysicalGroupIndexForTextureLayer(ProducerLayerIndex);
				FVirtualTexturePhysicalSpace* PhysicalSpace = Producer->GetPhysicalSpaceForPhysicalGroup(GroupIndex);
				FTexturePagePool& PagePool = PhysicalSpace->GetPagePool();
				const uint32 pAddress = PagePool.FindPageAddress(ProducerHandle, GroupIndex, Tile.Local_vAddress, Tile.Local_vLevel);
				if (pAddress == ~0u)
				{
					ProducerLayerMaskToLoad |= (1u << ProducerLayerIndex);
				}
				else
				{
					PagePool.Lock(pAddress);
				}
			}

			if (ProducerLayerMaskToLoad != 0u)
			{
				const bool bStreamingRequest = Producer->GetVirtualTexture()->IsPageStreamed(Tile.Local_vLevel, Tile.Local_vAddress);
				EVTProducerPriority ProducerPriority = Producer->GetDescription().Priority;
				FVirtualTextureLocalTileRequest TileRequest(Tile, ProducerPriority, EVTInvalidatePriority::Normal);
				const uint16 LoadRequestIndex = MergedRequestList->LockLoadRequest(TileRequest, ProducerLayerMaskToLoad, bStreamingRequest);
				if (LoadRequestIndex == 0xffff)
				{
					// Overflowed the request list...try to lock the tile again next frame
					TilesToLockForNextFrame.Add(Tile);
				}
			}
		}
	}
	TilesToLock.Reset();
}

void FVirtualTextureSystem::GatherPackedTileRequests(FConcurrentLinearBulkObjectAllocator& Allocator, const FVirtualTextureUpdateSettings& Settings, FUniqueRequestList* MergedRequestList)
{
	TArray<uint32> PackedTiles;
	if (RequestedPackedTiles.Num() > 0)
	{
		PackedTiles = MoveTemp(RequestedPackedTiles);
		RequestedPackedTiles.Reset();
	}

	if (PackedTiles.Num() > 0)
	{
		// Collect explicitly requested tiles
		// These tiles are generated on the current frame, so they are collected/processed in a separate list
		FUniquePageList* RequestedPageList = Allocator.Create<FUniquePageList>();
		RequestedPageList->Initialize();
		for (uint32 Tile : PackedTiles)
		{
			RequestedPageList->Add(Tile, 0xffff);
		}
		GatherRequests(MergedRequestList, RequestedPageList, Frame, Allocator, Settings);
	}
}

void FVirtualTextureSystem::BeginUpdate(FRDGBuilder& GraphBuilder, FVirtualTextureUpdater* Updater)
{
	if (CVarVTWaitBeforeUpdate.GetValueOnRenderThread())
	{
		// Wait for SceneRenderBuilder deferred deletion.
		// Otherwise scene renderer deletion may happen during virtual texture update.
		// This can trigger material proxy deletion and calls to RemoveAllProducerDestroyedCallbacks().
		// RemoveAllProducerDestroyedCallbacks() can't be called during a virtual texture update.
		FSceneRenderBuilder::WaitForAsyncDeleteTask();
	}

	// Mark updating to true now that we are potentially launching async tasks.
	bUpdating = true;

	if (Updater->Settings.bEnableFeedback)
	{
		SCOPE_CYCLE_COUNTER(STAT_FeedbackMap);
		Updater->FeedbackMapResult = GVirtualTextureFeedback.Map(GraphBuilder.RHICmdList);
	}

	Updater->AsyncTask = GraphBuilder.AddCommandListSetupTask([this, Updater, &Allocator = Updater->Allocator, Settings = Updater->Settings, FeedbackResult = Updater->FeedbackMapResult](FRHICommandList& RHICmdList)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::GatherAndSubmitRequests);

		Updater->MergedRequestList = Allocator.Create<FUniqueRequestList>(Allocator);
		Updater->MergedRequestList->Initialize();

		GatherFeedbackRequests(Allocator, Settings, FeedbackResult, Updater->MergedRequestList);
		GatherLockedTileRequests(Updater->MergedRequestList);
		GatherPackedTileRequests(Allocator, Settings, Updater->MergedRequestList);
		SubmitThrottledRequests(RHICmdList, Updater, EUpdatePhase::Begin);

		// Reset the request list for the gather in EndUpdate.
		const bool bResetContinousUpdates = false;
		Updater->MergedRequestList->Reset(bResetContinousUpdates);

	}, UE::Tasks::ETaskPriority::High, Updater->bAsyncTaskAllowed);
}

void FVirtualTextureSystem::CallPendingCallbacks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::CallPendingCallbacks);
	SCOPE_CYCLE_COUNTER(STAT_VirtualTextureSystem_Update);
	UE::TScopeLock Lock(Mutex);
	Producers.CallPendingCallbacks();
}

TUniquePtr<FVirtualTextureUpdater> FVirtualTextureSystem::BeginUpdate(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, ISceneRenderer* SceneRenderer, const FVirtualTextureUpdateSettings& Settings)
{
	check(IsInRenderingThread());
	check(!bUpdating);
	checkf(Producers.HasPendingCallbacks() == false, TEXT("FVirtualTextureSystem::CallPendingCallbacks(), called in UpdateAllPrimitiveSceneInfos(), must run before FVirtualTextureSystem::BeginUpdate()"));

	AllocateResources(GraphBuilder);

	// Optional early out after AllocateResources().
	if (!Settings.bEnablePageRequests)
	{
		return {};
	}

	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, VirtualTextureSystem_Update);
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::BeginUpdate);
	SCOPE_CYCLE_COUNTER(STAT_VirtualTextureSystem_Update);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, VirtualTexture, "VirtualTextureBeginUpdate");

	// Update Adaptive VTs. This can trigger allocation/destruction of VTs and must happen before the flush below.
	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_UpdateAdaptiveAllocations);
		for (int32 ID = 0; ID < AdaptiveVTs.Num(); ID++)
		{
			if (AdaptiveVTs[ID])
			{
				AdaptiveVTs[ID]->UpdateAllocations(this, GraphBuilder.RHICmdList, Frame);
			}
		}
	}
	
	FMaterialCacheVirtualTextureAllocator::Get().UpdateAllocations(GraphBuilder.RHICmdList, Frame);

	if (bFlushCaches)
	{
		SCOPE_CYCLE_COUNTER(STAT_FlushCache);
		INC_DWORD_STAT_BY(STAT_NumFlushCache, 1);

		for (int32 i = 0; i < PhysicalSpaces.Num(); ++i)
		{
			FVirtualTexturePhysicalSpace* PhysicalSpace = PhysicalSpaces[i];
			if (PhysicalSpace)
			{
				if (CVarVTProduceLockedTilesOnFlush.GetValueOnRenderThread())
				{
					// Collect locked pages to be produced again
					TSet<FVirtualTextureLocalTile> LockedTiles;
					PhysicalSpace->GetPagePool().GetAllLockedPages(this, LockedTiles);
					for (FVirtualTextureLocalTile LockedTile : LockedTiles)
					{
						// This is not really runtime-relevant (debug cache flush) so it's fine to re-produce the tile with normal priority again : 
						AddOrMergeTileRequest(FVirtualTextureLocalTileRequest(LockedTile, EVTProducerPriority::Normal, EVTInvalidatePriority::Normal), MappedTilesToProduce);
					}
				}
				// Flush unlocked pages
				PhysicalSpace->GetPagePool().EvictAllPages(this);
			}
		}

		bFlushCaches = false;
	}

	DestroyPendingVirtualTextures(false);
	ReleasePendingSpaces(false);

	UpdateResidencyTracking();

	// Early out when no allocated VTs
	if (AllocatedVTs.Num() == 0)
	{
		MappedTilesToProduce.Reset();
		return {};
	}

	// Flush any dirty runtime virtual textures for the current scene
	FScene* Scene = SceneRenderer != nullptr ? SceneRenderer->GetScene() : nullptr;
	if (Scene != nullptr)
	{
		// Only flush if we know that there is GPU feedback available to refill the visible data this frame
		// This prevents bugs when low frame rate causes feedback buffer to stall so that the physical cache isn't filled immediately which causes visible glitching
		if (GVirtualTextureFeedback.CanMap(GraphBuilder.RHICmdList))
		{
			// Each RVT will call FVirtualTextureSystem::FlushCache()
			Scene->FlushDirtyRuntimeVirtualTextures();
		}
	}

	TUniquePtr<FVirtualTextureUpdater> Updater(new FVirtualTextureUpdater());
	Updater->Settings = Settings;
	Updater->FeatureLevel = FeatureLevel;
	Updater->bAsyncTaskAllowed = Settings.bEnableAsyncTasks;
	Updater->PageUploadBudgetRVT = Settings.MaxRVTPageUploads;
	Updater->PageUploadBudgetSVT = Settings.MaxSVTPageUploads;

	if (Updater->bAsyncTaskAllowed)
	{
		BeginUpdate(GraphBuilder, Updater.Get());
	}

	return MoveTemp(Updater);
}

void FVirtualTextureSystem::WaitForTasks(FVirtualTextureUpdater* Updater)
{
	if (!Updater)
	{
		return;
	}

	// Set flag that will start dropping work from the updater task.
	// This can helps in extreme cases where the amount of virtual texture update work is causing us to wait here.
	// Setting FVirtualTextureUpdateSettings::MaxGatherTime and FVirtualTextureUpdateSettings::MaxSubmitTime are other ways to load balance the work.
	GAsyncTaskEarlyOut = true;

	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_WaitTasks);
		CSV_SCOPED_SET_WAIT_STAT(VirtualTexture);
		Updater->AsyncTask.Wait();
	}

	bUpdating = false;
	GAsyncTaskEarlyOut = false;
}

void FVirtualTextureSystem::EndUpdate(FRDGBuilder& GraphBuilder, TUniquePtr<FVirtualTextureUpdater>&& Updater, ERHIFeatureLevel::Type FeatureLevel)
{
	check(IsInRenderingThread());

	if (!Updater)
	{
		return;
	}

	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, VirtualTextureSystem_Update);
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::EndUpdate);
	SCOPE_CYCLE_COUNTER(STAT_VirtualTextureSystem_Update);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, VirtualTexture, "VirtualTextureEndUpdate");

	if (Updater->bAsyncTaskAllowed)
	{
		SCOPE_CYCLE_COUNTER(STAT_ProcessRequests_WaitTasks);
		CSV_SCOPED_SET_WAIT_STAT(VirtualTexture);
		Updater->AsyncTask.Wait();
	}
	else
	{
		BeginUpdate(GraphBuilder, Updater.Get());
	}
	bUpdating = false;

	if (Updater->FeedbackMapResult.Data)
	{
		GVirtualTextureFeedback.Unmap(GraphBuilder.RHICmdList, Updater->FeedbackMapResult.MapHandle);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::GatherAndSubmitRequests);
		const bool bContinousUpdates = true;

		// We only need to gather remaining requests if an async task was used early in the frame.
		if (Updater->bAsyncTaskAllowed)
		{
			GatherLockedTileRequests(Updater->MergedRequestList);
			GatherPackedTileRequests(Updater->Allocator, Updater->Settings, Updater->MergedRequestList);
		}

		SubmitThrottledRequests(GraphBuilder.RHICmdList, Updater.Get(), EUpdatePhase::End);
	}

	GrowPhysicalPools();

#if !UE_BUILD_SHIPPING
	UpdateCsvStats();
#endif

	TilesToLock = MoveTemp(TilesToLockForNextFrame);
}

void FVirtualTextureSystem::Update(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, ISceneRenderer* SceneRenderer, const FVirtualTextureUpdateSettings& InSettings)
{
	CallPendingCallbacks();

	FVirtualTextureUpdateSettings Settings = InSettings;
	Settings.EnableAsyncTasks(false);

	TUniquePtr<FVirtualTextureUpdater> Updater = BeginUpdate(GraphBuilder, FeatureLevel, SceneRenderer, Settings);
	EndUpdate(GraphBuilder, MoveTemp(Updater), FeatureLevel);

	FinalizeRequests(GraphBuilder, SceneRenderer);
}

void FVirtualTextureSystem::ReleasePendingResources()
{
	check(!bUpdating);
	UE::TScopeLock Lock(Mutex);
	DestroyPendingVirtualTextures(true);
	ReleasePendingSpaces(true);
}

FVector4f FVirtualTextureSystem::GetGlobalMipBias() const
{
	FVector4f MaxResidencyMipMapBias(EForceInit::ForceInitToZero);
	for (int32 SpaceIndex = 0; SpaceIndex < PhysicalSpaces.Num(); ++SpaceIndex)
	{
		if (FVirtualTexturePhysicalSpace const* PhysicalSpace = PhysicalSpaces[SpaceIndex])
		{
			const float ResidencyMipMapBias = PhysicalSpace->GetResidencyMipMapBias();
			const int32 GroupIndex = PhysicalSpace->GetDescriptionExt().ResidencyMipMapBiasGroup;
			MaxResidencyMipMapBias[GroupIndex] = FMath::Max(MaxResidencyMipMapBias[GroupIndex], ResidencyMipMapBias);
		}
	}

	const float GlobalBias = UTexture2D::GetGlobalMipMapLODBias();
	return MaxResidencyMipMapBias + FVector4f(GlobalBias, GlobalBias, GlobalBias, GlobalBias);
}

bool FVirtualTextureSystem::IsPendingRootPageMap(IAllocatedVirtualTexture* AllocatedVT) const
{
	UE::TScopeLock Lock(Mutex);
	return AllocatedVT->bIsWaitingToMap;
}

#if !UE_BUILD_SHIPPING

void FVirtualTextureSystem::GetOnScreenMessages(FCoreDelegates::FSeverityMessageMap& OutMessages)
{
	if (NumAllocatedSpaces > VIRTUALTEXTURE_MAX_FEEDBACK_SPACES)
	{
		// To reduce the number of spaces created, try using r.VT.PageTableMode=0 or modifying RVTs to not use private spaces.
		OutMessages.Add(
			FCoreDelegates::EOnScreenMessageSeverity::Warning,
			FText::Format(LOCTEXT("VTSpacesOversubscribed", "Using {0} virtual texture spaces. Feedback is disabled for some virtual textures. Consider using r.VT.PageTableMode=0."), FText::AsNumber(NumAllocatedSpaces)));
	}

	if (CVarVTResidencyNotify.GetValueOnRenderThread() == 0)
	{
		return;
	}

	for (int32 SpaceIndex = 0; SpaceIndex < PhysicalSpaces.Num(); ++SpaceIndex)
	{
		const FVirtualTexturePhysicalSpace* PhysicalSpace = PhysicalSpaces[SpaceIndex];
		if (PhysicalSpace != nullptr && Frame <= PhysicalSpace->GetLastFrameOversubscribed() + 60u)
		{
			FString const& FormatString = PhysicalSpace->GetFormatString();
			const float MipBias = PhysicalSpace->GetResidencyMipMapBias();
			const int32 GroupIndex = PhysicalSpace->GetDescriptionExt().ResidencyMipMapBiasGroup;

			OutMessages.Add(
				FCoreDelegates::EOnScreenMessageSeverity::Warning,
				FText::Format(LOCTEXT("VTOversubscribed", "VT Pool [{0}] is oversubscribed. Setting MipBias {1} on Group {2}"), FText::FromString(FormatString), FText::AsNumber(MipBias), FText::AsNumber(GroupIndex)));
		}
	}
}

void FVirtualTextureSystem::UpdateCsvStats()
{
#if CSV_PROFILER_STATS
	if (CVarVTCsvStats.GetValueOnRenderThread() == 0)
	{
		return;
	}

	float MaxResidencyMipMapBias = 0.f;
	for (int32 SpaceIndex = 0; SpaceIndex < PhysicalSpaces.Num(); ++SpaceIndex)
	{
		const FVirtualTexturePhysicalSpace* PhysicalSpace = PhysicalSpaces[SpaceIndex];
		if (PhysicalSpace != nullptr)
		{
			const float ResidencyMipMapBias = PhysicalSpace->GetResidencyMipMapBias();
			MaxResidencyMipMapBias = FMath::Max(MaxResidencyMipMapBias, ResidencyMipMapBias);

			if (CVarVTCsvStats.GetValueOnRenderThread() == 2)
			{
				PhysicalSpace->UpdateCsvStats();
			}
		}
	}

	CSV_CUSTOM_STAT(VirtualTexturing, ResidencyMipBias, MaxResidencyMipMapBias, ECsvCustomStatOp::Set);
#endif
}

void FVirtualTextureSystem::DrawResidencyHud(UCanvas* InCanvas, APlayerController* InController)
{
	if (CVarVTResidencyShow.GetValueOnGameThread() == 0)
	{
		return;
	}

	int32 NumGraphs = 0;
	for (int32 SpaceIndex = 0; SpaceIndex < PhysicalSpaces.Num(); ++SpaceIndex)
	{
		FVirtualTexturePhysicalSpace* PhysicalSpace = PhysicalSpaces[SpaceIndex];
		if (PhysicalSpace)
		{
			NumGraphs++;
		}
	}

	if (NumGraphs == 0)
	{
		return;
	}

	const FIntPoint GraphSize(250, 125);
	const FIntPoint BorderSize(25, 25);
	const FIntPoint GraphWithBorderSize = GraphSize + BorderSize * 2;

	const FIntPoint CanvasSize = FIntPoint(InCanvas->ClipX, InCanvas->ClipY);
	const int32 NumGraphsInRow = FMath::Max(CanvasSize.X / GraphWithBorderSize.X, 1);
	const int32 CanvasOffsetY = 90;

	int32 GraphIndex = 0;
	for (int32 SpaceIndex = 0; SpaceIndex < PhysicalSpaces.Num(); ++SpaceIndex)
	{
		FVirtualTexturePhysicalSpace* PhysicalSpace = PhysicalSpaces[SpaceIndex];
		if (PhysicalSpace && PhysicalSpace->IsInitialized())
		{
			int32 GraphX = GraphIndex % NumGraphsInRow;
			int32 GraphY = GraphIndex / NumGraphsInRow;

			FBox2D CanvasPosition;
			CanvasPosition.Min.X = GraphX * GraphWithBorderSize.X + BorderSize.X;
			CanvasPosition.Min.Y = GraphY * GraphWithBorderSize.Y + BorderSize.Y + CanvasOffsetY;
			CanvasPosition.Max = CanvasPosition.Min + GraphSize;

			if (CanvasPosition.Min.Y > CanvasSize.Y)
			{
				// Off screen so early out.
				break;
			}

			const bool bDrawKey = GraphIndex == 0;
			PhysicalSpace->DrawResidencyGraph(InCanvas->Canvas, CanvasPosition, bDrawKey);

			GraphIndex++;
		}
	}
}

#endif // !UE_BUILD_SHIPPING

#if WITH_EDITOR

void FVirtualTextureSystem::SetRequestRecordBuffer(uint64 Handle)
{
	UE::TScopeLock Lock(Mutex);
	check(PageRequestRecordHandle == ~0ull && PageRequestRecordBuffer.Num() == 0);
	
	PageRequestRecordHandle = Handle;
	PageRequestRecordBuffer.Reset();
}

void FVirtualTextureSystem::RecordPageRequests(FUniquePageList const* UniquePageList, TMap<uint64, uint32>& OutPages)
{
	const uint32 PageCount = UniquePageList->GetNum();

	OutPages.Reset();
	OutPages.Reserve(PageCount);

	for (uint32 PageIndex = 0; PageIndex < PageCount; ++PageIndex)
	{
		const uint32 PageId = UniquePageList->GetPage(PageIndex);
		const uint32 Count = UniquePageList->GetCount(PageIndex);

		const uint32 SpaceId = (PageId >> 28);
		const FVirtualTextureSpace* RESTRICT Space = GetSpace(SpaceId);
		if (Space == nullptr)
		{
			continue;
		}

		const uint32 vLevelPlusFixedOffset = ((PageId >> 24) & 0x0f);
		const uint32 vLevel = FMath::Max<uint32>(vLevelPlusFixedOffset, UE::HLSL::VirtualTextureFeedbackBias) - UE::HLSL::VirtualTextureFeedbackBias;
		const uint32 vPageX = (PageId & 0xfff) << vLevel;
		const uint32 vPageY = ((PageId >> 12) & 0xfff) << vLevel;

		const uint32 vAddress = FMath::MortonCode2(vPageX) | (FMath::MortonCode2(vPageY) << 1);
		const FAllocatedVirtualTexture* RESTRICT AllocatedVT = Space->GetAllocator().Find(vAddress);
		if (AllocatedVT == nullptr)
		{
			continue;
		}

		// PersistentHash is used to find associated AllocatedVT at playback.
		const uint32 PersistentHash = AllocatedVT->GetPersistentHash();
		// MaxLevel is used to detect and adjust for any TextureLODGroup bias that has been applied at playback.
		const uint32 MaxLevel = AllocatedVT->GetMaxLevel() & 0x0f;
		const uint32 LocalAddress = (vAddress - AllocatedVT->GetVirtualAddress()) & 0xffffff;
		const uint64 ExportPageId = ((uint64)PersistentHash << 32) | ((uint64)MaxLevel << 28) | ((uint64)vLevelPlusFixedOffset << 24) | (uint64)LocalAddress;

		// Store the PageId and associated count which we use for request priority.
		OutPages.FindOrAdd(ExportPageId) += Count;
	}
}

uint64 FVirtualTextureSystem::GetRequestRecordBuffer(TMap<uint64, uint32>& OutRequestData)
{
	UE::TScopeLock Lock(Mutex);

	if (PageRequestRecordHandle == ~0ull)
	{
		return ~0ull;
	}

	uint64 Ret = PageRequestRecordHandle;
	OutRequestData = MoveTemp(PageRequestRecordBuffer);
	PageRequestRecordBuffer.Reset();
	PageRequestRecordHandle = ~0ull;
	return Ret;
}

void FVirtualTextureSystem::PackRequestsToStream(TMap<uint64, uint32> const& InPageIdAndPriorities, TArray<uint32>& OutRequestStream)
{
	// Extract and sort requests
	struct FPackedRequest
	{
		uint64 PageId;
		uint16 Priority;

		bool operator<(const FPackedRequest& Other) const
		{
			return PageId != Other.PageId ? PageId < Other.PageId : Priority > Other.Priority;
		}
	};

	TArray<FPackedRequest> Requests;
	Requests.Reserve(InPageIdAndPriorities.Num());
	for (const TPair<uint64, uint32>& Entry : InPageIdAndPriorities)
	{
		uint16 Priority = (uint16)FMath::Min(Entry.Value, 0xffffu);
		Requests.Add(FPackedRequest{ Entry.Key, Priority });
	}
	Requests.Sort();

	// Count space required to store packed data.
	uint32 NumUniqueVirtualTextures = 0;
	{
		uint32 PreviousVirtualTextureId = ~0u;
		for (const FPackedRequest& Request : Requests)
		{
			const uint32 VirtualTextureId = (uint32)(Request.PageId >> 32);
			if (VirtualTextureId != PreviousVirtualTextureId)
			{
				NumUniqueVirtualTextures++;
			}
			PreviousVirtualTextureId = VirtualTextureId;
		}
	}

	// Write packed requests.
	// A request consists of three DWORDs. An allocated virtual texture DWORD and a two PageId/Priority/Repeat DWORDs.
	// The repeat bit indicates if the next request is to the same allocated virtual texture, so the allocated virtual texture DWORD can be omitted.
	// The final 2 DWORD packing itself is quite wasteful. We only actually need 24+16+1 bits. (The MaxLevel 4 bits could live with the allocated virtual texture).
	// Quantizing the priority is tempting, but using less than 10 bits can degrade final render quality because textures that cover more screen space don't update first.
	// If memory is an issue we could look at using FBitStream or similar to pack what we have more tightly. (Note that we would need to support appending two request streams without alignment issues.)
	{
		const uint32 NumOutputDwords = NumUniqueVirtualTextures + Requests.Num() * 2;
		OutRequestStream.SetNum(NumOutputDwords);

		uint32 WriteIndex = 0;
		uint32 PreviousVirtualTextureId = ~0u;
		for (const FPackedRequest& Request : Requests)
		{
			uint32 VirtualTextureId = (uint32)(Request.PageId >> 32);
			if (WriteIndex == 0 || VirtualTextureId != PreviousVirtualTextureId)
			{
				OutRequestStream[WriteIndex++] = VirtualTextureId;
			}
			else
			{
				// Mark repeat flag in previous packed dword.
				OutRequestStream[WriteIndex - 1] |= 1;
			}
			PreviousVirtualTextureId = VirtualTextureId;

			OutRequestStream[WriteIndex++] = (uint32)(Request.PageId & 0xffffffff);
			// Write priority in top 16 bits of second dword. 
			// Lowest bit is reserved for repeat flag. It defaults off and may be set in next loop iteration.
			OutRequestStream[WriteIndex++] = (uint32)(Request.Priority) << 16;
		}

		check(WriteIndex == NumOutputDwords);
	}
}

#endif // WITH_EDITOR

void FVirtualTextureSystem::RequestRecordedTiles(TArray<uint32>&& InRequestStream, FIntPoint const& InCaptureResolution)
{
	UE::TScopeLock Lock(Mutex);

	if (PageRequestPlaybackBuffer.Num() == 0)
	{
		PageRequestPlaybackBuffer = MoveTemp(InRequestStream);
	}
	else
	{
		PageRequestPlaybackBuffer.Append(InRequestStream);
	}

	// We expect all recorded requests to have been captured at the same resolution.
	// If that turns out not to be the case we will need to store an array of recorded requests per resolution.
	ensure(PageRequestCaptureResolution == FIntPoint::ZeroValue || PageRequestCaptureResolution == InCaptureResolution);
	PageRequestCaptureResolution = InCaptureResolution;
}

void FVirtualTextureSystem::AddRequestedTilesTask(const FAddRequestedTilesParameters& Parameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualTextureSystem::AddRequestedTilesTask);

	FUniquePageList* RESTRICT RequestedPageList = Parameters.UniquePageList;
	const uint32* RESTRICT Buffer = Parameters.RequestBuffer;
	const uint32 BufferSize = Parameters.NumRequests;
	const uint32 ViewLevelBias = Parameters.ViewLevelBias;

	uint32 Index = 0;
	while (Index < BufferSize)
	{
		const uint32 PersistentHash = Buffer[Index++];

		IAllocatedVirtualTexture** AllocatedVTPtr = PersistentVTMap.Find(PersistentHash);
		IAllocatedVirtualTexture* AllocatedVT = AllocatedVTPtr != nullptr ? *AllocatedVTPtr : nullptr;

		// Keep processing requests from this allocated virtual texture as long as they have the repeat bit set.
		bool bRepeat = true;
		while (bRepeat && Index + 1 < BufferSize)
		{
			const uint32 PageRequest = Buffer[Index++];
			const uint32 PackedPriorityAndRepeat = Buffer[Index++];

			// Repeat flag is set in first bit of second dword.
			bRepeat = (PackedPriorityAndRepeat & 1u) != 0u;

			if (AllocatedVT == nullptr || AllocatedVT->GetSpaceID() >= VIRTUALTEXTURE_MAX_FEEDBACK_SPACES)
			{
				continue;
			}

			const uint32 SourceLevelPlusFixedOffset = (uint32)(PageRequest >> 24) & 0x0f;
			const uint32 SourceLevel = FMath::Max<uint32>(SourceLevelPlusFixedOffset, UE::HLSL::VirtualTextureFeedbackBias) - UE::HLSL::VirtualTextureFeedbackBias;
			const uint32 SourceMaxLevel = (uint32)(PageRequest >> 28) & 0x0f;
			const uint32 SourceLocalAddress = (uint32)(PageRequest & 0xffffff);

			// Deal with the case of texture size being reduced from recording to playback due to LOD bias in cook.
			const uint32 MaxLevel = AllocatedVT->GetMaxLevel();
			const uint32 TextureSizeBias = SourceMaxLevel > MaxLevel ? SourceMaxLevel - MaxLevel : 0;
			const uint32 Level = FMath::Clamp((int32)SourceLevel - (int32)TextureSizeBias + (int32)ViewLevelBias, 0, MaxLevel);

			const uint32 LocalAddress = SourceLocalAddress >> (TextureSizeBias * 2u);
			const uint32 BaseAddress = AllocatedVT->GetVirtualAddress();
			const uint32 Address = BaseAddress + LocalAddress;
			const uint32 PageX = FMath::ReverseMortonCode2(Address) >> Level;
			const uint32 PageY = FMath::ReverseMortonCode2(Address >> 1) >> Level;

			const uint32 SpaceId = AllocatedVT->GetSpaceID();

			const uint32 PageId = PageX | (PageY << 12) | ((Level + UE::HLSL::VirtualTextureFeedbackBias) << 24) | (SpaceId << 28);
			const uint32 Priority = PackedPriorityAndRepeat >> 16;

			RequestedPageList->Add(PageId, Priority);
		}
	}
}

#undef LOCTEXT_NAMESPACE
