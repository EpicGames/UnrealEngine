// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if !UE_BUILD_SHIPPING
#include "Misc/CoreDelegates.h"
#endif
#include "RHI.h"
#include "RendererInterface.h"
#include "Templates/UniquePtr.h"
#include "VT/VirtualTextureProducer.h"
#include "VT/TexturePageLocks.h"
#include "VirtualTexturing.h"
#include "VT/VirtualTextureFeedback.h"
#include "Tasks/Task.h"
#include "Async/RecursiveMutex.h"

class FAdaptiveVirtualTexture;
class FAllocatedVirtualTexture;
class FScene;
class FUniquePageList;
class FUniqueRequestList;
class FVirtualTexturePhysicalSpace;
class FVirtualTextureProducer;
class FVirtualTextureSpace;
class FVirtualTextureSystem;
struct FVTSpaceDescription;
struct FVTPhysicalSpaceDescription;
union FPhysicalSpaceIDAndAddress;
struct FFeedbackAnalysisParameters;
struct FAddRequestedTilesParameters;
struct FGatherRequestsParameters;
struct FPageUpdateBuffer;
class ISceneRenderer;
class FViewFamilyInfo;

struct FVirtualTextureUpdateSettings
{
	FVirtualTextureUpdateSettings();
	FVirtualTextureUpdateSettings(FViewFamilyInfo const& ViewFamily);

	/** Force settings so that throttling page uploads is effectively disabled. */
	FVirtualTextureUpdateSettings& EnableThrottling(bool bEnable)
	{
		if (!bEnable)
		{
			bSyncProduceAllTiles = true;
			MaxRVTPageUploads = 99999;
			MaxSVTPageUploads = 99999;
			MaxPagesProduced = 99999;
			MaxGatherTime = 0.0f;
			MaxSubmitTime = 0.0f;
			MinSubmitAsyncEarlyOutTime = 0.0f;
		}
		return *this;
	}

	/** Force virtual texture updates to be done synchronously. */
	FVirtualTextureUpdateSettings& EnableAsyncTasks(bool bEnable = true)
	{
		bEnableAsyncTasks = bEnable;
		return *this;
	}

	/** Do not perform any updates related to read backs. */
	FVirtualTextureUpdateSettings& EnablePageRequests(bool bEnable = true)
	{
		bEnablePageRequests = bEnable;
		return *this;
	}

	/** Optionally disable feedback. */
	FVirtualTextureUpdateSettings& EnableFeedback(bool bEnable = true)
	{
		bEnableFeedback = bEnable;
		bEnableFeedbackProduce = bEnable;
		return *this;
	}

	bool bEnablePageRequests = true;
	bool bEnableFeedback = true;
	bool bEnableFeedbackProduce = true;
	bool bEnablePlayback = true;
	bool bForceContinuousUpdate = false;
	bool bSyncProduceAllTiles = false;
	bool bSyncProduceLockedTiles = false;
	bool bEnableAsyncTasks = true;
	int32 MaxRVTPageUploads = 0;
	int32 MaxSVTPageUploads = 0;
	int32 MaxPagesProduced = 0;
	int32 MaxContinuousUpdates = 0;
	float MaxGatherTime = 0.0f;
	float MaxSubmitTime = 0.0f;
	float MinSubmitAsyncEarlyOutTime = 0.0f;
	FIntPoint PlaybackViewSize = FIntPoint(0, 0);
};

class FVirtualTextureUpdater
{
public:
	UE::Tasks::FTask GetTask() const
	{
		return AsyncTask;
	}

private:
	FVirtualTextureUpdateSettings Settings;
	FConcurrentLinearBulkObjectAllocator Allocator;
	FUniqueRequestList* MergedRequestList = nullptr;
	FVirtualTextureFeedback::FMapResult FeedbackMapResult;
	UE::Tasks::FTask AsyncTask;
	ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::Num;
	bool bAsyncTaskAllowed = false;
	int32 PageUploadBudgetRVT = 0;
	int32 PageUploadBudgetSVT = 0;

	friend FVirtualTextureSystem;
};

class FVirtualTextureSystem
{
public:
	static void Initialize();
	static void Shutdown();
	static FVirtualTextureSystem& Get();

	uint32 GetFrame() const { return Frame; }

	void Update(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, ISceneRenderer* SceneRenderer, const FVirtualTextureUpdateSettings& Settings);

	// Called in FSceneRenderer::UpdateScene, before FVirtualTextureSystem::BeginUpdate -- see comments there for more info
	void CallPendingCallbacks();

	TUniquePtr<FVirtualTextureUpdater> BeginUpdate(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, ISceneRenderer* SceneRenderer, const FVirtualTextureUpdateSettings& Settings);
	void WaitForTasks(FVirtualTextureUpdater* Updater);
	void EndUpdate(FRDGBuilder& GraphBuilder, TUniquePtr<FVirtualTextureUpdater>&& Updater, ERHIFeatureLevel::Type FeatureLevel);

	void FinalizeRequests(FRDGBuilder& GraphBuilder, ISceneRenderer* SceneRenderer);

	void ReleasePendingResources();

	IAllocatedVirtualTexture* AllocateVirtualTexture(FRHICommandListBase& RHICmdList, const FAllocatedVTDescription& Desc);
	void DestroyVirtualTexture(IAllocatedVirtualTexture* AllocatedVT);

	FVirtualTextureProducerHandle RegisterProducer(FRHICommandListBase& RHICmdList, const FVTProducerDescription& InDesc, IVirtualTexture* InProducer);
	void ReleaseProducer(const FVirtualTextureProducerHandle& Handle);
	bool TryReleaseProducer(const FVirtualTextureProducerHandle& Handle);
	void AddProducerDestroyedCallback(const FVirtualTextureProducerHandle& Handle, FVTProducerDestroyedFunction* Function, void* Baton);
	uint32 RemoveAllProducerDestroyedCallbacks(const void* Baton);

	IAdaptiveVirtualTexture* AllocateAdaptiveVirtualTexture(FRHICommandListBase& RHICmdList, const FAdaptiveVTDescription& AdaptiveVTDesc, const FAllocatedVTDescription& AllocatedVTDesc);
	void DestroyAdaptiveVirtualTexture(IAdaptiveVirtualTexture* AdaptiveVT);

	void RequestTiles(const FVector2D& InScreenSpaceSize, int32 InMipLevel = -1);
	void RequestTiles(const FMaterialRenderProxy* InMaterialRenderProxy, const FVector2D& InScreenSpaceSize, ERHIFeatureLevel::Type InFeatureLevel);

	/**
	 * Helper function to request loading of tiles for a virtual texture that will be displayed in the UI. 
	 * It will request only the tiles that will be visible after clipping to the provided viewport.
	 * @param AllocatedVT			The virtual texture.
	 * @param InScreenSpaceSize		Size on screen at which the texture is to be displayed.
	 * @param InViewportPosition	Position in the viewport where the texture will be displayed.
	 * @param InViewportSize		Size of the viewport.
	 * @param InUV0					UV coordinate to use for the top left corner of the texture.
	 * @param InUV1					UV coordinate to use for the bottom right corner of the texture.
	 * @param InMipLevel [optional] Specific mip level to fetch tiles for.
	 */
	void RequestTiles(IAllocatedVirtualTexture* AllocatedVT, const FVector2D& InScreenSpaceSize, const FVector2D& InViewportPosition, const FVector2D& InViewportSize, const FVector2D& InUV0, const FVector2D& InUV1, int32 InMipLevel = -1);

	void LoadPendingTiles(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel);
	
	void SetMipLevelToLock(FVirtualTextureProducerHandle ProducerHandle, int32 InMipLevel);

#if WITH_EDITOR
	void SetRequestRecordBuffer(uint64 Handle);
	uint64 GetRequestRecordBuffer(TMap<uint64, uint32>& OutRequestData);
	static void PackRequestsToStream(TMap<uint64, uint32> const& InPageIdAndPriorities, TArray<uint32>& OutRequestStream);
#endif
	void RequestRecordedTiles(TArray<uint32>&& InRequestStream, FIntPoint const& InCaptureResolution);

	void FlushCache();
	void FlushCache(FVirtualTextureProducerHandle const& ProducerHandle, int32 SpaceID, FIntRect const& TextureRegion, uint32 MaxLevelToEvict, uint32 MaxAgeToKeepMapped, EVTInvalidatePriority InvalidatePriority);

	FVector4f GetGlobalMipBias() const;

	bool IsPendingRootPageMap(IAllocatedVirtualTexture* AllocatedVT) const;

	bool HasAllocatedVirtualTextures() const { return AllocatedVTs.Num() > 0; }

	/**
	 * Returns the finest (lowest index) mip level at which the given AllocatedVT was sampled
	 * within the last FrameWindow frames, or 0xFF if no tracking data is available within that window.
	 * Only producers registered with bTrackSamplingHistory contribute.
	 */
	uint8 GetFinestSampledMipInLastNFrames(const FAllocatedVirtualTexture* AllocatedVT, uint32 FrameWindow) const;
	
private:
	friend class FFeedbackAnalysisTask;
	friend class FAddRequestedTilesTask;
	friend class FGatherRequestsTask;
	friend class FAllocatedVirtualTexture;
	friend class FAdaptiveVirtualTexture;
	friend class FVirtualTextureProducer;
	friend class FVirtualTextureProducerCollection;
	friend class FTexturePageMap;
	friend class FTexturePagePool;
	friend class FRendererModule;
	friend class FMaterialCacheVirtualTextureAllocator;

	FVirtualTextureSystem();
	~FVirtualTextureSystem();

	FVirtualTextureSpace* AcquireSpace(FRHICommandListBase& RHICmdList, const FVTSpaceDescription& InDesc, uint8 InForceSpaceID, FAllocatedVirtualTexture* AllocatedVT);
	void ReleaseSpace(FVirtualTextureSpace* Space);

	FVirtualTextureProducer* FindProducer(const FVirtualTextureProducerHandle& Handle);
	FVirtualTexturePhysicalSpace* AcquirePhysicalSpace(FRHICommandListBase& RHICmdList, const FVTPhysicalSpaceDescription& InDesc);

	FVirtualTextureSpace* GetSpace(int32 ID) const { return Spaces.IsValidIndex(ID) ? Spaces[ID].Get() : nullptr; }
	FAdaptiveVirtualTexture* GetAdaptiveVirtualTexture(int32 ID) const { return AdaptiveVTs.IsValidIndex(ID) ? AdaptiveVTs[ID] : nullptr; }
	FVirtualTexturePhysicalSpace* GetPhysicalSpace(uint32 ID) const { return PhysicalSpaces.IsValidIndex(ID) ? PhysicalSpaces[ID] : nullptr; }

	void LockTile(const FVirtualTextureLocalTile& Tile);
	void UnlockTile(const FVirtualTextureLocalTile& Tile, const FVirtualTextureProducer* Producer);
	void ForceUnlockAllTiles(const FVirtualTextureProducerHandle& ProducerHandle, const FVirtualTextureProducer* Producer);

	void BeginUpdate(FRDGBuilder& GraphBuilder, FVirtualTextureUpdater* Updater);

	void AllocateResources(FRDGBuilder& GraphBuilder);
	void DestroyPendingVirtualTextures(bool bForceDestroyAll);
	void ReleasePendingSpaces(bool bForceDestroyAll);

	void RequestTilesForRegionInternal(const IAllocatedVirtualTexture* AllocatedVT, const FVector2D& InScreenSpaceSize, const FVector2D& InViewportPosition, const FVector2D& InViewportSize, const FVector2D& InUV0, const FVector2D& InUV1, int32 InMipLevel);
	void RequestTilesInternal(const IAllocatedVirtualTexture* AllocatedVT, int32 InMipLevel);
	void RequestTilesInternal(const IAllocatedVirtualTexture* AllocatedVT, const FVector2D& InScreenSpaceSize, int32 InMipLevel);

	int32 SubmitRequestsFromLocalTileRequests(FConcurrentLinearBulkObjectAllocator& Allocator, FRHICommandList& RHICmdList, TSet<FVirtualTextureLocalTileRequest>& OutDeferredTileRequests, const TSet<FVirtualTextureLocalTileRequest>& LocalTileRequests, EVTProducePageFlags Flags, ERHIFeatureLevel::Type FeatureLevel, uint32 MaxRequestsToProduce);
	TArrayView<FVTLocalTilePriorityAndIndex> SortLocalTileRequests(FConcurrentLinearBulkObjectAllocator& Allocator, const TConstArrayView<FVirtualTextureLocalTileRequest>& InLocalTileRequests);

	void GatherFeedbackRequests(FConcurrentLinearBulkObjectAllocator& Allocator, const FVirtualTextureUpdateSettings& Settings, const FVirtualTextureFeedback::FMapResult& FeedbackResult, FUniqueRequestList* MergedRequestList);
	void GatherLockedTileRequests(FUniqueRequestList* MergedRequestList);
	void GatherPackedTileRequests(FConcurrentLinearBulkObjectAllocator& Allocator, const FVirtualTextureUpdateSettings& Settings, FUniqueRequestList* MergedRequestList);

	enum class EUpdatePhase
	{
		Begin,
		End
	};

	void SubmitThrottledRequests(FRHICommandList& RHICmdList, FVirtualTextureUpdater* Updater, EUpdatePhase UpdatePhase);
	void SubmitRequests(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel, FConcurrentLinearBulkObjectAllocator& Allocator, FVirtualTextureUpdateSettings const& Settings, FUniqueRequestList* RequestList);

	void GatherRequests(FUniqueRequestList* MergedRequestList, const FUniquePageList* UniquePageList, uint32 FrameRequested, FConcurrentLinearBulkObjectAllocator& Allocator, FVirtualTextureUpdateSettings const& Settings);

	void AddPageUpdate(uint32 PhysicalSpaceID, uint16 pAddress, uint8 vLevelOffset, uint32 CurrentFrame);

	void FeedbackAnalysisTask(const FFeedbackAnalysisParameters& Parameters);
	void AddRequestedTilesTask(const FAddRequestedTilesParameters& Parameters);
	void GatherRequestsTask(const FGatherRequestsParameters& Parameters);

	void GetContinuousUpdatesToProduce(FUniqueRequestList const* RequestList, int32 MaxTilesToProduce, int32 MaxContinuousUpdates);

	void UpdateResidencyTracking() const;
	void GrowPhysicalPools() const;

#if WITH_EDITOR
	void RecordPageRequests(FUniquePageList const* UniquePageList, TMap<uint64, uint32>& OutPages);
#endif

	uint32	Frame;

	mutable UE::FRecursiveMutex Mutex;

	static const uint32 MaxNumTasks = 16;

	TArray<TUniquePtr<FVirtualTextureSpace>> Spaces;
	uint32 NumAllocatedSpaces = 0;

	TArray<FAdaptiveVirtualTexture*> AdaptiveVTs;

	TArray<FVirtualTexturePhysicalSpace*> PhysicalSpaces;
	FVirtualTextureProducerCollection Producers;

	TArray<IAllocatedVirtualTexture*> PendingDeleteAllocatedVTs;

	TMap<FAllocatedVTDescription, FAllocatedVirtualTexture*> AllocatedVTs;
	TArray<IAllocatedVirtualTexture*> AllocatedVTsToMap;
	TMap<uint32, IAllocatedVirtualTexture*> PersistentVTMap;

	bool bUpdating = false;
	bool bFlushCaches;
	void FlushCachesFromConsole();
	FAutoConsoleCommand FlushCachesCommand;

	void DumpFromConsole();
	FAutoConsoleCommand DumpCommand;

	void ListPhysicalPoolsFromConsole();
	FAutoConsoleCommand ListPhysicalPools;

	void DumpPoolUsageFromConsole();
	FAutoConsoleCommand DumpPoolUsageCommand;

#if WITH_EDITOR
	void SaveAllocatorImagesFromConsole();
	FAutoConsoleCommand SaveAllocatorImages;
#endif

	TArray<uint32> RequestedPackedTiles;

	TArray<FVirtualTextureLocalTile> TilesToLock;
	TArray<FVirtualTextureLocalTile> TilesToLockForNextFrame;
	FTexturePageLocks TileLocks;

	TSet<FVirtualTextureLocalTileRequest> ContinuousUpdateTilesToProduce;  
	TSet<FVirtualTextureLocalTileRequest> MappedTilesToProduce; 
	TSet<FVirtualTextureLocalTileRequest> TransientCollectedTilesToProduce; 
	TArray<IVirtualTextureFinalizer*> Finalizers;

#if WITH_EDITOR
	uint64 PageRequestRecordHandle;
	TMap<uint64, uint32> PageRequestRecordBuffer;
#endif
	TArray<uint32> PageRequestPlaybackBuffer;
	FIntPoint PageRequestCaptureResolution = FIntPoint::ZeroValue;

#if !UE_BUILD_SHIPPING
	void GetOnScreenMessages(FCoreDelegates::FSeverityMessageMap& OutMessages);
	FDelegateHandle OnScreenMessageDelegateHandle;
	void DrawResidencyHud(class UCanvas*, class APlayerController*);
	FDelegateHandle	DrawResidencyHudDelegateHandle;
	void UpdateCsvStats();
#endif
};
