// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContrastAdaptiveImageGenerator.h"
#include "StereoRenderTargetManager.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIDefinitions.h"
#include "RenderTargetPool.h"
#include "SystemTextures.h"
#include "SceneRendering.h"
#include "SceneTextures.h"
#include "SceneView.h"
#include "IEyeTracker.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "Engine/Engine.h"
#include "UnrealClient.h"
#include "PostProcess/PostProcessTonemap.h"
#include "DataDrivenShaderPlatformInfo.h"

/**
 * Contrast Adaptive Shading (CAS) is a Tier 2 Variable Rate Shading method which generates a shading rate image (SRI) by examining the contrast from the previous frame.
 * An image is generated which designates lower shading rates for areas of lower contrast in which reductions are unlikely to be noticed.
 * This image is then reprojected and rescaled in accordance with camera movement and dynamic resolution changes before being provided to the manager.
 */


/**
 * CAS Parameters
 */

TAutoConsoleVariable<int32> CVarCASContrastAdaptiveShading(
	TEXT("r.VRS.ContrastAdaptiveShading"),
	0,
	TEXT("Enables using Variable Rate Shading based on the luminance from the previous frame's post process output \n"),
	ECVF_RenderThreadSafe);
TAutoConsoleVariable<float> CVarCASEdgeThreshold(
	TEXT("r.VRS.ContrastAdaptiveShading.EdgeThreshold"),
	0.2f,
	TEXT(""),
	ECVF_RenderThreadSafe);
TAutoConsoleVariable<float> CVarCASConservativeEdgeThreshold(
	TEXT("r.VRS.ContrastAdaptiveShading.ConservativeEdgeThreshold"),
	0.1f,
	TEXT(""),
	ECVF_RenderThreadSafe);
TAutoConsoleVariable<float> CVarCAS_HDR10CorrectionMultiplier(
	TEXT("r.VRS.ContrastAdaptiveShading.HDR10CorrectionMultiplier"),
	0.55f,
	TEXT("Approximation multiplier to account for how perceptual values are spread out in SDR vs HDR10\n"),
	ECVF_RenderThreadSafe);

/**
 * Debug Settings 
 */

TAutoConsoleVariable<int32> CVarCASPreview(
	TEXT("r.VRS.ContrastAdaptiveShading.Preview"),
	1,
	TEXT("Whether to include CAS in VRS preview overlay.")
	TEXT("0 - off, 1 - on (default)"),
	ECVF_RenderThreadSafe);

/**
 * Shaders
 */

 //---------------------------------------------------------------------------------------------
class FCalculateShadingRateImageCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCalculateShadingRateImageCS);
	SHADER_USE_PARAMETER_STRUCT(FCalculateShadingRateImageCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSizeX = 16;
	static constexpr uint32 ThreadGroupSizeY = 8;

	class FHardwareTileSize : SHADER_PERMUTATION_SPARSE_INT("HARDWARE_TILE_SIZE", 8, 16);
	class FOutputHardwareImage : SHADER_PERMUTATION_BOOL("OUTPUT_HARDWARE_IMAGE");
	class FOutputSoftwareImage : SHADER_PERMUTATION_BOOL("OUTPUT_SOFTWARE_IMAGE");
	using FPermutationDomain = TShaderPermutationDomain<FHardwareTileSize, FOutputHardwareImage, FOutputSoftwareImage>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, LuminanceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LuminanceSampler)
		SHADER_PARAMETER(FVector4f, ViewRect)
		SHADER_PARAMETER(float, EdgeThreshold)
		SHADER_PARAMETER(float, ConservativeEdgeThreshold)
		SHADER_PARAMETER(FVector2f, InvLuminanceTextureSize)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, HardwareShadingRateImage)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, SoftwareShadingRateImage)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationDomain(Parameters.PermutationId);

		const bool bHardwareVRS = FDataDrivenShaderPlatformInfo::GetSupportsVariableRateShading(Parameters.Platform) && PermutationDomain.Get<FOutputHardwareImage>();
		const bool bSoftwareVRS = PermutationDomain.Get<FOutputSoftwareImage>();

		if (bSoftwareVRS && !IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM6))
		{
			return false;
		}

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && (bHardwareVRS || bSoftwareVRS);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), ThreadGroupSizeY);
	}

	static void InitParameters(
		FParameters& Parameters,
		FRDGTextureRef Luminance,
		const FIntRect& ViewRect,
		bool bIsHDR10,
		FRDGTextureUAV* HardwareShadingRateImage,
		FRDGTextureUAV* SoftwareShadingRateImage)
	{
		Parameters.LuminanceTexture = Luminance;
		Parameters.LuminanceSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Parameters.ViewRect = FVector4f(ViewRect.Min.X, ViewRect.Min.Y, ViewRect.Max.X, ViewRect.Max.Y);
		const float cEdgeThresholdCorrectionValue = bIsHDR10 ? CVarCAS_HDR10CorrectionMultiplier.GetValueOnRenderThread() : 1.0;
		Parameters.EdgeThreshold = cEdgeThresholdCorrectionValue * CVarCASEdgeThreshold.GetValueOnRenderThread();
		Parameters.ConservativeEdgeThreshold = cEdgeThresholdCorrectionValue * CVarCASConservativeEdgeThreshold.GetValueOnRenderThread();
		const FIntPoint LuminanceExtent = Luminance->Desc.Extent;
		Parameters.InvLuminanceTextureSize = FVector2f(1.0f / LuminanceExtent.X, 1.0f / LuminanceExtent.Y);
		Parameters.HardwareShadingRateImage = HardwareShadingRateImage;
		Parameters.SoftwareShadingRateImage = SoftwareShadingRateImage;
	}
};
IMPLEMENT_GLOBAL_SHADER(FCalculateShadingRateImageCS, "/Engine/Private/VariableRateShading/VRSShadingRateCalculate.usf", "CalculateShadingRateImage", SF_Compute);

//---------------------------------------------------------------------------------------------
class FRescaleVariableRateShadingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRescaleVariableRateShadingCS);
	SHADER_USE_PARAMETER_STRUCT(FRescaleVariableRateShadingCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSize = 8;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, InputSRITexture)
		SHADER_PARAMETER(FVector2f, TextureDimensions)
		SHADER_PARAMETER(FVector2f, InvTextureDimensions)
		SHADER_PARAMETER(FVector2f, InputSRIDimensions)
		SHADER_PARAMETER(FVector2f, ScaledSRIDimensions)
		SHADER_PARAMETER(FVector2f, ScaledUVOffset)
		SHADER_PARAMETER(float, InvDynamicResolutionScale)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, ScaledSRITexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, ScaledConservativeSRITexture)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}

	static void InitParameters(
		FParameters& Parameters,
		const FMinimalSceneTextures& SceneTextures,
		const FViewInfo& ViewInfo,
		FRDGTextureRef InputSRITexture,
		FVector2f InputSRIDimensions,
		FVector2f ScaledSRIDimensions,
		FVector2f TextureDimensions,
		FVector2f ScaledUVOffset,
		float DynamicResolutionScale,
		FRDGTextureUAVRef ScaledSRIUAV,
		FRDGTextureUAVRef ScaledConservateSRIUAV)
	{
		Parameters.SceneTextures = SceneTextures.UniformBuffer;
		Parameters.View = ViewInfo.GetShaderParameters();
		Parameters.InputSRITexture = InputSRITexture;
		Parameters.TextureDimensions = TextureDimensions;
		Parameters.InvTextureDimensions = FVector2f(1.0f / TextureDimensions.X, 1.0f / TextureDimensions.Y);
		Parameters.InputSRIDimensions = InputSRIDimensions;
		Parameters.ScaledSRIDimensions = ScaledSRIDimensions;
		Parameters.ScaledUVOffset = ScaledUVOffset;
		Parameters.InvDynamicResolutionScale = 1.0f / DynamicResolutionScale;
		Parameters.ScaledSRITexture = ScaledSRIUAV;
		Parameters.ScaledConservativeSRITexture = ScaledConservateSRIUAV;
	}
};
IMPLEMENT_GLOBAL_SHADER(FRescaleVariableRateShadingCS, "/Engine/Private/VariableRateShading/VRSShadingRateReproject.usf", "RescaleVariableRateShading", SF_Compute);

/**
 * Helper Functions and Structures
 */

namespace ESRITextureType
{
	enum Type
	{
		None = 0,
		ScaledSRIForRender,
		ScaledConservativeSRIForRender,
		ConstructedSRI,
		Num
	};
	bool IsInBounds(int32 TypeAsInt)
	{
		return TypeAsInt >= 0 && TypeAsInt < ESRITextureType::Num;
	}
	bool IsInBounds(ESRITextureType::Type TextureType)
	{
		return IsInBounds(static_cast<int32>(TextureType));
	}
	bool IsValidShadingRateTexture(int32 TextureType)
	{
		return IsInBounds(TextureType) && TextureType != ESRITextureType::None && TextureType != ESRITextureType::ConstructedSRI;
	}
	bool IsValidShadingRateTexture(ESRITextureType::Type TextureType)
	{
		return IsValidShadingRateTexture(static_cast<int32>(TextureType));
	}
	ESRITextureType::Type GetTextureType(FVariableRateShadingImageManager::EVRSImageType ImageType)
	{
		return static_cast<ESRITextureType::Type>(ImageType);
	}
};

namespace ESRIPreviewType
{
	enum Type
	{
		Off,
		Projected,
		ProjectedConservative,
		BeforeReprojection,
		Num
	};
	static const TCHAR* Names[] = {
		TEXT("Off"),
		TEXT("Projected"),
		TEXT("ProjectedConservative"),
		TEXT("BeforeReprojection"),
	};
	static const TCHAR* GetName(Type PreviewType)
	{
		if (PreviewType >= Off && PreviewType < Num)
		{
			return Names[static_cast<int32>(PreviewType)];
		}

		return TEXT("Invalid Type");
	}
};

static FIntRect GetFullPostProcessOutputRect(const FSceneViewFamily& ViewFamily)
{
	// Get initial ViewRect based on luminance texture from previous frame
	FIntRect FullLuminanceViewRect = FIntRect(0,0,0,0);
	int32 NumViews = ViewFamily.Views.Num();

	for (int32 ViewIndex = 0; ViewIndex < NumViews; ViewIndex++)
	{
		const FSceneView* View = ViewFamily.Views[ViewIndex];
		check(View->bIsViewInfo);

		if (ViewIndex == 0)
		{
			FullLuminanceViewRect = static_cast<const FViewInfo*>(View)->PrevViewInfo.LuminanceViewRectHistory;
		}
		else
		{
			FullLuminanceViewRect.Union(static_cast<const FViewInfo*>(View)->PrevViewInfo.LuminanceViewRectHistory); // Varies from luminance extent if cinematic bars are applied (constrained aspect ratio)
		}
	}

	return FullLuminanceViewRect;
}

static const TCHAR* ShadingRateTextureName = TEXT("ShadingRateTexture");
static const TCHAR* ScaledShadingRateTextureName = TEXT("ScaledShadingRateTexture");
static const TCHAR* ScaledConservativeShadingRateTextureName = TEXT("ConservativeScaledShadingRateTexture");
static const TCHAR* SoftwareShadingRateTextureName = TEXT("SoftwareShadingRateTexture");
static const TCHAR* SoftwareScaledShadingRateTextureName = TEXT("SoftwareScaledShadingRateTexture");
static const TCHAR* SoftwareScaledConservativeShadingRateTextureName = TEXT("SoftwareConservativeScaledShadingRateTexture");

struct RENDERER_API FCASImageData
{
	// Returns an FCASImageData created immutable instance from the builder blackboard. Asserts if none was created.
	static const FCASImageData& Get(FRDGBuilder& GraphBuilder)
	{
		const FCASImageData* CASImageData = GraphBuilder.Blackboard.Get<FCASImageData>();
		checkf(CASImageData, TEXT("FCASImageData was unexpectedly not initialized."));
		return *CASImageData;
	}
	static const bool IsInitialized(FRDGBuilder& GraphBuilder)
	{
		const FCASImageData* CASImageData = GraphBuilder.Blackboard.Get<FCASImageData>();
		return CASImageData != nullptr;
	}
	void Create(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, bool bCreateHardwareImages, bool bCreateSoftwareImages)
	{
		if (bCreateHardwareImages)
		{
			for (int Index = 0; Index < ViewFamily.Views.Num(); Index++)
			{
				const FViewInfo* ViewInfo = static_cast<const FViewInfo*>(ViewFamily.Views[Index]);
				FIntRect LuminanceRect = ViewInfo->PrevViewInfo.LuminanceViewRectHistory;
				FRDGTextureDesc ConstructedSRIDesc = CreateSRIDesc(ViewFamily, false, LuminanceRect, false);
				HardwareImages.ConstructedSRIArray.Add(GraphBuilder.CreateTexture(ConstructedSRIDesc, ShadingRateTextureName));
			}	

			FRDGTextureDesc ScaledSRIDesc = CreateSRIDesc(ViewFamily, true, FIntRect(), false);
			HardwareImages.ScaledSRI = GraphBuilder.CreateTexture(ScaledSRIDesc, ScaledShadingRateTextureName);
			HardwareImages.ScaledConservativeSRI = GraphBuilder.CreateTexture(ScaledSRIDesc, ScaledConservativeShadingRateTextureName);
		}
		
		if (bCreateSoftwareImages)
		{
			for (int Index = 0; Index < ViewFamily.Views.Num(); Index++)
			{
				const FViewInfo* ViewInfo = static_cast<const FViewInfo*>(ViewFamily.Views[Index]);
				FIntRect LuminanceRect = ViewInfo->PrevViewInfo.LuminanceViewRectHistory;
				FRDGTextureDesc ConstructedSRIDesc = CreateSRIDesc(ViewFamily, false, LuminanceRect, true);
				SoftwareImages.ConstructedSRIArray.Add(GraphBuilder.CreateTexture(ConstructedSRIDesc, SoftwareShadingRateTextureName));
			}

			FRDGTextureDesc ScaledSRIDesc = CreateSRIDesc(ViewFamily, true, FIntRect(), true);
			SoftwareImages.ScaledSRI = GraphBuilder.CreateTexture(ScaledSRIDesc, SoftwareScaledShadingRateTextureName);
			SoftwareImages.ScaledConservativeSRI = GraphBuilder.CreateTexture(ScaledSRIDesc, SoftwareScaledConservativeShadingRateTextureName);
		}
	}

	struct FCASImageSet
	{
		TArray<FRDGTextureRef> ConstructedSRIArray; // Initial images are created per-view
		FRDGTextureRef ScaledSRI; // Scaled and reprojected images are collated into a per-family texture matching the RT
		FRDGTextureRef ScaledConservativeSRI;
	};

	FCASImageSet HardwareImages;
	FCASImageSet SoftwareImages;

private:
	static FRDGTextureDesc CreateSRIDesc(const FSceneViewFamily& ViewFamily, bool bIsForDynResScaled, FIntRect PostProcessViewRect, bool bSoftwareVRS)
	{
		if (bIsForDynResScaled)
		{
			// Use final ViewRect for final scaled SRI
			return FVariableRateShadingImageManager::GetSRIDesc(ViewFamily, bSoftwareVRS);
		}
		else
		{
			// Use luminance ViewRect to create initial unscaled image
			const FIntPoint SRIDimensions = FMath::DivideAndRoundUp(PostProcessViewRect.Size(), FVariableRateShadingImageManager::GetSRITileSize(bSoftwareVRS));
			return FRDGTextureDesc::Create2D(
				SRIDimensions,
				bSoftwareVRS ? PF_R8_UINT : GRHIVariableRateShadingImageFormat,
				EClearBinding::ENoneBound,
				ETextureCreateFlags::DisableDCC |
				ETextureCreateFlags::ShaderResource |
				ETextureCreateFlags::UAV);
		}
	}
};
RDG_REGISTER_BLACKBOARD_STRUCT(FCASImageData);

static EDisplayOutputFormat GetDisplayOutputFormat(const FSceneView& View)
{
	FTonemapperOutputDeviceParameters Parameters = GetTonemapperOutputDeviceParameters(*View.Family);
	return (EDisplayOutputFormat)Parameters.OutputDevice;
}

void AddCreateShadingRateImagePass(
	FRDGBuilder& GraphBuilder,
	const FSceneViewFamily& ViewFamily,
	bool bCreateHardwareImages, bool bCreateSoftwareImages)
{
	FCASImageData& ImageData = GraphBuilder.Blackboard.Create<FCASImageData>();
	ImageData.Create(GraphBuilder, ViewFamily, bCreateHardwareImages, bCreateSoftwareImages);

	for (int Index = 0; Index < ViewFamily.Views.Num(); Index++)
	{
		const FViewInfo& ViewInfo = *static_cast<const FViewInfo*>(ViewFamily.Views[Index]);
		FRDGTextureRef LuminanceTexture = GraphBuilder.RegisterExternalTexture(ViewInfo.PrevViewInfo.LuminanceHistory);
		FIntRect LuminanceRect = ViewInfo.PrevViewInfo.LuminanceViewRectHistory;

		{
			// One thread per Luma texture quad
			const int32 HardwareTileSize = bCreateHardwareImages
				? FVariableRateShadingImageManager::GetSRITileSize(/*bSoftwareVRS=*/false).X
				: 8;
			const FIntPoint TileSize(FCalculateShadingRateImageCS::ThreadGroupSizeX, FCalculateShadingRateImageCS::ThreadGroupSizeY);
			const FIntPoint DispatchSize(
				FMath::DivideAndRoundUp(LuminanceRect.Size().X, 2),
				FMath::DivideAndRoundUp(LuminanceRect.Size().Y, 2));

			FCalculateShadingRateImageCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCalculateShadingRateImageCS::FHardwareTileSize>(HardwareTileSize);
			PermutationVector.Set<FCalculateShadingRateImageCS::FOutputHardwareImage>(bCreateHardwareImages);
			PermutationVector.Set<FCalculateShadingRateImageCS::FOutputSoftwareImage>(bCreateSoftwareImages);

			TShaderMapRef<FCalculateShadingRateImageCS> ComputeShader(ViewInfo.ShaderMap, PermutationVector);
			auto* PassParameters = GraphBuilder.AllocParameters<FCalculateShadingRateImageCS::FParameters>();

			const FTonemapperOutputDeviceParameters OutputDeviceParameters = GetTonemapperOutputDeviceParameters(*ViewInfo.Family);
			const EDisplayOutputFormat OutputFormat = (EDisplayOutputFormat) OutputDeviceParameters.OutputDevice;
			const bool bIsHDR10 = OutputFormat == EDisplayOutputFormat::HDR_ACES_1000nit_ST2084 
				|| OutputFormat == EDisplayOutputFormat::HDR_ACES_2000nit_ST2084;

			FCalculateShadingRateImageCS::InitParameters(
				*PassParameters,
				LuminanceTexture,
				LuminanceRect,
				bIsHDR10,
				bCreateHardwareImages ? GraphBuilder.CreateUAV(ImageData.HardwareImages.ConstructedSRIArray[Index]) : nullptr,
				bCreateSoftwareImages ? GraphBuilder.CreateUAV(ImageData.SoftwareImages.ConstructedSRIArray[Index]) : nullptr);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("CreateShadingRateImage"),
				ERDGPassFlags::AsyncCompute | ERDGPassFlags::NeverCull,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(DispatchSize, TileSize));
		}
	}
}

void AddReprojectImageBasedVRSPass(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FSceneViewFamily& ViewFamily,
	bool bReprojectSoftwareImages)
{
	SCOPED_DRAW_EVENT(GraphBuilder.RHICmdList, ScaleVariableRateShadingTexture);

	const FCASImageData& CASImageData = FCASImageData::Get(GraphBuilder);
	const FCASImageData::FCASImageSet& ImageSet = bReprojectSoftwareImages ? CASImageData.SoftwareImages : CASImageData.HardwareImages;

	FIntPoint TileSize = FVariableRateShadingImageManager::GetSRITileSize(bReprojectSoftwareImages);
	FIntPoint TextureSize = ImageSet.ScaledSRI->Desc.Extent;
	FVector2f TextureDimensions(TextureSize.X, TextureSize.Y);

	for (int32 ViewIndex = 0; ViewIndex < ViewFamily.Views.Num(); ViewIndex++)
	{
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, ViewFamily.Views.Num() > 1, "View%d", ViewIndex);

		const FViewInfo* ViewInfo = static_cast<const FViewInfo*>(ViewFamily.Views[ViewIndex]);
		
		int32 ViewportWidth = ViewInfo->ViewRect.Width();
		int32 ViewportHeight = ViewInfo->ViewRect.Height();

		int32 ScaledTilesWide = FMath::DivideAndRoundUp(ViewportWidth, TileSize.X);
		int32 ScaledTilesHigh = FMath::DivideAndRoundUp(ViewportHeight, TileSize.Y);
		FVector2f ScaledSRIDimensions(ScaledTilesWide, ScaledTilesHigh);

		// Only applies to the output SRI, since input SRIs generated in AddCreateShadingRateImagePass are always per-view and (0,0) aligned
		FVector2f UVOffset(
			static_cast<float>(ViewInfo->ViewRect.Min.X / TileSize.X) / TextureSize.X,
			static_cast<float>(ViewInfo->ViewRect.Min.Y / TileSize.Y) / TextureSize.Y);

		check(ViewInfo->PrevViewInfo.LuminanceViewRectHistory.Width());
		float DynamicResolutionScale = static_cast<float>(ViewportWidth) / (ViewInfo->PrevViewInfo.LuminanceViewRectHistory.Width());

		TShaderMapRef<FRescaleVariableRateShadingCS> RescaleVariableRateShadingCS(ViewInfo->ShaderMap);
		FRescaleVariableRateShadingCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRescaleVariableRateShadingCS::FParameters>();
		FRescaleVariableRateShadingCS::InitParameters(
			*PassParameters,
			SceneTextures,
			*ViewInfo,
			ImageSet.ConstructedSRIArray[ViewIndex],
			ImageSet.ConstructedSRIArray[ViewIndex]->Desc.Extent,
			ScaledSRIDimensions,
			TextureDimensions,
			UVOffset,
			DynamicResolutionScale,
			GraphBuilder.CreateUAV(ImageSet.ScaledSRI),
			GraphBuilder.CreateUAV(ImageSet.ScaledConservativeSRI));

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ReprojectShadingRateImage"),
			ERDGPassFlags::AsyncCompute | ERDGPassFlags::NeverCull,
			RescaleVariableRateShadingCS,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(FIntPoint(ScaledTilesWide, ScaledTilesHigh), FRescaleVariableRateShadingCS::ThreadGroupSize));
	}
}


/**
 * Interface Functions
 */

FRDGTextureRef FContrastAdaptiveImageGenerator::GetImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage)
{
	if (FCASImageData::IsInitialized(GraphBuilder))
	{
		ESRITextureType::Type TextureType = ESRITextureType::GetTextureType(ImageType);
		if (TextureType != ESRITextureType::None)
		{
			const FCASImageData& ImageData = FCASImageData::Get(GraphBuilder);
			const FCASImageData::FCASImageSet& ImageSet = bGetSoftwareImage ? ImageData.SoftwareImages : ImageData.HardwareImages;
			return (TextureType == ESRITextureType::ScaledSRIForRender) ? ImageSet.ScaledSRI : ImageSet.ScaledConservativeSRI;
		}
	}

	return nullptr;
}

void FContrastAdaptiveImageGenerator::PrepareImages(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const FMinimalSceneTextures& SceneTextures, bool bPrepareHardwareImages, bool bPrepareSoftwareImages)
{
	RDG_EVENT_SCOPE(GraphBuilder, "ContrastAdaptiveShading");

	check(!ViewFamily.Views.IsEmpty());
	check(ViewFamily.Views[0]->bIsViewInfo);
	check(bPrepareHardwareImages || bPrepareSoftwareImages);

	for (const FSceneView* View : ViewFamily.Views)
	{
		check(View->bIsViewInfo);
		const FViewInfo* ViewInfo = static_cast<const FViewInfo*>(View);
		if (View->bCameraCut || !FVariableRateShadingImageManager::IsVRSCompatibleWithView(*ViewInfo) || !ViewInfo->PrevViewInfo.LuminanceHistory)
		{
			return; // CAS is not supported unless all views are set up to support it
		}
	}

	AddCreateShadingRateImagePass(GraphBuilder, ViewFamily, bPrepareHardwareImages, bPrepareSoftwareImages);

	if (bPrepareHardwareImages)
	{
		AddReprojectImageBasedVRSPass(GraphBuilder, SceneTextures, ViewFamily, false);
	}
	if (bPrepareSoftwareImages)
	{
		AddReprojectImageBasedVRSPass(GraphBuilder, SceneTextures, ViewFamily, true);
	}
	
}

bool FContrastAdaptiveImageGenerator::IsEnabled() const
{
	return CVarCASContrastAdaptiveShading.GetValueOnRenderThread() > 0;
}

bool FContrastAdaptiveImageGenerator::IsSupportedByView(const FSceneView& View) const
{
	// See also the requirments under FVariableRateShadingImageManager::IsVRSCompatibleWithView
	// :todo-cf: Revise this to account for fractional vs. palette format if CAS support is extended to mobile
	return !IsMobilePlatform(View.GetShaderPlatform());
}

FRDGTextureRef FContrastAdaptiveImageGenerator::GetDebugImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage)
{
	if (!CVarCASPreview.GetValueOnRenderThread() || !FCASImageData::IsInitialized(GraphBuilder))
	{
		return nullptr;
	}

	ESRIPreviewType::Type PreviewType = static_cast<ESRIPreviewType::Type>(ImageType);

	const FCASImageData& ImageData = FCASImageData::Get(GraphBuilder);
	const FCASImageData::FCASImageSet& ImageSet = bGetSoftwareImage ? ImageData.SoftwareImages : ImageData.HardwareImages;

	switch (PreviewType)
	{
	case ESRIPreviewType::Projected:
		return ImageSet.ScaledSRI;
	case ESRIPreviewType::ProjectedConservative:
		return ImageSet.ScaledConservativeSRI;
	/*case ESRIPreviewType::BeforeReprojection:
		return ImageSet.ConstructedSRI;  */
	}

	return nullptr;
}

