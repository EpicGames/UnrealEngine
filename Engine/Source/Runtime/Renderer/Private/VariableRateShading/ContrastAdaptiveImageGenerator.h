// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RenderGraphDefinitions.h"
#include "VariableRateShadingImageManager.h"

class FContrastAdaptiveImageGenerator : public IVariableRateShadingImageGenerator
{
public:
	virtual ~FContrastAdaptiveImageGenerator() override = default;
	virtual FRDGTextureRef GetImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage = false) override;
	virtual void PrepareImages(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const FMinimalSceneTextures& SceneTextures, bool bPrepareHardwareImages, bool bPrepareSoftwareImages) override;
	virtual bool IsEnabled() const override;
	virtual bool IsSupportedByView(const FSceneView& View) const override;
	virtual FVariableRateShadingImageManager::EVRSSourceType GetType() const override
	{
		return FVariableRateShadingImageManager::EVRSSourceType::ContrastAdaptiveShading;
	}
	virtual FRDGTextureRef GetDebugImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage = false) override;
};

