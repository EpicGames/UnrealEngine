// Copyright Epic Games, Inc. All Rights Reserved.

#include "FoveatedImageGenerator.h"
#include "SystemTextures.h"
#include "GlobalShader.h"
#include "IXRTrackingSystem.h"
#include "StereoRendering.h"
#include "SceneView.h"
#include "UnrealClient.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneTexturesConfig.h"
#include "IEyeTracker.h"
#include "IHeadMountedDisplay.h"
#include "SceneRendering.h"
#include "UnrealEngine.h"

/* CVar values used to control generator behavior */

static TAutoConsoleVariable<int> CVarFoveationLevel(
	TEXT("xr.VRS.FoveationLevel"),
	0,
	TEXT("Level of foveated VRS to apply (when Variable Rate Shading is available)\n")
	TEXT(" 0: Disabled (default);\n")
	TEXT(" 1: Low;\n")
	TEXT(" 2: Medium;\n")
	TEXT(" 3: High;\n")
	TEXT(" 4: High Top;\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDynamicFoveation(
	TEXT("xr.VRS.DynamicFoveation"),
	0,
	TEXT("Whether foveation level should adjust based on GPU utilization\n")
	TEXT(" 0: Disabled (default);\n")
	TEXT(" 1: Enabled\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFoveationPreview(
	TEXT("xr.VRS.FoveationPreview"),
	1,
	TEXT("Include foveated VRS in the VRS debug overlay.")
	TEXT(" 0: Disabled;\n")
	TEXT(" 1: Enabled (default)\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarGazeTrackedFoveation(
	TEXT("xr.VRS.GazeTrackedFoveation"),
	0,
	TEXT("Enable gaze-tracking for foveated VRS\n")
	TEXT(" 0: Disabled (default);\n")
	TEXT(" 1: Enabled\n"),
	ECVF_RenderThreadSafe);


/* Image generation parameters, set up by Prepare() */
FVector2f HMDFieldOfView = FVector2f(90.0f, 90.0f);


/* Shader parameters and parameter structs */

constexpr int32 kComputeGroupSize = FComputeShaderUtils::kGolden2DGroupSize;
constexpr int32 kMaxCombinedSources = 4;

enum class EVRSGenerationFlags : uint32
{
	None = 0x0,
	StereoRendering = 0x1,
	SideBySideStereo = 0x2,
};

ENUM_CLASS_FLAGS(EVRSGenerationFlags);


/* The shader itself, which generates the foveated shading rate image */

class FComputeVariableRateShadingImageGeneration : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FComputeVariableRateShadingImageGeneration);
	SHADER_USE_PARAMETER_STRUCT(FComputeVariableRateShadingImageGeneration, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWOutputTexture)
		SHADER_PARAMETER(FVector2f, HMDFieldOfView)
		SHADER_PARAMETER(FVector2f, LeftEyeCenterPixelXY)
		SHADER_PARAMETER(FVector2f, RightEyeCenterPixelXY)
		SHADER_PARAMETER(float, ViewDiagonalSquaredInPixels)
		SHADER_PARAMETER(float, FoveationFullRateCutoffSquared)
		SHADER_PARAMETER(float, FoveationHalfRateCutoffSquared)
		SHADER_PARAMETER(uint32, ShadingRateAttachmentGenerationFlags)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsVariableRateShading(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// Shading rates:
		OutEnvironment.SetDefine(TEXT("SHADING_RATE_1x1"), VRSSR_1x1);
		OutEnvironment.SetDefine(TEXT("SHADING_RATE_1x2"), VRSSR_1x2);
		OutEnvironment.SetDefine(TEXT("SHADING_RATE_2x1"), VRSSR_2x1);
		OutEnvironment.SetDefine(TEXT("SHADING_RATE_2x2"), VRSSR_2x2);
		OutEnvironment.SetDefine(TEXT("SHADING_RATE_2x4"), VRSSR_2x4);
		OutEnvironment.SetDefine(TEXT("SHADING_RATE_4x2"), VRSSR_4x2);
		OutEnvironment.SetDefine(TEXT("SHADING_RATE_4x4"), VRSSR_4x4);

		OutEnvironment.SetDefine(TEXT("MAX_COMBINED_SOURCES_IN"), kMaxCombinedSources);
		OutEnvironment.SetDefine(TEXT("SHADING_RATE_TILE_WIDTH"), GRHIVariableRateShadingImageTileMinWidth);
		OutEnvironment.SetDefine(TEXT("SHADING_RATE_TILE_HEIGHT"), GRHIVariableRateShadingImageTileMinHeight);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), kComputeGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), kComputeGroupSize);

		OutEnvironment.SetDefine(TEXT("STEREO_RENDERING"), (uint32)EVRSGenerationFlags::StereoRendering);
		OutEnvironment.SetDefine(TEXT("SIDE_BY_SIDE_STEREO"), (uint32)EVRSGenerationFlags::SideBySideStereo);
	}
};

IMPLEMENT_GLOBAL_SHADER(FComputeVariableRateShadingImageGeneration, "/Engine/Private/VariableRateShading/VRSShadingRateFoveated.usf", "GenerateShadingRateTexture", SF_Compute);

FRDGTextureRef FFoveatedImageGenerator::GetImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage)
{
	// Generator only supports up to two side-by-side views
	if (ImageType == FVariableRateShadingImageManager::EVRSImageType::Disabled || ViewInfo.StereoViewIndex > 1 || bGetSoftwareImage)
	{
		return nullptr;
	}
	else
	{
		return CachedImage;
	}
}

void FFoveatedImageGenerator::PrepareImages(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const FMinimalSceneTextures& SceneTextures, bool bPrepareHardwareImages, bool bPrepareSoftwareImages)
{
	if (!bPrepareHardwareImages)
	{
		// Software images unsupported for now
		return;
	}

	// VRS level parameters - pretty arbitrary right now, later should depend on device characteristics
	static const TArray<float> kFoveationFullRateCutoffs = { 1.0f, 0.7f, 0.50f, 0.35f, 0.35f };
	static const TArray<float> kFoveationHalfRateCutoffs = { 1.0f, 0.9f, 0.75f, 0.55f, 0.55f };
	static const TArray<float> kFixedFoveationCenterX = { 0.5f, 0.5f, 0.5f, 0.5f, 0.5f };
	static const TArray<float> kFixedFoveationCenterY = { 0.5f, 0.5f, 0.5f, 0.5f, 0.42f };

	const int VRSMaxLevel = FMath::Clamp(CVarFoveationLevel.GetValueOnAnyThread(), 0, 4);

	// Set up dynamic VRS amount based on target framerate and use it to interpolate cutoffs (default to max level)
	float VRSDynamicAmount = 1.0f;
	if (CVarDynamicFoveation.GetValueOnAnyThread() && VRSMaxLevel > 0)
	{
		VRSDynamicAmount = UpdateDynamicVRSAmount();
	}

	if (VRSMaxLevel <= 0 || VRSDynamicAmount <= 0)
	{
		CachedImage = nullptr; // Early out if no VRS requested
		return;
	}

	const float FoveationFullRateCutoff = FMath::Lerp(kFoveationFullRateCutoffs[0], kFoveationFullRateCutoffs[VRSMaxLevel], VRSDynamicAmount);
	const float FoveationHalfRateCutoff = FMath::Lerp(kFoveationHalfRateCutoffs[0], kFoveationHalfRateCutoffs[VRSMaxLevel], VRSDynamicAmount);

	// Default to fixed center point
	float FoveationCenterX = kFixedFoveationCenterX[VRSMaxLevel];
	float FoveationCenterY = kFixedFoveationCenterY[VRSMaxLevel];

	// If gaze data is available and gaze-tracking is enabled, adjust foveation center point
	if (IsGazeTrackingEnabled())
	{
		TSharedPtr<IEyeTracker> EyeTracker = GEngine->EyeTrackingDevice;
		FEyeTrackerGazeData GazeData = FEyeTrackerGazeData();

		// Only use gaze if we have confident gaze data, otherwise stick with fixed
		if (EyeTracker->GetEyeTrackerGazeData(GazeData) && GazeData.ConfidenceValue > 0.5f)
		{
			// Get gaze orientation relative to HMD view
			FQuat ViewOrientation;
			FVector ViewPosition;
			GEngine->XRSystem->GetCurrentPose(0, ViewOrientation, ViewPosition);
			const FVector RelativeGazeDirection = ViewOrientation.UnrotateVector(GazeData.GazeDirection);

			// Switch from Unreal coordinate system to X right, Y up, Z forward before projecting
			const FMatrix StereoProjectionMatrix = GEngine->StereoRenderingDevice->GetStereoProjectionMatrix(0);
			const FVector4 GazePoint = StereoProjectionMatrix.GetTransposed().TransformVector(FVector(RelativeGazeDirection.Y, RelativeGazeDirection.Z, RelativeGazeDirection.X));

			// Transform into 0-1 screen coordinates. 0,0 is top left.  
			FoveationCenterX = 0.5f + GazePoint.X / 2.0f;
			FoveationCenterY = 0.5f - GazePoint.Y / 2.0f;

		}
	}

	// Sanity check VRS tile size.
	check(GRHIVariableRateShadingImageTileMinWidth >= 8 && GRHIVariableRateShadingImageTileMinWidth <= 64 && GRHIVariableRateShadingImageTileMinHeight >= 8 && GRHIVariableRateShadingImageTileMinHeight <= 64);

	// Create texture to hold shading rate image
	FRDGTextureDesc Desc = FVariableRateShadingImageManager::GetSRIDesc(ViewFamily);
	FRDGTextureRef ShadingRateTexture = GraphBuilder.CreateTexture(Desc, TEXT("FoveatedShadingRateTexture"));

	// Setup shader parameters and flags
	FComputeVariableRateShadingImageGeneration::FParameters* PassParameters = GraphBuilder.AllocParameters<FComputeVariableRateShadingImageGeneration::FParameters>();
	PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(ShadingRateTexture);

	PassParameters->HMDFieldOfView = HMDFieldOfView;

	PassParameters->FoveationFullRateCutoffSquared = FoveationFullRateCutoff * FoveationFullRateCutoff;
	PassParameters->FoveationHalfRateCutoffSquared = GRHISupportsLargerVariableRateShadingSizes ? (FoveationHalfRateCutoff * FoveationHalfRateCutoff) : 2.0f; // Set cutoff to outside screen edge if quarter-rate is unsupported

	PassParameters->LeftEyeCenterPixelXY = FVector2f(Desc.Extent.X * FoveationCenterX, Desc.Extent.Y * FoveationCenterY);
	PassParameters->RightEyeCenterPixelXY = PassParameters->LeftEyeCenterPixelXY;

	EVRSGenerationFlags GenFlags = EVRSGenerationFlags::None;

	// If stereo is enabled, side-by-side is assumed for now, may need to change this if we add support for mobile multi-view
	if (IStereoRendering::IsStereoEyeView(*ViewFamily.Views[0]))
	{
		EnumAddFlags(GenFlags, EVRSGenerationFlags::StereoRendering);
		EnumAddFlags(GenFlags, EVRSGenerationFlags::SideBySideStereo); 

		// Adjust eyes for side-by-side stereo and/or quadview
		PassParameters->LeftEyeCenterPixelXY.X /= ViewFamily.Views.Num();;
		PassParameters->RightEyeCenterPixelXY.X = PassParameters->LeftEyeCenterPixelXY.X + Desc.Extent.X / ViewFamily.Views.Num();
	}

	// Set up remaining parameters
	const FVector2f ViewCenterPoint = FVector2f(Desc.Extent.X / ViewFamily.Views.Num() * 0.5f, Desc.Extent.Y * 0.5f);
	PassParameters->ViewDiagonalSquaredInPixels = FVector2f::DotProduct(ViewCenterPoint, ViewCenterPoint);
	PassParameters->ShadingRateAttachmentGenerationFlags = (uint32)GenFlags;

	TShaderMapRef<FComputeVariableRateShadingImageGeneration> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel)); // If in stereo, shader sets up a single side-by-side image for both eyes at once
	FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(Desc.Extent, FComputeShaderUtils::kGolden2DGroupSize);
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("GenerateFoveatedVRSImage"),
		PassParameters,
		ERDGPassFlags::Compute,
		[PassParameters, ComputeShader, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *PassParameters, GroupCount);
		});

	CachedImage = ShadingRateTexture;
}

bool FFoveatedImageGenerator::IsEnabled() const
{
	return CVarFoveationLevel.GetValueOnRenderThread() > 0;
}

bool FFoveatedImageGenerator::IsSupportedByView(const FSceneView& View) const
{
	// On desktop, we always use palette rates, but may want this available for Software VRS even if DataType == VRSImage_Unsupported
	// On mobile, we may use palette or fractional rates, and this generator currently only supports palette rates
	return !IsMobilePlatform(View.GetShaderPlatform()) || GRHIVariableRateShadingImageDataType == VRSImage_Palette;
}

FVariableRateShadingImageManager::EVRSSourceType FFoveatedImageGenerator::GetType() const
{
	return IsGazeTrackingEnabled() ? FVariableRateShadingImageManager::EVRSSourceType::FixedFoveation : FVariableRateShadingImageManager::EVRSSourceType::EyeTrackedFoveation;
}

FRDGTextureRef FFoveatedImageGenerator::GetDebugImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage)
{
	if (CVarFoveationPreview.GetValueOnRenderThread() && ImageType != FVariableRateShadingImageManager::EVRSImageType::Disabled)
	{
		return CachedImage;
	}
	else
	{
		return nullptr;
	}
}

float FFoveatedImageGenerator::UpdateDynamicVRSAmount()
{
	const float		kUpdateIncrement = 0.1f;
	const int		kNumFramesToAverage = 10;
	const double	kRoundingErrorMargin = 0.5; // Add a small margin (.5 ms) to avoid oscillation
	const double	kDesktopMaxAllowedFrameTime = 12.50; // 80 fps
	const double	kMobileMaxAllowedFrameTime = 16.66; // 60 fps

	if (GFrameNumber != DynamicVRSData.LastUpdateFrame)
	{
		DynamicVRSData.LastUpdateFrame = GFrameNumber;

		const double GPUFrameTime = FPlatformTime::ToMilliseconds(RHIGetGPUFrameCycles());

		// Update sum for rolling average
		if (DynamicVRSData.NumFramesStored < kNumFramesToAverage)
		{
			DynamicVRSData.SumBusyTime += GPUFrameTime;
			DynamicVRSData.NumFramesStored++;
		}
		else if (DynamicVRSData.NumFramesStored == kNumFramesToAverage)
		{
			DynamicVRSData.SumBusyTime -= DynamicVRSData.SumBusyTime / kNumFramesToAverage;
			DynamicVRSData.SumBusyTime += GPUFrameTime;
		}

		// If rolling average has sufficient samples, check if update is necessary
		if (DynamicVRSData.NumFramesStored == kNumFramesToAverage)
		{
			const double AverageBusyTime = DynamicVRSData.SumBusyTime / kNumFramesToAverage;
			const double TargetBusyTime = IsMobilePlatform(GShaderPlatformForFeatureLevel[GMaxRHIFeatureLevel]) ? kMobileMaxAllowedFrameTime : kDesktopMaxAllowedFrameTime;

			if (AverageBusyTime > TargetBusyTime + kRoundingErrorMargin && DynamicVRSData.VRSAmount < 1.0f)
			{
				DynamicVRSData.VRSAmount = FMath::Clamp(DynamicVRSData.VRSAmount + kUpdateIncrement, 0.0f, 1.0f);
				DynamicVRSData.SumBusyTime = 0.0;
				DynamicVRSData.NumFramesStored = 0;
			}
			else if (AverageBusyTime < TargetBusyTime - kRoundingErrorMargin && DynamicVRSData.VRSAmount > 0.0f)
			{
				DynamicVRSData.VRSAmount = FMath::Clamp(DynamicVRSData.VRSAmount - kUpdateIncrement, 0.0f, 1.0f);
				DynamicVRSData.SumBusyTime = 0.0;
				DynamicVRSData.NumFramesStored = 0;
			}
		}
	}

	return DynamicVRSData.VRSAmount;
}

bool FFoveatedImageGenerator::IsGazeTrackingEnabled() const
{
	return CVarGazeTrackedFoveation.GetValueOnAnyThread()
		&& GEngine->EyeTrackingDevice.IsValid()
		&& GEngine->StereoRenderingDevice.IsValid();
}