// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RenderGraphDefinitions.h"
#include "VariableRateShadingImageManager.h"

class FFoveatedImageGenerator : public IVariableRateShadingImageGenerator
{
public:
	virtual ~FFoveatedImageGenerator() override = default;
	virtual FRDGTextureRef GetImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage = false) override;
	virtual void PrepareImages(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const FMinimalSceneTextures& SceneTextures, bool bPrepareHardwareImages, bool bPrepareSoftwareImages) override;
	virtual bool IsEnabled() const override;
	virtual bool IsSupportedByView(const FSceneView& View) const override;
	virtual FVariableRateShadingImageManager::EVRSSourceType GetType() const override;
	virtual FRDGTextureRef GetDebugImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage = false) override;
private:
	FRDGTextureRef CachedImage = nullptr;
	struct FDynamicVRSData
	{
		float	VRSAmount = 1.0f;
		double	SumBusyTime = 0.0;
		int		NumFramesStored = 0;
		uint32	LastUpdateFrame = 0;
	} DynamicVRSData;
	float UpdateDynamicVRSAmount();
	bool IsGazeTrackingEnabled() const;
};
