// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariableRateShadingImageManager.h"
#include "FoveatedImageGenerator.h"
#include "ContrastAdaptiveImageGenerator.h"
#include "StereoRenderTargetManager.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "RenderGraphUtils.h"
#include "RHIDefinitions.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderTargetPool.h"
#include "SceneRendering.h"
#include "SystemTextures.h"
#include "SceneRendering.h"
#include "SceneView.h"
#include "IEyeTracker.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "Engine/Engine.h"
#include "UnrealClient.h"
#include "PostProcess/PostProcessTonemap.h"


TGlobalResource<FVariableRateShadingImageManager> GVRSImageManager;

DEFINE_LOG_CATEGORY(LogVRS);

/**
 * Basic CVars
 */

TAutoConsoleVariable<int32> CVarSupportVRS(
	TEXT("r.VRS.Support"),
	1,
	TEXT("Toggles support for hardware Variable Rate Shading. Requires shader recompilation.")
	TEXT("0: Off, 1: On"),
	ECVF_ReadOnly);

void CVarEnableVRSCallback(IConsoleVariable* Var)
{
	const int32 Value = Var->GetInt();

	// If pipeline VRS is enabled, we need to update static meshes using per-material rates to reflect a change
	if (GRHISupportsPipelineVariableRateShading)
	{
		GVRSImageManager.SetNeedStaticMeshUpdate(true);
	}
}

TAutoConsoleVariable<int32> CVarEnableVRS(
	TEXT("r.VRS.Enable"),
	0,
	TEXT("Enables hardware Variable Rate Shading and Shading Rate Image generation (8x8 or 16x16 tile size).")
	TEXT("0: Off, 1: On"),
	FConsoleVariableDelegate::CreateStatic(&CVarEnableVRSCallback),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarEnableVRSSoftwareImage(
	TEXT("r.VRS.EnableSoftware"),
	0,
	TEXT("Enables software (2x2 tile size) Shading Rate Image generation for use with Nanite Software VRS. Allows generating iamges even when r.VRS.Enable/r.VRS.Support=0 or Tier 2 VRS is unsupported by the hardware.")
	TEXT("Image generation will only be enabled if r.Nanite.SoftwareVRS is also set to 1.")
	TEXT("0: Off, 1: On"),
	ECVF_RenderThreadSafe);

void CVarVRSPreviewCallback(IConsoleVariable* Var)
{
	const int32 RequestedPreview = Var->GetInt();
	if (RequestedPreview < 0 || RequestedPreview > 4)
	{
		UE_LOGF(LogVRS, Warning, "Selected invalid preview mode, disabling preview");
	}
}

TAutoConsoleVariable<int32> CVarVRSPreview(
	TEXT("r.VRS.Preview"),
	0,
	TEXT("Show a debug visualization of the VRS shading rate image texture. Conservative and software images are only available via Contrast Adaptive Shading.")
	TEXT("0 - off, 1 - full (hardware), 2 - conservative (hardware), 3 - full (software), 4 - conservative (software)"),
	FConsoleVariableDelegate::CreateStatic(&CVarVRSPreviewCallback),
	ECVF_RenderThreadSafe);

void CVarVRSDebugForceRateCallback(IConsoleVariable* Var)
{
	const int32 RequestedDebugForceRate = Var->GetInt();
	const int32 NumberOfAvailableRates = FVariableRateShadingImageManager::GetNumberOfSupportedRates();
	if (RequestedDebugForceRate >= NumberOfAvailableRates) 
	{
		UE_LOGF(LogVRS, Warning, "Selected forced shading rate exceeds maximum available, defaulting to %ls", GRHISupportsLargerVariableRateShadingSizes ? TEXT("4x4") : TEXT("2x2"));
	}
}

int GVRSDebugForceRate = -1;
FAutoConsoleVariableRef CVarVRSDebugForceRate(
	TEXT("r.VRS.DebugForceRate"),
	GVRSDebugForceRate,
	TEXT("-1: None, 0: Force 1x1, 1: Force 1x2, 2: Force 2x1, 3: Force 2x2, 4: Force 2x4, 5: Force 4x2, 6: Force 4x4"),
	FConsoleVariableDelegate::CreateStatic(&CVarVRSDebugForceRateCallback),
	ECVF_RenderThreadSafe);


/**
 * Pass Settings
 */

static void CVarVRSImagePassTypeCallback(IConsoleVariable* Var)
{
	const int32 RequestedImageType = Var->GetInt();
	if (RequestedImageType < 0 || RequestedImageType >= FVariableRateShadingImageManager::EVRSPassType::Num)
	{
		UE_LOGF(LogVRS, Warning, "Selected invalid image type, disabling VRS for pass");
	}
}

TAutoConsoleVariable<int32> CVarVRSBasePass(
	TEXT("r.VRS.BasePass"),
	2,
	TEXT("Whether to apply screen space VRS to the base pass.\n")
	TEXT("0: Disabled\n")
	TEXT("1: Full\n")
	TEXT("2: Conservative (default)"),
	FConsoleVariableDelegate::CreateStatic(&CVarVRSImagePassTypeCallback),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarVRSTranslucency(
	TEXT("r.VRS.Translucency"),
	1,
	TEXT("Whether to apply screen space VRS to the translucency pass.\n")
	TEXT("0: Disabled\n")
	TEXT("1: Full (default)\n")
	TEXT("2: Conservative"),
	FConsoleVariableDelegate::CreateStatic(&CVarVRSImagePassTypeCallback),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarVRSNaniteEmitGBuffer(
	TEXT("r.VRS.NaniteEmitGBuffer"),
	2,
	TEXT("Whether to apply screen space VRS to Nanite.\n")
	TEXT("0: Disabled\n")
	TEXT("1: Full\n")
	TEXT("2: Conservative (default)"),
	FConsoleVariableDelegate::CreateStatic(&CVarVRSImagePassTypeCallback),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarVRS_SSAO(
	TEXT("r.VRS.SSAO"),
	0,
	TEXT("Whether to apply screen space VRS to SSAO (not currently recommended).\n")
	TEXT("0: Disabled (default)\n")
	TEXT("1: Full\n")
	TEXT("2: Conservative"),
	FConsoleVariableDelegate::CreateStatic(&CVarVRSImagePassTypeCallback),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarVRS_SSR(
	TEXT("r.VRS.SSR"),
	2,
	TEXT("Whether to apply screen space VRS to screen space reflections (only supported when r.SSR.Compute=0).\n")
	TEXT("0: Disabled\n")
	TEXT("1: Full\n")
	TEXT("2: Conservative (default)"),
	FConsoleVariableDelegate::CreateStatic(&CVarVRSImagePassTypeCallback),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarVRSReflectionEnvironmentSky(
	TEXT("r.VRS.ReflectionEnvironmentSky"),
	2,
	TEXT("Whether to apply screen space VRS to reflections and indirect sky lighting (only supported when Lumen is disabled).\n")
	TEXT("0: Disabled\n")
	TEXT("1: Full\n")
	TEXT("2: Conservative (default)"),
	FConsoleVariableDelegate::CreateStatic(&CVarVRSImagePassTypeCallback),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarVRSLightFunctions(
	TEXT("r.VRS.LightFunctions"),
	1,
	TEXT("Whether to apply screen space VRS to light functions.\n")
	TEXT("0: Disabled\n")
	TEXT("1: Full (default)\n")
	TEXT("2: Conservative"),
	FConsoleVariableDelegate::CreateStatic(&CVarVRSImagePassTypeCallback),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarVRSDecals(
	TEXT("r.VRS.Decals"),
	2,
	TEXT("Whether to apply screen space VRS to decals.\n")
	TEXT("0: Disabled\n")
	TEXT("1: Full\n")
	TEXT("2: Conservative (default)"),
	FConsoleVariableDelegate::CreateStatic(&CVarVRSImagePassTypeCallback),
	ECVF_RenderThreadSafe);

/**
 * Shaders
 */

namespace VRSHelpers
{
	constexpr int32 kCombineGroupSize = FComputeShaderUtils::kGolden2DGroupSize;
}

// If false, we are using fragment shading rate images (selects rates using a palette)
// If true, we are using fragment density map images (suggests rates with floats as a percentage of full resolution)
class FFragmentDensityMap : SHADER_PERMUTATION_BOOL("FRAGMENT_DENSITY_MAP");

class FCombineShadingRateTexturesCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCombineShadingRateTexturesCS);
	SHADER_USE_PARAMETER_STRUCT(FCombineShadingRateTexturesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWOutputTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTexture0)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTexture1)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<FFragmentDensityMap>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsVariableRateShading(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_XY"), VRSHelpers::kCombineGroupSize);
	}

};

IMPLEMENT_GLOBAL_SHADER(FCombineShadingRateTexturesCS, "/Engine/Private/VariableRateShading/VRSShadingRateCombine.usf", "CombineShadingRateTextures", SF_Compute);

//---------------------------------------------------------------------------------------------
using FDebugVariableRateShadingVS = FScreenPassVS;

//---------------------------------------------------------------------------------------------
class FDebugVariableRateShadingPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDebugVariableRateShadingPS);
	SHADER_USE_PARAMETER_STRUCT(FDebugVariableRateShadingPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VariableRateShadingTextureIn)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<FFragmentDensityMap>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDebugVariableRateShadingPS, "/Engine/Private/VariableRateShading/VRSShadingRatePreview.usf", "PreviewVariableRateShadingTexturePS", SF_Pixel);


/**
 * Public functions
 */

FVariableRateShadingImageManager::FVariableRateShadingImageManager()
	: FRenderResource()
{
	InternalGenerators.Add(MakeUnique<FFoveatedImageGenerator>());
	InternalGenerators.Add(MakeUnique<FContrastAdaptiveImageGenerator>());

	FWriteScopeLock GeneratorsLock(GeneratorsMutex);
	for (const TUniquePtr<IVariableRateShadingImageGenerator>& Generator : InternalGenerators)
	{
		ImageGenerators.Add(Generator.Get());
	}
}

FVariableRateShadingImageManager::~FVariableRateShadingImageManager() {}

void FVariableRateShadingImageManager::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (GRHISupportsPipelineVariableRateShading && GRHISupportsAttachmentVariableRateShading)
	{
		UE_LOGF(LogVRS, Log, "Current RHI supports per-draw and screenspace Variable Rate Shading");
	}
	else if (GRHISupportsPipelineVariableRateShading)
	{
		UE_LOGF(LogVRS, Log, "Current RHI supports per-draw Variable Rate Shading");
	}
	else if (GRHISupportsAttachmentVariableRateShading)
	{
		UE_LOGF(LogVRS, Log, "Current RHI supports screenspace Variable Rate Shading");
	}
	else
	{
		UE_LOGF(LogVRS, Log, "Current RHI does not support Variable Rate Shading");
	}
}

bool FVariableRateShadingImageManager::IsHardwareVRSSupported()
{
	return (GRHISupportsPipelineVariableRateShading || GRHISupportsAttachmentVariableRateShading);
}

bool FVariableRateShadingImageManager::IsSoftwareVRSSupported()
{
	return IsFeatureLevelSupported(GMaxRHIShaderPlatform, ERHIFeatureLevel::SM6);
}

void FVariableRateShadingImageManager::ReleaseRHI()
{
	GRenderTargetPool.FreeUnusedResources();
}

bool FVariableRateShadingImageManager::IsPipelineVRSEnabled() const
{
	// This GRHI should be forced to false in DynamicRHI::Init() if r.VRS.Support=0 or bSupportsVariableRateShading=false in the platform's DDPI
	return GRHISupportsPipelineVariableRateShading && CVarEnableVRS.GetValueOnRenderThread() != 0;
}

bool FVariableRateShadingImageManager::IsAttachmentVRSEnabled() const
{
	// Ditto
	return GRHISupportsAttachmentVariableRateShading && CVarEnableVRS.GetValueOnRenderThread() != 0;
}

bool FVariableRateShadingImageManager::IsVRSEnabledForFrame() const
{
	return bHardwareVRSEnabledForFrame || bSoftwareVRSEnabledForFrame;
}

bool FVariableRateShadingImageManager::IsHardwareVRSEnabledForFrame() const
{
	return bHardwareVRSEnabledForFrame;
}

bool FVariableRateShadingImageManager::IsSoftwareVRSEnabledForFrame() const
{
	return bSoftwareVRSEnabledForFrame;
}

bool FVariableRateShadingImageManager::GetNeedStaticMeshUpdate() const
{
	return bNeedStaticMeshUpdate;
}

void FVariableRateShadingImageManager::SetNeedStaticMeshUpdate(bool bInNeedStaticMeshUpdate)
{
	bNeedStaticMeshUpdate = bInNeedStaticMeshUpdate;
}

bool FVariableRateShadingImageManager::IsVRSCompatibleWithOutputType(const EDisplayOutputFormat& OutputFormat)
{
	return OutputFormat == EDisplayOutputFormat::SDR_sRGB
		|| OutputFormat == EDisplayOutputFormat::SDR_Rec709
		|| OutputFormat == EDisplayOutputFormat::SDR_ExplicitGammaMapping
		|| OutputFormat == EDisplayOutputFormat::HDR_ACES_1000nit_ST2084
		|| OutputFormat == EDisplayOutputFormat::HDR_ACES_2000nit_ST2084;
}

bool FVariableRateShadingImageManager::IsVRSCompatibleWithView(const FViewInfo& ViewInfo)
{
	FTonemapperOutputDeviceParameters Parameters = GetTonemapperOutputDeviceParameters(*ViewInfo.Family);

	return !ViewInfo.bIsSceneCapture
		&& ViewInfo.Family->bRealtimeUpdate
		&& IsVRSCompatibleWithOutputType((EDisplayOutputFormat)Parameters.OutputDevice);
}

FIntPoint FVariableRateShadingImageManager::GetSRITileSize(bool bSoftwareVRS)
{
	return bSoftwareVRS ? FIntPoint(2, 2) : FIntPoint(GRHIVariableRateShadingImageTileMinWidth, GRHIVariableRateShadingImageTileMinHeight);
}

FRDGTextureDesc FVariableRateShadingImageManager::GetSRIDesc(const FSceneViewFamily& ViewFamily, bool bSoftwareVRS)
{
	check(!ViewFamily.Views.IsEmpty())
	check(ViewFamily.Views[0]->bIsViewInfo);

	const FViewInfo* ViewInfo = static_cast<const FViewInfo*>(ViewFamily.Views[0]);
	const FIntRect FamilyViewRect = ViewInfo->GetFamilyViewRect(); // May vary from the size of the scene textures if using constrained aspect ratios
	const FIntPoint SRISize = FMath::DivideAndRoundUp(FamilyViewRect.Size(), GetSRITileSize(bSoftwareVRS));

	ETextureCreateFlags Flags = TexCreate_UAV | TexCreate_ShaderResource | TexCreate_DisableDCC;
	if (GRHIGlobals.VariableRateShading.SupportsAttachment)
	{
		Flags |= TexCreate_Foveation;
	}

	return FRDGTextureDesc::Create2D(
		SRISize,
		bSoftwareVRS ? PF_R8_UINT : GRHIVariableRateShadingImageFormat,
		FClearValueBinding::None,
		Flags);
}

int32 FVariableRateShadingImageManager::GetNumberOfSupportedRates()
{
	// When using palette shading rate images, we will always support the four rates 1x1, 1x2, 2x1, and 2x2.
	// If the RHI supports larger rates, we can also use 2x4, 4x2, and 4x4, for 7 total.
	static const int32 NumBaseRates = 4;
	static const int32 NumExpandedRates = 7;

	// When using fractional shading rate images, we still use a palette rate to set r.VRS.DebugForceRate. The selected palette rate is automatically converted into a fractional rate.
	// As such, we should allow selecting expanded rates so we can fully represent fractional rates 0.25 (4x), 0.5 (2x), and 1.0 (1x).
	return (GRHISupportsLargerVariableRateShadingSizes || GRHIVariableRateShadingImageDataType == VRSImage_Fractional) ? NumExpandedRates : NumBaseRates;
}

FRDGTextureRef FVariableRateShadingImageManager::GetVariableRateShadingImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSPassType PassType, bool bRequestSoftwareImage)
{
	// If the view doesn't support VRS or this pass is disabled, bail immediately
	if (!IsVRSEnabledForFrame())
	{
		return nullptr;
	}

	EVRSImageType ImageType = GetImageTypeFromPassType(PassType);
	if (ImageType == EVRSImageType::Disabled || !IsVRSCompatibleWithView(ViewInfo))
	{
		return nullptr;
	}

	// Use debug rate if provided, otherwise bail if no generators available
	if (VRSForceRateForFrame >= 0)
	{
		return GetForceRateImage(GraphBuilder, *ViewInfo.Family, VRSForceRateForFrame, bRequestSoftwareImage);
	}

	if (ActiveGenerators.IsEmpty())
	{
		return nullptr;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "GetVariableRateShadingImage");
	SCOPED_NAMED_EVENT(GetVariableRateShadingImage, FColor::Yellow);

	// Collate all internal sources
	TArray<FRDGTextureRef> InternalVRSSources;
	for (IVariableRateShadingImageGenerator* const Generator : ActiveGenerators)
	{
		const bool bGetSoftwareImage = bSoftwareVRSEnabledForFrame && bRequestSoftwareImage;

		FRDGTextureRef Image = nullptr;
		if (Generator && Generator->IsSupportedByView(ViewInfo))
		{
			Image = Generator->GetImage(GraphBuilder, ViewInfo, ImageType, bGetSoftwareImage);
		}

		if (Image)
		{
			InternalVRSSources.Add(Image);
		}
	}

	// If we have more than one internal source, combine the first available two
	// If we have exactly one, the combiner will just return that
	if (InternalVRSSources.Num())
	{
		return CombineShadingRateImages(GraphBuilder, *ViewInfo.Family, InternalVRSSources);
	}

	// Default to nullptr if no sources are available
	else
	{
		return nullptr;
	}
}

void FVariableRateShadingImageManager::PrepareImageBasedVRS(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const FMinimalSceneTextures& SceneTextures)
{
	EShaderPlatform ShaderPlatform = ViewFamily.Scene->GetShaderPlatform();
	static const auto CVarLocalNaniteSoftwareVRS = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite.SoftwareVRS")); // "CVarNaniteSoftwareVRS" would shadow the static declaration in NaniteShading.cpp

	bHardwareVRSEnabledForFrame = 
		IsAttachmentVRSEnabled() &&
		HardwareVariableRateShadingSupportedByPlatform(ShaderPlatform); // Additional check is required here for preview levels

	bSoftwareVRSEnabledForFrame = 
		CVarEnableVRSSoftwareImage.GetValueOnRenderThread() > 0 &&
		UseNanite(ShaderPlatform) &&
		CVarLocalNaniteSoftwareVRS->GetInt() > 0 &&
		IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM6);

	if (!IsVRSEnabledForFrame())
	{
		return;
	}

	// If no generators or forced rate are active, bail
	{
		FReadScopeLock GeneratorsLock(GeneratorsMutex);
		ActiveGenerators = ImageGenerators.FilterByPredicate([](IVariableRateShadingImageGenerator* const InGenerator) { return InGenerator && InGenerator->IsEnabled(); });
	}

	VRSForceRateForFrame = CVarVRSDebugForceRate->GetInt();

	if (ActiveGenerators.IsEmpty() && VRSForceRateForFrame < 0)
	{
		return;
	}
	
	// If no views support VRS, bail
	bool bIsAnyViewVRSCompatible = false;
	for (const FSceneView* View : ViewFamily.Views)
	{
		check(View->bIsViewInfo);
		auto ViewInfo = static_cast<const FViewInfo*>(View);
		if (IsVRSCompatibleWithView(*ViewInfo))
		{
			bIsAnyViewVRSCompatible = true;
			break;
		}
	}

	if (!bIsAnyViewVRSCompatible)
	{
		return;
	}

	// Also bail if we're given a ViewFamily with no valid RenderTarget
	if (ViewFamily.RenderTarget == nullptr)
	{
		ensureMsgf(0, TEXT("VRS Image Manager does not support ViewFamilies with no valid RenderTarget"));
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "PrepareImageBasedVRS");
	SCOPED_NAMED_EVENT(PrepareImageBasedVRS, FColor::Red);

	VRSForceRateForFrame = CVarVRSDebugForceRate->GetInt();

	// Invoke active image generators
	for (IVariableRateShadingImageGenerator* const Generator : ActiveGenerators)
	{
		if (Generator && Generator->IsSupportedByView(*ViewFamily.Views[0]))
		{
			Generator->PrepareImages(GraphBuilder, ViewFamily, SceneTextures, bHardwareVRSEnabledForFrame, bSoftwareVRSEnabledForFrame);
		}
	}
}

bool FVariableRateShadingImageManager::IsTypeEnabledForView(const FSceneView& View, FVariableRateShadingImageManager::EVRSSourceType Type)
{
	for (IVariableRateShadingImageGenerator* const Generator : ActiveGenerators)
	{
		if (Generator && EnumHasAnyFlags(Type, Generator->GetType()) && Generator->IsSupportedByView(View))
		{
			return true;
		}
	}
	return false;
}

void FVariableRateShadingImageManager::DrawDebugPreview(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, FRDGTextureRef OutputSceneColor)
{
	if (!IsVRSEnabledForFrame() || !OutputSceneColor)
	{
		return;
	}

	uint32 ImageTypeAsInt = CVarVRSPreview.GetValueOnRenderThread();
	EVRSImageType PreviewImageType = EVRSImageType::Disabled;
	bool bUseSoftwareImage = false;
	
	switch (ImageTypeAsInt)
	{
		// Full hardware
		case 1: 
			PreviewImageType = EVRSImageType::Full;
			break;

		// Conservative hardware
		case 2:
			PreviewImageType = EVRSImageType::Conservative;
			break;

		// Full software
		case 3:
			PreviewImageType = EVRSImageType::Full;
			bUseSoftwareImage = true;
			break;

		// Conservative software
		case 4:
			PreviewImageType = EVRSImageType::Conservative;
			bUseSoftwareImage = true;
			break;

		default:
			return;
	}

	if ((bUseSoftwareImage && !IsSoftwareVRSEnabledForFrame()) || (!bUseSoftwareImage && !IsHardwareVRSEnabledForFrame()))
	{
		return;
	}

	for (const FSceneView* View : ViewFamily.Views)
	{
		check(View->bIsViewInfo);
		auto ViewInfo = static_cast<const FViewInfo*>(View);

		if (ViewInfo->ShouldRenderView() && IsVRSCompatibleWithView(*ViewInfo))
		{
			FRDGTextureRef PreviewTexture;
			
			// Use debug rate if provided
			if (VRSForceRateForFrame >= 0)
			{
				PreviewTexture = GetForceRateImage(GraphBuilder, ViewFamily, VRSForceRateForFrame, bUseSoftwareImage);
			}

			// Otherwise collate debug images
			else
			{
				TArray<FRDGTextureRef> InternalVRSSources;

				for (IVariableRateShadingImageGenerator* const Generator : ActiveGenerators)
				{
					FRDGTextureRef Image = nullptr;
					if (Generator && Generator->IsSupportedByView(*View))
					{
						Image = Generator->GetDebugImage(GraphBuilder, *ViewInfo, PreviewImageType, bUseSoftwareImage);
					}

					if (Image)
					{
						InternalVRSSources.Add(Image);
					}
				}

				PreviewTexture = CombineShadingRateImages(GraphBuilder, ViewFamily, InternalVRSSources);

				// Generate a dummy 1x1 image if we have no VRS sources
				if (!PreviewTexture)
				{
					PreviewTexture = GetForceRateImage(GraphBuilder, ViewFamily, VRSSR_1x1, bUseSoftwareImage);
				}
			}

			// If we have an active debug image, render it as a preview overlay
			auto& RHICmdList = GraphBuilder.RHICmdList;

			SCOPED_DRAW_EVENT(RHICmdList, VRSDebugPreview);

			FIntRect SrcViewRect = ViewInfo->ViewRect;

			const FIntRect& DestViewRect = ViewInfo->UnscaledViewRect;

			FDebugVariableRateShadingPS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FFragmentDensityMap>(GRHIVariableRateShadingImageDataType == VRSImage_Fractional);

			TShaderMapRef<FDebugVariableRateShadingVS> VertexShader(ViewInfo->ShaderMap);
			TShaderMapRef<FDebugVariableRateShadingPS> PixelShader(ViewInfo->ShaderMap, PermutationVector);

			auto* PassParameters = GraphBuilder.AllocParameters<FDebugVariableRateShadingPS::FParameters>();
			PassParameters->VariableRateShadingTextureIn = PreviewTexture;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputSceneColor, ERenderTargetLoadAction::ELoad);
			PassParameters->RenderTargets.MultiViewCount = ViewInfo->GetMultiViewCount();

			FRHIBlendState* BlendState =
				TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI();
			FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

			EScreenPassDrawFlags DrawFlags = EScreenPassDrawFlags::AllowHMDHiddenAreaMask;

			FIntRect ScaledSrcRect = FIntRect::DivideAndRoundUp(SrcViewRect, FVariableRateShadingImageManager::GetSRITileSize(bUseSoftwareImage));

			const FScreenPassTextureViewport InputViewport = FScreenPassTextureViewport(PreviewTexture, ScaledSrcRect);
			const FScreenPassTextureViewport OutputViewport(OutputSceneColor, DestViewRect);

			AddDrawScreenPass(
				GraphBuilder,
				RDG_EVENT_NAME("Display VRS Debug Preview"),
				*ViewInfo,
				OutputViewport,
				InputViewport,
				VertexShader,
				PixelShader,
				BlendState,
				DepthStencilState,
				PassParameters,
				DrawFlags);
		}
	}

}

void FVariableRateShadingImageManager::RegisterExternalImageGenerator(IVariableRateShadingImageGenerator* ExternalGenerator)
{
	if (ExternalGenerator == nullptr)
	{
		UE_LOGF(LogVRS, Warning, "Trying to register a null VRS generator. Generator will be ignored.");
		return;
	}
	FWriteScopeLock GeneratorsLock(GeneratorsMutex);
	ImageGenerators.Add(ExternalGenerator);
}

void FVariableRateShadingImageManager::UnregisterExternalImageGenerator(IVariableRateShadingImageGenerator* ExternalGenerator)
{
	if (ExternalGenerator == nullptr)
	{
		UE_LOGF(LogVRS, Warning, "Trying to unregister a null VRS generator. Generator will be ignored.");
		return;
	}
	FWriteScopeLock GeneratorsLock(GeneratorsMutex);
	ImageGenerators.Remove(ExternalGenerator);
}

/**
 * Private functions
 */

FRDGTextureRef FVariableRateShadingImageManager::CombineShadingRateImages(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, TArray<FRDGTextureRef> Sources)
{
	// If we have more than one source, combine the first available two
	// TODO: Support combining more textures
	if (Sources.Num() < 1)
	{
		return nullptr;
	}
	else if (Sources.Num() == 1)
	{
		return Sources[0];
	}
	else
	{
		SCOPED_NAMED_EVENT(CombineShadingRateImages, FColor::Green);

		// Create texture to hold shading rate image
		FRDGTextureRef CombinedShadingRateTexture = GraphBuilder.CreateTexture(Sources[0]->Desc, TEXT("CombinedShadingRateTexture"));

		FCombineShadingRateTexturesCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FFragmentDensityMap>(GRHIVariableRateShadingImageDataType == VRSImage_Fractional);

		TShaderMapRef<FCombineShadingRateTexturesCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationVector);

		FCombineShadingRateTexturesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCombineShadingRateTexturesCS::FParameters>();
		PassParameters->SourceTexture0 = Sources[0];
		PassParameters->SourceTexture1 = Sources[1];
		PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(CombinedShadingRateTexture);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("CombineShadingRateImages"),
			ERDGPassFlags::AsyncCompute | ERDGPassFlags::NeverCull,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(FSceneTexturesConfig::Get().Extent, FIntPoint(VRSHelpers::kCombineGroupSize, VRSHelpers::kCombineGroupSize)));

		return CombinedShadingRateTexture;
	}

}

FRDGTextureRef FVariableRateShadingImageManager::GetForceRateImage(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, int RateIndex /* = 0*/, bool bGetSoftwareImage)
{
	static const TArray<uint32> ValidShadingRates = { VRSSR_1x1, VRSSR_1x2, VRSSR_2x1, VRSSR_2x2, VRSSR_2x4, VRSSR_4x2, VRSSR_4x4 };
	static const TArray<FVector2D> ValidFragmentDensities = { FVector2D(1.0, 1.0), FVector2D(1.0, 0.5), FVector2D(0.5, 1.0), FVector2D(0.5, 0.5), FVector2D(0.5, 0.25), FVector2D(0.25, 0.5), FVector2D(0.25, 0.25) };

	const bool bImageTypeAvailable = bGetSoftwareImage ? bSoftwareVRSEnabledForFrame : bHardwareVRSEnabledForFrame;
	if (!bImageTypeAvailable || (GRHIVariableRateShadingImageDataType == VRSImage_NotSupported))
	{
		return nullptr;
	}

	FRDGTextureRef ForceShadingRateTexture = GraphBuilder.CreateTexture(GetSRIDesc(ViewFamily, bGetSoftwareImage), TEXT("ForceShadingRateTexture"));
	FRDGTextureUAVRef ForceShadingRateUAV = GraphBuilder.CreateUAV(ForceShadingRateTexture);

	RateIndex = FMath::Clamp(RateIndex, 0, GetNumberOfSupportedRates() - 1);

	if (GRHIVariableRateShadingImageDataType == VRSImage_Palette)
	{
		AddClearUAVPass(GraphBuilder, ForceShadingRateUAV, ValidShadingRates[RateIndex]);
	}
	else if (GRHIVariableRateShadingImageDataType == VRSImage_Fractional)
	{
		AddClearUAVPass(GraphBuilder, ForceShadingRateUAV, ValidFragmentDensities[RateIndex]);
	}
	else
	{
		checkNoEntry(); // Invalid GRHIVariableRateShadingImageDataType
		return nullptr;
	}
	

	return ForceShadingRateTexture;
}

FVariableRateShadingImageManager::EVRSImageType FVariableRateShadingImageManager::GetImageTypeFromPassType(EVRSPassType PassType)
{
	static struct FStaticPassToImageCVarData
	{
		TAutoConsoleVariable<int32>* CVarByPassType[EVRSPassType::Num] = {};
		FStaticPassToImageCVarData()
		{
			CVarByPassType[EVRSPassType::BasePass] = &CVarVRSBasePass;
			CVarByPassType[EVRSPassType::TranslucencyAll] = &CVarVRSTranslucency;
			CVarByPassType[EVRSPassType::NaniteEmitGBufferPass] = &CVarVRSNaniteEmitGBuffer;
			CVarByPassType[EVRSPassType::SSAO] = &CVarVRS_SSAO;
			CVarByPassType[EVRSPassType::SSR] = &CVarVRS_SSR;
			CVarByPassType[EVRSPassType::ReflectionEnvironmentAndSky] = &CVarVRSReflectionEnvironmentSky;
			CVarByPassType[EVRSPassType::LightFunctions] = &CVarVRSLightFunctions;
			CVarByPassType[EVRSPassType::Decals] = &CVarVRSDecals;
		}
	} StaticData;

	uint32 ImageTypeAsInt = StaticData.CVarByPassType[PassType]->GetValueOnRenderThread();
	if (ImageTypeAsInt >= 0 && ImageTypeAsInt <= EVRSImageType::Conservative)
	{
		return static_cast<EVRSImageType>(ImageTypeAsInt);
	}
	else
	{
		return EVRSImageType::Disabled;
	}
}