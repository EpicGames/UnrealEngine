// Copyright Epic Games, Inc. All Rights Reserved.

#include "VelocityRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneUtils.h"
#include "Materials/Material.h"
#include "PostProcess/SceneRenderTargets.h"
#include "MaterialShaderType.h"
#include "MaterialShader.h"
#include "MeshMaterialShader.h"
#include "ShaderBaseClasses.h"
#include "SceneRendering.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "ScreenSpaceRayTracing.h"
#include "PostProcess/PostProcessMotionBlur.h"
#include "UnrealEngine.h"
#if WITH_EDITOR
#include "Misc/CoreMisc.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif
#include "VisualizeTexture.h"
#include "MeshPassProcessor.inl"
#include "DebugProbeRendering.h"
#include "LogRenderer.h"
#include "RenderCore.h"

// Changing this causes a full shader recompile
static TAutoConsoleVariable<int32> CVarVelocityOutputPass(
	TEXT("r.VelocityOutputPass"),
	0,
	TEXT("When to write velocity buffer.\n") \
	TEXT(" 0: Renders during the depth pass. This splits the depth pass into 2 phases: with and without velocity.\n") \
	TEXT(" 1: Renders during the regular base pass. This adds an extra GBuffer target during base pass rendering.") \
	TEXT(" 2: Renders after the regular base pass.\n"), \
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarBasePassOutputsVelocity(
	TEXT("r.BasePassOutputsVelocity"),
	-1,
	TEXT("Deprecated CVar. Use r.VelocityOutputPass instead.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVertexDeformationOutputsVelocity(
	TEXT("r.VertexDeformationOutputsVelocity"),
	-1,
	TEXT("Deprecated CVar. Use r.Velocity.EnableVertexDeformation instead.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarParallelVelocity(
	TEXT("r.ParallelVelocity"),
	1,  
	TEXT("Toggles parallel velocity rendering. Parallel rendering must be enabled for this to have an effect."),
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarVelocityOutputTranslucentClippedDepthSupported(
	TEXT("r.Velocity.OutputTranslucentClippedDepth.Supported"),
	0,
	TEXT("Whether the translucent velocity clipped depth pass is supported on the current platform.\n"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarVelocityOutputTranslucentClippedDepthEnabled(
	TEXT("r.Velocity.OutputTranslucentClippedDepth.Enabled"),
	1,
	TEXT("Enable/Disable the translucent velocity clipped depth pass on the fly.\n")\
	TEXT("0: Skip this pass.\n")\
	TEXT("1: Provide functions e.g., mark before DoF translucency Temporal Responsiveness for opacity below clip value.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVelocityTemporalResponsivenessSupported(
	TEXT("r.Velocity.TemporalResponsiveness.Supported"),
	0,
	TEXT("Whether temporal Responsiveness is supported. use one more bit from the velocity texture.\n"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarVelocityPixelShaderMotionVectorWorldOffsetSupported(
	TEXT("r.Velocity.PixelShaderMotionVectorWorldOffset.Supported"),
	0,
	TEXT("Whether motion vector offset is supported in PS pass. Allow user to modify the motion vector per pixel.\n"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarVelocityDirectlyRenderOpenXRMotionVectors(
	TEXT("r.Velocity.DirectlyRenderOpenXRMotionVectors"),
	0,
	TEXT("If true and using the Vulkan mobile forward renderer, the engine will render velocity in the OpenXR motion vector format, at the size recommended by FOpenXRHMD::GetRecommendedMotionVectorTextureSize.\n")
	TEXT("Because the existing scene depth cannot be used to calculate flattened velocity for stationary objects due to the likely size mismatch, this requires including ALL meshes in the velocity pass, even stationary ones which would usually be excluded.\n")
	TEXT("This setting disables normal velocity rendering and all other features dependent on it, such as Temporal Anti-Aliasing and Motion Blur."),
	ECVF_ReadOnly);

DECLARE_GPU_DRAWCALL_STAT_NAMED(RenderVelocities, TEXT("Render Velocities"));

/** Validate that deprecated CVars are no longer set. */
inline void ValidateVelocityCVars()
{
#if !UE_BUILD_SHIPPING
	static bool bHasValidatedCVars = false;
	if (!bHasValidatedCVars)
	{
		{
			const int32 Value = CVarBasePassOutputsVelocity.GetValueOnAnyThread();
			if (Value != -1)
			{
				UE_LOGF(LogRenderer, Warning, "Deprecated CVar r.BasePassOutputsVelocity is set to %d. Remove and use r.VelocityOutputPass instead.", Value);
			}
		}
		{
			const int32 Value = CVarVertexDeformationOutputsVelocity.GetValueOnAnyThread();
			if (Value != -1)
			{
				UE_LOGF(LogRenderer, Warning, "Deprecated CVar r.VertexDeformationOutputsVelocity is set to %d. Remove and use r.Velocity.EnableVertexDeformation instead.", Value);
			}
		}
		bHasValidatedCVars = true;
	}
#endif
}

bool NeedVelocityDepth(EShaderPlatform ShaderPlatform)
{
	// Lumen needs velocity depth
	return (DoesProjectSupportDistanceFields() && FDataDrivenShaderPlatformInfo::GetSupportsLumenGI(ShaderPlatform)) 
		|| FDataDrivenShaderPlatformInfo::GetSupportsRayTracing(ShaderPlatform);
}

bool SupportsTemporalResponsiveness(EShaderPlatform ShaderPlatform)
{
	return NeedVelocityDepth(ShaderPlatform) && VelocitySupportsTemporalResponsiveness(ShaderPlatform);
}

bool SupportsPixelShaderMotionVectorWorldOffset(EShaderPlatform ShaderPlatform)
{
	return NeedVelocityDepth(ShaderPlatform) && VelocitySupportsPixelShaderMotionVectorWorldOffset(ShaderPlatform);
}

bool SupportsTranslucentClippedDepth(EShaderPlatform ShaderPlatform)
{
	// Translucent clipped depth requires atomics on uint64. So holds the same requirement of Nanite.
	static FShaderPlatformCachedIniValue<int32> PerPlatformCVar(TEXT("r.Velocity.OutputTranslucentClippedDepth.Supported"));
	return FDataDrivenShaderPlatformInfo::GetSupportsNanite(ShaderPlatform) && (PerPlatformCVar.Get(ShaderPlatform) != 0);
}

// Use template to allow shader pipeline binding for better performance on some platforms
template <EVelocityPassMode PassMode>
class TVelocityVS : public FVelocityVS
{
public:
	DECLARE_SHADER_TYPE(TVelocityVS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		// Compile for default material.
		const bool bIsDefault = Parameters.MaterialParameters.bIsSpecialEngineMaterial;

		// Compile for masked materials.
		const bool bIsMasked = !Parameters.MaterialParameters.bWritesEveryPixel;

		// Compile for opaque and two-sided materials.
		const bool bIsOpaqueAndTwoSided = (Parameters.MaterialParameters.bIsTwoSided && !IsTranslucentBlendMode(Parameters.MaterialParameters));

		// Compile for materials which modify meshes.
		const bool bMayModifyMeshes = Parameters.MaterialParameters.bMaterialMayModifyMeshPosition;

		// Compile for materials that modifies motion vector offset or uses temporal responsiveness to indicate motion vector mismatch.
		const bool bModifiesMotionVectorStatus = 
			(Parameters.MaterialParameters.bUsesMotionVectorWorldOffset && SupportsPixelShaderMotionVectorWorldOffset(Parameters.Platform)) || 
			(Parameters.MaterialParameters.bUsesTemporalResponsiveness && SupportsTemporalResponsiveness(Parameters.Platform));

		// Compile for SLW materials when SLW velocity is enabled and velocity outputs to the prepass.
		static const auto SLWVelocityOutputPass = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Water.SingleLayer.VelocityOutputPass"));
		const bool bSLWPrepassVelocity = SLWVelocityOutputPass && SLWVelocityOutputPass->GetValueOnAnyThread() >= /*Depth Prepass*/1
			&& Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater);

		bool bHasPlatformSupport = false;
		if constexpr (PassMode == EVelocityPassMode::Velocity_Standard)
		{
			bHasPlatformSupport = PlatformSupportsVelocityRendering(Parameters.Platform);
		}
		else if constexpr (PassMode == EVelocityPassMode::Velocity_ClippedDepth)
		{
			bHasPlatformSupport = PlatformSupportsVelocityRendering(Parameters.Platform) && SupportsTranslucentClippedDepth(Parameters.Platform);
		}
		else if constexpr (PassMode == EVelocityPassMode::Velocity_StereoMotionVectors)
		{
			bHasPlatformSupport = PlatformSupportsOpenXRMotionVectors(Parameters.Platform);
		}

		/**
		 * If we don't use base pass velocity then we may need to generate permutations for this shader. 
		 * We only need to compile shaders which aren't considered "simple" enough to swap against the default material. 
		 * This massively simplifies the calculations.
		 */
		const bool bIsSeparateVelocityPassRequired = 
			!FVelocityRendering::BasePassCanOutputVelocity(Parameters.Platform) &&
			(bIsMasked || bIsOpaqueAndTwoSided || bMayModifyMeshes || bModifiesMotionVectorStatus);

		// The material may explicitly request that it be rendered into the translucent velocity pass.
		const bool bIsSeparateVelocityPassRequiredByMaterial = Parameters.MaterialParameters.bIsTranslucencyWritingVelocity;

		const bool bIsNaniteFactory = Parameters.VertexFactoryType->SupportsNaniteRendering();
		return bHasPlatformSupport && !bIsNaniteFactory && (bIsDefault || bIsSeparateVelocityPassRequired || bIsSeparateVelocityPassRequiredByMaterial || bSLWPrepassVelocity);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FVelocityVS::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		constexpr bool bIsStereoMotionVectorPass = (PassMode == EVelocityPassMode::Velocity_StereoMotionVectors);
		OutEnvironment.SetDefine(TEXT("STEREO_MOTION_VECTORS"), bIsStereoMotionVectorPass ? 1 : 0);
	}

	TVelocityVS()
	{
	}

	TVelocityVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FVelocityVS(Initializer)
	{}
};

// Use template to allow shader pipeline binding for better performance on some platforms
template <EVelocityPassMode PassMode>
class TVelocityPS : public FVelocityPS
{
public:
	DECLARE_SHADER_TYPE(TVelocityPS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return TVelocityVS<PassMode>::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FVelocityPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		constexpr bool bIsVelocityClippedDepthPass = (PassMode == EVelocityPassMode::Velocity_ClippedDepth);
		OutEnvironment.SetDefine(TEXT("VELOCITY_CLIPPED_DEPTH_PASS"), bIsVelocityClippedDepthPass ? 1 : 0);

		constexpr bool bIsMotionVectorPass = (PassMode == EVelocityPassMode::Velocity_StereoMotionVectors);
		OutEnvironment.SetDefine(TEXT("STEREO_MOTION_VECTORS"), bIsMotionVectorPass ? 1 : 0);

		if (bIsVelocityClippedDepthPass)
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}
	}

	TVelocityPS()
	{
	}

	TVelocityPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FVelocityPS(Initializer)
	{}
};

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVelocityClippedDepthUniformParameters, )
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<UlongType>, RWVelocity)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_SHADER_TYPE(,FVelocityVS, TEXT("/Engine/Private/VelocityShader.usf"), TEXT("MainVertexShader"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(,FVelocityPS, TEXT("/Engine/Private/VelocityShader.usf"), TEXT("MainPixelShader"), SF_Pixel);

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TVelocityVS<EVelocityPassMode::Velocity_Standard>, TEXT("/Engine/Private/VelocityShader.usf"), TEXT("MainVertexShader"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TVelocityVS<EVelocityPassMode::Velocity_ClippedDepth>, TEXT("/Engine/Private/VelocityShader.usf"), TEXT("MainVertexShader"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TVelocityVS<EVelocityPassMode::Velocity_StereoMotionVectors>, TEXT("/Engine/Private/VelocityShader.usf"), TEXT("MainVertexShader"), SF_Vertex);

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TVelocityPS<EVelocityPassMode::Velocity_Standard>, TEXT("/Engine/Private/VelocityShader.usf"), TEXT("MainPixelShader"), SF_Pixel);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TVelocityPS<EVelocityPassMode::Velocity_ClippedDepth>, TEXT("/Engine/Private/VelocityShader.usf"), TEXT("MainPixelShader"), SF_Pixel);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TVelocityPS<EVelocityPassMode::Velocity_StereoMotionVectors>, TEXT("/Engine/Private/VelocityShader.usf"), TEXT("MainPixelShader"), SF_Pixel);

IMPLEMENT_SHADERPIPELINE_TYPE_VSPS(StandardVelocityPipeline, TVelocityVS<EVelocityPassMode::Velocity_Standard>, TVelocityPS<EVelocityPassMode::Velocity_Standard>, true);
IMPLEMENT_SHADERPIPELINE_TYPE_VSPS(VelocityClippedDepthPipeline, TVelocityVS<EVelocityPassMode::Velocity_ClippedDepth>, TVelocityPS<EVelocityPassMode::Velocity_ClippedDepth>, true);
IMPLEMENT_SHADERPIPELINE_TYPE_VSPS(VelocityMotionVectorsPipeline, TVelocityVS<EVelocityPassMode::Velocity_StereoMotionVectors>, TVelocityPS<EVelocityPassMode::Velocity_StereoMotionVectors>, true);

class FMotionVectorWorldOffsetVelocityResolveCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMotionVectorWorldOffsetVelocityResolveCS);
	SHADER_USE_PARAMETER_STRUCT(FMotionVectorWorldOffsetVelocityResolveCS, FGlobalShader);

	class FSLWResolve : SHADER_PERMUTATION_BOOL("SLW_RESOLVE");
	using FPermutationDomain = TShaderPermutationDomain<FSLWResolve>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, DepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, VelocityTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWMotionVectorWorldOffset)
		//SLW permutation needs 2 inputs and 1 output
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, MotionVectorWorldOffset)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWOutputVelocity)
	END_GLOBAL_SHADER_PARAMETER_STRUCT()

	static int GetGroupSize()
	{
		return FComputeShaderUtils::kGolden2DGroupSize;
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		//TODO: don't cache if per-pixel velocity offset is not used.
		return EShaderPermutationPrecacheRequest::Precached;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"),GetGroupSize());
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMotionVectorWorldOffsetVelocityResolveCS, "/Engine/Private/VelocityUpdate.usf", "MainCS", SF_Compute);

EMeshPass::Type GetMeshPassFromVelocityPass(EVelocityPass VelocityPass)
{
	switch (VelocityPass)
	{
	case EVelocityPass::Opaque:
		return EMeshPass::Velocity;
	case EVelocityPass::Translucent:
		return EMeshPass::TranslucentVelocity;
	case EVelocityPass::TranslucentClippedDepth:
		return EMeshPass::TranslucentVelocityClippedDepth;
	}
	check(false);
	return EMeshPass::Velocity;
}

static const TCHAR* GetVelocityPassName(EVelocityPass VelocityPass)
{
	static const TCHAR* const kPassNames[] = {
		TEXT("Opaque"),
		TEXT("Translucent"),
		TEXT("TranslucentClippedDepth")
	};
	static_assert(UE_ARRAY_COUNT(kPassNames) == int32(EVelocityPass::Count), "Fix me");
	return kPassNames[int32(VelocityPass)];
}

bool FDeferredShadingSceneRenderer::ShouldRenderVelocities() const
{
	if (!FVelocityRendering::IsVelocityPassSupported(ShaderPlatform) || ViewFamily.UseDebugViewPS())
	{
		return false;
	}
	if (FVelocityRendering::DepthPassCanOutputVelocity(Scene->GetFeatureLevel()))
	{
		// Always render velocity when it is part of the depth pass to avoid dropping things from the depth pass.
		// This means that we will pay the cost of velocity in the pass even if we don't really need it according to the view logic below.
		// But requiring velocity is by far the most common case.
		// And the alternative approach is for the depth pass to also incorporate the logic below to avoid dropping velocity primitives.
		return true;
	}

	bool bNeedsVelocity = false;
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);

		bool bTemporalAA = IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod) && !View.bCameraCut;
		bool bMotionBlur = IsMotionBlurEnabled(View);
		bool bVisualizeMotionblur = View.Family->EngineShowFlags.VisualizeMotionBlur || View.Family->EngineShowFlags.VisualizeTemporalUpscaler;
		bool bDistanceFieldAO = ShouldPrepareForDistanceFieldAO(Scene, ViewFamily, AnyViewHasGIMethodSupportingDFAO());

		bool bSceneSSREnabled = ViewPipelineState.ReflectionsMethod == EReflectionsMethod::SSR && ScreenSpaceRayTracing::ShouldRenderScreenSpaceReflections(View);
		bool bWaterSSREnabled = ViewPipelineState.ReflectionsMethodWater == EReflectionsMethod::SSR && ScreenSpaceRayTracing::ShouldRenderScreenSpaceReflectionsWater(View);
		bool bSSRTemporal = (bSceneSSREnabled || bWaterSSREnabled) && ScreenSpaceRayTracing::IsSSRTemporalPassRequired(View);

		bool bRayTracing = IsRayTracingEnabled() && View.IsRayTracingAllowedForView();
		bool bDenoise = bRayTracing;

		bool bSSGI = ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::SSGI;
		bool bLumen = ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen || ViewPipelineState.ReflectionsMethod == EReflectionsMethod::Lumen;
		
		bool bDistortion = ShouldRenderDistortion();

		bNeedsVelocity |= bVisualizeMotionblur || bMotionBlur || bTemporalAA || bDistanceFieldAO || bSSRTemporal || bDenoise || bSSGI || bLumen || bDistortion;
	}

	return bNeedsVelocity;
}

bool FMobileSceneRenderer::ShouldRenderVelocities() const
{
	if (!FVelocityRendering::IsVelocityPassSupported(ShaderPlatform) || ViewFamily.UseDebugViewPS() || !PlatformSupportsVelocityRendering(ShaderPlatform))
	{
		return false;
	}

	bool bNeedsVelocity = false;
	for (int32 ViewIndex = 0; ViewIndex < Views.Num() && !bNeedsVelocity; ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];

		const bool bTemporalAA = IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod);
		const bool bIsUsingTemporalUpscaler = View.Family->GetTemporalUpscalerInterface() != nullptr;
		const bool bVelocityRendering = (bIsUsingTemporalUpscaler || bTemporalAA) && !View.bCameraCut;

		bNeedsVelocity |= bVelocityRendering;
	}

	return bNeedsVelocity;
}

IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(VelocityClippedDepth);
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FVelocityClippedDepthUniformParameters, "VelocityClippedDepth", VelocityClippedDepth);

static FRDGTextureUAVRef CreateDummyVelocityUAV(FRDGBuilder& GraphBuilder, EPixelFormat PixelFormat)
{
	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
		FIntPoint(1, 1),
		PixelFormat,
		FClearValueBinding::None,
		/* InFlags = */ TexCreate_ShaderResource | TexCreate_UAV);

	FRDGTextureRef DummyTexture = GraphBuilder.CreateTexture(Desc, TEXT("VelocityClippedDepth.UAVDummy"));

	return GraphBuilder.CreateUAV(FRDGTextureUAVDesc(DummyTexture, 0, PixelFormat));
};

TRDGUniformBufferRef<FVelocityClippedDepthUniformParameters> BindTranslucentVelocityClippedDepthPassUniformParameters(FRDGBuilder& GraphBuilder, 
	FRDGTextureRef TargetVelocity, bool bWriteVelocity, EShaderPlatform ShaderPlatform)
{
	FVelocityClippedDepthUniformParameters& VelocityClippedDepthUniformParameters = *GraphBuilder.AllocParameters<FVelocityClippedDepthUniformParameters>();

	const bool bNeedVelocityDepth = NeedVelocityDepth(ShaderPlatform);
	
	if (bNeedVelocityDepth && bWriteVelocity)
	{
		VelocityClippedDepthUniformParameters.RWVelocity = GraphBuilder.CreateUAV(TargetVelocity);
	}
	else
	{
		const EPixelFormat DummpyPixelFormat = GPixelFormats[PF_R64_UINT].Supported ? PF_R64_UINT : PF_R32G32_UINT;
		VelocityClippedDepthUniformParameters.RWVelocity = CreateDummyVelocityUAV(GraphBuilder, DummpyPixelFormat);
	}

	return GraphBuilder.CreateUniformBuffer(&VelocityClippedDepthUniformParameters);
}

BEGIN_SHADER_PARAMETER_STRUCT(FVelocityPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVelocityClippedDepthUniformParameters, VelocityClippedDepth)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void GetMotionVectorOutputFlag(
	TArrayView<FViewInfo> InViews,
	EMeshPass::Type MeshPass,
	bool bForceVelocity,
	bool& bUsesAnyMotionVectorWorldOffsetMaterials)
{
	bUsesAnyMotionVectorWorldOffsetMaterials = false;

	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ViewIndex++)
	{
		FViewInfo& View = InViews[ViewIndex];
		if (View.ShouldRenderView())
		{
			const bool bHasAnyDraw = HasAnyDraw(View.ParallelMeshDrawCommandPasses[MeshPass]);
			if (!bHasAnyDraw && !bForceVelocity)
			{
				continue;
			}
			bUsesAnyMotionVectorWorldOffsetMaterials |= View.bUsesMotionVectorWorldOffset;
		}
	}
}

void FSceneRenderer::RenderVelocities(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> InViews,
	const FSceneTextures& SceneTextures,
	EVelocityPass VelocityPass,
	bool bForceVelocity,
	bool bBindRenderTarget,
	FRDGTextureRef TargetVelocity,
	FRDGTextureRef TargetDepth)
{
	const bool bIsTranslucentClippedDepthPass = VelocityPass == EVelocityPass::TranslucentClippedDepth;
	const bool bSupportsTranslucentClippedDepth = SupportsTranslucentClippedDepth(ShaderPlatform);
	const bool bIsTranslucentClippedDepthEnabled = CVarVelocityOutputTranslucentClippedDepthEnabled.GetValueOnRenderThread() != 0;
	if (bIsTranslucentClippedDepthPass && (!bSupportsTranslucentClippedDepth || !bIsTranslucentClippedDepthEnabled))
	{
		return;
	}

	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderVelocities);
	SCOPED_NAMED_EVENT(FSceneRenderer_RenderVelocities, FColor::Emerald);
	SCOPE_CYCLE_COUNTER(STAT_RenderVelocities);

	// Create mask for which GPUs we need clearing on
	uint32 bNeedsClearMask = HasBeenProduced(TargetVelocity) ? 0 : ((1u << GNumExplicitGPUsForRendering) - 1);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, RenderVelocities, "RenderVelocities(%s)",GetVelocityPassName(VelocityPass));

	const EMeshPass::Type MeshPass = GetMeshPassFromVelocityPass(VelocityPass);
	const bool bIsOpaquePass = VelocityPass == EVelocityPass::Opaque;

	FExclusiveDepthStencil ExclusiveDepthStencil = (bIsOpaquePass && !(Scene->EarlyZPassMode == DDM_AllOpaqueNoVelocity))
														? FExclusiveDepthStencil::DepthRead_StencilWrite
														: FExclusiveDepthStencil::DepthWrite_StencilWrite;
	ExclusiveDepthStencil = bIsTranslucentClippedDepthPass ? FExclusiveDepthStencil::DepthRead_StencilNop : ExclusiveDepthStencil;

	//Only call the per pixel velocity resolve when we have at least one material using it.
	bool bHasAnyPixelShaderMotionVectorWorldOffsetMaterials = false;
	GetMotionVectorOutputFlag(InViews, MeshPass, bForceVelocity, bHasAnyPixelShaderMotionVectorWorldOffsetMaterials);
	const bool bSupportPixelShaderMotionVectorWorldOffset = SupportsPixelShaderMotionVectorWorldOffset(ShaderPlatform) && bIsOpaquePass && bHasAnyPixelShaderMotionVectorWorldOffsetMaterials;

	//Only opaque pass supports per pixel override.
	FRDGTextureRef MotionVectorWorldOffsetTexture = nullptr;
	if (bSupportPixelShaderMotionVectorWorldOffset)
	{
		MotionVectorWorldOffsetTexture = GraphBuilder.CreateTexture(TargetVelocity->Desc,TEXT("MotionVectorWorldOffsetTexture"));
		AddClearRenderTargetPass(GraphBuilder, MotionVectorWorldOffsetTexture);
	}

	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ViewIndex++)
	{
		FViewInfo& View = InViews[ViewIndex];

		checkf(!(View.Family->EngineShowFlags.StereoMotionVectors && PlatformSupportsOpenXRMotionVectors(View.GetShaderPlatform())),
			TEXT("Normal velocity rendering is not supported alongside motion vector rendering. If this is causing problems in your project, disable r.Velocity.DirectlyRenderOpenXRMotionVectors."));

		if (View.ShouldRenderView())
		{
			const bool bHasAnyDraw = HasAnyDraw(View.ParallelMeshDrawCommandPasses[MeshPass]);
			if (!bHasAnyDraw && !bForceVelocity)
			{
				continue;
			}

			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			const bool bIsParallelVelocity = FVelocityRendering::IsParallelVelocity(ShaderPlatform);

			// Clear velocity render target explicitly when velocity rendering in parallel or no draw but force to.
			// Avoid adding a separate clear pass in non parallel rendering.
			const bool bExplicitlyClearVelocity = (bNeedsClearMask & View.GPUMask.GetNative()) && (bIsParallelVelocity || (bForceVelocity && !bHasAnyDraw));

			if (bExplicitlyClearVelocity)
			{
				AddClearRenderTargetPass(GraphBuilder, TargetVelocity);
				bNeedsClearMask &= ~View.GPUMask.GetNative();
			}

			if (!bHasAnyDraw)
			{
				continue;
			}

			View.BeginRenderView();

			FParallelMeshDrawCommandPass& ParallelMeshPass = *View.ParallelMeshDrawCommandPasses[MeshPass];

			FVelocityPassParameters* PassParameters = GraphBuilder.AllocParameters<FVelocityPassParameters>();
			PassParameters->View = View.GetShaderParameters();
			ParallelMeshPass.BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);
			PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
				TargetDepth,
				ERenderTargetLoadAction::ELoad,
				ERenderTargetLoadAction::ELoad,
				ExclusiveDepthStencil);

			if (bBindRenderTarget)
			{
				ERenderTargetLoadAction LoadAction = (bNeedsClearMask & View.GPUMask.GetNative()) ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad;
				if (MotionVectorWorldOffsetTexture)
				{
					// Switch Velocity and Offset texture to avoid an additional copy
					// 
					// Write Velocity into the Offset texture and Offset into the Velocity texture so that
					// When we resolve (e.g., RWOffset[Position]+= Velocity[ResolvedPosition]), the resolved velocity 
					// is stored in the Velocity texture (RWOffset) instead of Offset texture to avoid an additional copy
					// from Offset texture to Velocity texture.
					// From
					//      V			=	v
					//      Offset		=	o
					//		Offset[p]	+= V[rp]
					//		V			= Offset
					// To
					//		Offset		= v
					//		V			= o
					//		V[p]		+= Offset[rp]

					PassParameters->RenderTargets[0] = FRenderTargetBinding(MotionVectorWorldOffsetTexture,LoadAction);
					PassParameters->RenderTargets[1] = FRenderTargetBinding(TargetVelocity, LoadAction);
				}
				else
				{
					PassParameters->RenderTargets[0] = FRenderTargetBinding(TargetVelocity, LoadAction);
				}
				
				bNeedsClearMask &= ~View.GPUMask.GetNative();
			}

			PassParameters->VelocityClippedDepth = BindTranslucentVelocityClippedDepthPassUniformParameters(GraphBuilder, TargetVelocity, bIsTranslucentClippedDepthPass, ShaderPlatform);

			PassParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();

			if (bIsParallelVelocity)
			{
				GraphBuilder.AddDispatchPass(
					RDG_EVENT_NAME("VelocityParallel"),
					PassParameters,
					ERDGPassFlags::Raster,
					[&View, &ParallelMeshPass, PassParameters](FRDGDispatchPassBuilder& DispatchPassBuilder)
				{
					ParallelMeshPass.Dispatch(DispatchPassBuilder, &PassParameters->InstanceCullingDrawParams);
				});
			}
			else
			{
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("Velocity"),
					PassParameters,
					ERDGPassFlags::Raster,
					[&View, &ParallelMeshPass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					SetStereoViewport(RHICmdList, View);
					ParallelMeshPass.Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
				});
			}
		}
	}

	if (bSupportPixelShaderMotionVectorWorldOffset)
	{
		for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ViewIndex++)
		{
			FViewInfo& View = InViews[ViewIndex];

			if (View.ShouldRenderView())
			{
				const bool bHasAnyDraw = HasAnyDraw(View.ParallelMeshDrawCommandPasses[MeshPass]);
				if ((!bHasAnyDraw && !bForceVelocity) || !View.bUsesMotionVectorWorldOffset)
				{
					continue;
				}

				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

				// Resolve
				{
					typedef FMotionVectorWorldOffsetVelocityResolveCS SHADER;
					SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
					PassParameters->View = View.GetShaderParameters();
					PassParameters->DepthTexture = GraphBuilder.CreateSRV(TargetDepth);

					// Switch back to avoid an additional copy.
					PassParameters->VelocityTexture = GraphBuilder.CreateSRV(MotionVectorWorldOffsetTexture);
					PassParameters->RWMotionVectorWorldOffset = GraphBuilder.CreateUAV(TargetVelocity);

					FMotionVectorWorldOffsetVelocityResolveCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FMotionVectorWorldOffsetVelocityResolveCS::FSLWResolve>(false);
					TShaderMapRef<FMotionVectorWorldOffsetVelocityResolveCS> ComputeShader(View.ShaderMap, PermutationVector);

					FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), SHADER::GetGroupSize());
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("MotionVectorWorldOffsetVelocityResolve %dx%d",
							View.ViewRect.Width(),
							View.ViewRect.Height()),
						ComputeShader,
						PassParameters,
						GroupCount);
				}
			}
		}
	}

#if !(UE_BUILD_SHIPPING)
	const bool bForwardShadingEnabled = IsForwardShadingEnabled(ShaderPlatform);
	if (!bForwardShadingEnabled)
	{
		FRenderTargetBindingSlots VelocityRenderTargets;
		VelocityRenderTargets[0] = FRenderTargetBinding(TargetVelocity, bNeedsClearMask ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad);
		VelocityRenderTargets.DepthStencil = FDepthStencilBinding(
			TargetDepth,
			ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad,
			ExclusiveDepthStencil);

		StampDeferredDebugProbeVelocityPS(GraphBuilder, InViews, VelocityRenderTargets);
	}
#endif


}

void FSceneRenderer::RenderVelocities(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> InViews,
	const FSceneTextures& SceneTextures,
	EVelocityPass VelocityPass,
	bool bForceVelocity,
	bool bBindRenderTarget)
{
	RenderVelocities(GraphBuilder, InViews, SceneTextures, VelocityPass,
		bForceVelocity, bBindRenderTarget,
		SceneTextures.Velocity, SceneTextures.Depth.Resolve);
}

void ResolveSLWMotionVectorWorldOffset(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> InViews,
	FRDGTextureRef DepthResolve,
	FRDGTextureRef StandardVelocityTexture,
	FRDGTextureRef MVWOOffsetTexture,
	FRDGTextureRef OutputVelocityTexture,
	EShaderPlatform ShaderPlatform)
{
	if (!SupportsPixelShaderMotionVectorWorldOffset(ShaderPlatform)
		|| !StandardVelocityTexture || !MVWOOffsetTexture || !OutputVelocityTexture)
	{
		return;
	}

	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ViewIndex++)
	{
		FViewInfo& View = InViews[ViewIndex];

		if (!View.ShouldRenderView() || !View.bSLWUsesMotionVectorWorldOffset)
		{
			continue;
		}

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

		typedef FMotionVectorWorldOffsetVelocityResolveCS SHADER;
		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		PassParameters->View = View.GetShaderParameters();
		PassParameters->DepthTexture = GraphBuilder.CreateSRV(DepthResolve);
		PassParameters->VelocityTexture = GraphBuilder.CreateSRV(StandardVelocityTexture);
		PassParameters->MotionVectorWorldOffset = GraphBuilder.CreateSRV(MVWOOffsetTexture);
		PassParameters->RWOutputVelocity = GraphBuilder.CreateUAV(OutputVelocityTexture);

		SHADER::FPermutationDomain PermutationVector;
		PermutationVector.Set<SHADER::FSLWResolve>(true);
		TShaderMapRef<SHADER> ComputeShader(View.ShaderMap, PermutationVector);

		FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), SHADER::GetGroupSize());
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("SLWMotionVectorWorldOffsetResolve %dx%d",
				View.ViewRect.Width(),
				View.ViewRect.Height()),
			ComputeShader,
			PassParameters,
			GroupCount);
	}
}

void FMobileSceneRenderer::RenderVelocityPass(FRHICommandList& RHICmdList, const FViewInfo& View, const FInstanceCullingDrawParams* InstanceCullingDrawParams)
{
	checkSlow(RHICmdList.IsInsideRenderPass());
	checkf(!(View.Family->EngineShowFlags.StereoMotionVectors && PlatformSupportsOpenXRMotionVectors(View.GetShaderPlatform())),
		TEXT("Normal velocity rendering is not supported alongside motion vector rendering. If this is causing problems in your project, disable r.Velocity.DirectlyRenderOpenXRMotionVectors."));

	if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::Velocity])
	{
		SCOPED_NAMED_EVENT(FMobileSceneRenderer_RenderVelocityPass, FColor::Emerald);
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, RenderVelocities, "MobileRenderVelocityPass");

		SCOPE_CYCLE_COUNTER(STAT_RenderVelocities);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RenderVelocityPass);

		SetStereoViewport(RHICmdList, View);
		Pass->Draw(RHICmdList, InstanceCullingDrawParams);
	}
}

EPixelFormat FVelocityRendering::GetFormat(EShaderPlatform ShaderPlatform)
{
	bool bNeedVelocityDepth = NeedVelocityDepth(ShaderPlatform); 

	// Android GLES platform doesn't support R16G16_UNORM and R16G16B16A16_UNORM format, use R16G16_UINT or R16G16B16A16_UINT instead.
	const bool bIsOpenGLPlatform = IsOpenGLPlatform(ShaderPlatform);
	if (bIsOpenGLPlatform)
	{
		return bNeedVelocityDepth ? PF_R16G16B16A16_UINT : PF_R16G16_UINT;
	}
	else
	{
		return bNeedVelocityDepth ? PF_A16B16G16R16 : PF_G16R16;
	}
}

ETextureCreateFlags FVelocityRendering::GetCreateFlags(EShaderPlatform ShaderPlatform)
{
	const ETextureCreateFlags FastVRamFlag = BasePassCanOutputVelocity(ShaderPlatform) ? GFastVRamConfig.GBufferVelocity : TexCreate_None;
	const ETextureCreateFlags Atomic64CompatibleFlag = (NeedVelocityDepth(ShaderPlatform) && SupportsTranslucentClippedDepth(ShaderPlatform)) ? ETextureCreateFlags::Atomic64Compatible : TexCreate_None;
	return TexCreate_RenderTargetable | TexCreate_UAV | TexCreate_ShaderResource | FastVRamFlag | Atomic64CompatibleFlag;
}

FRDGTextureDesc FVelocityRendering::GetRenderTargetDesc(EShaderPlatform ShaderPlatform, FIntPoint Extent, const bool bRequireMultiView)
{
	return FRDGTextureDesc::CreateRenderTargetTextureDesc(Extent, GetFormat(ShaderPlatform), FClearValueBinding::Transparent, GetCreateFlags(ShaderPlatform), bRequireMultiView);
}

bool FVelocityRendering::IsVelocityPassSupported(EShaderPlatform ShaderPlatform)
{
	ValidateVelocityCVars();

	return GPixelFormats[GetFormat(ShaderPlatform)].Supported;
}

bool FVelocityRendering::DepthPassCanOutputVelocity(ERHIFeatureLevel::Type FeatureLevel)
{
	static bool bRequestedDepthPassVelocity = CVarVelocityOutputPass.GetValueOnAnyThread() == 0;
	const bool bMSAAEnabled = GetDefaultMSAACount(FeatureLevel) > 1;
	return !bMSAAEnabled && bRequestedDepthPassVelocity;
}

bool FVelocityRendering::BasePassCanOutputVelocity(EShaderPlatform ShaderPlatform)
{
	return IsUsingBasePassVelocity(ShaderPlatform);
}

bool FVelocityRendering::IsParallelVelocity(EShaderPlatform ShaderPlatform)
{
	return GRHICommandList.UseParallelAlgorithms() && CVarParallelVelocity.GetValueOnRenderThread()
		// Parallel dispatch is not supported on mobile platform
		&& !IsMobilePlatform(ShaderPlatform);
}

bool FVelocityMeshProcessor::PrimitiveHasVelocityForView(const FViewInfo& View, const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	// Skip if velocity rendering is unsupported.
	if (!PlatformSupportsVelocityRendering(View.GetShaderPlatform()))
	{
		return false;
	}
	// Skip camera cuts which effectively reset velocity for the new frame.
	if (View.bCameraCut && !View.PreviousViewTransform.IsSet())
	{
		return false;
	}
	// Velocity pass not rendered for debug views.
	if (View.Family->UseDebugViewPS())
	{
		return false;
	}
	// Only enabled on mobile when TAA is enabled or we're rendering OpenXR motion vectors.
	const bool bUsesTAA = (View.AntiAliasingMethod == AAM_TemporalAA);
	if (IsMobilePlatform(View.GetShaderPlatform()) && !(bUsesTAA || View.Family->EngineShowFlags.StereoMotionVectors))
	{
		return false;
	}

	const FBoxSphereBounds& PrimitiveBounds = PrimitiveSceneProxy->GetBounds();
	const float PrimitiveScreenRadiusSq = ComputeBoundsScreenRadiusSquared(PrimitiveBounds.Origin, PrimitiveBounds.SphereRadius, View);

	const float MinScreenRadiusForVelocityPass = (View.FinalPostProcessSettings.MotionBlurPerObjectSize * 0.5f / 100.0f) * View.LODDistanceFactor;
	const float MinScreenRadiusForVelocityPassSquared = FMath::Square(MinScreenRadiusForVelocityPass);

	if (PrimitiveScreenRadiusSq < MinScreenRadiusForVelocityPassSquared)
	{
		return false;
	}

	return true;
}

bool FOpaqueVelocityMeshProcessor::PrimitiveCanHaveVelocity(EShaderPlatform ShaderPlatform, const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	const bool bDrawsVelocity = PrimitiveSceneProxy->DrawsVelocity() || VelocityIncludeStationaryPrimitives(ShaderPlatform);
	return PrimitiveCanHaveVelocity(ShaderPlatform, bDrawsVelocity, PrimitiveSceneProxy->HasStaticLighting());
}

bool FOpaqueVelocityMeshProcessor::PrimitiveCanHaveVelocity(EShaderPlatform ShaderPlatform, bool bDrawVelocity, bool bHasStaticLighting)
{
	if (!FVelocityRendering::IsVelocityPassSupported(ShaderPlatform) || !PlatformSupportsVelocityRendering(ShaderPlatform))
	{
		return false;
	}

	if (!bDrawVelocity)
	{
		return false;
	}

	return true;
}

bool FOpaqueVelocityMeshProcessor::PrimitiveHasVelocityForFrame(EShaderPlatform ShaderPlatform, const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	if (!PrimitiveSceneProxy->AlwaysHasVelocity() && !VelocityIncludeStationaryPrimitives(ShaderPlatform))
	{
		// Check if the primitive has moved.
		const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
		const FScene* Scene = PrimitiveSceneInfo->Scene;
		const FMatrix& LocalToWorld = PrimitiveSceneProxy->GetLocalToWorld();
		FMatrix PreviousLocalToWorld = LocalToWorld;
		Scene->VelocityData.GetComponentPreviousLocalToWorld(PrimitiveSceneInfo->PrimitiveComponentId, PreviousLocalToWorld);

		if (LocalToWorld.Equals(PreviousLocalToWorld, 0.0001f))
		{
			// Hasn't moved (treat as background by not rendering any special velocities)
			return false;
		}
	}

	return true;
}

static bool UseDefaultMaterial(const FMaterial* Material, bool bVFTypeSupportsNullPixelShader, bool bMaterialModifiesMeshPosition)
{
	// Materials without masking or custom vertex modifications can be swapped out
	// for the default material, which simplifies the shader. However, the default
	// material also does not support being two-sided.
	return Material->WritesEveryPixel(false, bVFTypeSupportsNullPixelShader) && !Material->IsTwoSided() && !bMaterialModifiesMeshPosition;
}

bool FOpaqueVelocityMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FMaterial* Material)
{
	const bool bIsNotTranslucent = IsOpaqueOrMaskedBlendMode(*Material);

	bool bResult = true;
	if (MeshBatch.bUseForMaterial && bIsNotTranslucent && ShouldIncludeMaterialInDefaultOpaquePass(*Material))
	{
		// This is specifically done *before* the material swap, as swapped materials may have different fill / cull modes.
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(*Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(*Material, OverrideSettings);
		const bool bVFTypeSupportsNullPixelShader = MeshBatch.VertexFactory->SupportsNullPixelShader();
		const bool bModifiesMeshPosition = DoMaterialAndPrimitiveModifyMeshPosition(*Material, PrimitiveSceneProxy);
		const bool bModifiesMotionVectorStatus = Material->MaterialUsesMotionVectorWorldOffset_RenderThread() || Material->MaterialUsesTemporalResponsiveness_RenderThread();
		const bool bSwapWithDefaultMaterial = UseDefaultMaterial(Material, bVFTypeSupportsNullPixelShader, bModifiesMeshPosition) && !bModifiesMotionVectorStatus;
		if (bSwapWithDefaultMaterial)
		{
			MaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		}

		check(Material && MaterialRenderProxy);

		bResult = Process(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, *MaterialRenderProxy, *Material, MeshFillMode, MeshCullMode);
	}
	return bResult;
}

void FOpaqueVelocityMeshProcessor::AddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	if (!PrimitiveCanHaveVelocity(ShaderPlatform, PrimitiveSceneProxy))
	{
		return;
	}

	if (ViewIfDynamicMeshCommand)
	{
		if (!PrimitiveHasVelocityForFrame(ShaderPlatform, PrimitiveSceneProxy))
		{
			return;
		}

		checkSlow(ViewIfDynamicMeshCommand->bIsViewInfo);
		FViewInfo* ViewInfo = (FViewInfo*)ViewIfDynamicMeshCommand;

		if (!PrimitiveHasVelocityForView(*ViewInfo, PrimitiveSceneProxy))
		{
			return;
		}
	}

	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material && Material->GetRenderingThreadShaderMap())
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

void FOpaqueVelocityMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	bool bDrawsVelocity = (PreCacheParams.Mobility == EComponentMobility::Movable || PreCacheParams.Mobility == EComponentMobility::Stationary);
	bDrawsVelocity = bDrawsVelocity || (/*VertexDeformationOutputsVelocity() &&*/ (PreCacheParams.bAnyMaterialHasWorldPositionOffset || Material.MaterialUsesWorldPositionOffset_GameThread()));
	bDrawsVelocity = bDrawsVelocity || VelocityIncludeStationaryPrimitives(ShaderPlatform);

	if (!PrimitiveCanHaveVelocity(ShaderPlatform, bDrawsVelocity, PreCacheParams.bStaticLighting))
	{
		return;
	}

	const FMaterial* EffectiveMaterial = &Material;
	
	bool bCollectPSOs = false;
	if (PreCacheParams.bDefaultMaterial)
	{
		// Precache all cull modes for default material?
		bCollectPSOs = true;
	}
	else
	{
		const bool bIsNotTranslucent = IsOpaqueOrMaskedBlendMode(Material);

		if (PreCacheParams.bRenderInMainPass && bIsNotTranslucent && ShouldIncludeMaterialInDefaultOpaquePass(Material))
		{
			const bool bVFTypeSupportsNullPixelShader = VertexFactoryData.VertexFactoryType->SupportsNullPixelShader();
			const bool bModifiesMotionVectorStatus = Material.MaterialUsesMotionVectorWorldOffset_GameThread() || Material.MaterialUsesTemporalResponsiveness_GameThread();
			const bool bUseDefaultMaterial = UseDefaultMaterial(&Material, bVFTypeSupportsNullPixelShader, Material.MaterialModifiesMeshPosition_GameThread())
				&& !bModifiesMotionVectorStatus;
			if (!bUseDefaultMaterial)
			{
				bCollectPSOs = true;
			}
			else if (VertexFactoryData.CustomDefaultVertexDeclaration)
			{
				EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
				EffectiveMaterial = UMaterial::GetDefaultMaterial(MD_Surface)->GetMaterialResource(ShaderPlatform, ActiveQualityLevel);
				bCollectPSOs = true;
			}

		}
	}

	if (bCollectPSOs)
	{
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
		if (!CollectPSOInitializersInternal(SceneTexturesConfig, VertexFactoryData, *EffectiveMaterial, MeshFillMode, MeshCullMode, PSOInitializers))
		{
			// try again with default material (should use fallback material proxy here but currently only have FMaterial during PSO precaching)
			EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
			const FMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface)->GetMaterialResource(ShaderPlatform, ActiveQualityLevel);
			if (DefaultMaterial != EffectiveMaterial)
			{
				CollectPSOInitializersInternal(SceneTexturesConfig, VertexFactoryData, *DefaultMaterial, MeshFillMode, MeshCullMode, PSOInitializers);
			}
		}
	}
}

bool FTranslucentVelocityMeshProcessor::PrimitiveCanHaveVelocity(EShaderPlatform ShaderPlatform, const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	/**
	 * Velocity for translucency is always relevant because the pass also writes depth.
	 * Therefore, the primitive can't be filtered based on motion, or it will break post
	 * effects like depth of field which rely on depth information.
	 */
	return FVelocityRendering::IsVelocityPassSupported(ShaderPlatform) && PlatformSupportsVelocityRendering(ShaderPlatform);
}

bool FTranslucentVelocityMeshProcessor::PrimitiveHasVelocityForFrame(const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	return true;
}

bool FTranslucentVelocityMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FMaterial* Material)
{
	// Whether the primitive is marked to write translucent velocity / depth.
	const bool bMaterialWritesVelocity = Material->IsTranslucencyWritingVelocity();

	bool bResult = true;
	if (MeshBatch.bUseForMaterial && bMaterialWritesVelocity)
	{
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(*Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(*Material, OverrideSettings);

		bResult = Process(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, *MaterialRenderProxy, *Material, MeshFillMode, MeshCullMode);
	}
	return bResult;
}

void FTranslucentVelocityMeshProcessor::AddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);

	if (!PrimitiveCanHaveVelocity(ShaderPlatform, PrimitiveSceneProxy))
	{
		return;
	}

	if (ViewIfDynamicMeshCommand)
	{
		if (!PrimitiveHasVelocityForFrame(PrimitiveSceneProxy))
		{
			return;
		}

		checkSlow(ViewIfDynamicMeshCommand->bIsViewInfo);
		FViewInfo* ViewInfo = (FViewInfo*)ViewIfDynamicMeshCommand;

		if (!PrimitiveHasVelocityForView(*ViewInfo, PrimitiveSceneProxy))
		{
			return;
		}
	}

	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

void FTranslucentVelocityMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers
)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	if (!PrimitiveCanHaveVelocity(ShaderPlatform, nullptr))
	{
		return;
	}

	// Whether the primitive is marked to write translucent velocity / depth.
	const bool bMaterialWritesVelocity = Material.IsTranslucencyWritingVelocity();
	if (PreCacheParams.bRenderInMainPass && bMaterialWritesVelocity)
	{
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
		CollectPSOInitializersInternal(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, PSOInitializers);
	}
}

bool GetVelocityPassShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	EVelocityPassMode PassMode,
	TShaderRef<FVelocityVS>& VertexShader,
	TShaderRef<FVelocityPS>& PixelShader)
{
	FMaterialShaderTypes ShaderTypes;

	switch (PassMode)
	{
		case EVelocityPassMode::Velocity_Standard:
			ShaderTypes.PipelineType = &StandardVelocityPipeline;
			ShaderTypes.AddShaderType<TVelocityVS<EVelocityPassMode::Velocity_Standard>>();
			ShaderTypes.AddShaderType<TVelocityPS<EVelocityPassMode::Velocity_Standard>>();
			break;

		case EVelocityPassMode::Velocity_ClippedDepth:
			ShaderTypes.PipelineType = &VelocityClippedDepthPipeline;
			ShaderTypes.AddShaderType<TVelocityVS<EVelocityPassMode::Velocity_ClippedDepth>>();
			ShaderTypes.AddShaderType<TVelocityPS<EVelocityPassMode::Velocity_ClippedDepth>>();
			break;

		case EVelocityPassMode::Velocity_StereoMotionVectors:
			ShaderTypes.PipelineType = &VelocityMotionVectorsPipeline;
			ShaderTypes.AddShaderType<TVelocityVS<EVelocityPassMode::Velocity_StereoMotionVectors>>();
			ShaderTypes.AddShaderType<TVelocityPS<EVelocityPassMode::Velocity_StereoMotionVectors>>();
			break;

		default:
			checkNoEntry();
			break;
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}

bool FVelocityMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	int32 StaticMeshId,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		FVelocityVS,
		FVelocityPS> VelocityPassShaders;

	const bool bIsTranslucentClippedDepthPass = (MeshPassType == EMeshPass::TranslucentVelocityClippedDepth);

	EVelocityPassMode PassMode = EVelocityPassMode::Velocity_Standard;
	if (PlatformSupportsOpenXRMotionVectors(GetFeatureLevelShaderPlatform(FeatureLevel)))
	{
		checkf(!bIsTranslucentClippedDepthPass, TEXT("Translucent velocity clipped depth is not supported alongside motion vector rendering. If this is causing problems in your project, disable r.Velocity.DirectlyRenderOpenXRMotionVectors."));
		PassMode = EVelocityPassMode::Velocity_StereoMotionVectors;
	}
	else if (bIsTranslucentClippedDepthPass)
	{
		PassMode = EVelocityPassMode::Velocity_ClippedDepth;
	}

	if (!GetVelocityPassShaders(
		MaterialResource,
		VertexFactory->GetType(),
		FeatureLevel,
		PassMode,
		VelocityPassShaders.VertexShader,
		VelocityPassShaders.PixelShader))
	{
		return false;
	}

	// When velocity is used as a depth pass we need to set a correct stencil state on mobile
	bool bAllowDither = false;
	
	if (FeatureLevel == ERHIFeatureLevel::ES3_1 && EarlyZPassMode == DDM_AllOpaqueNoVelocity)
	{
		extern TOptional<uint32> GetOptionalDitheringFlag(
			const FSceneView * SceneView,
			const FMeshBatch & RESTRICT Mesh,
			int32 StaticMeshId);

		extern void SetMobileBasePassDepthState(
			FMeshPassProcessorRenderState& DrawRenderState, 
			const FPrimitiveSceneProxy* PrimitiveSceneProxy,
			const FMaterial& Material, 
			FMaterialShadingModelField ShadingModels, 
			bool bUsesDeferredShading,
			TOptional<uint32> DitheringFlag, 
			bool bKeepStencilDither);

		// *Don't* get shading models from MaterialResource since it's for a default material
		FMaterialShadingModelField ShadingModels = MeshBatch.MaterialRenderProxy->GetIncompleteMaterialWithFallback(ERHIFeatureLevel::ES3_1).GetShadingModels();
		bool bUsesDeferredShading = IsMobileDeferredShadingEnabled(GetFeatureLevelShaderPlatform(FeatureLevel));
		SetMobileBasePassDepthState(
			PassDrawRenderState, 
			PrimitiveSceneProxy, 
			MaterialResource, 
			ShadingModels, 
			bUsesDeferredShading, 
			GetOptionalDitheringFlag(ViewIfDynamicMeshCommand, MeshBatch, StaticMeshId),
			true);
			
		bAllowDither = true;
	}

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, bAllowDither);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(VelocityPassShaders.VertexShader, VelocityPassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		VelocityPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

bool FVelocityMeshProcessor::CollectPSOInitializersInternal(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FMaterial& RESTRICT MaterialResource, 
	ERasterizerFillMode MeshFillMode, 
	ERasterizerCullMode MeshCullMode,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	TMeshProcessorShaders<
		FVelocityVS,
		FVelocityPS> VelocityPassShaders;

	const bool bIsTranslucentClippedDepthPass = (MeshPassType == EMeshPass::TranslucentVelocityClippedDepth);
			
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);

	// Only enabled on mobile when any AA mode is enabled
	EAntiAliasingMethod DefaultAntiAliasingMethod = GetDefaultAntiAliasingMethod(FeatureLevel);
	if (IsMobilePlatform(ShaderPlatform) && DefaultAntiAliasingMethod == AAM_None)
	{
		return false;
	}

	EVelocityPassMode PassMode = EVelocityPassMode::Velocity_Standard;
	if (PlatformSupportsOpenXRMotionVectors(GetFeatureLevelShaderPlatform(FeatureLevel)))
	{
		checkf(!bIsTranslucentClippedDepthPass, TEXT("Translucent velocity clipped depth is not supported alongside motion vector rendering. If this is causing problems in your project, disable r.Velocity.DirectlyRenderOpenXRMotionVectors."));
		PassMode = EVelocityPassMode::Velocity_StereoMotionVectors;
	}
	else if (bIsTranslucentClippedDepthPass)
	{
		PassMode = EVelocityPassMode::Velocity_ClippedDepth;
	}

	if (!GetVelocityPassShaders(
		MaterialResource,
		VertexFactoryData.VertexFactoryType,
		FeatureLevel,
		PassMode,
		VelocityPassShaders.VertexShader,
		VelocityPassShaders.PixelShader))
	{
		return false;
	}

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;
	AddRenderTargetInfo(FVelocityRendering::GetFormat(ShaderPlatform), FVelocityRendering::GetCreateFlags(ShaderPlatform), RenderTargetsInfo);
	{
		ETextureCreateFlags DepthStencilCreateFlags = SceneTexturesConfig.DepthCreateFlags;
		SetupDepthStencilInfo(PF_DepthStencil, DepthStencilCreateFlags, ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite, RenderTargetsInfo);
	}

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		MaterialResource,
		PassDrawRenderState,
		RenderTargetsInfo,
		VelocityPassShaders,
		MeshFillMode,
		MeshCullMode,
		PT_TriangleList,
		EMeshPassFeatures::Default,
		true /*bRequired*/,
		PSOInitializers);

	return true;
}

FVelocityMeshProcessor::FVelocityMeshProcessor(EMeshPass::Type InMeshPassType, const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(InMeshPassType, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InPassDrawRenderState)
{
	MeshPassType = InMeshPassType;
}

FOpaqueVelocityMeshProcessor::FOpaqueVelocityMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext, EDepthDrawingMode InEarlyZPassMode)
	: FVelocityMeshProcessor(EMeshPass::Velocity, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InPassDrawRenderState, InDrawListContext)
{
	EarlyZPassMode = InEarlyZPassMode;
}

FMeshPassProcessor* CreateVelocityPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	EDepthDrawingMode EarlyZPassMode;
	bool bEarlyZPassMovable;
	FScene::GetEarlyZPassMode(FeatureLevel, EarlyZPassMode, bEarlyZPassMovable);
	
	FMeshPassProcessorRenderState VelocityPassState;
	VelocityPassState.SetBlendState(TStaticBlendState<CW_RGBA>::GetRHI());

	const bool bNeedStationaryPrimitiveDepth = VelocityIncludeStationaryPrimitives(GetFeatureLevelShaderPlatform(FeatureLevel));
	VelocityPassState.SetDepthStencilState(bNeedStationaryPrimitiveDepth || (EarlyZPassMode == DDM_AllOpaqueNoVelocity) // if the depth mode is all opaque except velocity, it relies on velocity to write the depth of the remaining meshes
										    ? TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI()
											: TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

	return new FOpaqueVelocityMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, VelocityPassState, InDrawListContext, EarlyZPassMode);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(VelocityPass, CreateVelocityPassProcessor, EShadingPath::Deferred,  EMeshPass::Velocity, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileVelocityPass, CreateVelocityPassProcessor, EShadingPath::Mobile,  EMeshPass::Velocity, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);

FTranslucentVelocityMeshProcessor::FTranslucentVelocityMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext, EMeshPass::Type MeshPass)
	: FVelocityMeshProcessor(MeshPass, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InPassDrawRenderState, InDrawListContext)
{}

FMeshPassProcessor* CreateTranslucentVelocityPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState VelocityPassState;
	VelocityPassState.SetBlendState(TStaticBlendState<CW_RGBA>::GetRHI());
	VelocityPassState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());

	return new FTranslucentVelocityMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, VelocityPassState, InDrawListContext, EMeshPass::TranslucentVelocity);
}

FMeshPassProcessor* CreateTranslucentVelocityClippedDepthPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState VelocityPassState;
	VelocityPassState.SetBlendState(TStaticBlendState<CW_RGBA>::GetRHI());
	VelocityPassState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

	return new FTranslucentVelocityMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, VelocityPassState, InDrawListContext, EMeshPass::TranslucentVelocityClippedDepth);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(TranslucentVelocityPass, CreateTranslucentVelocityPassProcessor, EShadingPath::Deferred, EMeshPass::TranslucentVelocity, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(TranslucentVelocityClippedDepthPass, CreateTranslucentVelocityClippedDepthPassProcessor, EShadingPath::Deferred, EMeshPass::TranslucentVelocityClippedDepth, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileTranslucentVelocityPass, CreateTranslucentVelocityPassProcessor, EShadingPath::Mobile, EMeshPass::TranslucentVelocity, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
//TODO: Add Mobile translucent velocity clipped depth pass process when it is ready to support.
