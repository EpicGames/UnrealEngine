// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "HitProxies.h"
#include "RendererInterface.h"
#include "DepthRendering.h"

class FPrimitiveSceneInfo;
class FPrimitiveSceneProxy;
class FScene;
class FStaticMeshBatch;
class FViewInfo;

class FVertexFactoryType;
struct FPrimitiveViewRelevance;

enum class EVelocityPass : uint32
{
	// Renders a separate velocity pass for opaques.
	Opaque = 0,

	// Renders a separate velocity / depth pass for translucency AFTER the translucent pass.
	Translucent,

	// Renders a secondary translucent velocity / depth pass for pixels that are clipped by opacity mask clip value (e.g., write has pixel animation mask).
	TranslucentClippedDepth,

	Count
};

EMeshPass::Type GetMeshPassFromVelocityPass(EVelocityPass VelocityPass);

enum EVelocityPassMode
{
	Velocity_Standard,
	Velocity_ClippedDepth,
	Velocity_StereoMotionVectors,
};

class FVelocityVS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FVelocityVS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return false;
	}

	FVelocityVS() = default;
	FVelocityVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}
};

class FVelocityPS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FVelocityPS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return false;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_A16B16G16R16);
		OutEnvironment.SetRenderTargetOutputFormat(1, PF_A16B16G16R16);

		// We support velocity on thin translucent only with masking, and only if the material is only made of thin translucent shading model.
		OutEnvironment.SetDefine(TEXT("VELOCITY_THIN_TRANSLUCENT_MODE"), Parameters.MaterialParameters.ShadingModels.HasOnlyShadingModel(MSM_ThinTranslucent));
	}

	FVelocityPS() = default;
	FVelocityPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}
};

bool GetVelocityPassShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	EVelocityPassMode PassMode,
	TShaderRef<FVelocityVS>& VertexShader,
	TShaderRef<FVelocityPS>& PixelShader);

bool SupportsTemporalResponsiveness(EShaderPlatform ShaderPlatform);
bool SupportsPixelShaderMotionVectorWorldOffset(EShaderPlatform ShaderPlatform);

/** 
* Resolves motion vector world offset and writes the merged velocity into an output texture.
* The SLW variant writes only to pixels that have MVWO data, leaving the rest untouched.
*
* @param StandardVelocityTexture  Contains standard velocity
* @param MVWOOffsetTexture        Contains the MVWO offset
* @param OutputVelocityTexture    Receives the merged velocity. Only pixels with MVWO data are written.
*/
void ResolveSLWMotionVectorWorldOffset(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> InViews,
	FRDGTextureRef DepthResolve,
	FRDGTextureRef StandardVelocityTexture,
	FRDGTextureRef MVWOOffsetTexture,
	FRDGTextureRef OutputVelocityTexture,
	EShaderPlatform ShaderPlatform);

struct FTranslucencyVelocityContext
{
	/** Velocity texture produced by early translucent velocity pass. nullptr if early path not taken. */
	FRDGTextureRef EarlyVelocity = nullptr;

	/** Scratch depth texture used by early translucent velocity pass. nullptr if early path not taken. */
	FRDGTextureRef EarlyDepth = nullptr;

	/** Holds the original scene depth with stencil bits. Translucency depth is not included. */
	FRDGTextureRef OriginalSceneDepthForStencil = nullptr;
};

// Group Velocity Rendering accessors, types, etc.
struct FVelocityRendering
{
	/** Returns the texture format for the velocity buffer. */
	static EPixelFormat GetFormat(EShaderPlatform ShaderPlatform);

	/** Returns the texture create flags for the velocity buffer. */
	static ETextureCreateFlags GetCreateFlags(EShaderPlatform ShaderPlatform);
	
	/** Returns the render target description for the velocity buffer. */
	static FRDGTextureDesc GetRenderTargetDesc(EShaderPlatform ShaderPlatform, FIntPoint Extent, const bool bRequireMultiView = false);

	/** Returns true if a velocity pass is supported. */
	static bool IsVelocityPassSupported(EShaderPlatform ShaderPlatform);

	/** Returns true if the velocity can be output during depth pass. */
	static bool DepthPassCanOutputVelocity(ERHIFeatureLevel::Type FeatureLevel);

	/** Returns true if the velocity can be output in the BasePass. */
	static bool BasePassCanOutputVelocity(EShaderPlatform ShaderPlatform);

	/** Returns true if the velocity pass is using parallel dispatch. */
	static bool IsParallelVelocity(EShaderPlatform ShaderPlatform);
};

/**
 * Base velocity mesh pass processor class. Used for both opaque and translucent velocity passes.
 */
class FVelocityMeshProcessor : public FMeshPassProcessor
{
public:
	FVelocityMeshProcessor(
		EMeshPass::Type MeshPassType,
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand,
		const FMeshPassProcessorRenderState& InPassDrawRenderState,
		FMeshPassDrawListContext* InDrawListContext);

	FMeshPassProcessorRenderState PassDrawRenderState;

	/* Checks whether the primitive should emit velocity for the current view by comparing screen space size against a threshold. */
	static bool PrimitiveHasVelocityForView(const FViewInfo& View, const FPrimitiveSceneProxy* PrimitiveSceneProxy);

protected:
	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	bool CollectPSOInitializersInternal(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		TArray<FPSOPrecacheData>& PSOInitializers);

protected:
	EDepthDrawingMode EarlyZPassMode = DDM_None;
	EMeshPass::Type MeshPassType = EMeshPass::Type::Num;
};

/**
 * Velocity pass processor for rendering opaques into a separate velocity pass (i.e. separate from the base pass).
 */
class FOpaqueVelocityMeshProcessor : public FSceneRenderingAllocatorObject<FOpaqueVelocityMeshProcessor>, public FVelocityMeshProcessor
{
public:
	FOpaqueVelocityMeshProcessor(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand,
		const FMeshPassProcessorRenderState& InPassDrawRenderState,
		FMeshPassDrawListContext* InDrawListContext,
		EDepthDrawingMode InEarlyZPassMode);

	/** Returns true if the object is capable of having velocity for any frame. */
	static bool PrimitiveCanHaveVelocity(EShaderPlatform ShaderPlatform, const FPrimitiveSceneProxy* PrimitiveSceneProxy);
	static bool PrimitiveCanHaveVelocity(EShaderPlatform ShaderPlatform, bool bDrawVelocity, bool bHasStaticLighting);

	/** Returns true if the primitive has velocity for the current frame. */
	static bool PrimitiveHasVelocityForFrame(EShaderPlatform ShaderPlatform, const FPrimitiveSceneProxy* PrimitiveSceneProxy);

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy* MaterialRenderProxy,
		const FMaterial* Material);

	virtual void AddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId = -1) override final;

	virtual void CollectPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams,
		TArray<FPSOPrecacheData>& PSOInitializers) override final;
};

/**
 * Velocity pass processor for rendering translucent object velocity and depth. This pass is rendered AFTER the
 * translucent pass so that depth can safely be written.
 */
class FTranslucentVelocityMeshProcessor : public FSceneRenderingAllocatorObject<FTranslucentVelocityMeshProcessor>, public FVelocityMeshProcessor
{
public:
	FTranslucentVelocityMeshProcessor(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand,
		const FMeshPassProcessorRenderState& InPassDrawRenderState,
		FMeshPassDrawListContext* InDrawListContext,
		EMeshPass::Type Meshpass);

	/** Returns true if the object is capable of having velocity for any frame. */
	static bool PrimitiveCanHaveVelocity(EShaderPlatform ShaderPlatform, const FPrimitiveSceneProxy* PrimitiveSceneProxy);

	/** Returns true if the primitive has velocity for the current frame. */
	static bool PrimitiveHasVelocityForFrame(const FPrimitiveSceneProxy* PrimitiveSceneProxy);

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy* MaterialRenderProxy,
		const FMaterial* Material);

	virtual void AddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId = -1) override final;

	virtual void CollectPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material, 
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams,
		TArray<FPSOPrecacheData>& PSOInitializers) override final;
};