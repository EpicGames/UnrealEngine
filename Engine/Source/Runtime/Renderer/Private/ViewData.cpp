// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewData.h"
#include "ScenePrivate.h"

FRendererViewDataManager::FRendererViewDataManager(FRDGBuilder& GraphBuilder, const FScene& InScene, FSceneUniformBuffer& InSceneUniforms, TArray<FViewInfo*> &InOutSceneRendererPrimaryViews)
	: Scene(InScene), GPUScene(InScene.GPUScene), SceneUniforms(InSceneUniforms), SceneRendererPrimaryViews(InOutSceneRendererPrimaryViews), bIsEnabled(GPUScene.IsEnabled())
{
	if (bIsEnabled)
	{
		{
			TArray<uint32> PersistentIdToIndexMap;
			PersistentIdToIndexMap.SetNumZeroed(InScene.GetMaxPersistentViewId());

			NumSceneRendererPrimaryViews = SceneRendererPrimaryViews.Num();
			AllocateViews(NumSceneRendererPrimaryViews);
			for (int32 ViewIndex = 0; ViewIndex < SceneRendererPrimaryViews.Num(); ++ViewIndex)
			{
				const FViewInfo &View = *SceneRendererPrimaryViews[ViewIndex];
				ensure(View.SceneRendererPrimaryViewId == RegisterPrimaryView(View));

				if (View.PersistentViewId.IsValid())
				{
					// Allow zero to mean invalid.
					PersistentIdToIndexMap[View.PersistentViewId.Index] = ViewIndex + 1;
				}
			}
	
			PersistentIdToIndexMapRDG = CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCullingManager.PersistentIdToIndexMap"), MoveTemp(PersistentIdToIndexMap));
		}
		CullingShaderParameters.InViews = nullptr;
		CullingShaderParameters.NumCullingViews = 0;

		// Create a deferred buffer and flush when all views are registered. This doesn't work when immediate mode is used, and in that case we need to re-upload partial buffers when the buffer is requested
		if (!GraphBuilder.IsImmediateMode())
		{
			CullingViewsRDG = CreateStructuredBuffer(GraphBuilder, TEXT("ViewData.CullingViews"), [this]() -> TArray<Nanite::FPackedView>& { return CullingViews; });
			CullingShaderParameters.InViews = GraphBuilder.CreateSRV(CullingViewsRDG);
			CullingShaderParameters.NumCullingViews = NumSceneRendererPrimaryViews;
		}
		else
		{
			FlushRegisteredViews(GraphBuilder);
		}
		PrimaryViewsRDG = CreateStructuredBuffer(GraphBuilder, TEXT("ViewData.PrimaryViews"), CullingViews);
		CullingShaderParameters.NumSceneRendererPrimaryViews = NumSceneRendererPrimaryViews;
	}
}

void FRendererViewDataManager::InitInstanceState(FRDGBuilder& GraphBuilder)
{
	if (bIsEnabled)
	{
		int32 NumInstances = GPUScene.GetNumInstances();
		InstanceMaskWordStride = FMath::DivideAndRoundUp(NumInstances, 32);
		int32 NumWords = InstanceMaskWordStride * NumSceneRendererPrimaryViews;
	
		DeformingInstanceViewMaskRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(4u, FMath::Max(1, NumWords)), TEXT("ViewData.DeformingInstanceViewMask"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(DeformingInstanceViewMaskRDG), 0u);

		ShaderParameters.DeformingInstanceViewMask = GraphBuilder.CreateSRV(DeformingInstanceViewMaskRDG);
		ShaderParameters.PersistentIdToIndexMap = GraphBuilder.CreateSRV(PersistentIdToIndexMapRDG);
		ShaderParameters.Common.InstanceMaskWordStride = InstanceMaskWordStride;
		ShaderParameters.Common.NumSceneRendererPrimaryViews = NumSceneRendererPrimaryViews;
		ShaderParameters.Common.MaxPersistentViewId = Scene.GetMaxPersistentViewId();
		ShaderParameters.Common.InViews = GraphBuilder.CreateSRV(PrimaryViewsRDG);
		SceneUniforms.Set(SceneUB::ViewData, ShaderParameters);

		WriterShaderParameters.OutDeformingInstanceViewMask = GraphBuilder.CreateUAV(DeformingInstanceViewMaskRDG,ERDGUnorderedAccessViewFlags::SkipBarrier);
		WriterShaderParameters.Common = ShaderParameters.Common;
	}
}

void FRendererViewDataManager::AllocateViews(int32 NumViews)
{
	if (bIsEnabled)
	{
		CullingViews.AddUninitialized(CullingViews.Num() + NumViews);
	}
}

int32 FRendererViewDataManager::RegisterPrimaryView(const FViewInfo& ViewInfo)
{
	Nanite::FPackedViewParams Params;
	Params.ViewMatrices = ViewInfo.ViewMatrices;
	Params.PrevViewMatrices = ViewInfo.PrevViewInfo.ViewMatrices;
	Params.ViewRect = ViewInfo.ViewRect;
	// TODO: faking this here (not needed for culling, until we start involving multi-view and HZB)
	Params.RasterContextSize = ViewInfo.ViewRect.Size();
	Params.ViewLODDistanceFactor = ViewInfo.LODDistanceFactor;
	Params.HZBTestViewRect = FIntRect(0, 0, ViewInfo.PrevViewInfo.ViewRect.Width(), ViewInfo.PrevViewInfo.ViewRect.Height());	// needs to be in HZB space, which is 0,0-based for any view, even stereo/splitscreen ones
	Params.MaxPixelsPerEdgeMultipler = 1.0f;
	Params.InstanceOcclusionQueryMask = ViewInfo.PrevViewInfo.InstanceOcclusionQueryMask;
	Params.SceneRendererPrimaryViewId = ViewInfo.SceneRendererPrimaryViewId;
	Nanite::SetViewFrameIndex(&ViewInfo, Params);

	return RegisterView(Params);
}

int32 FRendererViewDataManager::RegisterView(const Nanite::FPackedViewParams& Params)
{
	if (!bIsEnabled)
	{
		return 0;
	}

	const int32 ViewIndex = NumRegisteredViews.fetch_add(1, std::memory_order_relaxed);
	check(ViewIndex <= CullingViews.Num());
	CullingViews[ViewIndex] = CreatePackedView(Params);
	return ViewIndex;
}

void FRendererViewDataManager::FlushRegisteredViews(FRDGBuilder& GraphBuilder)
{
	const int32 LocalNumRegisteredViews = NumRegisteredViews.load(std::memory_order_relaxed);

	if (CullingShaderParameters.NumCullingViews != LocalNumRegisteredViews)
	{
		// No need to recreate in deferred upload mode
		if (GraphBuilder.IsImmediateMode())
		{
			CullingViewsRDG = CreateStructuredBuffer(GraphBuilder, TEXT("ViewData.CullingViews"), MakeArrayView<const Nanite::FPackedView>(CullingViews.GetData(), LocalNumRegisteredViews));
			CullingShaderParameters.InViews = GraphBuilder.CreateSRV(CullingViewsRDG);
		}
		CullingShaderParameters.NumCullingViews = LocalNumRegisteredViews;
	}
}

RendererViewData::FCullingShaderParameters FRendererViewDataManager::GetCullingParameters(FRDGBuilder& GraphBuilder)
{
	FlushRegisteredViews(GraphBuilder);
	return CullingShaderParameters;
}

void GetDefaultParameters(RendererViewData::FParameters& OutParameters, FRDGBuilder& GraphBuilder)
{
	OutParameters.Common.NumSceneRendererPrimaryViews = 0u;
	OutParameters.Common.InstanceMaskWordStride = 0u;
	OutParameters.Common.MaxPersistentViewId = 0;
	OutParameters.Common.InViews = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(Nanite::FPackedView)));

	OutParameters.DeformingInstanceViewMask = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 4u, 0xFFFFFFFFu));
	OutParameters.PersistentIdToIndexMap = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 4u));
}

IMPLEMENT_SCENE_UB_STRUCT(RendererViewData::FParameters, ViewData, GetDefaultParameters);
