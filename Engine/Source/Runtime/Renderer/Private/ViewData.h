// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShaderParameterMacros.h"
#include "SceneUniformBuffer.h"
//#include "Nanite/NaniteShared.h"

class FViewInfo;
class FRDGBuilder;
class FScene;
class FGPUScene;

namespace Nanite
{
	struct FPackedView;
	struct FPackedViewParams;
}

namespace RendererViewData
{

BEGIN_SHADER_PARAMETER_STRUCT(FCommonParameters, RENDERER_API)
	// InViews represents the scene renderer primary views (passed to FRendererViewDataManager constructor), and are named "InViews" to be compatible with Nanite conventions (see GetNaniteView),
	// The buffer may also contain all the non-primary views but these are not generally accessible.
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPackedNaniteView >, InViews)
	SHADER_PARAMETER(int32, MaxPersistentViewId)
	// Number of scene renderer primary views.
	SHADER_PARAMETER(uint32, NumSceneRendererPrimaryViews)
	// Stride between each bit vector in the per-view bit masks in dwords.
	SHADER_PARAMETER(uint32, InstanceMaskWordStride)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FCommonParameters, Common)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, DeformingInstanceViewMask)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PersistentIdToIndexMap)
END_SHADER_PARAMETER_STRUCT()

/**
 * Parameters to use in kernels modifying instance visibility state.
 * Use the API to abstract any access, see: ViewData.ush.
 */
BEGIN_SHADER_PARAMETER_STRUCT(FWriterParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FCommonParameters, Common)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutDeformingInstanceViewMask)
END_SHADER_PARAMETER_STRUCT()


BEGIN_SHADER_PARAMETER_STRUCT(FCullingShaderParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPackedNaniteView >, InViews)
	SHADER_PARAMETER(uint32, NumSceneRendererPrimaryViews)
	SHADER_PARAMETER(uint32, NumCullingViews)
END_SHADER_PARAMETER_STRUCT()

}

DECLARE_SCENE_UB_STRUCT(RendererViewData::FParameters, ViewData, RENDERER_API)

/**
 * Takes care of all view data management that belongs to a given FSceneRenderer,
 */
class FRendererViewDataManager
{
public:
	/**
	 * Construct the renderer
	 */
	FRendererViewDataManager(FRDGBuilder& GraphBuilder, const FScene& InScene, FSceneUniformBuffer& InSceneUniforms, TArray<FViewInfo*> &InOutSceneRendererPrimaryViews);

	bool IsEnabled() const { return bIsEnabled; }

	/**
	 * Register a non-primary view for culling, returns integer ID of the view.
	 * Note that these views are not accessible through the SceneUB and general ViewData.ush API as they are added later in the frame. 
	 * This could be changed in the future.
	 */ 
	int32 RegisterView(const Nanite::FPackedViewParams& Params);

	// Allocate space for views ahead of time prior to calling RegisterView.
	void AllocateViews(int32 NumViews);

	/**
	 * Must be called _after_ dynamic primitives are allocated & before instance visibility and state processing has begun (i.e., anything that calls GetWriterShaderParameters).
	 */
	void InitInstanceState(FRDGBuilder& GraphBuilder);

	/**
	 * Shader parameters used for culling where all registered views are concerned (not just the primary)
	 * The secondary views are not accessible throught the scene UB.
	 */
	RendererViewData::FCullingShaderParameters GetCullingParameters(FRDGBuilder& GraphBuilder);

	/**
	 */
	RendererViewData::FWriterParameters GetWriterShaderParameters(FRDGBuilder& GraphBuilder) const { return WriterShaderParameters; }
	
	void FlushRegisteredViews(FRDGBuilder& GraphBuilder);

	const TArray<FViewInfo*> &GetRegisteredPrimaryViews() const { return SceneRendererPrimaryViews; }

	int32 GetNumCullingViews() const { return NumRegisteredViews.load(std::memory_order_relaxed); }


private:

	// Register a primary view
	int32 RegisterPrimaryView(const FViewInfo& ViewInfo);

	FRendererViewDataManager() = delete;
	FRendererViewDataManager(FRendererViewDataManager&) = delete;

	const FScene& Scene;
	const FGPUScene& GPUScene;
	FSceneUniformBuffer& SceneUniforms;
	TArray<FViewInfo*> &SceneRendererPrimaryViews;

	std::atomic_int32_t NumRegisteredViews = {0};
	TArray<Nanite::FPackedView> CullingViews;
	bool bIsEnabled;

	FRDGBuffer* CullingViewsRDG = nullptr;
	FRDGBuffer* PrimaryViewsRDG = nullptr;
	int32 NumSceneRendererPrimaryViews = 0;
	int32 InstanceMaskWordStride = 0;
	// Non view index strided buffer of bits, one per instance that is deforming (animating or something like that), indexed by InstanceId.
	// Initialized to zero each frame & updated by interested scene extensions / systems.
	// Also in the same buffer laid out after the above
	// 1. counter of number of instances marked & 
	FRDGBuffer* DeformingInstanceViewMaskRDG = nullptr;
	FRDGBuffer* PersistentIdToIndexMapRDG = nullptr;

	RendererViewData::FCullingShaderParameters CullingShaderParameters;
	RendererViewData::FParameters	ShaderParameters;
	RendererViewData::FWriterParameters	WriterShaderParameters;
};
