// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewDebug.h"

#include "SkeletalRenderPublic.h"
#include "Kismet/GameplayStatics.h"

#if !UE_BUILD_SHIPPING

#include "ScenePrivate.h"
#include "SceneInterface.h"
#include "SceneView.h"
#include "Materials/Material.h"
#include "MaterialShared.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/LocalPlayer.h"
#include "Engine/GameViewportClient.h"
#include "UnrealClient.h"
#include "Materials/MaterialInterface.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"

static bool bDumpPrimitiveDrawCallsNextFrame = false;
static bool bDumpDetailedPrimitivesNextFrame = false;

static FAutoConsoleCommand CVarDumpPrimitives(
	TEXT("DumpPrimitiveDrawCalls"),
	TEXT("Writes the draw call count of all primitives tracked by the PrimitiveDebugger to a CSV file"),
	FConsoleCommandDelegate::CreateStatic([] { bDumpPrimitiveDrawCallsNextFrame = true; }),
	ECVF_Default);

static FAutoConsoleCommand CVarDrawPrimitiveDebugData(
	TEXT("DumpDetailedPrimitives"),
	TEXT("Writes the detailed information of all primitives tracked by the PrimitiveDebugger to a CSV file"),
	FConsoleCommandDelegate::CreateStatic([] { bDumpDetailedPrimitivesNextFrame = !bDumpDetailedPrimitivesNextFrame; }),
	ECVF_Default);

FViewDebugInfo FViewDebugInfo::Instance;

FViewDebugInfo::FViewDebugInfo()
{
	bHasEverUpdated = false;
	bIsOutdated = true;
	bShouldUpdate = false;
	bShouldCaptureSingleFrame = false;
	bShouldClearCapturedData = false;
}

int32 FViewDebugInfo::FPrimitiveInfo::ComputeCurrentLODIndex(int32 PlayerIndex, int32 ViewIndex) const
{
	if (!IsPrimitiveValid() || !ComponentInterface->GetSceneProxy()) return INDEX_NONE;
	if (const USkinnedMeshComponent* SkinnedMesh = ComponentInterface->GetUObject<USkinnedMeshComponent>())
	{
		if (SkinnedMesh->MeshObject)
		{
			// Skinned meshes do not implement the GetLOD function for proxies, instead grab it from the mesh object
			return SkinnedMesh->MeshObject->GetLOD();
		}
	}
	APlayerController* PlayerController = UGameplayStatics::GetPlayerController(Owner.Get(), PlayerIndex);
	if (!IsValid(PlayerController))
	{
		return INDEX_NONE;
	}
	ULocalPlayer* LocalPlayer = Cast<ULocalPlayer>(PlayerController->Player);
	if (IsValid(LocalPlayer) && IsValid(LocalPlayer->ViewportClient))
	{
		// see: AHUD::GetCoordinateOffset() and UGameViewportClient::Draw(FViewport* InViewport, FCanvas* SceneCanvas)
		
		// Create a view family for the game viewport
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			LocalPlayer->ViewportClient->Viewport,
			Owner->GetWorld()->Scene,
			LocalPlayer->ViewportClient->EngineShowFlags)
			.SetRealtimeUpdate(false));

		// Calculate a view where the player is
		FVector ViewLocation;
		FRotator ViewRotation;
		const FSceneView* SceneView = LocalPlayer->CalcSceneView(&ViewFamily, /*out*/ ViewLocation, /*out*/ ViewRotation, LocalPlayer->ViewportClient->Viewport, nullptr, ViewIndex);

		const int32 LOD = SceneView ? ComponentInterface->GetSceneProxy()->GetLOD(SceneView) : INDEX_NONE;

		if (IsLODIndexValid(LOD))
		{
			return LOD;
		}
	}
	return INDEX_NONE;
}

void FViewDebugInfo::ProcessPrimitive(FPrimitiveSceneInfo* PrimitiveSceneInfo, const FViewInfo& View, FScene* Scene, IPrimitiveComponent* DebugComponentInterface)
{
	if (!DebugComponentInterface || !DebugComponentInterface->IsRegistered() || !PrimitiveSceneInfo || !PrimitiveSceneInfo->Proxy)
	{
		return;
	}
	UObject* Owner = DebugComponentInterface->GetOwner();
	if (!IsValid(Owner)) return;
	FString FullName = DebugComponentInterface->GetName();
	
	FPrimitiveStats Stats;
	DebugComponentInterface->GetPrimitiveStats(Stats);
	
	TArray<TWeakObjectPtr<UMaterialInterface>> Materials;
	UMaterialInterface* OverlayMaterial = nullptr;
	int32 CurrentLOD = PrimitiveSceneInfo->Proxy->GetLOD(&View);
	
	if (const UPrimitiveComponent* DebugComponent = DebugComponentInterface->GetUObject<UPrimitiveComponent>())
	{
		const int32 NumMaterials = DebugComponent->GetNumMaterials();
		Materials.Reserve(NumMaterials);
		for (int32 Idx = 0; Idx < NumMaterials; Idx++)
		{
			if (UMaterialInterface* MaterialInterface = DebugComponent->GetMaterial(Idx))
			{
				Materials.Add(MaterialInterface);
			}
		}

		if (const UMeshComponent* MeshComponent = Cast<UMeshComponent>(DebugComponent))
		{
			OverlayMaterial = MeshComponent->GetOverlayMaterial();
			if (const USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(MeshComponent))
			{
				//CurrentLOD = SkinnedMeshComponent->GetPredictedLODLevel();
				CurrentLOD = SkinnedMeshComponent->MeshObject->GetLOD();
			}
		}
	}

	const FPrimitiveInfo PrimitiveInfo = {
		Owner,
		PrimitiveSceneInfo->PrimitiveComponentId,
		DebugComponentInterface,
		DebugComponentInterface->GetUObject(),
		PrimitiveSceneInfo,
		MoveTemp(FullName),
		MoveTemp(Stats),
		MoveTemp(Materials),
		OverlayMaterial,
		CurrentLOD
	};

	Primitives.Add(PrimitiveSceneInfo->PrimitiveComponentId, PrimitiveInfo);
}

void FViewDebugInfo::DumpToCSV() const
{
	const FString OutputPath = FPaths::ProfilingDir() / TEXT("Primitives") / FString::Printf(TEXT("PrimitivesDetailed-%s.csv"), *FDateTime::Now().ToString());
	const bool bSuppressViewer = true;
	FDiagnosticTableViewer DrawViewer(*OutputPath, bSuppressViewer);
	DrawViewer.AddColumn(TEXT("Name"));
	DrawViewer.AddColumn(TEXT("ActorClass"));
	DrawViewer.AddColumn(TEXT("Actor"));
	DrawViewer.AddColumn(TEXT("Location"));
	DrawViewer.AddColumn(TEXT("NumMaterials"));
	DrawViewer.AddColumn(TEXT("Materials"));
	DrawViewer.AddColumn(TEXT("NumDraws"));
	DrawViewer.AddColumn(TEXT("LOD"));
	DrawViewer.AddColumn(TEXT("Triangles"));
	DrawViewer.CycleRow();

	FRWScopeLock ScopeLock(Lock, SLT_ReadOnly);
	const FPrimitiveSceneInfo* LastPrimitiveSceneInfo = nullptr;
	for (const TTuple<FPrimitiveComponentId, FPrimitiveInfo>& Entry : Primitives)
	{
		const FPrimitiveInfo& Primitive = Entry.Value;
		if (Primitive.PrimitiveSceneInfo != LastPrimitiveSceneInfo)
		{
			const FPrimitiveLODStats* Stats = Primitive.GetCurrentLOD();
			DrawViewer.AddColumn(*Primitive.Name);
			DrawViewer.AddColumn(Primitive.Owner.IsValid() ? *Primitive.Owner->GetClass()->GetName() : TEXT(""));
			DrawViewer.AddColumn(Primitive.Owner.IsValid() ? *Primitive.Owner->GetFullName() : TEXT(""));
			DrawViewer.AddColumn(Primitive.IsPrimitiveValid() ?
				*FString::Printf(TEXT("{%s}"), *Primitive.GetPrimitiveLocation().ToString()) : TEXT(""));
			DrawViewer.AddColumn(*FString::Printf(TEXT("%d"), Primitive.Materials.Num()));
			FString Materials = "[";
			for (int i = 0; i < Primitive.Materials.Num(); i++)
			{
				if (Primitive.Materials[i].IsValid() && Primitive.Materials[i]->GetMaterial())
				{
					Materials += Primitive.Materials[i]->GetMaterial()->GetName();
				}
				else
				{
					Materials += "Null";
				}

				if (i < Primitive.Materials.Num() - 1)
				{
					Materials += ", ";
				}
			}
			Materials += "]";
			DrawViewer.AddColumn(*FString::Printf(TEXT("%s"), *Materials));
			DrawViewer.AddColumn(*FString::Printf(TEXT("%d"), Stats ? Stats->GetDrawCount() : 0));
			DrawViewer.AddColumn(*FString::Printf(TEXT("%d"), Stats ? Stats->LODIndex : -1));
			DrawViewer.AddColumn(*FString::Printf(TEXT("%u"), Stats ? Stats->Triangles : 0));
			DrawViewer.CycleRow();

			LastPrimitiveSceneInfo = Primitive.PrimitiveSceneInfo;
		}
	}
}

void FViewDebugInfo::CaptureNextFrame()
{
	ENQUEUE_RENDER_COMMAND(CmdShouldCaptureNextFrame)(
		[this](const FRHICommandListImmediate& RHICmdList)
		{
			bShouldCaptureSingleFrame = true;
			bShouldUpdate = true;
		});
}

void FViewDebugInfo::EnableLiveCapture()
{
	ENQUEUE_RENDER_COMMAND(CmdEnableLiveDebugCapture)(
    	[this](const FRHICommandListImmediate& RHICmdList)
    	{
    		bShouldCaptureSingleFrame = false;
    		bShouldUpdate = true;
    	});
}

void FViewDebugInfo::DisableLiveCapture()
{
	ENQUEUE_RENDER_COMMAND(CmdDisableLiveDebugCapture)(
    	[this](const FRHICommandListImmediate& RHICmdList)
    	{
    		bShouldCaptureSingleFrame = false;
    		bShouldUpdate = false;
    	});
}

void FViewDebugInfo::ClearCaptureData()
{
	ENQUEUE_RENDER_COMMAND(CmdShouldCaptureNextFrame)(
		[this](const FRHICommandListImmediate& RHICmdList)
		{
			bShouldClearCapturedData = true;
		});
}

bool FViewDebugInfo::HasEverUpdated() const
{
	FRWScopeLock ScopeLock(Lock, SLT_ReadOnly);
	return bHasEverUpdated;
}

bool FViewDebugInfo::IsOutOfDate() const
{
	FRWScopeLock ScopeLock(Lock, SLT_ReadOnly);
	return bIsOutdated;
}

void FViewDebugInfo::ProcessPrimitives(FScene* Scene, const FViewInfo& View, const FViewCommands& ViewCommands)
{
	if (bDumpPrimitiveDrawCallsNextFrame)
	{
		bDumpPrimitiveDrawCallsNextFrame = false;
		DumpDrawCallsToCSV();
	}

	{
		FRWScopeLock ScopeLock(Lock, SLT_Write);
		bIsOutdated = true;

		if (bShouldClearCapturedData)
		{
			Primitives.Empty();
			bShouldClearCapturedData = false;
		}
		
		if (!bShouldUpdate && !bDumpDetailedPrimitivesNextFrame)
		{
			return;
		}

		if (bShouldCaptureSingleFrame)
		{
			bShouldCaptureSingleFrame = false;
			bShouldUpdate = false;
		}
		
		Primitives.Empty();
 
		for (FSceneSetBitIterator BitIt(View.PrimitiveVisibilityMap); BitIt; ++BitIt)
		{
			const int32 PrimitiveIndex = BitIt.GetIndex();
			FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene->Primitives[PrimitiveIndex];
			ProcessPrimitive(PrimitiveSceneInfo, View, Scene, PrimitiveSceneInfo->GetComponentInterfaceForDebugOnly());
		}

		bHasEverUpdated = true;
		bIsOutdated = false;
	}
	AsyncTask(ENamedThreads::GameThread, [this]()
	{
		OnUpdate.Broadcast();
	});

	if (bDumpDetailedPrimitivesNextFrame)
	{
		DumpToCSV();
		bDumpDetailedPrimitivesNextFrame = false;
	}
}

void FViewDebugInfo::DumpDrawCallsToCSV()
{
	const FString OutputPath = FPaths::ProfilingDir() / TEXT("Primitives") / FString::Printf(TEXT("Primitives-%s.csv"), *FDateTime::Now().ToString());
	const bool bSuppressViewer = true;
	FDiagnosticTableViewer DrawViewer(*OutputPath, bSuppressViewer);
	DrawViewer.AddColumn(TEXT("Name"));
	DrawViewer.AddColumn(TEXT("NumDraws"));
	DrawViewer.CycleRow();

	FRWScopeLock ScopeLock(Lock, SLT_ReadOnly);
	const FPrimitiveSceneInfo* LastPrimitiveSceneInfo = nullptr;
	for (const TTuple<FPrimitiveComponentId, FPrimitiveInfo>& Entry : Primitives)
	{
		const FPrimitiveInfo& Primitive = Entry.Value;
		if (Primitive.PrimitiveSceneInfo != LastPrimitiveSceneInfo)
		{
			const FPrimitiveLODStats* Stats = Primitive.GetCurrentLOD();
			DrawViewer.AddColumn(*Primitive.Name);
			DrawViewer.AddColumn(*FString::Printf(TEXT("%d"), Stats ? Stats->GetDrawCount() : 0));
			DrawViewer.CycleRow();

			LastPrimitiveSceneInfo = Primitive.PrimitiveSceneInfo;
		}
	}
}
#endif