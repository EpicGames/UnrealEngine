// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualShadowMapArray.h"
#include "CoreMinimal.h"
#include "CanvasTypes.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "Containers/StringView.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DebugViewModeHelpers.h"
#include "DecalRenderingShared.h"
#include "DynamicRenderScaling.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "ExtensibleUniformBuffer.h"
#include "FrontLayerTranslucency.h"
#include "Froxel/Froxel.h"
#include "GlobalShader.h"
#include "GPUMessaging.h"
#include "GPUScene.h"
#include "GPUSkinCache.h"
#include "HairStrands/HairStrandsData.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingLoadBalancer.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "InstanceCulling/InstanceCullingMergedContext.h"
#include "LightComponentId.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "MegaLights/MegaLights.h"
#include "MeshDrawCommands.h"
#include "Misc/LargeWorldCoordinates.h"
#include "MultiGPU.h"
#include "Nanite/NaniteCullRaster.h"
#include "Nanite/NaniteShared.h"
#include "NaniteDefinitions.h"
#include "PipelineStateCache.h"
#include "PostProcess/DiaphragmDOF.h"
#include "PrimitiveSceneInfo.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/CsvProfilerTrace.h"
#include "ProfilingDebugging/TagTrace.h"
#include "RenderCommandTag.h"
#include "RendererInterface.h"
#include "LogRenderer.h"
#include "RendererPrivateUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "RenderingVisualizationUtils.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneCulling/SceneCullingRenderer.h"
#include "SceneExtensions.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShaderPrintParameters.h"
#include "ShadowRendering.h"
#include "Shadows/ShadowSceneRenderer.h"
#include "ShowFlags.h"
#include "SingleLayerWaterRendering.h"
#include "Stats/Stats.h"
#include "StochasticLighting/StochasticLightingViewState.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "Trace/Detail/Channel.h"
#include "UnifiedBuffer.h"
#include "VirtualShadowMapDefinitions.h"
#include "VirtualShadowMaps/VirtualShadowMapCacheManager.h"
#include "VirtualShadowMaps/VirtualShadowMapClipmap.h"
#include "VirtualShadowMaps/VirtualShadowMapProjection.h"
#include "VirtualShadowMaps/VirtualShadowMapShaders.h"
#include "VirtualShadowMapVisualizationData.h"
#include "VisualizationData/VisualizationDataShared.h"
#include "SceneViewState.h"

#define LOCTEXT_NAMESPACE "VirtualShadowMap"

IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(VirtualShadowMapUbSlot);

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FVirtualShadowMapUniformParameters, "VirtualShadowMap", VirtualShadowMapUbSlot);

// Disabled by default: use either console command "CsvCategory VSM" or command line argument "-CsvCategories=VSM[,...]" to enable.
CSV_DEFINE_CATEGORY(VSM, false);

UE_TRACE_CHANNEL_DEFINE(VSMChannel,
	"Virtual Shadow Map per-frame page statistics and cache diagnostics; "
	"emits Insights counters (use 'trace.enable counters,vsm' to view).");

DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Nanite Views (Primary)"), STAT_VSMNaniteViewsPrimary, STATGROUP_ShadowRendering);
DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Single Page Count"), STAT_VSMSinglePageCount, STATGROUP_ShadowRendering);
DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Full Count"), STAT_VSMFullCount, STATGROUP_ShadowRendering);

extern int32 GVSMMaxPageAgeSinceLastRequest;
extern TAutoConsoleVariable<float> CVarNaniteMaxPixelsPerEdge;
extern TAutoConsoleVariable<float> CVarNaniteMinPixelsPerEdgeHW;
extern float GMinScreenRadiusForShadowCaster;

static TAutoConsoleVariable<int32> CVarVSMDeferredInvalidationBudget(
	TEXT("r.Shadow.Virtual.DeferredInvalidationBudget"),
	-1,
	TEXT("Maximum number of VSM physical pages whose deferred-invalidation extended flag may be consumed per frame.\n")
	TEXT("Excess pages keep the flag set and retry next frame, so the total work is preserved but spread across frames.\n")
	TEXT("Today's only source of this flag is the streaming-LOD-delta channel from Nanite (see r.Nanite.VSMInvalidateOnLODDelta).\n")
	TEXT("  0  disables consumption entirely.\n")
	TEXT(" -1  (default) is unbounded (pre-throttle behavior, may spike)."),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

int32 GVSMShowLightDrawEvents = 0;
FAutoConsoleVariableRef CVarVSMShowLightDrawEvents(
	TEXT("r.Shadow.Virtual.ShowLightDrawEvents"),
	GVSMShowLightDrawEvents,
	TEXT("Enable Virtual Shadow Maps per-light draw events - may affect performance especially when there are many small lights in the scene."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarMarkPagesUseFroxels(
	TEXT("r.Shadow.Virtual.MarkPagesUsingFroxels"),
	0,
	TEXT("Experimental: If enabled the virtual shadow map pages are marked using froxels that are generated during HZB build.\n")
	TEXT("  Higher throughput as it is not bandwidth limited. Is approximate as it only marks the center of each froxel representing 8x8 pixels."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarDebugDrawFroxels(
	TEXT("r.Shadow.Virtual.DebugDrawFroxels"),
	0,
	TEXT("Render the froxels using shaderprint (which needs to be enabled) r.ShaderPrint.MaxLine also needs to be set to a high value as this produces many lines."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarDebugDrawFroxelRange(
	TEXT("r.Shadow.Virtual.DebugDrawFroxelRange"),
	20.0f,
	TEXT("Range in froxel tiles from the mouse cursor which to draw debug froxels in."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarEnableVirtualShadowMaps(
	TEXT("r.Shadow.Virtual.Enable"),
	0,
	TEXT("Enable Virtual Shadow Maps. Renders geometry into virtualized shadow depth maps for shadowing.\n")
	TEXT("Provides high - quality shadows for next - gen projects with simplified setup.High efficiency culling when used with Nanite."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		// Needed because the depth state changes with method (so cached draw commands must be re-created) see SetStateForShadowDepth
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarMaxPhysicalPages(
	TEXT("r.Shadow.Virtual.MaxPhysicalPages"),
	2048,
	TEXT("Maximum number of physical pages in the pool.\n")
	TEXT("More space for pages means more memory usage, but allows for higher resolution shadows.\n")
	TEXT("Ideally this value is large enough to fit enough pages for all the lights in the scene, but not too large to waste memory.\n")
	TEXT("Enable 'ShowStats' to see how many pages are allocated in the pool right now.\n")
	TEXT("For more page pool control, see the 'ResolutionLodBias*' cvars."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarBuildDynamicHZB(
	TEXT("r.Shadow.Virtual.DynamicHZB"),
	0,
	TEXT("When enabled, a separate HZB is built for dynamic cached pages.\n")
	TEXT("This can improve performance in cached scenes with a lot of dynamic overdraw, e.g. a forest with a static sun light.\n")
	TEXT("Constructing separate HZB doubles the memory cost of the HZB (1/4 of the page pool) and incurs some cost for building the second HZB."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarShowStatsVisible(
	TEXT("r.Shadow.Virtual.Stats.Visible"),
	false,
	TEXT("Render VSM Stats to screen"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarVirtualShadowMapNaniteAllowTessellationDirectional(
	TEXT( "r.Shadow.Virtual.Nanite.AllowTessellationDirectional" ),
	1,
	TEXT( "When enabled, allows Nanite Tessellation to be rendered in directional light shadows." ),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		// Needed because Nanite raster pipelines will need to be recached due to forcing displacement fallback bins
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarVirtualShadowMapNaniteAllowTessellationLocal(
	TEXT( "r.Shadow.Virtual.Nanite.AllowTessellationLocal" ),
	1,
	TEXT( "When enabled, allows Nanite Tessellation to be rendered in local light shadows." ),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		// Needed because Nanite raster pipelines will need to be recached due to forcing displacement fallback bins
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static EVSMStatSection::Type GVSMStatSectionsEnabled = EVSMStatSection::All;
EVSMStatSection::Type FVirtualShadowMap::GetEnabledStatSections()
{
	if (!CVarShowStatsVisible.GetValueOnAnyThread())
	{
		return EVSMStatSection::None;
	}

	return GVSMStatSectionsEnabled;
}

static FString SetEnabledStatSections(const FString& Sections)
{
	FString OutputWarning;

	if (Sections == TEXT("basic"))
	{
		GVSMStatSectionsEnabled = EVSMStatSection::Basic;
	}
	else if (Sections == TEXT("all"))
	{
		GVSMStatSectionsEnabled = EVSMStatSection::All;
	}
	else
	{
		// Iterate over flags in comma-separated string
		auto Tokenize = [](const FStringView& String, TCHAR Separator, TFunctionRef<void(const FStringView&)> Func)
		{
			FStringView RemainingString { String };
			for (int TokenEnd; RemainingString.FindChar(Separator, TokenEnd);)
			{
				FStringView Token = RemainingString.SubStr(0, TokenEnd);
				Func(Token);
				RemainingString.RightChopInline(TokenEnd + 1);
			}
			if (RemainingString.Len() > 0)
			{
				Func(RemainingString);
			}
		};

		EVSMStatSection::Type SectionsEnabled = EVSMStatSection::Basic;

		// Match string to section flag and turn it on
		auto ParseFlag = [&](EVSMStatSection::Type Flag, FString FlagLabel, FStringView Token)
		{
			if (Token.Equals(FlagLabel, ESearchCase::IgnoreCase))
			{
				SectionsEnabled = EVSMStatSection::Type(SectionsEnabled | Flag);
				return true;
			}
			return false;
		};

		Tokenize(Sections, ',', [&](const FStringView& Token)
		{
			#define PARSE_FLAG(x) ParseFlag(EVSMStatSection::x, TEXT(#x), Token) ||
			bool bFlagValid = ENUMERATE_VSM_STAT_SECTIONS(PARSE_FLAG) 0;
			#undef PARSE_FLAG

			if(!bFlagValid)
			{
				FString TokenStr(Token);
				OutputWarning += FString::Printf(TEXT("Invalid argument '%s'\n"), *TokenStr);
			}
		});

		GVSMStatSectionsEnabled = SectionsEnabled;
	}
	return OutputWarning;
}

static TAutoConsoleVariable<FString> CVarShowStatsSections(
	TEXT("r.Shadow.Virtual.Stats.Sections"),
	"all",
	TEXT("Select which parts of the stats visualization should be visible."),
	FConsoleVariableDelegate::CreateLambda(
		[](IConsoleVariable* CVar)
		{
			FString Value = CVar->GetString();
			FString Output = SetEnabledStatSections(Value);
			if (Output.Len() > 0)
			{
				UE_LOGF(LogRenderer, Warning, "%ls", *Output);
			}
		}
	)
);

static FAutoConsoleCommand CCmdVSMShowStats = FAutoConsoleCommand(
	TEXT("r.Shadow.Virtual.Stats"),
	TEXT("Show/hide VSM statistics.\n")
	TEXT("Usage: ShowStats <0|1|basic|all>\n")
	TEXT("Usage: ShowStats <section1,section2,...>\n")
	TEXT("Sections:\n")
	#define PRINT_LABEL_LINE(x) TEXT("       " #x "\n")
	ENUMERATE_VSM_STAT_SECTIONS(PRINT_LABEL_LINE),
	#undef PRINT_LABEL_LINE
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateStatic([](const TArray<FString>& Args, FOutputDevice& Output)
	{
		if (Args.Num() < 1)
		{
			if (!CVarShowStatsVisible.GetValueOnAnyThread())
			{
				Output.Log(TEXT("0"));
			}
			else if (GVSMStatSectionsEnabled == EVSMStatSection::All)
			{
				Output.Log(TEXT("1"));
			}
			else
			{
				FString FlagString;
				#define CONCAT_FLAG_LABEL(x) if ((GVSMStatSectionsEnabled & EVSMStatSection::x ) > 0) { FlagString += TEXT( #x ); FlagString += TEXT(","); }
				ENUMERATE_VSM_STAT_SECTIONS(CONCAT_FLAG_LABEL)
				#undef CONCAT_FLAG_LABEL
				Output.Log(FlagString);
			}
			return;
		}

		FString ValueString = Args[0];

		if (ValueString == TEXT("0"))
		{
			CVarShowStatsVisible->Set(false);
		}
		else if (ValueString == TEXT("1"))
		{
			CVarShowStatsVisible->Set(true);
		}
		else
		{
			CVarShowStatsVisible->Set(true);
			FString OutputString = SetEnabledStatSections(ValueString);
			if (OutputString.Len() > 0)
			{
				Output.Log(TEXT("LogRenderer"), ELogVerbosity::Warning, *OutputString);
			}
		}
	})
);

static FAutoConsoleVariableDeprecated CVarVSMShowStats_Deprecated(TEXT("r.Shadow.Virtual.ShowStats"), TEXT("r.Shadow.Virtual.Stats.Visible"), TEXT("5.7"));

static TAutoConsoleVariable<float> CVarPageDilationBorderSizeDirectional(
	TEXT("r.Shadow.Virtual.PageDilationBorderSizeDirectional"),
	0.05f,
	TEXT("If a screen pixel falls within this fraction of a page border for directional lights, the adacent page will also be mapped.")
	TEXT("Higher values can reduce page misses at screen edges or disocclusions, but increase total page counts."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarFirstPersonPixelRequestBias(
	TEXT("r.FirstPerson.Shadow.Virtual.Clipmap.PixelRequestBias"),
	2.0f,
	TEXT("Pixels marked with as coming from first person geometry can request a biased resolution as they are not self-shadowing and also very close to the camera (if scaled).\n")
	TEXT("  Setting to a negative value disables page marking from the FP geometry, which can be used to avoid marking high-res pages for geometry that is scaled to be very small."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarFirstPersonPixelRequestLevelClamp(
	TEXT("r.FirstPerson.Shadow.Virtual.Clipmap.RequestMinLevelClamp"),
	8,
	TEXT("Clamp to avoid high-resolution requests from first-person geometry close to the camera, while still allowing more distant first-person geometry to request full resolution from the environment.\n")
	TEXT("  Note that this interacts with r.Shadow.Virtual.Clipmap.FirstLevel (as this is what is being clamped) and so may need to be configured in scalability settings."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarMaxDOFResolutionBias(
	TEXT("r.Shadow.Virtual.MaxDOFResolutionBias"),
	1.0f,
	TEXT("Determine which pixels are out of focus, and request a lower VSM resolution in those areas.\n")
	TEXT("Since DOF will blur these anyway, the lowered resolution should not be noticable.\n")
	TEXT("Set to 0 to turn off this feature. A higher value more aggressively lowers resolution."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarPageDilationBorderSizeLocal(
	TEXT("r.Shadow.Virtual.PageDilationBorderSizeLocal"),
	0.05f,
	TEXT("If a screen pixel falls within this fraction of a page border for local lights, the adacent page will also be mapped.")
	TEXT("Higher values can reduce page misses at screen edges or disocclusions, but increase total page counts."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarMarkPixelPages(
	TEXT("r.Shadow.Virtual.MarkPixelPages"),
	1,
	TEXT("Marks pages in virtual shadow maps based on depth buffer pixels. Ability to disable is primarily for profiling and debugging."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarMarkPixelPagesMipModeLocal(
	TEXT("r.Shadow.Virtual.MarkPixelPagesMipModeLocal"),
	0,
	TEXT("When enabled, this uses a subset of mips to reduce instance duplication in VSMs. Will result in better performance but a harsher falloff on mip transitions.\n")
	TEXT(" 0 - Disabled: Use all 8 mips\n")
	TEXT(" 1 - Quality Mode: Use 4 higher res mips (16k, 4k, 1k, 256)\n")
	TEXT(" 2 - Performance Mode: Use 4 lower res mips (8k, 2k, 512, 128)\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

TAutoConsoleVariable<int32> CVarMarkCoarsePagesLocal(
	TEXT("r.Shadow.Virtual.MarkCoarsePagesLocal"),
	2,
	TEXT("Marks coarse pages in local light virtual shadow maps so that low resolution data is available everywhere.\n")
	TEXT(" 0 - Disabled\n")
	TEXT(" 1 - Always mark the last mip level.\n")
	TEXT(" 2 - (default) Performance Mode: Suppress dynamic invalidations due to geometry changes (e.g., moving objects or WPO or animation).\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarCoarsePagesIncludeNonNanite(
	TEXT("r.Shadow.Virtual.NonNanite.IncludeInCoarsePages"),
	1,
	TEXT("Include non-Nanite geometry in coarse pages.")
	TEXT("Rendering non-Nanite geometry into large coarse pages can be expensive; disabling this can be a significant performance win."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarNonNaniteCulledInstanceAllocationFactor(
	TEXT("r.Shadow.Virtual.NonNanite.CulledInstanceAllocationFactor"),
	1.0f,
	TEXT("Allocation size scale factor for the buffer used to store instances after culling.\n")
	TEXT("The total size accounts for the worst-case scenario in which all instances are emitted into every clip or mip level.\n")
	TEXT("This is far more than we'd expect in reasonable circumstances, so this scale factor is used to reduce memory pressure.\n")
	TEXT("The actual number cannot be known on the CPU as the culling emits an instance for each clip/mip level that is overlapped.\n")
	TEXT("Setting to 1.0 is fully conservative. Lowering this is likely to produce artifacts unless you're certain the buffer won't overflow."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarNonNaniteMaxCulledInstanceAllocationSize(
	TEXT("r.Shadow.Virtual.NonNanite.MaxCulledInstanceAllocationSize"),
	128 * 1024 * 1024,
	TEXT("Maximum number of instances that may be output from the culling pass into all VSM mip/clip levels. At 12 byte per instance reference this represents a 1.5GB clamp."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarShowClipmapStats(
	TEXT("r.Shadow.Virtual.ShowClipmapStats"),
	-1,
	TEXT("Set to the number of clipmap you want to show stats for (-1 == off)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarCullBackfacingPixels(
	TEXT("r.Shadow.Virtual.CullBackfacingPixels"),
	1,
	TEXT("When enabled does not generate shadow data for pixels that are backfacing to the light."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarNonNaniteVsmUseHzb(
	TEXT("r.Shadow.Virtual.NonNanite.UseHZB"),
	1,
	TEXT("Enable two-pass Nanite culling with HZB from the current frame."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarVirtualShadowOnePassProjectionMaxLights(
	TEXT("r.Shadow.Virtual.OnePassProjection.MaxLightsPerPixel"),
	16,
	TEXT("Maximum lights per pixel that get full filtering when using one pass projection and clustered shading.")
	TEXT("Generally set to 8 (32bpp), 16 (64bpp) or 32 (128bpp). Lower values require less transient VRAM during the lighting pass."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarDoNonNaniteBatching(
	TEXT("r.Shadow.Virtual.NonNanite.Batch"),
	1,
	TEXT("."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarNonNaniteUseRadiusThreshold(
	TEXT("r.Shadow.Virtual.NonNanite.UseRadiusThreshold"),
	1,
	TEXT("If enabled (default) the r.Shadow.RadiusThreshold cvar is also used for uncached virtual shadow maps to cull small non-nanite instances."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarCoarsePagePixelThresholdDynamic(
	TEXT("r.Shadow.Virtual.CoarsePagePixelThresholdDynamic"),
	16.0f,
	TEXT("If a dynamic (non-nanite) instance has a smaller estimated pixel footprint than this value, it should not be drawn into a coarse page. Higher values cull away more instances."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarCoarsePagePixelThresholdStatic(
	TEXT("r.Shadow.Virtual.CoarsePagePixelThresholdStatic"),
	1.0f,
	TEXT("If a static (non-nanite) instance has a smaller estimated pixel footprint than this value, it should not be drawn into a coarse page. Higher values cull away more instances.\n")
	TEXT("This value is typically lower than the non-static one because the static pages have better caching."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarCoarsePagePixelThresholdDynamicNanite(
	TEXT("r.Shadow.Virtual.CoarsePagePixelThresholdDynamicNanite"),
	4.0f,
	TEXT("If a dynamic Nanite instance has a smaller estimated pixel footprint than this value, it should not be drawn into a coarse page. Higher values cull away more instances.\n")
	TEXT("This value is typically lower than the non-Nanite one because Nanite has lower overhead for drawing small objects."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarCacheAllocateViaLRU(
	TEXT("r.Shadow.Virtual.Cache.AllocateViaLRU"),
	1,
	TEXT("Prioritizes keeping more recently requested cached physical pages when allocating for new requests."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarShadowsVirtualForceFullPageClears(
	TEXT("r.Shadow.Virtual.ForceFullPageClears"),
	0,
	TEXT("Always clear physical pages, even when nothing has been written to them."),
	ECVF_RenderThreadSafe
);

#if VSM_ENABLE_VISUALIZATION
bool GDumpVSMLightNames = false;
void DumpVSMLightNames()
{
	ENQUEUE_RENDER_COMMAND(DumpVSMLightNames)(
		[](FRHICommandList& RHICmdList)
		{
			GDumpVSMLightNames = true;
		});
}

FAutoConsoleCommand CmdDumpVSMLightNames(
	TEXT("r.Shadow.Virtual.Visualize.DumpLightNames"),
	TEXT("Dump light names with virtual shadow maps (for developer use in non-shipping builds)"),
	FConsoleCommandDelegate::CreateStatic(DumpVSMLightNames)
);

FLightComponentId GVirtualShadowMapLastSelectedVisualizeLightId;
FLightComponentId GVirtualShadowMapVisualizeLightId;
bool GVirtualShadowMapVisualizeByLightId = false;

FString GVirtualShadowMapVisualizeLightName;
FAutoConsoleVariableRef CVarVisualizeLightName(
	TEXT("r.Shadow.Virtual.Visualize.LightName"),
	GVirtualShadowMapVisualizeLightName,
	TEXT("Sets the name of a specific light to visualize (for developer use in non-shipping builds)"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		GVirtualShadowMapVisualizeByLightId = false;
	}),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVisualizeLayout(
	TEXT("r.Shadow.Virtual.Visualize.Layout"),
	0,
	TEXT("Overlay layout when virtual shadow map visualization is enabled:\n")
	TEXT("  0: Full screen\n")
	TEXT("  1: Thumbnail\n")
	TEXT("  2: Split screen"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVisualizeDynamicMeshBounds(
	TEXT("r.Shadow.Virtual.VisualizeDynamicMeshBounds"),
	0,
	TEXT("\n"),
	ECVF_RenderThreadSafe
);
#endif // VSM_ENABLE_VISUALIZATION

#if !UE_BUILD_SHIPPING
TAutoConsoleVariable<int32> CVarDebugSkipMergePhysical(
	TEXT("r.Shadow.Virtual.DebugSkipMergePhysical"),
	0,
	TEXT("Skip the merging of the static VSM cache into the dynamic one. This will create obvious visual artifacts when disabled."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarDebugSkipDynamicPageInvalidation(
	TEXT("r.Shadow.Virtual.Cache.DebugSkipDynamicPageInvalidation"),
	0,
	TEXT("Skip invalidation of cached pages when geometry moves for debugging purposes. This will create obvious visual artifacts when disabled."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarNumPageAreaDiagSlots(
	TEXT("r.Shadow.Virtual.NonNanite.NumPageAreaDiagSlots"),
	0,
	TEXT("Number of slots in diagnostics to report non-nanite instances with the largest page area coverage, < 0 uses the max number allowed, 0 disables."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarLargeInstancePageAreaThreshold(
	TEXT("r.Shadow.Virtual.NonNanite.LargeInstancePageAreaThreshold"),
	-1,
	TEXT("How large area is considered a 'large' footprint, summed over all overlapped levels, if set to -1 uses the physical page pool size / 8.\n")
	TEXT("Used as a threshold when storing page area coverage stats for diagnostics."),
	ECVF_RenderThreadSafe
);
#endif // !UE_BUILD_SHIPPING

static TAutoConsoleVariable<int32> CVarShadowsVirtualUseHZB(
	TEXT("r.Shadow.Virtual.UseHZB"),
	1,
	TEXT("Enables two pass occlusion culling for (Nanite) Virtual Shadow Maps\n")
	TEXT("Non-Nanite has a separate flag: r.Shadow.Virtual.NonNanite.UseHZB."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowsVirtualForceFullHZBUpdate(
	TEXT("r.Shadow.Virtual.ForceFullHZBUpdate"),
	0,
	TEXT("Forces full HZB update every frame rather than just dirty pages.\n"),
	ECVF_RenderThreadSafe);


static TAutoConsoleVariable<int32> CVarVirtualShadowSinglePassBatched(
	TEXT("r.Shadow.Virtual.NonNanite.SinglePassBatched"),
	1,
	TEXT("."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVirtualShadowMapPageMarkingPixelStrideX(
	TEXT("r.Shadow.Virtual.PageMarkingPixelStrideX"),
	2,
	TEXT("During page marking, instead of testing every screen pixel, test every Nth pixel.\n")
	TEXT("Page marking from screen pixels is used to determine which VSM pages are seen from the camera and need to be rendered.\n")
	TEXT("Increasing this value reduces page-marking costs, but could introduce artifacts due to missing pages.\n")
	TEXT("With sufficiently low values, it is likely a neighbouring pixel will mark the required page anyway."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVirtualShadowMapPageMarkingPixelStrideY(
	TEXT("r.Shadow.Virtual.PageMarkingPixelStrideY"),
	2,
	TEXT("Same as PageMarkingPixelStrideX, but on the vertical axis of the screen."),
	ECVF_RenderThreadSafe);


static TAutoConsoleVariable<float> CVarScreenRayLength(
	TEXT("r.Shadow.Virtual.ScreenRayLength"),
	0.015f,
	TEXT("Length of the screen space shadow trace away from receiver surface (smart shadow bias) before the VSM / SMRT lookup."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarNormalBias(
	TEXT("r.Shadow.Virtual.NormalBias"),
	0.5f,
	TEXT("Receiver offset along surface normal for shadow lookup. Scaled by distance to camera.")
	TEXT("Higher values avoid artifacts on surfaces nearly parallel to the light, but also visibility offset shadows and increase the chance of hitting unmapped pages."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSMRTAdaptiveRayCount(
	TEXT("r.Shadow.Virtual.SMRT.AdaptiveRayCount"),
	1,
	TEXT("Shoot fewer rays in fully shadowed and unshadowed regions. Currently only supported with OnePassProjection. "),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSMRTRayCountLocal(
	TEXT("r.Shadow.Virtual.SMRT.RayCountLocal"),
	7,
	TEXT("Ray count for shadow map tracing of local lights. 0 = disabled."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSMRTSamplesPerRayLocal(
	TEXT("r.Shadow.Virtual.SMRT.SamplesPerRayLocal"),
	8,
	TEXT("Shadow map samples per ray for local lights"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarSMRTMaxRayAngleFromLight(
	TEXT("r.Shadow.Virtual.SMRT.MaxRayAngleFromLight"),
	0.03f,
	TEXT("Max angle (in radians) a ray is allowed to span from the light's perspective for local lights.")
	TEXT("Smaller angles limit the screen space size of shadow penumbra. ")
	TEXT("Larger angles lead to more noise. "),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarSMRTExtrapolateMaxSlopeLocal(
	TEXT("r.Shadow.Virtual.SMRT.ExtrapolateMaxSlopeLocal"),
	0.05f,
	TEXT("Maximum depth slope when extrapolating behind occluders for local lights.\n")
	TEXT("Higher values allow softer penumbra edges but can introduce light leaks behind second occluders.\n")
	TEXT("Setting to 0 will disable slope extrapolation slightly improving projection performance, at the cost of reduced penumbra quality."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarSMRTTexelDitherScaleLocal(
	TEXT("r.Shadow.Virtual.SMRT.TexelDitherScaleLocal"),
	2.0f,
	TEXT("Applies a dither to the shadow map ray casts for local lights to help hide aliasing due to insufficient shadow resolution.\n")
	TEXT("Setting this too high can cause shadows light leaks near occluders."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarSMRTMaxSlopeBiasLocal(
	TEXT("r.Shadow.Virtual.SMRT.MaxSlopeBiasLocal"),
	50.0f,
	TEXT("Maximum depth slope. Low values produce artifacts if shadow resolution is insufficient. High values can worsen light leaks near occluders and sparkly pixels in shadowed areas."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSMRTRayCountDirectional(
	TEXT("r.Shadow.Virtual.SMRT.RayCountDirectional"),
	7,
	TEXT("Ray count for shadow map tracing of directional lights. 0 = disabled."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSMRTSamplesPerRayDirectional(
	TEXT("r.Shadow.Virtual.SMRT.SamplesPerRayDirectional"),
	8,
	TEXT("Shadow map samples per ray for directional lights"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarSMRTExtrapolateMaxSlopeDirectional(
	TEXT("r.Shadow.Virtual.SMRT.ExtrapolateMaxSlopeDirectional"),
	5.0f,
	TEXT("Maximum depth slope when extrapolating behind occluders for directional lights.\n")
	TEXT("Higher values allow softer penumbra edges but can introduce light leaks behind second occluders.\n")
	TEXT("Setting to 0 will disable slope extrapolation slightly improving projection performance, at the cost of reduced penumbra quality."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarSMRTTexelDitherScaleDirectional(
	TEXT("r.Shadow.Virtual.SMRT.TexelDitherScaleDirectional"),
	2.0f,
	TEXT("Applies a dither to the shadow map ray casts for directional lights to help hide aliasing due to insufficient shadow resolution.\n")
	TEXT("Setting this too high can cause shadows light leaks near occluders."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarSMRTRayLengthScaleDirectional(
	TEXT("r.Shadow.Virtual.SMRT.RayLengthScaleDirectional"),
	1.5f,
	TEXT("Length of ray to shoot for directional lights, scaled by distance to camera.")
	TEXT("Shorter rays limit the screen space size of shadow penumbra. ")
	TEXT("Longer rays require more samples to avoid shadows disconnecting from contact points. "),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSMRTRayCountHair(
	TEXT("r.Shadow.Virtual.SMRT.SamplesPerRayHair"),
	1,
	TEXT("Shadow map samples per ray for hair"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarVSMUseReceiverMaskLocal(
	TEXT("r.Shadow.Virtual.UseReceiverMaskLocal"),
	false,
	TEXT("Use receiver page masks with local lights. This enables much more effective culling especially at lower resolutions."),	
	ECVF_RenderThreadSafe
);

extern TAutoConsoleVariable<float> CVarVSMDynamicResolutionMaxLodBias;
extern TAutoConsoleVariable<float> CVarVSMDynamicResolutionMaxLodBiasDirectional;
extern TAutoConsoleVariable<float> CVarVSMDynamicResolutionMaxLodBiasLocal;

static TAutoConsoleVariable<float> CVarTimeBudgetMs(
	TEXT("r.Shadow.Virtual.DynamicRes.TimeBudgetMs"),
	DynamicRenderScaling::FHeuristicSettings::kBudgetMsDisabled,
	TEXT("Frame's time budget for VSM ShadowDepths in milliseconds. Non-Nanite not included.\n")
	TEXT("Disabled when set to 0. Experimental."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarThrottleLoadBudget(
	TEXT("r.Shadow.Virtual.DynamicRes.LoadBudget"),
	0.0f,
	TEXT("Scale LOD bias based on an estimate of the ShadowDepths load. Lower value means more aggressive resolution downscaling.\n")
	TEXT("Disabled when TimeBudgetMs is active, or if set to 0. Experimental."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarThrottleEMAHistoryWeight(
	TEXT("r.Shadow.Virtual.DynamicRes.PerVSMBiasHistoryWeight"),
	0.9f,
	TEXT("Smoothen out the per-VSM LOD bias over time. Value between 0 and 1."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarVSMPrefilteredDistantHistoryBlendFactor(
	TEXT( "r.Shadow.Virtual.PrefilteredDistant.HistoryBlendFactor" ),
	0.87f,
	TEXT( "How large a proportion of history to blend into the current frame. Larger values means slower shadow update and more temporal aliasing. 0.0f turns the reprojection off." ),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarVirtualShadowMapPrefilteredDistant(
	TEXT( "r.Shadow.Virtual.PrefilteredDistant" ),
	false,
	TEXT( "Enable/Disable the experimental prefiltered distant shadow feature at runtime. r.Shadow.Virtual.PrefilteredDistant.ProjectEnable must be enabled for this to have any effect." ),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

bool GVirtualShadowMapPrefilteredDistantProjectEnable = false;
static FAutoConsoleVariableRef CVarVirtualShadowMapPrefilteredDistantProjectEnable(
	TEXT( "r.Shadow.Virtual.PrefilteredDistant.ProjectEnable" ),
	GVirtualShadowMapPrefilteredDistantProjectEnable,
	TEXT( "Enable/Disable the experimental prefiltered distant shadow feature for the project (enabling compiles additional shader permutations, e.g., for shadow projection)" ),
	ECVF_ReadOnly
);

static DynamicRenderScaling::FHeuristicSettings GetDynamicVSMResolutionSettings()
{
	DynamicRenderScaling::FHeuristicSettings BucketSetting;
	BucketSetting.Model = DynamicRenderScaling::EHeuristicModel::Linear;
	BucketSetting.bModelScalesWithPrimaryScreenPercentage = false;
	// Resolution fraction is used as throttle strength, not directly as resolution
	BucketSetting.MinResolutionFraction = DynamicRenderScaling::PercentageToFraction(0.0f);
	BucketSetting.MaxResolutionFraction = DynamicRenderScaling::PercentageToFraction(100.0f);
	BucketSetting.BudgetMs = CVarTimeBudgetMs.GetValueOnAnyThread();
	BucketSetting.ChangeThreshold = DynamicRenderScaling::PercentageToFraction(1.0f);
	BucketSetting.TargetedHeadRoom = DynamicRenderScaling::PercentageToFraction(5.0f); // 5% headroom
	BucketSetting.UpperBoundQuantization = DynamicRenderScaling::FHeuristicSettings::kDefaultUpperBoundQuantization;
	return BucketSetting;
}
DynamicRenderScaling::FBudget GDynamicResolutionVSMNaniteBudget(TEXT("DynamicResolutionVSMNanite"), &GetDynamicVSMResolutionSettings);

bool IsVirtualShadowMapLocalReceiverMaskEnabled()
{
	return CVarVSMUseReceiverMaskLocal.GetValueOnRenderThread() != 0;
}

namespace Nanite
{
	extern bool IsStatFilterActive(const FString& FilterName);
}

bool IsVSMTranslucentHighQualityEnabled();
bool IsFrontLayerHistoryValid(const FViewInfo& View);
extern bool LightGridUses16BitBuffers(EShaderPlatform Platform);

bool DoesVSMWantFroxels(EShaderPlatform ShaderPlatform)
{
	return UseVirtualShadowMaps(ShaderPlatform)
		&& CVarMarkPagesUseFroxels.GetValueOnRenderThread() != 0
		// fall back to per-pixel marking if the front layer translucency path is enabled as it doesn't generate froxels.
		&& !IsVSMTranslucentHighQualityEnabled();
}

FMatrix CalcTranslatedWorldToShadowUVMatrix(
	const FMatrix& TranslatedWorldToShadowView,
	const FMatrix& ViewToClip)
{
	FMatrix TranslatedWorldToShadowClip = TranslatedWorldToShadowView * ViewToClip;
	FMatrix ScaleAndBiasToSmUV = FScaleMatrix(FVector(0.5f, -0.5f, 1.0f)) * FTranslationMatrix(FVector(0.5f, 0.5f, 0.0f));
	FMatrix TranslatedWorldToShadowUv = TranslatedWorldToShadowClip * ScaleAndBiasToSmUV;
	return TranslatedWorldToShadowUv;
}

FMatrix CalcTranslatedWorldToShadowUVNormalMatrix(
	const FMatrix& TranslatedWorldToShadowView,
	const FMatrix& ViewToClip)
{
	return CalcTranslatedWorldToShadowUVMatrix(TranslatedWorldToShadowView, ViewToClip).GetTransposed().Inverse();
}

static float GetNormalBiasForShader()
{
	return CVarNormalBias.GetValueOnRenderThread() / 1000.0f;
}

uint32 FVirtualShadowMapProjectionShaderData::PackCullingViewId(int32 SceneRendererPrimaryViewId, const FPersistentViewId& PersistentViewId)
{
	// TODO: define constants
	check(SceneRendererPrimaryViewId >= -1 && SceneRendererPrimaryViewId < ((1 << 16u) - 1));
	check(PersistentViewId.Index >= -1);
	// Pack such that invalid == 0
	return (uint32(SceneRendererPrimaryViewId + 1) << 16u) | uint32(PersistentViewId.Index + 1);
}

FVirtualShadowMapArray::FVirtualShadowMapArray(FScene& InScene) 
	: Scene(InScene)
{
}

void FVirtualShadowMapArray::UpdateNextData(int32 PrevVirtualShadowMapId, const FNextVirtualShadowMapData& Data)
{
	if (PrevVirtualShadowMapId >= NextData.Num())
	{
		// Will set flags to 0 which means "invalid/no caching"
		NextData.SetNumZeroed(PrevVirtualShadowMapId + 1);
	}

	NextData[PrevVirtualShadowMapId] = Data;
}

static FVirtualShadowMapPerViewParameters MakeEmptyVirtualShadowMapPerViewParameters(FRDGBuilder& GraphBuilder)
{
	FVirtualShadowMapPerViewParameters PerViewData;
	PerViewData.NumCulledLightsGrid = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32), 0u));
	PerViewData.LightGridData = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32), 0u));
	PerViewData.MaxLightGridEntryIndex = 0u;
	PerViewData.DirectionalLightIds = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(int32), 0u));
	PerViewData.DirectionalLightIdCount = 0u;

	return PerViewData;
}

void FVirtualShadowMapArray::Initialize(
	FRDGBuilder& GraphBuilder,
	FVirtualShadowMapArrayCacheManager* InCacheManager,
	bool bInEnabled,
	const FEngineShowFlags& EngineShowFlags)
{
	bInitialized = true;
	bEnabled = bInEnabled;
	CacheManager = InCacheManager;

	// Always retain the VSM cache data if the feature is enabled.
	const bool bShouldRetainCacheInfo = UseVirtualShadowMaps(Scene.GetShaderPlatform(), Scene.GetFeatureLevel());

	bCullBackfacingPixels = CVarCullBackfacingPixels.GetValueOnRenderThread() != 0;
	bUseHzbOcclusion = CVarShadowsVirtualUseHZB.GetValueOnRenderThread() != 0;

	bEnablePrefilteredDistant = GVirtualShadowMapPrefilteredDistantProjectEnable && CVarVirtualShadowMapPrefilteredDistant.GetValueOnRenderThread();

	UniformParameters.NumFullShadowMaps = 0;
	UniformParameters.NumSinglePageShadowMaps = 0;
	UniformParameters.NumShadowMapSlots = 0;
	UniformParameters.MaxPhysicalPages = 0;
	UniformParameters.StaticCachedArrayIndex = 0;
	UniformParameters.StaticHZBArrayIndex = 0;
	// NOTE: Most uniform values don't matter when VSM is disabled

	UniformParameters.bExcludeNonNaniteFromCoarsePages = !CVarCoarsePagesIncludeNonNanite.GetValueOnRenderThread();
	UniformParameters.CoarsePagePixelThresholdDynamic = CVarCoarsePagePixelThresholdDynamic.GetValueOnRenderThread();
	UniformParameters.CoarsePagePixelThresholdStatic = CVarCoarsePagePixelThresholdStatic.GetValueOnRenderThread();
	UniformParameters.CoarsePagePixelThresholdDynamicNanite = CVarCoarsePagePixelThresholdDynamicNanite.GetValueOnRenderThread();
	UniformParameters.MipModeLocal = CVarMarkPixelPagesMipModeLocal.GetValueOnRenderThread();

	UniformParameters.SceneFrameNumber = Scene.GetFrameNumberRenderThread();
	
	// Global SMRT settings so they can be shared between different passes that call into them
	UniformParameters.ScreenRayLength = CVarScreenRayLength.GetValueOnRenderThread();
	UniformParameters.NormalBias = GetNormalBiasForShader();

	UniformParameters.SMRTAdaptiveRayCount = CVarSMRTAdaptiveRayCount.GetValueOnRenderThread();

	UniformParameters.SMRTRayCountLocal = CVarSMRTRayCountLocal.GetValueOnRenderThread();
	UniformParameters.SMRTSamplesPerRayLocal = CVarSMRTSamplesPerRayLocal.GetValueOnRenderThread();
	UniformParameters.SMRTExtrapolateMaxSlopeLocal = CVarSMRTExtrapolateMaxSlopeLocal.GetValueOnRenderThread();
	UniformParameters.SMRTTexelDitherScaleLocal = CVarSMRTTexelDitherScaleLocal.GetValueOnRenderThread();
	UniformParameters.SMRTMaxSlopeBiasLocal = CVarSMRTMaxSlopeBiasLocal.GetValueOnRenderThread();
	UniformParameters.SMRTCotMaxRayAngleFromLight = 1.0f / FMath::Tan(CVarSMRTMaxRayAngleFromLight.GetValueOnRenderThread());
	
	UniformParameters.SMRTRayCountDirectional = CVarSMRTRayCountDirectional.GetValueOnRenderThread();
	UniformParameters.SMRTSamplesPerRayDirectional = CVarSMRTSamplesPerRayDirectional.GetValueOnRenderThread();
	UniformParameters.SMRTExtrapolateMaxSlopeDirectional = CVarSMRTExtrapolateMaxSlopeDirectional.GetValueOnRenderThread();
	UniformParameters.SMRTTexelDitherScaleDirectional = CVarSMRTTexelDitherScaleDirectional.GetValueOnRenderThread();
	UniformParameters.SMRTRayLengthScale = CVarSMRTRayLengthScaleDirectional.GetValueOnRenderThread();

	UniformParameters.SMRTHairRayCount = CVarSMRTRayCountHair.GetValueOnRenderThread();

	UniformParameters.PageTableSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	// Reference dummy data in the UB initially
	const uint32 DummyPageTableElement = 0xFFFFFFFF;
	UniformParameters.PageTable = GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R32_UINT, DummyPageTableElement);
	UniformParameters.ProjectionData = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, sizeof(FVirtualShadowMapProjectionShaderData)));
	UniformParameters.PageFlags = GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R32_UINT, 0u);
	UniformParameters.PageReceiverMasks = GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R32_UINT, 0xFFFFFFFFu);

	UniformParameters.UncachedPageRectBounds = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FIntVector4)));
	UniformParameters.AllocatedPageRectBounds = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FIntVector4)));
	UniformParameters.PerViewData = MakeEmptyVirtualShadowMapPerViewParameters(GraphBuilder);
	UniformParameters.CachePrimitiveAsDynamic = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32)));
	UniformParameters.BilinearClampedSampler = TStaticSamplerState< SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI();

	// These get updated/allocated lazily after we know that we have VSMs present in the scene
	UniformParameters.PhysicalPagePool = GSystemTextures.GetZeroUIntArrayAtomicCompatDummy(GraphBuilder);
	HZBPhysicalArrayRDG = nullptr;

	if (bShouldRetainCacheInfo)
	{
		// Always reserve slots for the single-page SMs.
		NumShadowMapSlots = VSM_MAX_SINGLE_PAGE_SHADOW_MAPS;

		// Fixed physical page pool width, we adjust the height to accomodate the requested maximum
		// NOTE: Row size in pages has to be POT since we use mask & shift in place of integer ops
		// NOTE: This assumes GetMax2DTextureDimension() is a power of two on supported platforms
		// Note: If prefiltered distant is enabled, then we must leave space for borders for the filtered page pool (currently 1:1)
		const uint32 UsablePagePoolWidth = IsPrefilteredDistantEnabled() ? (CacheManager->GetPhysicalMaxWidth() / 2u) : CacheManager->GetPhysicalMaxWidth();
		const uint32 PhysicalPagesX = FMath::DivideAndRoundDown(UsablePagePoolWidth, FVirtualShadowMap::PageSize);
		check(FMath::IsPowerOfTwo(PhysicalPagesX));
		const int32 MaxPhysicalPages = CVarMaxPhysicalPages.GetValueOnRenderThread();
		uint32 PhysicalPagesY = FMath::DivideAndRoundUp((uint32)FMath::Max(1, MaxPhysicalPages), PhysicalPagesX);	

		UniformParameters.MaxPhysicalPages = PhysicalPagesX * PhysicalPagesY;
		
		if (CacheManager->IsCacheEnabled())
		{
			// Only set up the dynamic separate HZB build allocation if 
			// 1. caching is enabled & we cache static separate (otherwise they are combined anyway)
			// 2. the cvar is enabled
			if (CVarBuildDynamicHZB.GetValueOnRenderThread() != 0)
			{
				UniformParameters.StaticHZBArrayIndex = 1;
			}

			// Enable separate static caching in the second texture array element
			UniformParameters.StaticCachedArrayIndex = 1;
		}

		uint32 PhysicalX = PhysicalPagesX * FVirtualShadowMap::PageSize;
		uint32 PhysicalY = PhysicalPagesY * FVirtualShadowMap::PageSize;

		// TODO: Some sort of better fallback with warning?
		// All supported platforms support at least 16384 texture dimensions which translates to 16384 max pages with default 128x128 page size
		check(PhysicalX <= GetMax2DTextureDimension());
		check(PhysicalY <= GetMax2DTextureDimension());

		UniformParameters.PhysicalPageRowMask = (PhysicalPagesX - 1);
		UniformParameters.PhysicalPageRowShift = FMath::FloorLog2( PhysicalPagesX );
		UniformParameters.RecPhysicalPoolSize = FVector4f( 1.0f / PhysicalX, 1.0f / PhysicalY, 1.0f, 1.0f );
		UniformParameters.PhysicalPoolSize = FIntPoint( PhysicalX, PhysicalY );
		UniformParameters.PhysicalPoolSizePages = FIntPoint( PhysicalPagesX, PhysicalPagesY );

		UniformParameters.GlobalResolutionLodBias = CacheManager->GetGlobalResolutionLodBias();

		// Note: at this point we don't know these yet, so we use previous frame info, which is the only data we could access using these anyway (the new data is not set up yet).
		UniformParameters.PageTableRowMask = CacheManager->PrevUniformParameters.PageTableRowMask;
		UniformParameters.PageTableRowShift = CacheManager->PrevUniformParameters.PageTableRowShift;
		UniformParameters.PageTableTextureSizeInvSize = CacheManager->PrevUniformParameters.PageTableTextureSizeInvSize;
		UniformParameters.PageReceiverMaskTextureSizeInvSize = CacheManager->PrevUniformParameters.PageReceiverMaskTextureSizeInvSize;

		// TODO: Parameterize this in a useful way; potentially modify it automatically
		// when there are fewer lights in the scene and/or clustered shading settings differ.
		UniformParameters.PackedShadowMaskMaxLightCount = FMath::Min(CVarVirtualShadowOnePassProjectionMaxLights.GetValueOnRenderThread(), 32);

		// Set up nanite visualization if enabled. We use an extra array slice in the physical page pool for debug output
		// so need to set this up in advance.
		if (EngineShowFlags.VisualizeVirtualShadowMap)
		{
			bEnableVisualization = true;

			FVirtualShadowMapVisualizationData& VisualizationData = GetVirtualShadowMapVisualizationData();
			if (VisualizationData.GetActiveModeID() == VIRTUAL_SHADOW_MAP_VISUALIZE_NANITE_OVERDRAW)
			{
				bEnableNaniteVisualization = true;
			}
		}

		UniformParameters.CachePrimitiveAsDynamic = GraphBuilder.CreateSRV(CacheManager->UploadCachePrimitiveAsDynamic(GraphBuilder));
		UniformParameters.RecFilterablePhysicalPoolSize = FVector4f( 0.0f, 0.0f, 1.0f, 1.0f );
	}
	else
	{
		if (CacheManager)
		{
			CacheManager->FreePhysicalPool(GraphBuilder);
			CacheManager->FreeHZBPhysicalPool(GraphBuilder);
		}
	}

	UpdateCachedUniformBuffers(GraphBuilder);
}

int32 FVirtualShadowMapArray::AllocateShadowMapSlots(bool bSinglePageShadowMap, int32 Count)
{
	check(IsEnabled());
	int32 VirtualShadowMapId = INDEX_NONE;
	if (bSinglePageShadowMap)
	{
		if (ensure((NumSinglePageShadowMaps + Count) <= VSM_MAX_SINGLE_PAGE_SHADOW_MAPS))
		{
			VirtualShadowMapId = NumSinglePageShadowMaps;
			NumSinglePageShadowMaps += Count;
		}
	}
	else
	{
		// Full shadow maps come after single page shadow maps
		VirtualShadowMapId = NumShadowMapSlots;
		NumShadowMapSlots += Count;
	}
	return VirtualShadowMapId;
}

int32 FVirtualShadowMapArray::AllocateDirectional(int32 Count)
{	
	check(IsEnabled());
	check(NumUnreferencedShadowMaps == 0);
	check(NumLocalShadowMaps == 0);
	NumDirectionalShadowMaps += Count;
	return AllocateShadowMapSlots(false, Count);
}

int32 FVirtualShadowMapArray::AllocateLocal(bool bSinglePageShadowMap, int32 Count)
{
	check(IsEnabled());
	check(NumUnreferencedShadowMaps == 0);
	NumLocalShadowMaps += Count;
	return AllocateShadowMapSlots(bSinglePageShadowMap, Count);
}

int32 FVirtualShadowMapArray::AllocateUnreferenced(bool bSinglePageShadowMap, int32 Count)
{
	check(IsEnabled());
	NumUnreferencedShadowMaps += Count;
	return AllocateShadowMapSlots(bSinglePageShadowMap, Count);
}

FVirtualShadowMapArray::~FVirtualShadowMapArray()
{
}

EPixelFormat FVirtualShadowMapArray::GetPackedShadowMaskFormat() const
{
	// TODO: Check if we're after any point that determines the format later too (light setup)
	check(bInitialized);
	// NOTE: Currently 4bpp/light
	if (UniformParameters.PackedShadowMaskMaxLightCount <= 8)
	{
		return PF_R32_UINT;
	}
	else if (UniformParameters.PackedShadowMaskMaxLightCount <= 16)
	{
		return PF_R32G32_UINT;
	}
	else
	{
		check(UniformParameters.PackedShadowMaskMaxLightCount <= 32);
		return PF_R32G32B32A32_UINT;
	}
}

FIntPoint FVirtualShadowMapArray::GetPhysicalPoolSize() const
{
	check(bInitialized);
	return FIntPoint(UniformParameters.PhysicalPoolSize.X, UniformParameters.PhysicalPoolSize.Y);
}

FIntPoint FVirtualShadowMapArray::GetHZBPhysicalPoolSize() const
{
	check(bInitialized);
	FIntPoint PhysicalPoolSize = GetPhysicalPoolSize();
	FIntPoint HZBSize(FMath::Max(FPlatformMath::RoundUpToPowerOfTwo(PhysicalPoolSize.X) >> 1, 1u),
	                  FMath::Max(FPlatformMath::RoundUpToPowerOfTwo(PhysicalPoolSize.Y) >> 1, 1u));
	return HZBSize;
}

uint32 FVirtualShadowMapArray::GetTotalAllocatedPhysicalPages() const
{
	check(bInitialized);
	// 2 array elements (static/dynamic)
	return 2U * UniformParameters.MaxPhysicalPages;
}

TRDGUniformBufferRef<FVirtualShadowMapUniformParameters> FVirtualShadowMapArray::GetUncachedUniformBuffer(FRDGBuilder& GraphBuilder) const
{
	// NOTE: Need to allocate new parameter space since the UB changes over the frame as dummy references are replaced
	FVirtualShadowMapUniformParameters* VersionedParameters = GraphBuilder.AllocParameters<FVirtualShadowMapUniformParameters>();
	*VersionedParameters = UniformParameters;
	return GraphBuilder.CreateUniformBuffer(VersionedParameters);
}

void FVirtualShadowMapArray::UpdateCachedUniformBuffers(FRDGBuilder& GraphBuilder)
{
	CachedUniformBuffers.Reset();

	// If we haven't yet initialized per-view parameters and are still using dummy data
	if (PerViewParameters.Num() == 0)
	{
		CachedUniformBuffers.Add(GetUncachedUniformBuffer(GraphBuilder));
	}
	// If per-view parameters are initialized
	else
	{
		CachedUniformBuffers.SetNum(PerViewParameters.Num());

		for (int ViewIndex = 0; ViewIndex < PerViewParameters.Num(); ++ViewIndex)
		{
			FVirtualShadowMapUniformParameters* VersionedParameters = GraphBuilder.AllocParameters<FVirtualShadowMapUniformParameters>(&UniformParameters);
			VersionedParameters->PerViewData = PerViewParameters[ViewIndex];
			CachedUniformBuffers[ViewIndex] = GraphBuilder.CreateUniformBuffer(VersionedParameters);
		}
	}
}

void FVirtualShadowMapArray::SetShaderDefines(FShaderCompilerEnvironment& OutEnvironment)
{
	static_assert(FVirtualShadowMap::Log2Level0DimPagesXY * 2U + NANITE_MAX_VIEWS_PER_CULL_RASTERIZE_PASS_BITS <= 32U, "Page indirection plus view index must fit into 32-bits for page-routing storage!");
	OutEnvironment.SetDefine(TEXT("MAX_PAGE_AREA_DIAGNOSTIC_SLOTS"), MaxPageAreaDiagnosticSlots);
	OutEnvironment.SetDefine(TEXT("MAX_NPF_DIAGNOSTIC_SLOTS_SINGLEPAGE"), MaxNPFDiagnosticSlotsSinglePage);
	OutEnvironment.SetDefine(TEXT("MAX_NPF_DIAGNOSTIC_SLOTS_MULTIPAGE"), MaxNPFDiagnosticSlotsMultiPage);
	OutEnvironment.SetDefine(TEXT("INDEX_NONE"), INDEX_NONE);
}

FVirtualShadowMapSamplingParameters FVirtualShadowMapArray::GetSamplingParameters(FRDGBuilder& GraphBuilder, int32 ViewIndex) const
{
	// Sanity check: either VSMs are disabled and it's expected to be relying on dummy data, or we should have valid data
	// If this fires, it is likely because the caller is trying to sample VSMs before they have been rendered by the ShadowDepths pass
	// This should not crash, but it is not an intended production path as it will not return valid shadow data.
	// TODO: Disabled warning until SkyAtmosphereLUT is moved after ShadowDepths
	//ensureMsgf(!IsEnabled() || IsAllocated(),
	//	TEXT("Attempt to use Virtual Shadow Maps before they have been rendered by ShadowDepths."));

	FVirtualShadowMapSamplingParameters Parameters;
	Parameters.VirtualShadowMap = GetUniformBuffer(ViewIndex);
	return Parameters;
}

FVirtualShadowMapMarkingParameters FVirtualShadowMapArray::GetMarkingParameters(FRDGBuilder& GraphBuilder, int32 ViewIndex) const
{
	FVirtualShadowMapMarkingParameters Parameters;
	Parameters.VirtualShadowMap = GetUniformBuffer(ViewIndex);
	Parameters.OutPageRequestFlags = GraphBuilder.CreateUAV(PageRequestFlagsRDG);
	Parameters.OutPageReceiverMasks = GraphBuilder.CreateUAV(PageReceiverMasksRDG);
	return Parameters;
}

class FPruneLightGridCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FPruneLightGridCS);
	SHADER_USE_PARAMETER_STRUCT(FPruneLightGridCS, FVirtualShadowMapPageManagementShader)
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER(uint32, MinLocalLightIndex)
		SHADER_PARAMETER(uint32, MaxLocalLightIndex)
		SHADER_PARAMETER(int32, bIncludeMegaLights)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutPrunedLightGridData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutPrunedNumCulledLightsGrid)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FPruneLightGridCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPageMarking.usf", "PruneLightGridCS", SF_Compute);

BEGIN_SHADER_PARAMETER_STRUCT(FThrottlingParameters, )
	SHADER_PARAMETER(float, DynResThrottleStrength)
	SHADER_PARAMETER(float, ThrottleLoadBudget)
	SHADER_PARAMETER(float, ThrottleEMAHistoryWeight)
	SHADER_PARAMETER(float, ThrottleMaxBiasDirectional)
	SHADER_PARAMETER(float, ThrottleMaxBiasLocal)
END_SHADER_PARAMETER_STRUCT()

class FProcessPrevFramePerfDataCS : public FVirtualShadowMapGlobalShader
{
	DECLARE_GLOBAL_SHADER(FProcessPrevFramePerfDataCS);
	SHADER_USE_PARAMETER_STRUCT(FProcessPrevFramePerfDataCS, FVirtualShadowMapGlobalShader)
public:
	static constexpr int32 NumThreadsPerGroup = 64;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PrevThrottleBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PrevNanitePerformanceFeedback)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FNextVirtualShadowMapData>, NextVirtualShadowMapData)
		SHADER_PARAMETER(uint32, NextVirtualShadowMapDataCount)

		SHADER_PARAMETER_STRUCT_INCLUDE(FThrottlingParameters, ThrottlingParameters)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutThrottleBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FProcessPrevFramePerfDataCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapThrottle.usf", "ProcessPrevFramePerfDataCS", SF_Compute);

class FUpdateThrottleParametersCS : public FVirtualShadowMapGlobalShader
{
	DECLARE_GLOBAL_SHADER(FUpdateThrottleParametersCS);
	SHADER_USE_PARAMETER_STRUCT(FUpdateThrottleParametersCS, FVirtualShadowMapGlobalShader)
public:
	static constexpr int32 NumThreadsPerGroup = 64;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PrevThrottleBuffer)

		SHADER_PARAMETER_STRUCT_INCLUDE(FThrottlingParameters, ThrottlingParameters)

		SHADER_PARAMETER(uint32, NumThrottleBufferEntries)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutThrottleBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, InOutProjectionData)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FUpdateThrottleParametersCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapThrottle.usf", "UpdateThrottleParametersCS", SF_Compute);

BEGIN_SHADER_PARAMETER_STRUCT(FThrottlingShaderParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, NanitePerformanceFeedback)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FBaseGeneratePageFlagsParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapMarkingParameters, MarkingParameters)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, DirectionalLightIds)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
	SHADER_PARAMETER(float, PageDilationBorderSizeDirectional)
	SHADER_PARAMETER(float, PageDilationBorderSizeLocal)
	SHADER_PARAMETER(uint32, bCullBackfacingPixels)
	SHADER_PARAMETER(uint32, MipModeLocal)
	SHADER_PARAMETER(float, FirstPersonPixelRequestBias)	
	SHADER_PARAMETER(uint32, FirstPersonPixelRequestLevelClamp)	
	SHADER_PARAMETER(float, DOFBiasStrength)
	SHADER_PARAMETER_STRUCT_INCLUDE(DiaphragmDOF::FDOFCocModelShaderParameters, CocModel)
END_SHADER_PARAMETER_STRUCT()

class FGeneratePageFlagsFromPixelsCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FGeneratePageFlagsFromPixelsCS);
	SHADER_USE_PARAMETER_STRUCT(FGeneratePageFlagsFromPixelsCS, FVirtualShadowMapPageManagementShader)

	class FInputType : SHADER_PERMUTATION_INT("PERMUTATION_INPUT_TYPE", 2); 
	class FWaterDepth : SHADER_PERMUTATION_BOOL("PERMUTATION_WATER_DEPTH"); 
	class FTranslucencyDepth : SHADER_PERMUTATION_BOOL("PERMUTATION_TRANSLUCENCY_DEPTH"); 
	class FThrottling : SHADER_PERMUTATION_BOOL("PERMUTATION_THROTTLING"); 
	using FPermutationDomain = TShaderPermutationDomain<FInputType, FWaterDepth, FTranslucencyDepth, FThrottling>;
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FInputType>() != 0 && (PermutationVector.Get<FWaterDepth>() || PermutationVector.Get<FTranslucencyDepth>()))
		{
			return false;
		}
		return FVirtualShadowMapPageManagementShader::ShouldCompilePermutation(Parameters);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FBaseGeneratePageFlagsParameters, Base)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		// PERMUTATION_WATER_DEPTH
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, SingleLayerWaterDepthTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint >, SingleLayerWaterTileMask)
		SHADER_PARAMETER(FIntPoint, SingleLayerWaterTileViewRes)
		// PERMUTATION_TRANSLUCENCY_DEPTH
		// FRONT LAYER 
		SHADER_PARAMETER(uint32, FrontLayerMode)
		SHADER_PARAMETER(FVector4f, FrontLayerHistoryUVMinMax)
		SHADER_PARAMETER(FVector4f, FrontLayerHistoryScreenPositionScaleBias)
		SHADER_PARAMETER(FVector4f, FrontLayerHistoryBufferSizeAndInvSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, FrontLayerTranslucencyDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, FrontLayerTranslucencyNormalTexture)

		RDG_BUFFER_ACCESS(IndirectBufferArgs, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER(uint32, InputType)
		SHADER_PARAMETER(FIntPoint, PixelStride)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FGeneratePageFlagsFromPixelsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPageMarking.usf", "GeneratePageFlagsFromPixels", SF_Compute);

class FGeneratePageFlagsFromFroxelsCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FGeneratePageFlagsFromFroxelsCS);
	SHADER_USE_PARAMETER_STRUCT(FGeneratePageFlagsFromFroxelsCS, FVirtualShadowMapPageManagementShader)

	class FDebugRenderDim : SHADER_PERMUTATION_BOOL("DEBUG_DRAW_GENERATE_FROM_FROXELS"); 
	class FThrottling : SHADER_PERMUTATION_BOOL("PERMUTATION_THROTTLING"); 
	using FPermutationDomain = TShaderPermutationDomain<FDebugRenderDim, FThrottling>;

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FDebugRenderDim>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}
		return FVirtualShadowMapPageManagementShader::ShouldPrecachePermutation(Parameters);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FBaseGeneratePageFlagsParameters, Base)
		SHADER_PARAMETER(uint32, bShouldMarkLocaLights)
		SHADER_PARAMETER(int32, PassId)
		SHADER_PARAMETER(float, DebugRange)
		SHADER_PARAMETER_STRUCT_INCLUDE(Froxel::FParameters, FroxelParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE( ShaderPrint::FShaderParameters, ShaderPrintStruct )
		RDG_BUFFER_ACCESS(IndirectBufferArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FGeneratePageFlagsFromFroxelsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPageMarking.usf", "GeneratePageFlagsFromFroxelsCS", SF_Compute);


class FMarkCoarsePagesCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FMarkCoarsePagesCS);
	SHADER_USE_PARAMETER_STRUCT(FMarkCoarsePagesCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapMarkingParameters, MarkingParameters)
		SHADER_PARAMETER(uint32, bMarkCoarsePagesLocal)
		SHADER_PARAMETER(uint32, bIncludeNonNaniteGeometry)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FMarkCoarsePagesCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPageMarking.usf", "MarkCoarsePages", SF_Compute);


class FGenerateHierarchicalPageFlagsCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FGenerateHierarchicalPageFlagsCS);
	SHADER_USE_PARAMETER_STRUCT(FGenerateHierarchicalPageFlagsCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPhysicalPageMetaData >, PhysicalPageMetaData)
		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<uint>, OutPageFlagMips, [VSM_LOG2_PAGE_SIZE - 1u])
		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<uint>, OutPageReceiverMaskMips, [VSM_LOG2_PAGE_SIZE])
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint>, InPageFlags)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint>, InPageReceiverMasks)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FIntVector4>, OutUncachedPageRectBounds)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FIntVector4>, OutAllocatedPageRectBounds)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FGenerateHierarchicalPageFlagsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPageManagement.usf", "GenerateHierarchicalPageFlags", SF_Compute);


class FUpdatePhysicalPageAddresses : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FUpdatePhysicalPageAddresses);
	SHADER_USE_PARAMETER_STRUCT(FUpdatePhysicalPageAddresses, FVirtualShadowMapPageManagementShader )

	class FHasCacheDataDim : SHADER_PERMUTATION_BOOL("HAS_CACHE_DATA");
	class FGenerateStatsDim : SHADER_PERMUTATION_BOOL("VSM_GENERATE_STATS");
	using FPermutationDomain = TShaderPermutationDomain<FHasCacheDataDim, FGenerateStatsDim>;

	BEGIN_SHADER_PARAMETER_STRUCT( FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPhysicalPageMetaData>,		OutPhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FNextVirtualShadowMapData>,	NextVirtualShadowMapData )
		SHADER_PARAMETER(uint32,														NextVirtualShadowMapDataCount )		
		// Required if using FHasCacheDataDim
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, 									PrevPageRequestFlags)
		// Required if using FGenerateStatsDim
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer< uint >,					OutStatsBuffer )
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FUpdatePhysicalPageAddresses, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "UpdatePhysicalPageAddresses", SF_Compute );


class FUpdatePhysicalPages : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FUpdatePhysicalPages);
	SHADER_USE_PARAMETER_STRUCT(FUpdatePhysicalPages, FVirtualShadowMapPageManagementShader )

	class FHasCacheDataDim : SHADER_PERMUTATION_BOOL("HAS_CACHE_DATA");
	class FGenerateStatsDim : SHADER_PERMUTATION_BOOL("VSM_GENERATE_STATS"); 
	using FPermutationDomain = TShaderPermutationDomain<FHasCacheDataDim, FGenerateStatsDim>;

	BEGIN_SHADER_PARAMETER_STRUCT( FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer< FPhysicalPageMetaData >,	OutPhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer< int >,						OutPhysicalPageLists )
		// Required if using FHasCacheDataDim
		SHADER_PARAMETER_RDG_TEXTURE( Texture2D<uint>,									PageRequestFlags )
		SHADER_PARAMETER_RDG_TEXTURE_UAV( RWTexture2D<uint>,OutPageFlags )
		SHADER_PARAMETER_RDG_TEXTURE_UAV( RWTexture2D<uint>,OutPageTable )
		SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< int >,						PrevPhysicalPageLists )
		SHADER_PARAMETER( uint32,														MaxPageAgeSinceLastRequest )
		// TODO: encode into options bitfield?
		SHADER_PARAMETER( int32,														bDynamicPageInvalidation )
		SHADER_PARAMETER( int32,														bAllocateViaLRU )
		SHADER_PARAMETER( int32,														DeferredInvalidationBudget )
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer< uint >,					DirtyPageFlagsInOut )
		// Required if using FGenerateStatsDim
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer< uint >,					OutStatsBuffer )
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FUpdatePhysicalPages, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "UpdatePhysicalPages", SF_Compute );

/**
* Utility to help schedule kernels that do processing for each page to facilitate not processing mip levels that can't be reached.
*/
class FVirtualShadowMapPerPageShader : public FVirtualShadowMapPageManagementShader
{
public:
	FVirtualShadowMapPerPageShader()
	{
	}

	FVirtualShadowMapPerPageShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FVirtualShadowMapPageManagementShader(Initializer)
	{
	}

	static constexpr int32 ThreadGroupSizeXY = 8;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< uint >, VirtualShadowMapIds)
		SHADER_PARAMETER( uint32, VirtualShadowMapIdsOffset ) 
		SHADER_PARAMETER( uint32, NumVirtualShadowMapIds ) 
		SHADER_PARAMETER( uint32, PerPageDispatchDimXY ) 
		SHADER_PARAMETER( uint32, bUseThreadPerId ) 
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("PER_PAGE_THREAD_GROUP_SIZE_XY"), ThreadGroupSizeXY);
		OutEnvironment.SetDefine(TEXT("PER_PAGE_DISPATCH_SETUP"), 1);
		FVirtualShadowMapPageManagementShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

/**
 * Utility to help schedule kernels that do processing for each page to facilitate not processing mip levels that can't be reached.
 */
class FPerPageShaderDispatcher
{
public:
	static constexpr int32 ThreadGroupSizeXY = FVirtualShadowMapPerPageShader::ThreadGroupSizeXY;

	struct FBinConfig
	{
		int32 DispatchDimX;
	};

	struct FBin
	{
		int32 VirtualShadowMapIdsOffset = 0;
		int32 NumVirtualShadowMapIds = 0;
	};

	// 1, Small, medium, large
	static constexpr int32 BinCount = 4;
	static constexpr int32 BinDispatchGroupDimXY[BinCount] = 
	{
		8, // * ThreadGroupSizeXY) ^ 2 = 4096 threads
		4, // * ThreadGroupSizeXY) ^ 2 = 1024 threads
		1, // * ThreadGroupSizeXY) ^ 2 = 64 threads
		0  // special: number of threads == number of Ids
	};

	static int32 CalcBin(int32 VirtualShadowMapId, int32 MinMipLevel)
	{
		if (IsSinglePageVirtualShadowMap(VirtualShadowMapId))
		{
			// last bin, single thread per shadow map
			return BinCount - 1;
		}

		if (MinMipLevel < 6)
		{
			return MinMipLevel / 2; 
		}

		// last bin, single thread per shadow map
		return BinCount - 1;
	}

	TStaticArray<FBin, BinCount> Bins;
	FRDGBufferSRV* VirtualShadowMapIdsSRV = nullptr;

	struct FBuilder
	{
		struct FIdBinIndex
		{
			uint32 VirtualShadowMapId : 28;
			uint32 BinIndex : 4;
		};

		void Add(int32 VirtualShadowMapId, uint32 MinMipLevel)
		{
			if (VirtualShadowMapId != INDEX_NONE)
			{
				FIdBinIndex IdBinIndex;
				IdBinIndex.BinIndex = CalcBin(VirtualShadowMapId, MinMipLevel);
				IdBinIndex.VirtualShadowMapId = VirtualShadowMapId;
				Tmp.Add(IdBinIndex);
				Bins[IdBinIndex.BinIndex].NumVirtualShadowMapIds += 1;
			}

		}

		void Reserve(int32 Num)
		{
			Tmp.Reserve(Num);
		}

		TStaticArray<FBin, BinCount> Bins;
		TArray<FIdBinIndex,SceneRenderingAllocator> Tmp;
	};

	void Init(FRDGBuilder& GraphBuilder, const FBuilder& Builder)
	{
		Bins = Builder.Bins;
		// counting sort the temp data into buffer of IDs
		int32 Offset = 0;
		for (int32 BinIndex = 0; BinIndex < BinCount; ++BinIndex)
		{
			Bins[BinIndex].VirtualShadowMapIdsOffset = Offset;
			Offset += Bins[BinIndex].NumVirtualShadowMapIds;
			Bins[BinIndex].NumVirtualShadowMapIds = 0;
		}
		TArray<uint32,SceneRenderingAllocator> VirtualShadowMapIds;
		VirtualShadowMapIds.SetNumUninitialized(Builder.Tmp.Num());
		for (FBuilder::FIdBinIndex IdBinIndex : Builder.Tmp)
		{
			FBin& Bin = Bins[IdBinIndex.BinIndex];
			VirtualShadowMapIds[Bin.VirtualShadowMapIdsOffset + Bin.NumVirtualShadowMapIds++] = IdBinIndex.VirtualShadowMapId;
		}

		VirtualShadowMapIdsSRV = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.PerPageBinIds"), MoveTemp(VirtualShadowMapIds)));
	}

	template <typename TShaderClass>
	void AddPass(FRDGBuilder& GraphBuilder, FRDGEventName&& PassName, const TShaderRef<TShaderClass>& ComputeShader, typename TShaderClass::FParameters* Parameters)
	{
		const FShaderParametersMetadata* ParametersMetadata = TShaderClass::FParameters::FTypeInfo::GetStructMetadata();
		Parameters->VirtualShadowMapPerPageShader.VirtualShadowMapIds = VirtualShadowMapIdsSRV;
		ClearUnusedGraphResources(ComputeShader, ParametersMetadata, Parameters);

		GraphBuilder.AddPass(Forward<FRDGEventName>(PassName), ParametersMetadata, Parameters, ERDGPassFlags::Compute,
		[ParametersMetadata, Parameters, ComputeShader, Bins = Bins](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			for (int32 BinIndex = 0; BinIndex < BinCount; ++BinIndex)
			{
				const FBin& Bin = Bins[BinIndex];
				if (Bins[BinIndex].NumVirtualShadowMapIds > 0)
				{
					// Set args for each pass
					Parameters->VirtualShadowMapPerPageShader.VirtualShadowMapIdsOffset = Bin.VirtualShadowMapIdsOffset;
					Parameters->VirtualShadowMapPerPageShader.NumVirtualShadowMapIds = Bin.NumVirtualShadowMapIds;
					bool bUseThreadPerId = BinDispatchGroupDimXY[BinIndex] == 0;
					Parameters->VirtualShadowMapPerPageShader.bUseThreadPerId = bUseThreadPerId ? 1 : 0;

					FIntVector GroupCount = FIntVector(0,0,1);
					if (bUseThreadPerId)
					{
						int32 NumThreadGroups = FMath::DivideAndRoundUp(Bin.NumVirtualShadowMapIds, FMath::Square(ThreadGroupSizeXY));
						// Note: here it is just a row pitch
						Parameters->VirtualShadowMapPerPageShader.PerPageDispatchDimXY = NumThreadGroups * ThreadGroupSizeXY; 
						// Each group contains ThreadGroupSizeXY^2 threads, so we launch enough groups for all IDs.
						GroupCount.X = NumThreadGroups;
						GroupCount.Y = 1;
						GroupCount.Z = 1;
					}
					else
					{
						Parameters->VirtualShadowMapPerPageShader.PerPageDispatchDimXY = BinDispatchGroupDimXY[BinIndex] * ThreadGroupSizeXY; 
						GroupCount.X = BinDispatchGroupDimXY[BinIndex];
						GroupCount.Y = BinDispatchGroupDimXY[BinIndex];
						GroupCount.Z = Bin.NumVirtualShadowMapIds;
					}

					FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, ParametersMetadata, *Parameters, GroupCount);
				}
			}
		});
	}
};

class FClearPageTableCS : public FVirtualShadowMapPerPageShader
{
	DECLARE_GLOBAL_SHADER(FClearPageTableCS);
	SHADER_USE_PARAMETER_STRUCT(FClearPageTableCS, FVirtualShadowMapPerPageShader)

	class FNumMipLevelsDim : SHADER_PERMUTATION_SPARSE_INT("NUM_MIP_LEVELS", 1, VSM_LOG2_PAGE_SIZE, VSM_LOG2_PAGE_SIZE + 1);
	using FPermutationDomain = TShaderPermutationDomain<FNumMipLevelsDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER( FVirtualShadowMapUniformParameters,		VirtualShadowMap )
		SHADER_PARAMETER_STRUCT_INCLUDE( FVirtualShadowMapPerPageShader::FParameters,	VirtualShadowMapPerPageShader)
		SHADER_PARAMETER( uint32,														ClearValue )
		SHADER_PARAMETER( uint32,														SampleStride )
		SHADER_PARAMETER_RDG_TEXTURE_UAV( RWTexture2D< uint >,							OutDestBuffer )
		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<uint>, OutDestBufferMips, [VSM_LOG2_PAGE_SIZE])

	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FClearPageTableCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "ClearPageTableCS", SF_Compute);


class FAllocateNewPageMappingsCS : public FVirtualShadowMapPerPageShader
{
	DECLARE_GLOBAL_SHADER(FAllocateNewPageMappingsCS);
	SHADER_USE_PARAMETER_STRUCT(FAllocateNewPageMappingsCS, FVirtualShadowMapPerPageShader)

	class FGenerateStatsDim : SHADER_PERMUTATION_BOOL("VSM_GENERATE_STATS"); 
	using FPermutationDomain = TShaderPermutationDomain<FGenerateStatsDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER( FVirtualShadowMapUniformParameters,		VirtualShadowMap )
		SHADER_PARAMETER_STRUCT_INCLUDE( FVirtualShadowMapPerPageShader::FParameters,	VirtualShadowMapPerPageShader)
		SHADER_PARAMETER_RDG_TEXTURE( Texture2D<uint>,									PageRequestFlags )
		SHADER_PARAMETER_RDG_TEXTURE_UAV( RWTexture2D<uint>,							OutPageFlags )
		SHADER_PARAMETER_RDG_TEXTURE_UAV( RWTexture2D<uint>,							OutPageTable )
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer< int >,						OutPhysicalPageLists )
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer< FPhysicalPageMetaData >,	OutPhysicalPageMetaData )
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer< uint >,					OutStatsBuffer )
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FAllocateNewPageMappingsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "AllocateNewPageMappingsCS", SF_Compute);

class FPackAvailablePagesCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FPackAvailablePagesCS);
	SHADER_USE_PARAMETER_STRUCT(FPackAvailablePagesCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters,			VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int>,						OutPhysicalPageLists)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>,						OutStatsBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FPackAvailablePagesCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "PackAvailablePages", SF_Compute );

class FAppendPhysicalPageListsCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FAppendPhysicalPageListsCS);
	SHADER_USE_PARAMETER_STRUCT(FAppendPhysicalPageListsCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters,			VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>,						OutPhysicalPageLists)
		SHADER_PARAMETER(uint32, bAppendEmptyToAvailable)
		SHADER_PARAMETER(uint32, bUpdateCounts)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FAppendPhysicalPageListsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "AppendPhysicalPageLists", SF_Compute );

class FPropagateMappedMipsCS : public FVirtualShadowMapPerPageShader
{
	DECLARE_GLOBAL_SHADER(FPropagateMappedMipsCS);
	SHADER_USE_PARAMETER_STRUCT(FPropagateMappedMipsCS, FVirtualShadowMapPerPageShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER( FVirtualShadowMapUniformParameters,		VirtualShadowMap )
		SHADER_PARAMETER_STRUCT_INCLUDE( FVirtualShadowMapPerPageShader::FParameters,	VirtualShadowMapPerPageShader)
		SHADER_PARAMETER_RDG_TEXTURE_UAV( RWTexture2D<uint>,							OutPageTable )
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FPropagateMappedMipsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPageManagement.usf", "PropagateMappedMips", SF_Compute);

class FPropagateFilterableRequestsCS : public FVirtualShadowMapPerPageShader
{
	DECLARE_GLOBAL_SHADER(FPropagateFilterableRequestsCS);
	SHADER_USE_PARAMETER_STRUCT(FPropagateFilterableRequestsCS, FVirtualShadowMapPerPageShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER( FVirtualShadowMapUniformParameters,		VirtualShadowMap )
		SHADER_PARAMETER_STRUCT_INCLUDE( FVirtualShadowMapPerPageShader::FParameters,	VirtualShadowMapPerPageShader)
		SHADER_PARAMETER_RDG_TEXTURE_UAV( RWTexture2D<uint>,							InOutPageRequestFlags )
		SHADER_PARAMETER_RDG_TEXTURE_UAV( RWTexture2D<uint>,							InOutPageReceiverMasks )
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FPropagateFilterableRequestsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPageManagement.usf", "PropagateFilterableRequestsCS", SF_Compute);

class FSelectPagesToInitializeCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FSelectPagesToInitializeCS);
	SHADER_USE_PARAMETER_STRUCT(FSelectPagesToInitializeCS, FVirtualShadowMapPageManagementShader)

	class FGenerateStatsDim : SHADER_PERMUTATION_BOOL("VSM_GENERATE_STATS");
	using FPermutationDomain = TShaderPermutationDomain<FGenerateStatsDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPhysicalPageMetaData>, OutPhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutInitializePagesIndirectArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutPhysicalPagesToInitialize)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutStatsBuffer)
		SHADER_PARAMETER(uint32, bUseClearedFlags)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FSelectPagesToInitializeCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "SelectPagesToInitializeCS", SF_Compute);

class FInitializePhysicalPagesIndirectCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FInitializePhysicalPagesIndirectCS);
	SHADER_USE_PARAMETER_STRUCT(FInitializePhysicalPagesIndirectCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPhysicalPageMetaData>, PhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint >, PhysicalPagesToInitialize)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutPhysicalPagePool)
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FInitializePhysicalPagesIndirectCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "InitializePhysicalPagesIndirectCS", SF_Compute);

class FSelectPagesToPostProcessCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FSelectPagesToPostProcessCS);
	SHADER_USE_PARAMETER_STRUCT(FSelectPagesToPostProcessCS, FVirtualShadowMapPageManagementShader)

	class FGenerateStatsDim : SHADER_PERMUTATION_BOOL("VSM_GENERATE_STATS");
	using FPermutationDomain = TShaderPermutationDomain<FGenerateStatsDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER( uint32, PostProcessPagesIndirectArgsBufferStride )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPhysicalPageMetaData>, OutPhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutPostProcessPagesIndirectArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, OutPhysicalPagesToMerge)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, OutPhysicalPagesWithFilterable)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, OutPhysicalPagesWithSeparableFilterable)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, OutStatsBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FSelectPagesToPostProcessCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "SelectPagesToPostProcessCS", SF_Compute);

class FMergeStaticPhysicalPagesIndirectCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FMergeStaticPhysicalPagesIndirectCS);
	SHADER_USE_PARAMETER_STRUCT(FMergeStaticPhysicalPagesIndirectCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint >, PhysicalPagesToMerge)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutPhysicalPagePool)
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FMergeStaticPhysicalPagesIndirectCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "MergeStaticPhysicalPagesIndirectCS", SF_Compute);

// Helper to gate relevant shaders with the project setting.
class FVirtualShadowMapPrefilteredDistantShader : public FVirtualShadowMapPageManagementShader
{
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GVirtualShadowMapPrefilteredDistantProjectEnable && FVirtualShadowMapPageManagementShader::ShouldCompilePermutation(Parameters);
	}
};

class FUpdateFilterablePagesSeparableX_CS : public FVirtualShadowMapPrefilteredDistantShader
{
	DECLARE_GLOBAL_SHADER(FUpdateFilterablePagesSeparableX_CS);
	SHADER_USE_PARAMETER_STRUCT(FUpdateFilterablePagesSeparableX_CS, FVirtualShadowMapPrefilteredDistantShader)

	// using FPermutationDomain = FUpdateFilterablePagesSeparablePermutationDomain;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPhysicalPageMetaData>, PhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint >, PhysicalPagesWithSeparableFilterable)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, OutPrefilteredDistantPhysicalPagePoolX)
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FUpdateFilterablePagesSeparableX_CS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "UpdateFilterablePagesSeparableX_CS", SF_Compute);

class FUpdateFilterablePagesSeparableY_CS : public FVirtualShadowMapPrefilteredDistantShader
{
	DECLARE_GLOBAL_SHADER(FUpdateFilterablePagesSeparableY_CS);
	SHADER_USE_PARAMETER_STRUCT(FUpdateFilterablePagesSeparableY_CS, FVirtualShadowMapPrefilteredDistantShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPhysicalPageMetaData>, PhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint >, PhysicalPagesWithSeparableFilterable)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, PrefilteredDistantPhysicalPagePoolX)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, OutPrefilteredDistantPhysicalPagePool)
		
		
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PrevVirtualShadowMapIds)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, PrevPrefilteredDistantPhysicalPagePool)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PrevProjectionData)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, PrevPageTable)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, PrevPageReceiverMasks)
		SHADER_PARAMETER( float, HistoryBlendFactor )
		SHADER_PARAMETER( uint32, bHavePrevReceiverMasks )

		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FUpdateFilterablePagesSeparableY_CS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "UpdateFilterablePagesSeparableY_CS", SF_Compute);

class FGatherFilterableBorderTexels_CS : public FVirtualShadowMapPrefilteredDistantShader
{
	DECLARE_GLOBAL_SHADER(FGatherFilterableBorderTexels_CS);
	SHADER_USE_PARAMETER_STRUCT(FGatherFilterableBorderTexels_CS, FVirtualShadowMapPrefilteredDistantShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPhysicalPageMetaData>, PhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint >, PhysicalPagesWithSeparableFilterable)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, InOutPrefilteredDistantPhysicalPagePool)

		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FGatherFilterableBorderTexels_CS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "GatherFilterableBorderTexels_CS", SF_Compute);

class FMergeDownFilterablePagesCS : public FVirtualShadowMapPrefilteredDistantShader
{
	DECLARE_GLOBAL_SHADER(FMergeDownFilterablePagesCS);
	SHADER_USE_PARAMETER_STRUCT(FMergeDownFilterablePagesCS, FVirtualShadowMapPrefilteredDistantShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPhysicalPageMetaData>, PhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint >, PhysicalPagesWithFilterable)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, PrefilteredDistantPhysicalPagePool)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, OutPrefilteredDistantPhysicalPagePool)

		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FMergeDownFilterablePagesCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "MergeDownFilterablePagesCS", SF_Compute);

class FUpdateAndClearDirtyFlagsCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FUpdateAndClearDirtyFlagsCS);
	SHADER_USE_PARAMETER_STRUCT(FUpdateAndClearDirtyFlagsCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, DirtyPageFlagsInOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< FPhysicalPageMetaData >, OutPhysicalPageMetaData)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FUpdateAndClearDirtyFlagsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "UpdateAndClearDirtyFlagsCS", SF_Compute);


void FVirtualShadowMapArray::PostRender(FRDGBuilder& GraphBuilder)
{
	check(IsEnabled());		
	if (GetNumShadowMaps() == 0)
	{
		return;
	}

	// Update the dirty page flags & the page table meta data for invalidations.
	{
		FUpdateAndClearDirtyFlagsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FUpdateAndClearDirtyFlagsCS::FParameters>();
		PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
		PassParameters->OutPhysicalPageMetaData = GraphBuilder.CreateUAV(PhysicalPageMetaDataRDG);
		PassParameters->DirtyPageFlagsInOut = GraphBuilder.CreateUAV(DirtyPageFlagsRDG);
		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FUpdateAndClearDirtyFlagsCS>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("UpdateAndClearDirtyFlags"),
			ComputeShader,
			PassParameters,
			FIntVector(FMath::DivideAndRoundUp(UniformParameters.MaxPhysicalPages, FUpdateAndClearDirtyFlagsCS::DefaultCSGroupX), 1, 1)
		);
	}

	{
		RDG_EVENT_SCOPE(GraphBuilder, "FVirtualShadowMapArray::PostProcessPages");

		// Note: We use GetTotalAllocatedPhysicalPages() to size the buffer as the selection shader emits both static/dynamic pages separately when enabled.
		FRDGBufferRef PhysicalPagesToMergeRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), GetTotalAllocatedPhysicalPages() + 1), TEXT("Shadow.Virtual.PhysicalPagesToMerge"));
		FRDGBufferRef PhysicalPagesWithFilterableRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), FMath::Max(1u, CacheManager->GetMaxFilterablePages())), TEXT("Shadow.Virtual.PhysicalPagesWithFilterable"));
		FRDGBufferRef PhysicalPagesWithSeparableFilterableRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), FMath::Max(1u, CacheManager->GetMaxFilterablePages())), TEXT("Shadow.Virtual.PhysicalPagesWithSeparableFilterable"));

		// 1. Initialize the indirect args buffer for both (potential) post passes
		// Note: we use stride 4 for the args because we use the 4th element to count allocated filterable pages.
		FRDGBufferRef PostProcessPagesIndirectArgsRDG = CreateAndClearIndirectDispatchArgs(GraphBuilder, Scene.GetFeatureLevel(), TEXT("Shadow.Virtual.PostProcessPagesIndirectArgs"), FIntVector3(0,1,1), 4u, 4u);

		// 2. Select the pages for different processing steps
		{
			FSelectPagesToPostProcessCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSelectPagesToPostProcessCS::FParameters>();
			PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
			PassParameters->OutPhysicalPageMetaData = GraphBuilder.CreateUAV(PhysicalPageMetaDataRDG);
			PassParameters->OutPostProcessPagesIndirectArgsBuffer = GraphBuilder.CreateUAV(PostProcessPagesIndirectArgsRDG);
			PassParameters->PostProcessPagesIndirectArgsBufferStride = 4u;
			PassParameters->OutPhysicalPagesToMerge = GraphBuilder.CreateUAV(PhysicalPagesToMergeRDG);
			PassParameters->OutPhysicalPagesWithFilterable = GraphBuilder.CreateUAV(PhysicalPagesWithFilterableRDG);
			PassParameters->OutPhysicalPagesWithSeparableFilterable = GraphBuilder.CreateUAV(PhysicalPagesWithSeparableFilterableRDG);

			FSelectPagesToPostProcessCS::FPermutationDomain PermutationVector;
			SetStatsArgsAndPermutation<FSelectPagesToPostProcessCS>(ShouldGenerateStats(), StatsBufferUAV, PassParameters, PermutationVector);

			auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FSelectPagesToPostProcessCS>(PermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SelectPagesToPostProcess"),
				ComputeShader,
				PassParameters,
				FIntVector(FMath::DivideAndRoundUp(GetMaxPhysicalPages(), FSelectPagesToPostProcessCS::DefaultCSGroupX), 1, 1));
		}

		// 3. Indirect dispatch to merge the selected pages
#if !UE_BUILD_SHIPPING
		if (CVarDebugSkipMergePhysical.GetValueOnRenderThread() == 0)
#endif			
		{
			FMergeStaticPhysicalPagesIndirectCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMergeStaticPhysicalPagesIndirectCS::FParameters>();
			PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
			PassParameters->OutPhysicalPagePool = GraphBuilder.CreateUAV(PhysicalPagePoolRDG);
			PassParameters->IndirectArgs = PostProcessPagesIndirectArgsRDG;
			PassParameters->PhysicalPagesToMerge = GraphBuilder.CreateSRV(PhysicalPagesToMergeRDG);
			auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FMergeStaticPhysicalPagesIndirectCS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("MergeStaticPhysicalPagesIndirect"),
				ComputeShader,
				PassParameters,
				PassParameters->IndirectArgs,
				0
			);
		}

		// Indirect dispatch(es) to process any pages with filterable SM (outside in)
		if (IsPrefilteredDistantEnabled())
		{
			{
				// NOTE: offset 4 to get second set of args in the buffer.
				const uint32 IndirectArgsOffset = 4U * sizeof(uint32);
				// separable
				// 1. allocate temp separable target:
				FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
					CacheManager->GetFilterablePagePoolSizePages() * FVirtualShadowMap::PageSize, 
					PF_R8, 
					FClearValueBinding::Black, 
					ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV | ETextureCreateFlags::RenderTargetable);
				FRDGTexture* PrefilteredDistantPhysicalPagePoolX = GraphBuilder.CreateTexture(Desc, TEXT("Shadow.Virtual.PrefilteredDistantPhysicalPagePoolX"));
				AddClearRenderTargetPass(GraphBuilder, PrefilteredDistantPhysicalPagePoolX);

				// 1. horizontal pass
				{
					FUpdateFilterablePagesSeparableX_CS::FParameters* PassParameters = GraphBuilder.AllocParameters<FUpdateFilterablePagesSeparableX_CS::FParameters>();
					PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
					PassParameters->OutPrefilteredDistantPhysicalPagePoolX = GraphBuilder.CreateUAV(PrefilteredDistantPhysicalPagePoolX);
					PassParameters->PhysicalPageMetaData = GraphBuilder.CreateSRV(PhysicalPageMetaDataRDG);
					PassParameters->PhysicalPagesWithSeparableFilterable = GraphBuilder.CreateSRV(PhysicalPagesWithSeparableFilterableRDG);
					PassParameters->IndirectArgs = PostProcessPagesIndirectArgsRDG;
			
					auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FUpdateFilterablePagesSeparableX_CS>();
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("FUpdateFilterablePagesSeparableX_CS"),
						ComputeShader,
						PassParameters,
						PostProcessPagesIndirectArgsRDG,
						IndirectArgsOffset
					);
				}
				// 2. vertical pass + history reprojection
				{
					FUpdateFilterablePagesSeparableY_CS::FParameters* PassParameters = GraphBuilder.AllocParameters<FUpdateFilterablePagesSeparableY_CS::FParameters>();
					PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
					PassParameters->PrefilteredDistantPhysicalPagePoolX = PrefilteredDistantPhysicalPagePoolX;
					PassParameters->OutPrefilteredDistantPhysicalPagePool = GraphBuilder.CreateUAV(PrefilteredDistantPhysicalPagePoolSeparateRDG);
					PassParameters->PhysicalPageMetaData = GraphBuilder.CreateSRV(PhysicalPageMetaDataRDG);
					PassParameters->PhysicalPagesWithSeparableFilterable = GraphBuilder.CreateSRV(PhysicalPagesWithSeparableFilterableRDG);
					PassParameters->HistoryBlendFactor = FMath::Clamp(CVarVSMPrefilteredDistantHistoryBlendFactor.GetValueOnRenderThread(), 0.0f, 1.0f);
					PassParameters->IndirectArgs = PostProcessPagesIndirectArgsRDG;

					const FVirtualShadowMapArrayFrameData& PrevBuffers = CacheManager->GetPrevBuffers();
					PassParameters->PrevProjectionData = GraphBuilder.CreateSRV(PrevProjectionDataRDG);
					if (PassParameters->HistoryBlendFactor > 0.0f && PrevBuffers.PageTable && PrevBuffers.ProjectionData && PrevBuffers.PrevPrefilteredDistantPhysicalPagePoolSeparate && PrevBuffers.PageReceiverMasks)
					{
						PassParameters->PrevPageTable = GraphBuilder.RegisterExternalTexture( PrevBuffers.PageTable, TEXT("Shadow.Virtual.PrevPageTable"));
						PassParameters->bHavePrevReceiverMasks = IsVirtualShadowMapDirectionalReceiverMaskEnabled();
						PassParameters->PrevPageReceiverMasks = GraphBuilder.RegisterExternalTexture( PrevBuffers.PageReceiverMasks, TEXT("Shadow.Virtual.PrevPageReceiverMasks"));
						PassParameters->PrevPrefilteredDistantPhysicalPagePool = GraphBuilder.RegisterExternalTexture( PrevBuffers.PrevPrefilteredDistantPhysicalPagePoolSeparate, TEXT("Shadow.Virtual.PrevPrefilteredDistantPhysicalPagePoolSeparate"));

						TArray<int32> PrevVirtualShadowMapIds;
						PrevVirtualShadowMapIds.SetNumZeroed(GetNumShadowMapSlots());
						for (int32 PrevId = 0; PrevId < NextData.Num(); ++PrevId)
						{
							FNextVirtualShadowMapData& Next = NextData[PrevId];
							if ((Next.Flags & VSM_NEXT_FLAG_ID_VALID) != 0u)
							{
								PrevVirtualShadowMapIds[Next.NextVirtualShadowMapId] = PrevId + 1;
							}
						}

						FRDGBufferRef PrevVirtualShadowMapIdsRDG = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.PrevVirtualShadowMapIds"), MoveTemp(PrevVirtualShadowMapIds));
						PassParameters->PrevVirtualShadowMapIds = GraphBuilder.CreateSRV(PrevVirtualShadowMapIdsRDG);
					}
					else
					{
						PassParameters->PrevPageReceiverMasks = 0u;
						PassParameters->HistoryBlendFactor = 0.0f;
						PassParameters->PrevPageTable = GSystemTextures.GetZeroUIntDummy(GraphBuilder);
						PassParameters->PrevPrefilteredDistantPhysicalPagePool = GSystemTextures.GetBlackDummy(GraphBuilder);
						PassParameters->PrevPageReceiverMasks = PageReceiverMasksRDG;
						PassParameters->PrevVirtualShadowMapIds = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 4, 0u));
					}

					auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FUpdateFilterablePagesSeparableY_CS>();
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("FUpdateFilterablePagesSeparableY_CS"),
						ComputeShader,
						PassParameters,
						PostProcessPagesIndirectArgsRDG,
						IndirectArgsOffset
					);				
				}
			}
			// Copy borders pass
			auto AddPassBorderCopy = [&](FRDGTextureRef PrefilteredDistantPhysicalPagePoolRDG)
			{
				// NOTE: offset to get fourth set of args in the buffer.
				const uint32 IndirectArgsOffset = 12U * sizeof(uint32);
				FGatherFilterableBorderTexels_CS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGatherFilterableBorderTexels_CS::FParameters>();
				PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
				PassParameters->InOutPrefilteredDistantPhysicalPagePool = GraphBuilder.CreateUAV(PrefilteredDistantPhysicalPagePoolRDG);
				PassParameters->PhysicalPageMetaData = GraphBuilder.CreateSRV(PhysicalPageMetaDataRDG);
				PassParameters->PhysicalPagesWithSeparableFilterable = GraphBuilder.CreateSRV(PhysicalPagesWithSeparableFilterableRDG);
				PassParameters->IndirectArgs = PostProcessPagesIndirectArgsRDG;

				auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FGatherFilterableBorderTexels_CS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("GatherFilterableBorderTexels_CS"),
					ComputeShader,
					PassParameters,
					PostProcessPagesIndirectArgsRDG,
					IndirectArgsOffset
				);

			};
			AddPassBorderCopy(PrefilteredDistantPhysicalPagePoolSeparateRDG);
			// 3. Merge down pass
			{
				// NOTE: offset 8 to get third set of args in the buffer.
				const uint32 IndirectArgsOffset = 8U * sizeof(uint32);
				FMergeDownFilterablePagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMergeDownFilterablePagesCS::FParameters>();
				PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
				PassParameters->PrefilteredDistantPhysicalPagePool = PrefilteredDistantPhysicalPagePoolSeparateRDG;
				PassParameters->OutPrefilteredDistantPhysicalPagePool = GraphBuilder.CreateUAV(PrefilteredDistantPhysicalPagePoolMergedRDG);
				PassParameters->PhysicalPageMetaData = GraphBuilder.CreateSRV(PhysicalPageMetaDataRDG);
				PassParameters->PhysicalPagesWithFilterable = GraphBuilder.CreateSRV(PhysicalPagesWithFilterableRDG);
				PassParameters->IndirectArgs = PostProcessPagesIndirectArgsRDG;

				auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FMergeDownFilterablePagesCS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("FMergeDownFilterablePagesCS"),
					ComputeShader,
					PassParameters,
					PostProcessPagesIndirectArgsRDG,
					IndirectArgsOffset
				);

			}
			AddPassBorderCopy(PrefilteredDistantPhysicalPagePoolMergedRDG);
		}
	}
}


class FInitPageRectBoundsCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FInitPageRectBoundsCS);
	SHADER_USE_PARAMETER_STRUCT(FInitPageRectBoundsCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FIntVector4>, OutUncachedPageRectBounds)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FIntVector4>, OutAllocatedPageRectBounds)
		SHADER_PARAMETER(uint32, NumPageRectsToClear)		
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int>, OutPhysicalPageLists)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPhysicalPageRequest>, OutPhysicalPageAllocationRequests)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FInitPageRectBoundsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "InitPageRectBounds", SF_Compute);


class FVirtualSmFeedbackStatusCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FVirtualSmFeedbackStatusCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualSmFeedbackStatusCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< int >, PhysicalPageLists)
		SHADER_PARAMETER_STRUCT_INCLUDE(GPUMessage::FParameters, GPUMessageParams)
		SHADER_PARAMETER(uint32, StatusMessageId)
	END_SHADER_PARAMETER_STRUCT()


	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FVirtualShadowMapPageManagementShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FVirtualSmFeedbackStatusCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "FeedbackStatusCS", SF_Compute);

void FVirtualShadowMapVisualizeLightSearch::CheckLight(const FLightSceneProxy* CheckProxy, int CheckVirtualShadowMapId)
{
#if VSM_ENABLE_VISUALIZATION
	FString CheckLightName = CheckProxy->GetOwnerNameOrLabel();
	if (GDumpVSMLightNames)
	{
		UE_LOGF(LogRenderer, Display, "%ls", *CheckLightName);
	}

	const FLightComponentId CheckProxyId = CheckProxy->GetLightComponentId();

	const int SelectAdjacentVisualizeLight = GetVirtualShadowMapVisualizationData().SelectAdjacentVisualizeLight;
	const bool SelectNextLight = SelectAdjacentVisualizeLight > 0;
	const bool SelectPrevLight = SelectAdjacentVisualizeLight < 0;
	GVirtualShadowMapVisualizeByLightId = GVirtualShadowMapVisualizeByLightId || SelectAdjacentVisualizeLight != 0;

	// When the user clicks a light, visualize the selected light.
	if (CheckProxy->IsSelected() && CheckProxyId != GVirtualShadowMapLastSelectedVisualizeLightId)
	{
		GVirtualShadowMapVisualizeByLightId = false;
		GVirtualShadowMapLastSelectedVisualizeLightId = CheckProxyId;
	}

	if (GVirtualShadowMapVisualizeByLightId)
	{
		FLightComponentId FoundAdjacentLightId = FoundProxy ? FoundProxy->GetLightComponentId() : FLightComponentId();
		bool bIsFoundAdjacentLightInvalid = !FoundAdjacentLightId.IsValid();

		if (SelectPrevLight
			// Light comes before current selection
			&& CheckProxyId < GVirtualShadowMapVisualizeLightId
			// Light comes after best match so far
			&& FoundAdjacentLightId < CheckProxyId)
		{
			FoundProxy = CheckProxy;
			FoundVirtualShadowMapId = CheckVirtualShadowMapId;
		}
		else if (SelectNextLight
			// Light comes after current selection
			&& GVirtualShadowMapVisualizeLightId < CheckProxyId
			// Light comes before best match so far
			&& (CheckProxyId < FoundAdjacentLightId || bIsFoundAdjacentLightInvalid))
		{
			FoundProxy = CheckProxy;
			FoundVirtualShadowMapId = CheckVirtualShadowMapId;
		}
		else if (SelectAdjacentVisualizeLight == 0 
			&& CheckProxyId == GVirtualShadowMapVisualizeLightId)
		{
			FoundProxy = CheckProxy;
			FoundVirtualShadowMapId = CheckVirtualShadowMapId;
		}
	}
	else
	{
		// Fill out new sort key and compare to our best found so far
		SortKey CheckKey;
		CheckKey.Packed = 0;
		CheckKey.Fields.bExactNameMatch = (CheckLightName == GVirtualShadowMapVisualizeLightName);
		CheckKey.Fields.bPartialNameMatch = CheckKey.Fields.bExactNameMatch || (!GVirtualShadowMapVisualizeLightName.IsEmpty() && CheckLightName.Contains(GVirtualShadowMapVisualizeLightName));
		CheckKey.Fields.bSelected = CheckProxy->IsSelected();

		if (CheckKey.Packed > FoundKey.Packed)		//-V547
		{
			FoundKey = CheckKey;
			FoundProxy = CheckProxy;
			FoundVirtualShadowMapId = CheckVirtualShadowMapId;
		}
	}
#endif //VSM_ENABLE_VISUALIZATION
}

void FVirtualShadowMapVisualizeLightSearch::ChooseLight()
{
#if VSM_ENABLE_VISUALIZATION
	const int SelectAdjacentVisualizeLight = GetVirtualShadowMapVisualizationData().SelectAdjacentVisualizeLight;
	GetVirtualShadowMapVisualizationData().SelectAdjacentVisualizeLight = 0;

	if (FoundProxy)
	{
		GVirtualShadowMapVisualizeLightId = FoundProxy->GetLightComponentId();
	}
	else if (SelectAdjacentVisualizeLight) // selected past first or last light, select none
	{
		GVirtualShadowMapVisualizeLightId = FLightComponentId();
	}
#endif //VSM_ENABLE_VISUALIZATION
}

const FString FVirtualShadowMapVisualizeLightSearch::GetLightName() const
{
	return FoundProxy->GetOwnerNameOrLabel();
}

static FRDGTextureRef CreateDebugVisualizationTexture(FRDGBuilder& GraphBuilder, FIntPoint Extent)
{
	const FLinearColor ClearColor(0.0f, 0.0f, 0.0f, 0.0f);

	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
		Extent,
		PF_FloatRGBA,
		FClearValueBinding(ClearColor),
		TexCreate_ShaderResource | TexCreate_UAV);

	FRDGTextureRef Texture = GraphBuilder.CreateTexture(Desc, TEXT("Shadow.Virtual.DebugProjection"));
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Texture), ClearColor);
	return Texture;
}

void FVirtualShadowMapArray::UpdateVisualizeLight(const TConstArrayView<FViewInfo>& Views, const FVisibleLightInfoArray& VisibleLightInfos)
{
#if VSM_ENABLE_VISUALIZATION
	for (const FVisibleLightInfo& VisibleLightInfo : VisibleLightInfos)
	{
		if (VisibleLightInfo.VirtualShadowMapClipmaps.Num() > 0)
		{
			// Directional light
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				TSharedPtr<FVirtualShadowMapClipmap> Clipmap = VisibleLightInfo.FindVirtualShadowMapShadowClipmapForView(&Views[ViewIndex]);
				if (Clipmap.IsValid())
				{
					VisualizeLight[ViewIndex].CheckLight(Clipmap->GetLightSceneInfo().Proxy, Clipmap->GetVirtualShadowMapId());
				}
			}
		}
		else
		{
			// Local light
			int32 VirtualShadowMapId = VisibleLightInfo.GetVirtualShadowMapId(&Views[0]);
			if (VirtualShadowMapId != INDEX_NONE)
			{
				// NOTE: Assumes all the light scene infos for the different projections are the same!
				if (VisibleLightInfo.AllProjectedShadows.Num() > 0)
				{
					const FLightSceneProxy* Proxy = VisibleLightInfo.AllProjectedShadows[0]->GetLightSceneInfo().Proxy;
					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
					{
						VisualizeLight[ViewIndex].CheckLight(Proxy, VirtualShadowMapId);
					}
				}
			}
		}
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		VisualizeLight[ViewIndex].ChooseLight();
	}
#endif //VSM_ENABLE_VISUALIZATION
}

void FVirtualShadowMapArray::AppendPhysicalPageList(FRDGBuilder& GraphBuilder, bool bEmptyToAvailable)
{
	auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FAppendPhysicalPageListsCS>();
	
	FAppendPhysicalPageListsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAppendPhysicalPageListsCS::FParameters>();
	PassParameters->VirtualShadowMap		= GetUncachedUniformBuffer(GraphBuilder);
	PassParameters->OutPhysicalPageLists	= GraphBuilder.CreateUAV(PhysicalPageListsRDG);
	PassParameters->bAppendEmptyToAvailable = bEmptyToAvailable ? 1 : 0;
	PassParameters->bUpdateCounts			= 0;

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("AppendPhysicalPageList"),
		ComputeShader,
		PassParameters,
		FIntVector(FMath::DivideAndRoundUp(GetMaxPhysicalPages(), FAppendPhysicalPageListsCS::DefaultCSGroupX), 1, 1)
	);

	FAppendPhysicalPageListsCS::FParameters* CountsParameters = GraphBuilder.AllocParameters<FAppendPhysicalPageListsCS::FParameters>();
	*CountsParameters = *PassParameters;
	CountsParameters->bUpdateCounts = 1;
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("AppendPhysicalPageList(Counts)"),
		ComputeShader,
		CountsParameters,
		FIntVector(1, 1, 1)
	);
}

static TAutoConsoleVariable<int32> CVarVSMDynamicDeallocationMode(
	TEXT("r.Shadow.Virtual.DynamicDeallocationMode"),
	0,
	TEXT("Controls the behavior of VSM resources being dynamically allocated/deallocated based on whether VSM shadows are present:\n")
	TEXT("0 (default): Enables dynamic deallocation when MegaLights are also enabled, otherwise disabled\n")
	TEXT("1: Enables dynamic deallocation always\n")
	TEXT("2: Disables dynamic deallocation always\n")
	TEXT("Dynamic deallocation can save memory if there are significant times when VSMs are not in use, but can add allocator churn."),
	ECVF_RenderThreadSafe
);

static bool AllowVirtualShadowMapDynamicDeallocation(const FSceneViewFamily& ViewFamily)
{
	int32 Mode = CVarVSMDynamicDeallocationMode.GetValueOnRenderThread();
	return
		(Mode == 1 ||
		(Mode == 0 && MegaLights::IsEnabled(ViewFamily)));
}

void FVirtualShadowMapArray::UpdatePhysicalPageAddresses(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily)
{
	if (!IsEnabled() ||
		(GetNumShadowMaps() == 0 && NextData.IsEmpty()))
	{
		// If we get here with no shadow maps and no cached data even for unreferenced entries it means we legitimately have no
		// useful shadow data. Optionally we can deallocate the buffers here to save VRAM. There's a potential for allocator churn
		// so we only do this in situations that we expect there to be a good chance that we aren't using VSMs at all for some
		// reasonable chunk of time (i.e. MegaLights in use, etc)
		if (CacheManager && AllowVirtualShadowMapDynamicDeallocation(ViewFamily))
		{
			CacheManager->FreePhysicalPool(GraphBuilder);
			CacheManager->FreeHZBPhysicalPool(GraphBuilder);
		}
		return;
	}

	// At this point, ensure we have a properly-sized physical page pool and HZB
	{
		// 2 elements normally (static/dynamic cache), one extra if we want to store nanite visualization outputs in light space
		const int PoolArraySize = bEnableNaniteVisualization ? 3 : 2;
		CacheManager->SetPhysicalPoolSize(GraphBuilder, GetPhysicalPoolSize(), PoolArraySize, GetMaxPhysicalPages());
		PhysicalPagePoolRDG = GraphBuilder.RegisterExternalTexture(CacheManager->GetPhysicalPagePool());
		PhysicalPageMetaDataRDG = GraphBuilder.RegisterExternalBuffer(CacheManager->GetPhysicalPageMetaData());
		UniformParameters.PhysicalPagePool = PhysicalPagePoolRDG;

		if (bUseHzbOcclusion)
		{
			const int HZBPoolArraySize = HasSeparateDynamicHZB() ? 2 : 1;
			CacheManager->SetHZBPhysicalPoolSize(GraphBuilder, GetHZBPhysicalPoolSize(), HZBPoolArraySize, PF_R32_FLOAT);
			HZBPhysicalArrayRDG = GraphBuilder.RegisterExternalTexture(CacheManager->GetHZBPhysicalPool());
		}
	}

	if (IsPrefilteredDistantEnabled())
	{
		FIntPoint FilterablePhysicalPoolSizePages = CacheManager->GetFilterablePagePoolSizePages();
		FIntPoint FilterablePhysicalPoolSize = FilterablePhysicalPoolSizePages * VSM_PREFILTERED_DISTANT_PAGE_SIZE;
		UniformParameters.RecFilterablePhysicalPoolSize = FVector4f( 1.0f / float(FilterablePhysicalPoolSize.X), 1.0f / float(FilterablePhysicalPoolSize.Y), 1.0f, 1.0f );

		FRDGTextureDesc FilterableDesc = FRDGTextureDesc::Create2D(
			FilterablePhysicalPoolSize,
			PF_R8, 
			FClearValueBinding::Black,
			ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV | ETextureCreateFlags::RenderTargetable);


		PrefilteredDistantPhysicalPagePoolSeparateRDG = GraphBuilder.CreateTexture(FilterableDesc, TEXT("Shadow.Virtual.PrefilteredDistantPhysicalPagePoolSeparate"));
		AddClearRenderTargetPass(GraphBuilder, PrefilteredDistantPhysicalPagePoolSeparateRDG);
		PrefilteredDistantPhysicalPagePoolMergedRDG = GraphBuilder.CreateTexture(FilterableDesc, TEXT("Shadow.Virtual.PrefilteredDistantPhysicalPagePoolMerged"));
		AddClearRenderTargetPass(GraphBuilder, PrefilteredDistantPhysicalPagePoolMergedRDG);
	}
	// NOTE: This pass MUST run on all GPUs, as we still need to propogate changes to the VSM IDs even if
	// a given GPU may not do any rendering during this phase.
	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	FUpdatePhysicalPageAddresses::FParameters* PassParameters = GraphBuilder.AllocParameters<FUpdatePhysicalPageAddresses::FParameters>();
	PassParameters->VirtualShadowMap		= GetUncachedUniformBuffer(GraphBuilder);
	PassParameters->OutPhysicalPageMetaData = GraphBuilder.CreateUAV(PhysicalPageMetaDataRDG);

	// Upload our prev -> next shadow data mapping (FNextVirtualShadowMapData) to the GPU
	FRDGBufferRef NextVirtualShadowMapData = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.NextVirtualShadowMapData"), NextData);
	PassParameters->NextVirtualShadowMapData = GraphBuilder.CreateSRV(NextVirtualShadowMapData);
	PassParameters->NextVirtualShadowMapDataCount = NextData.Num();

	FUpdatePhysicalPageAddresses::FPermutationDomain PermutationVector;

	const bool bCacheDataValid = CacheManager->IsCacheDataAvailable();
	if (CacheManager->IsCacheDataAvailable())
	{
		TRefCountPtr<IPooledRenderTarget> PrevPageRequestFlags = CacheManager->GetPrevBuffers().PageRequestFlags;
		PassParameters->PrevPageRequestFlags = GraphBuilder.RegisterExternalTexture(PrevPageRequestFlags, TEXT("Shadow.Virtual.PrevPageRequestFlags"));
	}
	PermutationVector.Set<FUpdatePhysicalPageAddresses::FHasCacheDataDim>(bCacheDataValid);

	SetStatsArgsAndPermutation<FUpdatePhysicalPageAddresses>(ShouldGenerateStats(), StatsBufferUAV, PassParameters, PermutationVector);

	auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FUpdatePhysicalPageAddresses>(PermutationVector);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FVirtualShadowMapArray::UpdatePhysicalPageAddresses"),
		ComputeShader,
		PassParameters,
		FIntVector(FMath::DivideAndRoundUp(GetMaxPhysicalPages(), FUpdatePhysicalPageAddresses::DefaultCSGroupX), 1, 1)
	);
}

bool FVirtualShadowMapArray::ShouldGenerateStats() const
{
#if !UE_BUILD_SHIPPING
	const bool bRunPageAreaDiagnostics = CVarNumPageAreaDiagSlots.GetValueOnRenderThread() != 0;
	const bool bInsightsVSMChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(VSMChannel);
#else
	constexpr bool bRunPageAreaDiagnostics = false;
	constexpr bool bInsightsVSMChannelEnabled = false;
#endif
	return FVirtualShadowMap::GetEnabledStatSections() != EVSMStatSection::None || CacheManager->IsAccumulatingStats() || bRunPageAreaDiagnostics || IsCsvLogEnabled() || bInsightsVSMChannelEnabled;
}

bool FVirtualShadowMapArray::IsCsvLogEnabled() const
{
#if CSV_PROFILER_STATS
	return FCsvProfiler::Get()->IsCapturing_Renderthread() && FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(VSM));
#else
	return false;
#endif
}

class FVirtualShadowMapSetupContext
{
public:
	FPerPageShaderDispatcher PerPageShaderDispatcher;
	FPerPageShaderDispatcher PerPageShaderDispatcherDirectionalOnly;
};

void FVirtualShadowMapArray::BeginMarkPages(
	FRDGBuilder& GraphBuilder,	
	const FSceneRenderer& SceneRenderer,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult,
	const FFrontLayerTranslucencyData& FrontLayerTranslucencyData,
	const Froxel::FRenderer& FroxelRenderer,
	bool bAnyLocalLightsWithVSMs)
{
	check(IsEnabled());
	check(!VirtualShadowMapSetupContext);

	if (GetNumShadowMaps() == 0)
	{
		return;
	}

	VirtualShadowMapSetupContext = GraphBuilder.AllocObject<FVirtualShadowMapSetupContext>();

	const FMinimalSceneTextures& SceneTextures = SceneRenderer.GetActiveSceneTextures();
	const TConstArrayView<FViewInfo>& Views = SceneRenderer.Views;
	const FShadowSceneRenderer& ShadowSceneRenderer = SceneRenderer.GetSceneExtensionsRenderers().GetRenderer<FShadowSceneRenderer>();

	check(Views.Num() > 0);

	RDG_EVENT_SCOPE(GraphBuilder, "FVirtualShadowMapArray::BeginMarkPages");
	SCOPED_NAMED_EVENT(FVirtualShadowMapArray_BeginMarkPages, FColor::Emerald);

	VisualizeLight.Reset(Views.Num());
	VisualizeLight.AddDefaulted(Views.Num());

	PerViewParameters.Reset(Views.Num());
	PerViewParameters.AddDefaulted(Views.Num());

#if VSM_ENABLE_VISUALIZATION
	if (GDumpVSMLightNames)
	{
		UE_LOGF(LogRenderer, Display, "Lights with Virtual Shadow Maps:");
	}

	// Setup debug visualization output if enabled
	if (bEnableVisualization)
	{
		FVirtualShadowMapVisualizationData& VisualizationData = GetVirtualShadowMapVisualizationData();
	
		for (const FViewInfo& View : Views)
		{
			VisualizationData.Update(View.CurrentVirtualShadowMapVisualizationMode);
			if (VisualizationData.IsActive())
			{
				// for stereo views that aren't multi-view, don't account for the left
				FIntPoint Extent = View.ViewRect.Max - View.ViewRect.Min;
				DebugVisualizationOutput.Add(CreateDebugVisualizationTexture(GraphBuilder, Extent));
			}
		}
	}

	UpdateVisualizeLight(Views, SceneRenderer.VisibleLightInfos);
#endif //VSM_ENABLE_VISUALIZATION

	{
		FPerPageShaderDispatcher::FBuilder Builder;
		FPerPageShaderDispatcher::FBuilder BuilderDirectionalOnly;

		// Create large enough to hold all the unused elements too (wastes GPU memory but allows direct indexing via the ID)
		uint32 DataSize = sizeof(FVirtualShadowMapProjectionShaderData) * uint32(GetNumShadowMapSlots());
		FRDGBufferDesc Desc;
		Desc.Usage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::ByteAddressBuffer | EBufferUsageFlags::StructuredBuffer;
		Desc.BytesPerElement = 4;
		Desc.NumElements = DataSize / 4U;
		ProjectionDataRDG = GraphBuilder.CreateBuffer(Desc, TEXT("Shadow.Virtual.ProjectionData"));
		
		FRDGScatterUploadBuffer Uploader;
		Uploader.Init(GraphBuilder, GetNumShadowMaps(), sizeof(FVirtualShadowMapProjectionShaderData), false, TEXT("Shadow.Virtual.ProjectionData.UploadBuffer"));

		for (auto It = CacheManager->CreateConstEntryIterator(); It; ++It)
		{
			const TSharedPtr<FVirtualShadowMapPerLightCacheEntry>& CacheEntry = It.Value();
			const int32 VirtualShadowMapId = CacheEntry->GetVirtualShadowMapId();
			check(VirtualShadowMapId >= 0);
			for (int32 Index = 0; Index < CacheEntry->ShadowMapEntries.Num(); ++Index)
			{
				const auto& Entry = CacheEntry->ShadowMapEntries[Index];
				int32 EntryVirtualShadowMapId = VirtualShadowMapId + Index;
				Uploader.Add(EntryVirtualShadowMapId, &Entry.ProjectionData);
				Builder.Add(EntryVirtualShadowMapId, Entry.ProjectionData.MinMipLevel);

				if (IsDirectional(VirtualShadowMapId))
				{
					BuilderDirectionalOnly.Add(EntryVirtualShadowMapId, Entry.ProjectionData.MinMipLevel);
				}
			}
		}
		Uploader.ResourceUploadTo(GraphBuilder, ProjectionDataRDG);

		
		VirtualShadowMapSetupContext->PerPageShaderDispatcher.Init(GraphBuilder, Builder);
		VirtualShadowMapSetupContext->PerPageShaderDispatcherDirectionalOnly.Init(GraphBuilder, BuilderDirectionalOnly);
	}

	// Stats
	SET_DWORD_STAT(STAT_VSMSinglePageCount, GetNumSinglePageShadowMaps());
	SET_DWORD_STAT(STAT_VSMFullCount, GetNumFullShadowMaps());
	// And _other_ stats...
	CSV_CUSTOM_STAT(VSM, SinglePageCount, GetNumSinglePageShadowMaps(), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(VSM, FullCount, GetNumFullShadowMaps(), ECsvCustomStatOp::Set);

	UniformParameters.NumFullShadowMaps = GetNumFullShadowMaps();
	UniformParameters.NumSinglePageShadowMaps = GetNumSinglePageShadowMaps();
	UniformParameters.NumShadowMapSlots = GetNumShadowMapSlots();
	UniformParameters.ProjectionData = GraphBuilder.CreateSRV(ProjectionDataRDG);

	UniformParameters.bExcludeNonNaniteFromCoarsePages = !CVarCoarsePagesIncludeNonNanite.GetValueOnRenderThread();
	UniformParameters.CoarsePagePixelThresholdDynamic = CVarCoarsePagePixelThresholdDynamic.GetValueOnRenderThread();
	UniformParameters.CoarsePagePixelThresholdStatic = CVarCoarsePagePixelThresholdStatic.GetValueOnRenderThread();
	UniformParameters.CoarsePagePixelThresholdDynamicNanite = CVarCoarsePagePixelThresholdDynamicNanite.GetValueOnRenderThread();

	bool bCsvLogEnabled = IsCsvLogEnabled();

	// Stats buffer
	{
		StatsBufferRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), VSM_STAT_NUM + MaxPageAreaDiagnosticSlots * 2), TEXT("Shadow.Virtual.StatsBuffer"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(StatsBufferRDG), 0);

		// For the rest of the frame we don't want the stats buffer adding additional barriers that are not otherwise present.
		// Even though this is not a high performance path with stats enabled, we don't want to change behavior.
		StatsBufferUAV = GraphBuilder.CreateUAV(StatsBufferRDG, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}

	// Create and clear the requested page flags

	// For the texture 2D version we pack all single-page SMs into the first page table entry (which is 128x128, or 16k)
	static_assert(VSM_MAX_SINGLE_PAGE_SHADOW_MAPS <= VSM_PAGE_SIZE * VSM_PAGE_SIZE, "All single-page SMs must fit in a single page.");
	// One extra for single page SMs
	const int32 NumEntriesRequired = GetNumFullShadowMaps() + 1;

	// Note: we use the max GetMax2DTextureDimension() / 2 to allow for the 2x page mask table to fit.
	const uint32 NumPageTablesEntriesPerRow = FMath::DivideAndRoundDown(GetMax2DTextureDimension() / 2u, VSM_PAGE_TABLE_TEX2D_SIZE_X);
	check(FMath::IsPowerOfTwo(NumPageTablesEntriesPerRow));
	const uint32 NumPageTableRows = FMath::DivideAndRoundUp((uint32)NumEntriesRequired, NumPageTablesEntriesPerRow);
	FIntPoint PageTableTextureSize(NumPageTablesEntriesPerRow * VSM_PAGE_TABLE_TEX2D_SIZE_X, NumPageTableRows * VSM_PAGE_TABLE_TEX2D_SIZE_Y);
	UniformParameters.PageTableRowShift = uint32(FMath::FloorLog2(NumPageTablesEntriesPerRow));
	UniformParameters.PageTableRowMask = NumPageTablesEntriesPerRow - 1u;
	UniformParameters.PageTableTextureSizeInvSize = FVector4f(FVector2f(PageTableTextureSize), FVector2f(1.0f, 1.0f) / FVector2f(PageTableTextureSize));

	auto AllocatePageTable = [&](bool bAllocateMipLevels, EPixelFormat PixelFormat, const TCHAR* DebugName, uint32 SampleStride = 1u)
	{
		return GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(
				PageTableTextureSize * SampleStride, 
				PixelFormat, 
				FClearValueBinding(0), 
				TexCreate_ShaderResource | TexCreate_UAV | TexCreate_AtomicCompatible,
				// Technically one Hmip too little but the 1x1 level is meaningless as it mixes the info from all the various mip levels. 
				// TODO: Should this be Log2(SampleStride)...? Only used with 2 currently
				bAllocateMipLevels ? (VSM_LOG2_PAGE_SIZE + SampleStride / 2u): 1u), 
			DebugName);
	};
	auto AddClearPageTableUAVPass = [&](FRDGTextureRef Dest, uint32 ClearValue, uint32 SampleStride = 1u, bool bDirectionalOnly = false)
	{
		FRDGTextureUAV* DestUAV = GraphBuilder.CreateUAV(Dest);
		FClearPageTableCS::FParameters* PassParameters = GraphBuilder.AllocParameters< FClearPageTableCS::FParameters >();
		PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
		PassParameters->ClearValue = ClearValue;
		PassParameters->OutDestBuffer = DestUAV;
		PassParameters->SampleStride = SampleStride;

		if (Dest->Desc.NumMips > 1u)
		{
			check(int(Dest->Desc.NumMips) - 1 <= PassParameters->OutDestBufferMips.Num());
			for (uint32 MipLevel = 1u; MipLevel < Dest->Desc.NumMips; ++MipLevel)
			{
				PassParameters->OutDestBufferMips[MipLevel - 1u] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Dest, MipLevel));
			}
		}
		
		FClearPageTableCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FClearPageTableCS::FNumMipLevelsDim>(Dest->Desc.NumMips);
		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FClearPageTableCS>(PermutationVector);

		if (bDirectionalOnly)
		{
			VirtualShadowMapSetupContext->PerPageShaderDispatcherDirectionalOnly.AddPass(GraphBuilder, RDG_EVENT_NAME("ClearPageTableDir(%s Size=%ux%u texels)", DestUAV->GetParent()->Name, DestUAV->GetParent()->Desc.GetSize().X, DestUAV->GetParent()->Desc.GetSize().Y), ComputeShader, PassParameters);
		}
		else
		{
			VirtualShadowMapSetupContext->PerPageShaderDispatcher.AddPass(GraphBuilder, RDG_EVENT_NAME("ClearPageTable(%s Size=%ux%u texels)", DestUAV->GetParent()->Name, DestUAV->GetParent()->Desc.GetSize().X, DestUAV->GetParent()->Desc.GetSize().Y), ComputeShader, PassParameters);
		}
	};
	auto GetOrCreatePageTableDummy = [&]()
	{
		if (PageTableDummyRDG)
		{
		}
		else if (CacheManager->PageTableDummy)
		{
			PageTableDummyRDG = GraphBuilder.RegisterExternalTexture(CacheManager->PageTableDummy);
		}
		else
		{
			PageTableDummyRDG = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(
					FIntPoint(VSM_PAGE_TABLE_TEX2D_SIZE_X, VSM_PAGE_TABLE_TEX2D_SIZE_Y), 
					PF_R32_UINT, 
					FClearValueBinding(0), 
					TexCreate_ShaderResource | TexCreate_UAV | TexCreate_AtomicCompatible,
					VSM_LOG2_PAGE_SIZE + 1u), 
				TEXT("Shadow.Virtual.PageTableDummy"));

			for (uint32 MipLevel = 0u; MipLevel < PageTableDummyRDG->Desc.NumMips; ++MipLevel)
			{
				AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(FRDGTextureUAVDesc(PageTableDummyRDG, MipLevel)), 0u);
			}

			GraphBuilder.QueueTextureExtraction(PageTableDummyRDG, &CacheManager->PageTableDummy);
		}
		return PageTableDummyRDG;
	};



	// TODO: should be uint8 - but atomics prevent this
	PageRequestFlagsRDG = AllocatePageTable(false, PF_R32_UINT, TEXT("Shadow.Virtual.PageRequestFlags"));
	AddClearPageTableUAVPass(PageRequestFlagsRDG, 0u);

	if (IsVirtualShadowMapDirectionalReceiverMaskEnabled() || IsVirtualShadowMapLocalReceiverMaskEnabled())
	{
		// If local light receiver masks are enabled (and present), allocate and clear it as another full page table
		if (IsVirtualShadowMapLocalReceiverMaskEnabled())
		{
			PageReceiverMasksRDG = AllocatePageTable(true, PF_R32_UINT, TEXT("Shadow.Virtual.PageReceiverMasks"), 2u);
			// TODO: If we don't actually have any local lights perhaps worth doing the simpler ClearUAVPass below instead?
			AddClearPageTableUAVPass(PageReceiverMasksRDG, 0u, 2u);
			//AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PageReceiverMasksRDG), FUintVector4(0, 0, 0, 0));
		}
		else
		{
			// Directional goes before local so that if local receiver masks are disabled we don't need to allocate them
			// NOTE: One extra for the single page entries
			int32 NumDirectionalPageReceiverMaskEntriesRequired = NumDirectionalShadowMaps + 1;
			uint32 NumPageReceiverMaskRows = FMath::DivideAndRoundUp((uint32)NumDirectionalPageReceiverMaskEntriesRequired, NumPageTablesEntriesPerRow);

			// Special but common case with a single directional light; we don't need to allocate a full row if we only have one
			uint32 RowEntries = (NumPageReceiverMaskRows == 1) ? NumDirectionalPageReceiverMaskEntriesRequired : NumPageTablesEntriesPerRow;

			// Each page stores 2x2 masks, each representing a 4x4 tile
			FIntPoint PageReceiverMaskTextureSize(2U * RowEntries * VSM_PAGE_TABLE_TEX2D_SIZE_X, 2U * NumPageReceiverMaskRows * VSM_PAGE_TABLE_TEX2D_SIZE_Y);

			// TODO: should be uint16 - but atomics prevent this
			PageReceiverMasksRDG = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(
					PageReceiverMaskTextureSize,
					PF_R32_UINT, 
					FClearValueBinding(0), 
					TexCreate_ShaderResource | TexCreate_UAV | TexCreate_AtomicCompatible,
					// Technically one Hmip too few but the 1x1 level is meaningless as it mixes the info from all the various mip levels. 
					// One extra mip level for the 2x2 tiles
					(VSM_LOG2_PAGE_SIZE + 1)),
				TEXT("Shadow.Virtual.PageReceiverMasks"));
			AddClearPageTableUAVPass(PageReceiverMasksRDG, 0u, 2u, true);
		}

		const FIntPoint TextureSize = PageReceiverMasksRDG->Desc.Extent;
		UniformParameters.PageReceiverMaskTextureSizeInvSize = FVector4f(FVector2f(TextureSize), FVector2f(1.0f, 1.0f) / FVector2f(TextureSize));
	}
	else
	{
		PageReceiverMasksRDG = GetOrCreatePageTableDummy();
		UniformParameters.PageReceiverMaskTextureSizeInvSize = FVector4f(1, 1, 1, 1);
	}

	const uint32 DirtyFlagsPerPageCount = VSM_DIRTY_PAGE_FLAGS_NUM_SLICES;
	// Note: +1 trailing element used as the deferred-invalidation budget atomic in UpdatePhysicalPages. Co-located here to avoid extra barriers / clear passes.
	DirtyPageFlagsRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), GetMaxPhysicalPages() * DirtyFlagsPerPageCount + 1U), TEXT("Shadow.Virtual.DirtyPageFlags"));
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(DirtyPageFlagsRDG), 0);

	// One additional element as the last element is used as an atomic counter
	const uint32 ItemsPerPhysicalPageList = GetMaxPhysicalPages() + 1;
	const uint32 PhysicalPageListsCount = 4;		// See VirtualShadowMapPhysicalPageManagement.usf
	PhysicalPageListsRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), PhysicalPageListsCount * ItemsPerPhysicalPageList), TEXT("Shadow.Virtual.PhysicalPageLists"));

	const uint32 NumPageRects = GetNumShadowMapSlots() * FVirtualShadowMap::MaxMipLevels;
	const uint32 NumPageRectsToAllocate = FMath::RoundUpToPowerOfTwo(NumPageRects);
	UncachedPageRectBoundsRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FIntVector4), NumPageRectsToAllocate), TEXT("Shadow.Virtual.PageRectBounds"));
	AllocatedPageRectBoundsRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FIntVector4), NumPageRectsToAllocate), TEXT("Shadow.Virtual.AllocatedPageRectBounds"));
	const uint32 NumPageRectsToClear = (GetNumFullShadowMaps() + GetNumSinglePageShadowMaps()) * FVirtualShadowMap::MaxMipLevels;
	{
		FInitPageRectBoundsCS::FParameters* PassParameters = GraphBuilder.AllocParameters< FInitPageRectBoundsCS::FParameters >();
		PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
		PassParameters->OutUncachedPageRectBounds = GraphBuilder.CreateUAV(UncachedPageRectBoundsRDG);
		PassParameters->OutAllocatedPageRectBounds = GraphBuilder.CreateUAV(AllocatedPageRectBoundsRDG);
		PassParameters->NumPageRectsToClear = NumPageRectsToClear;
		PassParameters->OutPhysicalPageLists = GraphBuilder.CreateUAV(PhysicalPageListsRDG);

		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FInitPageRectBoundsCS>();
		ClearUnusedGraphResources(ComputeShader, PassParameters);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("InitPageRectBounds"),
			ComputeShader,
			PassParameters,
			FIntVector(FMath::DivideAndRoundUp(NumPageRectsToClear, FInitPageRectBoundsCS::DefaultCSGroupX), 1, 1)
		);
	}

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	bThrottlingEnabled = 
		CVarTimeBudgetMs.GetValueOnRenderThread() > 0 ||
		CVarThrottleLoadBudget.GetValueOnRenderThread() > 0;

	if (bThrottlingEnabled)
	{
		ThrottleBufferRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), VSM_TB_SIZEOF_HEADER + VSM_TB_SIZEOF_ENTRY * GetNumShadowMapSlots()), TEXT("ThrottleBuffer"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(ThrottleBufferRDG), 0);
	}
	else
	{
		ThrottleBufferRDG = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32), 0U);
	}

	FThrottlingParameters ThrottlingParameters {};
	{
		if (GDynamicResolutionVSMNaniteBudget.GetSettings().IsEnabled())
		{
			ThrottlingParameters.DynResThrottleStrength = FMath::Clamp(1.0f - SceneRenderer.DynamicResolutionFractions[GDynamicResolutionVSMNaniteBudget], 0.0f, 1.0f);
		}
		else
		{
			ThrottlingParameters.DynResThrottleStrength = -1.0f;
		}

		float ThrottleLoadBudget = CVarThrottleLoadBudget.GetValueOnRenderThread();
		ThrottleLoadBudget = (ThrottleLoadBudget <= 0) ? TNumericLimits<float>::Max() : ThrottleLoadBudget;
		ThrottlingParameters.ThrottleLoadBudget = ThrottleLoadBudget;

		// Memory pressure lod bias is applied first. How much lod bias do we have left for compute-time pressure?
		const float GlobalMaxBias = CVarVSMDynamicResolutionMaxLodBias.GetValueOnRenderThread();
		const float DirectionalMaxBias = CVarVSMDynamicResolutionMaxLodBiasDirectional.GetValueOnRenderThread();
		const float LocalMaxBias = CVarVSMDynamicResolutionMaxLodBiasLocal.GetValueOnRenderThread();
		
		float MaxLodBiasDirectional = FMath::Max(0, FMath::Min(GlobalMaxBias, DirectionalMaxBias) 	- CacheManager->GetGlobalResolutionLodBias());
		float MaxLodBiasLocal 		= FMath::Max(0, FMath::Min(GlobalMaxBias, LocalMaxBias) 		- CacheManager->GetGlobalResolutionLodBias());
		
		ThrottlingParameters.ThrottleMaxBiasDirectional = MaxLodBiasDirectional;
		ThrottlingParameters.ThrottleMaxBiasLocal = MaxLodBiasLocal;

		ThrottlingParameters.ThrottleEMAHistoryWeight = CVarThrottleEMAHistoryWeight.GetValueOnRenderThread();
	}

	// Is used during logging
	{
		if (CacheManager->GetPrevBuffers().ProjectionData)
		{
			PrevProjectionDataRDG = GraphBuilder.RegisterExternalBuffer(CacheManager->GetPrevBuffers().ProjectionData);
		}
		else
		{
			PrevProjectionDataRDG = GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, sizeof(FVirtualShadowMapProjectionShaderData));
		}
	}

	if (bThrottlingEnabled && CacheManager->GetPrevBuffers().NanitePerformanceFeedback)
	{
		FProcessPrevFramePerfDataCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FProcessPrevFramePerfDataCS::FParameters>();
		
		check(CacheManager->GetPrevBuffers().ThrottleBuffer);
		FRDGBufferRef PrevThrottleBuffer = GraphBuilder.RegisterExternalBuffer(CacheManager->GetPrevBuffers().ThrottleBuffer);
		PassParameters->PrevThrottleBuffer = GraphBuilder.CreateSRV(PrevThrottleBuffer);

		FRDGBufferRef PrevNanitePerformanceFeedback = GraphBuilder.RegisterExternalBuffer(CacheManager->GetPrevBuffers().NanitePerformanceFeedback);
		PassParameters->PrevNanitePerformanceFeedback = GraphBuilder.CreateSRV(PrevNanitePerformanceFeedback);

		FRDGBufferRef NextVirtualShadowMapData = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.NextVirtualShadowMapData"), NextData);
		PassParameters->NextVirtualShadowMapData = GraphBuilder.CreateSRV(NextVirtualShadowMapData);
		PassParameters->NextVirtualShadowMapDataCount = NextData.Num();

		PassParameters->ThrottlingParameters = ThrottlingParameters;

		PassParameters->OutThrottleBuffer = GraphBuilder.CreateUAV(ThrottleBufferRDG);

		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FProcessPrevFramePerfDataCS>();
		ClearUnusedGraphResources(ComputeShader, PassParameters);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ProcessPrevFramePerfData"),
			ComputeShader,
			PassParameters,
			FIntVector(FMath::Max(1, FMath::DivideAndRoundUp(NextData.Num(), FProcessPrevFramePerfDataCS::NumThreadsPerGroup)), 1, 1)
		);
	}

	if (bThrottlingEnabled && CacheManager->GetPrevBuffers().ThrottleBuffer)
	{
		FUpdateThrottleParametersCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FUpdateThrottleParametersCS::FParameters>();
		
		FRDGBufferRef PrevThrottleBuffer = GraphBuilder.RegisterExternalBuffer(CacheManager->GetPrevBuffers().ThrottleBuffer);
		PassParameters->PrevThrottleBuffer = GraphBuilder.CreateSRV(PrevThrottleBuffer);

		PassParameters->ThrottlingParameters = ThrottlingParameters;

		PassParameters->NumThrottleBufferEntries = GetNumShadowMapSlots();
		PassParameters->OutThrottleBuffer = GraphBuilder.CreateUAV(ThrottleBufferRDG);

		PassParameters->InOutProjectionData = GraphBuilder.CreateUAV(ProjectionDataRDG);

		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FUpdateThrottleParametersCS>();
		ClearUnusedGraphResources(ComputeShader, PassParameters);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("UpdateThrottleParameters"),
			ComputeShader,
			PassParameters,
			FIntVector(FMath::DivideAndRoundUp(GetNumShadowMapSlots(), FUpdateThrottleParametersCS::NumThreadsPerGroup), 1, 1)
		);
	}

	uint32 NanitePerformanceFeedbackSize = VSM_NPF_SIZEOF_HEADER + VSM_NPF_SIZEOF_ENTRY * uint32(GetNumShadowMapSlots());
	NanitePerformanceFeedbackRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NanitePerformanceFeedbackSize), TEXT("NanitePerformanceFeedback"));
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(NanitePerformanceFeedbackRDG), 0);

	PageTableRDG = AllocatePageTable(false, PF_R32_UINT, TEXT("Shadow.Virtual.PageTable"));		
	// TODO: should be uint8 - but atomics prevent this
	PageFlagsRDG = AllocatePageTable(true, PF_R32_UINT, TEXT("Shadow.Virtual.PageFlags"));

	AddClearPageTableUAVPass(PageTableRDG, 0u);
	AddClearPageTableUAVPass(PageFlagsRDG, 0u);

	// Prune light grid to remove lights without VSMs
	// Light grids are per-view, so this is a per-view operation that will update the "PerViewData"
	// and then generate per-view cached uniform buffers.
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);
		const FViewInfo &View = Views[ViewIndex];
		FVirtualShadowMapPerViewParameters& PerViewData = PerViewParameters[ViewIndex];

		// Gather directional light virtual shadow maps
		TArray<int32, SceneRenderingAllocator> DirectionalLightIds = ShadowSceneRenderer.GatherClipmapIds(ViewIndex);

		// This view contained no local lights (that were stored in the light grid), and no directional lights, so nothing to do.
		if (View.ForwardLightingResources.ForwardLightUniformParameters->NumLocalLights + DirectionalLightIds.Num() == 0)
		{
			PerViewData = MakeEmptyVirtualShadowMapPerViewParameters(GraphBuilder);
			continue;
		}

		FRDGBufferRef DirectionalLightIdsRDG = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.DirectionalLightIds"), DirectionalLightIds);
		PerViewData.DirectionalLightIds = GraphBuilder.CreateSRV(DirectionalLightIdsRDG);
		PerViewData.DirectionalLightIdCount = uint32(DirectionalLightIds.Num());

		// Prune light grid to remove lights without VSMs
		{
			const bool bLightGridUses16BitBuffers = LightGridUses16BitBuffers(View.GetShaderPlatform());
			const FRDGBufferSRVRef CulledLightDataGrid = bLightGridUses16BitBuffers ? View.ForwardLightingResources.ForwardLightUniformParameters->CulledLightDataGrid16Bit : View.ForwardLightingResources.ForwardLightUniformParameters->CulledLightDataGrid32Bit;
			const FRDGBufferDesc PrunedLightGridDataDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), CulledLightDataGrid->Desc.Buffer->Desc.NumElements);
			FRDGBufferRef PrunedLightGridDataRDG = GraphBuilder.CreateBuffer(PrunedLightGridDataDesc, TEXT("Shadow.Virtual.LightGridData"));
				
			uint32 NumLightGridCells = View.ForwardLightingResources.ForwardLightUniformParameters->NumGridCells;
			if (View.bIsSinglePassStereo)
			{
				// NumCulledLightsGrid holds info like so: [view 0 lights] [view 0 reflections] [view 1 lights] [view 1 reflections].
				// We don't care about reflections here, but we need [view 0 lights] and [view 1 lights] to be at the same offsets
				// in the pruned buffer as they are in the original, so [view 0 reflections] is included but left blank. [view 1 reflections] is omitted entirely.
				NumLightGridCells += View.ForwardLightingResources.ForwardLightUniformParameters->CulledBufferOffsetISR;
			}
			const FRDGBufferDesc PrunedNumCulledLightsGridDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NumLightGridCells);
			FRDGBufferRef PrunedNumCulledLightsGridRDG = GraphBuilder.CreateBuffer(PrunedNumCulledLightsGridDesc, TEXT("Shadow.Virtual.NumCulledLightsGrid"));

			{
				// TODO: Make this a more dynamic bound rather than just this special case
				uint32 MinLocalLightIndex = 0;
				uint32 MaxLocalLightIndex = View.ForwardLightingResources.ForwardLightUniformParameters->NumLocalLights;
				if (!bAnyLocalLightsWithVSMs)
				{
					MaxLocalLightIndex = 0;
				}

				FPruneLightGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters< FPruneLightGridCS::FParameters >();
				PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
				PassParameters->View = View.ViewUniformBuffer;
				PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
				PassParameters->MinLocalLightIndex = MinLocalLightIndex;
				PassParameters->MaxLocalLightIndex = MaxLocalLightIndex;
				// Include megalights if they are *not* being marked directly by that system
				PassParameters->bIncludeMegaLights = MegaLights::IsMarkingVSMPages() ? 0 : 1;
				PassParameters->OutPrunedLightGridData = GraphBuilder.CreateUAV(PrunedLightGridDataRDG);
				PassParameters->OutPrunedNumCulledLightsGrid = GraphBuilder.CreateUAV(PrunedNumCulledLightsGridRDG);
				auto ComputeShader = View.ShaderMap->GetShader<FPruneLightGridCS>();

				FComputeShaderUtils::AddPass(GraphBuilder,
					RDG_EVENT_NAME("PruneLightGrid(Min=%d,Max=%d)", MinLocalLightIndex, MaxLocalLightIndex),
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(NumLightGridCells, FPruneLightGridCS::DefaultCSGroupX));
			};

			PerViewData.LightGridData = GraphBuilder.CreateSRV(PrunedLightGridDataRDG);
			PerViewData.NumCulledLightsGrid = GraphBuilder.CreateSRV(PrunedNumCulledLightsGridRDG);
			PerViewData.MaxLightGridEntryIndex = NumLightGridCells - 1u;
		}
	}

	// Update cached uniform buffers for upcoming marking passes (see GetMarkingParameters)
	// This will also generate the per-view versions. Past this point we should use GetUniformBuffer
	// with a ViewIndex for any passes that need per-view data (i.e. pruned light grid)
	UpdateCachedUniformBuffers(GraphBuilder);
	

	// Mark coarse pages (view-independent)
	// NOTE: Must do this *first*. In the case where bIncludeNonNaniteGeometry is false we need to ensure that the request
	// can be over-written by any pixel pages that *do* want Non-Nanite geometry. We avoid writing with atomics since that
	// is much slower.
	// Because of this we also cannot overlap this pass with the following ones.	
	// Note: always run this pass such that the distant lights may be marked if need be
	{
		FMarkCoarsePagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters< FMarkCoarsePagesCS::FParameters >();
		PassParameters->MarkingParameters = GetMarkingParameters(GraphBuilder, 0);
		PassParameters->bMarkCoarsePagesLocal = CVarMarkCoarsePagesLocal.GetValueOnRenderThread() != 0 ? 1 : 0;
		PassParameters->bIncludeNonNaniteGeometry = CVarCoarsePagesIncludeNonNanite.GetValueOnRenderThread();

		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FMarkCoarsePagesCS>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("MarkCoarsePages"),
			ComputeShader,
			PassParameters,
			FIntVector(FMath::DivideAndRoundUp(uint32(GetNumShadowMaps()), FMarkCoarsePagesCS::DefaultCSGroupX), 1, 1)
		);
	}

	if (CVarMarkPixelPages.GetValueOnRenderThread() != 0)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);
			const FViewInfo &View = Views[ViewIndex];
					
			// It's currently safe to overlap these passes that all write to same page request flags
			// TODO: Extend this to include the UAVs that get put into the marking parameters
			//auto PageRequestFlagsUAV = GraphBuilder.CreateUAV(PageRequestFlagsRDG, ERDGUnorderedAccessViewFlags::SkipBarrier);
			//auto PageReceiverMasksUAV = GraphBuilder.CreateUAV(PageReceiverMasksRDG, ERDGUnorderedAccessViewFlags::SkipBarrier);

			FBaseGeneratePageFlagsParameters BaseParameters;
			BaseParameters.MarkingParameters = GetMarkingParameters(GraphBuilder, ViewIndex);
			BaseParameters.SceneTexturesStruct = SceneTextures.UniformBuffer;
			BaseParameters.View = View.ViewUniformBuffer;
			BaseParameters.ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
			BaseParameters.PageDilationBorderSizeLocal = CVarPageDilationBorderSizeLocal.GetValueOnRenderThread();
			BaseParameters.PageDilationBorderSizeDirectional = CVarPageDilationBorderSizeDirectional.GetValueOnRenderThread();
			BaseParameters.bCullBackfacingPixels = ShouldCullBackfacingPixels() ? 1 : 0;
			BaseParameters.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
			BaseParameters.MipModeLocal = CVarMarkPixelPagesMipModeLocal.GetValueOnRenderThread();
			BaseParameters.FirstPersonPixelRequestBias = CVarFirstPersonPixelRequestBias.GetValueOnRenderThread();
			BaseParameters.FirstPersonPixelRequestLevelClamp = CVarFirstPersonPixelRequestLevelClamp.GetValueOnRenderThread();

			DiaphragmDOF::FPhysicalCocModel CocModel;
			CocModel.Compile(View);
			DiaphragmDOF::SetCocModelParameters(GraphBuilder, &BaseParameters.CocModel, CocModel, View.ViewRect.Size().X);
			BaseParameters.DOFBiasStrength = FMath::Max(0.0f, DiaphragmDOF::IsEnabled(View) ? CVarMaxDOFResolutionBias.GetValueOnRenderThread() : 0.0f);

			auto GeneratePageFlags = [&](const EVirtualShadowMapProjectionInputType InputType)
			{
				FGeneratePageFlagsFromPixelsCS::FParameters* PassParameters = GraphBuilder.AllocParameters< FGeneratePageFlagsFromPixelsCS::FParameters >();
				PassParameters->Base = BaseParameters;

				const FIntPoint PixelStride(
					FMath::Clamp(CVarVirtualShadowMapPageMarkingPixelStrideX.GetValueOnRenderThread(), 1, 128),
					FMath::Clamp(CVarVirtualShadowMapPageMarkingPixelStrideY.GetValueOnRenderThread(), 1, 128));
					
				// If Lumen has valid front layer history data use it, otherwise use same frame front layer depth
				bool bFrontLayerEnabled = false;
				if (IsVSMTranslucentHighQualityEnabled())
				{
					if (FrontLayerTranslucencyData.IsValid())
					{
						PassParameters->FrontLayerMode = 0;
						PassParameters->FrontLayerTranslucencyDepthTexture = FrontLayerTranslucencyData.SceneDepth;
						PassParameters->FrontLayerTranslucencyNormalTexture = FrontLayerTranslucencyData.GBufferA;
						bFrontLayerEnabled = true;
					}
					else if (IsFrontLayerHistoryValid(View))
					{
						const FStochasticLightingViewState& State = View.ViewState->StochasticLighting;
						const FIntPoint HistoryResolution = State.FrontLayerTranslucencyDepthHistory->GetDesc().Extent;
						const FVector2f InvBufferSize(1.0f / SceneTextures.Config.Extent.X, 1.0f / SceneTextures.Config.Extent.Y);
						PassParameters->FrontLayerMode = 1;
						PassParameters->FrontLayerHistoryUVMinMax = FVector4f(
							(State.HistoryViewRect.Min.X + 0.5f) * InvBufferSize.X,
							(State.HistoryViewRect.Min.Y + 0.5f) * InvBufferSize.Y,
							(State.HistoryViewRect.Max.X - 0.5f) * InvBufferSize.X,
							(State.HistoryViewRect.Max.Y - 0.5f) * InvBufferSize.Y);
						PassParameters->FrontLayerHistoryScreenPositionScaleBias = State.HistoryScreenPositionScaleBias;
						PassParameters->FrontLayerHistoryBufferSizeAndInvSize = FVector4f(HistoryResolution.X, HistoryResolution.Y, 1.f/HistoryResolution.X, 1.f/HistoryResolution.Y);
						PassParameters->FrontLayerTranslucencyDepthTexture = GraphBuilder.RegisterExternalTexture(State.FrontLayerTranslucencyDepthHistory, TEXT("VSM.FrontLayerHistoryDepth"));
						PassParameters->FrontLayerTranslucencyNormalTexture = GraphBuilder.RegisterExternalTexture(State.FrontLayerTranslucencyNormalHistory, TEXT("VSM.FrontLayerHistoryNormal"));
						bFrontLayerEnabled = true;
					}
				}

				PassParameters->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
				bool bWaterEnabled = false;
				if (SingleLayerWaterPrePassResult && InputType == EVirtualShadowMapProjectionInputType::GBuffer)
				{
					PassParameters->SingleLayerWaterDepthTexture = SingleLayerWaterPrePassResult->DepthPrepassTexture.Resolve;
					PassParameters->SingleLayerWaterTileMask = GraphBuilder.CreateSRV(SingleLayerWaterPrePassResult->ViewTileClassification[ViewIndex].TileMaskBuffer != nullptr ?
						SingleLayerWaterPrePassResult->ViewTileClassification[ViewIndex].TileMaskBuffer :
						GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32), 0xFFFFFFFF));
					PassParameters->SingleLayerWaterTileViewRes = SingleLayerWaterPrePassResult->ViewTileClassification[ViewIndex].TiledViewRes;
					bWaterEnabled = true;
				}
				PassParameters->PixelStride = PixelStride;
					
				const FIntPoint StridedPixelSize = FIntPoint::DivideAndRoundUp(View.ViewRect.Size(), PixelStride);
				// Note: we use the tile size defined by the water as the group-size - this is needed because the tile mask testing code relies on the size being the same to scalarize efficiently.
				const FIntPoint GridSize = FIntPoint::DivideAndRoundUp(StridedPixelSize, SLW_TILE_SIZE_XY);

				if (InputType == EVirtualShadowMapProjectionInputType::HairStrands)
				{
					FGeneratePageFlagsFromPixelsCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FGeneratePageFlagsFromPixelsCS::FInputType>(static_cast<uint32>(InputType));
					PermutationVector.Set<FGeneratePageFlagsFromPixelsCS::FThrottling>(bThrottlingEnabled);
					auto ComputeShader = View.ShaderMap->GetShader<FGeneratePageFlagsFromPixelsCS>(PermutationVector);

					check(View.HairStrandsViewData.VisibilityData.TileData.IsValid());
					PassParameters->IndirectBufferArgs = View.HairStrandsViewData.VisibilityData.TileData.TileIndirectDispatchBuffer;
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("GeneratePageFlagsFromPixels(HairStrands,Tile)"),
						ComputeShader,
						PassParameters,
						View.HairStrandsViewData.VisibilityData.TileData.TileIndirectDispatchBuffer,
						View.HairStrandsViewData.VisibilityData.TileData.GetIndirectDispatchArgOffset(FHairStrandsTiles::ETileType::HairAll));
				}
				else
				{						
					FGeneratePageFlagsFromPixelsCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FGeneratePageFlagsFromPixelsCS::FInputType>(static_cast<uint32>(InputType));
					PermutationVector.Set<FGeneratePageFlagsFromPixelsCS::FWaterDepth>(bWaterEnabled);
					PermutationVector.Set<FGeneratePageFlagsFromPixelsCS::FTranslucencyDepth>(bFrontLayerEnabled);
					PermutationVector.Set<FGeneratePageFlagsFromPixelsCS::FThrottling>(bThrottlingEnabled);
					auto ComputeShader = View.ShaderMap->GetShader<FGeneratePageFlagsFromPixelsCS>(PermutationVector);
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("GeneratePageFlagsFromPixels(%s,%s%sNumShadowMaps=%d,{%d,%d})", ToString(InputType), (bWaterEnabled ? TEXT("Water,") : TEXT("")), (bFrontLayerEnabled ? TEXT("FrontLayer,") : TEXT("")), GetNumFullShadowMaps(), GridSize.X, GridSize.Y),
						ComputeShader,
						PassParameters,
						FIntVector(GridSize.X, GridSize.Y, 1));
				}
			};

			if (FroxelRenderer.IsEnabled() && CVarMarkPagesUseFroxels.GetValueOnRenderThread() != 0)
			{
				auto AddPass_FroxelBuild = [&](const Froxel::FViewData* ViewFroxelData, int32 PassId, bool bShouldMarkLocaLights)
				{
					if (ViewFroxelData == nullptr)
					{
						return;
					}

					FGeneratePageFlagsFromFroxelsCS::FParameters* PassParameters = GraphBuilder.AllocParameters< FGeneratePageFlagsFromFroxelsCS::FParameters >();
					PassParameters->Base = BaseParameters;
					PassParameters->PassId = PassId;
					PassParameters->bShouldMarkLocaLights = bShouldMarkLocaLights ? 1u : 0u;
					PassParameters->DebugRange = CVarDebugDrawFroxelRange.GetValueOnRenderThread();

					PassParameters->FroxelParameters = ViewFroxelData->GetShaderParameters(GraphBuilder);
					PassParameters->IndirectBufferArgs = ViewFroxelData->FroxelArgsRDG;

					FGeneratePageFlagsFromFroxelsCS::FPermutationDomain PermutationVector;
					bool bDebugRender = CVarDebugDrawFroxels.GetValueOnRenderThread() < 0 || CVarDebugDrawFroxels.GetValueOnRenderThread() == (PassId + 1);
					if (bDebugRender)
					{
						PermutationVector.Set<FGeneratePageFlagsFromFroxelsCS::FDebugRenderDim>(true);
						ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintStruct);
					}
					PermutationVector.Set<FGeneratePageFlagsFromFroxelsCS::FThrottling>(bThrottlingEnabled);

					auto ComputeShader = View.ShaderMap->GetShader<FGeneratePageFlagsFromFroxelsCS>(PermutationVector);
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("GeneratePageFlagsFromFroxels(NumShadowMaps=%d)", GetNumFullShadowMaps()),
						ComputeShader,
						PassParameters,
						ViewFroxelData->FroxelArgsRDG,
						ViewFroxelData->ArgsOffset);
				};

				AddPass_FroxelBuild(FroxelRenderer.GetView(ViewIndex), 0, true);
				if (SingleLayerWaterPrePassResult && SingleLayerWaterPrePassResult->Froxels.GetView(ViewIndex))
				{
					// Not marking local lights since SLW does not support sampling shadow for these.
					AddPass_FroxelBuild(SingleLayerWaterPrePassResult->Froxels.GetView(ViewIndex), 1, false);
				}
			}
			else
			{
				GeneratePageFlags(EVirtualShadowMapProjectionInputType::GBuffer);
			}

			if (HairStrands::HasViewHairStrandsData(View))
			{
				GeneratePageFlags(EVirtualShadowMapProjectionInputType::HairStrands);
			}
		}
	}
}

void FVirtualShadowMapArray::BuildPageAllocations(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const TConstArrayView<FViewInfo> &Views)
{
	check(IsEnabled());

	// TODO: Base this on instead if the other pass ran/early'd-out
	if (GetNumShadowMaps() == 0)
	{
		// Nothing to do
		return;
	}
	// BeginMarkPages was not run, this indicates a systemic error
	if (!ensure(VirtualShadowMapSetupContext))
	{
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "FVirtualShadowMapArray::BuildPageAllocation");
	SCOPED_NAMED_EVENT(FVirtualShadowMapArray_BuildPageAllocation, FColor::Emerald);

#if WITH_MGPU
	bool bCopyOtherGPUsPhysicalPageLists = false;
#endif

	// Propagate the page requests to secondary (filterable) requests
	if (IsPrefilteredDistantEnabled() && GetNumFullShadowMaps() > 0)
	{
		FPropagateFilterableRequestsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPropagateFilterableRequestsCS::FParameters>();
		PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
		PassParameters->InOutPageRequestFlags	= GraphBuilder.CreateUAV(PageRequestFlagsRDG);
		PassParameters->InOutPageReceiverMasks	= GraphBuilder.CreateUAV(PageReceiverMasksRDG);

		auto ComputeShader = Views[0].ShaderMap->GetShader<FPropagateFilterableRequestsCS>();
		VirtualShadowMapSetupContext->PerPageShaderDispatcherDirectionalOnly.AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("PropagateFilterableRequests"),
			ComputeShader,
			PassParameters
		);
	}

	// Update cached or newly invalidated pages with respect to the new requests
	{	
		// Cached data from previous frames is available and valid.  Note that we currently don't support GPUMask varying within
		// a view family, so just use the first view's GPU mask.
		const bool bCacheDataAvailable = CacheManager->IsCacheDataAvailable();
		const bool bCacheDataValid = bCacheDataAvailable && CacheManager->GetCacheValidGPUMask().ContainsAll(Views[0].GPUMask);

		FUpdatePhysicalPages::FParameters* PassParameters = GraphBuilder.AllocParameters<FUpdatePhysicalPages::FParameters>();
		PassParameters->VirtualShadowMap		   = GetUncachedUniformBuffer(GraphBuilder);
		PassParameters->OutPhysicalPageMetaData    = GraphBuilder.CreateUAV(PhysicalPageMetaDataRDG);
		PassParameters->OutPhysicalPageLists	   = GraphBuilder.CreateUAV(PhysicalPageListsRDG);
		PassParameters->DeferredInvalidationBudget = CVarVSMDeferredInvalidationBudget.GetValueOnRenderThread();
		// Note: The last (extra_ element of DirtyPageFlagsRDG is used here as the atomic budget counter
		PassParameters->DirtyPageFlagsInOut        = GraphBuilder.CreateUAV(DirtyPageFlagsRDG);

		if (bCacheDataValid)
		{
			PassParameters->PageRequestFlags		   = PageRequestFlagsRDG;
			PassParameters->OutPageTable			   = GraphBuilder.CreateUAV(PageTableRDG);
			PassParameters->OutPageFlags			   = GraphBuilder.CreateUAV(PageFlagsRDG);
			PassParameters->PrevPhysicalPageLists      = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(CacheManager->GetPrevBuffers().PhysicalPageLists));
			PassParameters->MaxPageAgeSinceLastRequest = GVSMMaxPageAgeSinceLastRequest;
			PassParameters->bDynamicPageInvalidation   = 1;
#if !UE_BUILD_SHIPPING
			PassParameters->bDynamicPageInvalidation   = CVarDebugSkipDynamicPageInvalidation.GetValueOnRenderThread() == 0 ? 1 : 0;
#endif
			PassParameters->bAllocateViaLRU			   = CVarCacheAllocateViaLRU.GetValueOnRenderThread();
		}

		FUpdatePhysicalPages::FPermutationDomain PermutationVector;
		PermutationVector.Set<FUpdatePhysicalPages::FHasCacheDataDim>(bCacheDataValid);
		SetStatsArgsAndPermutation<FUpdatePhysicalPages>(ShouldGenerateStats(), StatsBufferUAV, PassParameters, PermutationVector);
		auto ComputeShader = Views[0].ShaderMap->GetShader<FUpdatePhysicalPages>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("UpdatePhysicalPages"),
			ComputeShader,
			PassParameters,
			FIntVector(FMath::DivideAndRoundUp(GetMaxPhysicalPages(), FUpdatePhysicalPages::DefaultCSGroupX), 1, 1)
		);

#if WITH_MGPU
		// Need to propagate previous physical page lists for GPUs we didn't update in the above pass.  This is only a
		// 64 KB structure, so copying it is cheap.  Ping ponging and independently tracking which is the current buffer
		// separately per GPU is another option, which avoids the copy, but adds a bunch of complexity.
		FRHIGPUMask InverseGPUMask;
		if (CacheManager->GetPrevBuffers().PhysicalPageLists && Views[0].GPUMask.Invert(InverseGPUMask) && InverseGPUMask.Intersects(CacheManager->GetCacheValidGPUMask()))
		{
			bCopyOtherGPUsPhysicalPageLists = true;
		}

		// Track which GPUs the cache has been initialized on.  Merges GPU mask if cache data was already available, otherwise sets mask to initialize it.
		CacheManager->UpdateCacheValidGPUMask(Views[0].GPUMask, bCacheDataAvailable);
#endif
	}

	{
		FPackAvailablePagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPackAvailablePagesCS::FParameters>();
		PassParameters->VirtualShadowMap		= GetUncachedUniformBuffer(GraphBuilder);
		PassParameters->OutPhysicalPageLists	= GraphBuilder.CreateUAV(PhysicalPageListsRDG);						
		auto ComputeShader = Views[0].ShaderMap->GetShader<FPackAvailablePagesCS>();

		// NOTE: We run a single CS group here (see shader)
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("PackAvailablePages"),
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1)
		);
	}

	// Add any newly empty pages to the list of available pages to allocate
	// We add them at the end so that they take priority over any pages with valid cached data
	AppendPhysicalPageList(GraphBuilder, true);

	{
		FAllocateNewPageMappingsCS::FParameters* PassParameters = GraphBuilder.AllocParameters< FAllocateNewPageMappingsCS::FParameters >();
		PassParameters->VirtualShadowMap		= GetUncachedUniformBuffer(GraphBuilder);
		PassParameters->PageRequestFlags		= PageRequestFlagsRDG;
		PassParameters->OutPageTable			= GraphBuilder.CreateUAV(PageTableRDG);
		PassParameters->OutPageFlags			= GraphBuilder.CreateUAV(PageFlagsRDG);
		PassParameters->OutPhysicalPageLists	= GraphBuilder.CreateUAV(PhysicalPageListsRDG);
		PassParameters->OutPhysicalPageMetaData	= GraphBuilder.CreateUAV(PhysicalPageMetaDataRDG);

		FAllocateNewPageMappingsCS::FPermutationDomain PermutationVector;
		SetStatsArgsAndPermutation<FAllocateNewPageMappingsCS>(ShouldGenerateStats(), StatsBufferUAV, PassParameters, PermutationVector);
		auto ComputeShader = Views[0].ShaderMap->GetShader<FAllocateNewPageMappingsCS>(PermutationVector);

		VirtualShadowMapSetupContext->PerPageShaderDispatcher.AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("AllocateNewPageMappings"),
			ComputeShader,
			PassParameters
		);
	}

	{
		// Run pass building hierarchical page flags to make culling acceptable performance.
		FGenerateHierarchicalPageFlagsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGenerateHierarchicalPageFlagsCS::FParameters>();
		PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
		PassParameters->PhysicalPageMetaData = GraphBuilder.CreateSRV(PhysicalPageMetaDataRDG);
		PassParameters->InPageFlags = GraphBuilder.CreateSRV(GRHISupportsTextureViews ? FRDGTextureSRVDesc::CreateForMipLevel(PageFlagsRDG, 0) : FRDGTextureSRVDesc::Create(PageFlagsRDG));
		PassParameters->InPageReceiverMasks = GraphBuilder.CreateSRV(GRHISupportsTextureViews ? FRDGTextureSRVDesc::CreateForMipLevel(PageReceiverMasksRDG, 0) : FRDGTextureSRVDesc::Create(PageReceiverMasksRDG));
		PassParameters->OutUncachedPageRectBounds = GraphBuilder.CreateUAV(UncachedPageRectBoundsRDG);
		PassParameters->OutAllocatedPageRectBounds = GraphBuilder.CreateUAV(AllocatedPageRectBoundsRDG);

		for (uint32 MipLevel = 1u; MipLevel < PageFlagsRDG->Desc.NumMips; ++MipLevel)
		{
			PassParameters->OutPageFlagMips[MipLevel - 1u] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(PageFlagsRDG, MipLevel));
		}
		for (uint32 MipLevel = 1u; MipLevel < PageReceiverMasksRDG->Desc.NumMips; ++MipLevel)
		{
			PassParameters->OutPageReceiverMaskMips[MipLevel - 1u] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(PageReceiverMasksRDG, MipLevel));
		}
		auto ComputeShader = Views[0].ShaderMap->GetShader<FGenerateHierarchicalPageFlagsCS>();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("GenerateHierarchicalPageFlags"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(GetMaxPhysicalPages(), FGenerateHierarchicalPageFlagsCS::DefaultCSGroupX)
		);
	}

	// NOTE: We could skip this (in shader) for shadow maps that only have 1 mip (ex. clipmaps)
	if (GetNumFullShadowMaps() > 0)
	{
		// Propagate mapped mips down the hierarchy to allow O(1) lookup of coarser mapped pages
		FPropagateMappedMipsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPropagateMappedMipsCS::FParameters>();
		PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
		PassParameters->OutPageTable	= GraphBuilder.CreateUAV(PageTableRDG);

		auto ComputeShader = Views[0].ShaderMap->GetShader<FPropagateMappedMipsCS>();
		VirtualShadowMapSetupContext->PerPageShaderDispatcher.AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("PropagateMappedMips"),
			ComputeShader,
			PassParameters
		);
	}

	// Initialize the physical page pool
	FRDGBufferRef InitializePagesIndirectArgsRDG = CreateAndClearIndirectDispatchArgs1D(GraphBuilder, Scene.GetFeatureLevel(), TEXT("Shadow.Virtual.InitializePagesIndirectArgs"));

	check(PhysicalPagePoolRDG != nullptr);
	{
		RDG_EVENT_SCOPE( GraphBuilder, "InitializePhysicalPages" );
		
		// Note: We use GetTotalAllocatedPhysicalPages() to size the buffer as the selection shader emits both static/dynamic pages separately when enabled.
		FRDGBufferRef PhysicalPagesToInitializeRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), GetTotalAllocatedPhysicalPages() + 1), TEXT("Shadow.Virtual.PhysicalPagesToInitialize"));

		// 2. Filter the relevant physical pages and set up the indirect args
		{
			FSelectPagesToInitializeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSelectPagesToInitializeCS::FParameters>();
			PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
			PassParameters->OutPhysicalPageMetaData = GraphBuilder.CreateUAV(PhysicalPageMetaDataRDG);
			PassParameters->OutInitializePagesIndirectArgsBuffer = GraphBuilder.CreateUAV(InitializePagesIndirectArgsRDG);
			PassParameters->OutPhysicalPagesToInitialize = GraphBuilder.CreateUAV(PhysicalPagesToInitializeRDG);
			PassParameters->bUseClearedFlags = CVarShadowsVirtualForceFullPageClears.GetValueOnRenderThread() == 0 ? 1 : 0;
			FSelectPagesToInitializeCS::FPermutationDomain PermutationVector;
			SetStatsArgsAndPermutation<FSelectPagesToInitializeCS>(ShouldGenerateStats(), StatsBufferUAV, PassParameters, PermutationVector);

			auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FSelectPagesToInitializeCS>(PermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SelectPagesToInitialize"),
				ComputeShader,
				PassParameters,
				FIntVector(FMath::DivideAndRoundUp(GetMaxPhysicalPages(), FSelectPagesToInitializeCS::DefaultCSGroupX), 1, 1)
			);

		}
		// 3. Indirect dispatch to clear the selected pages
		{
			FInitializePhysicalPagesIndirectCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInitializePhysicalPagesIndirectCS::FParameters>();
			PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
			PassParameters->PhysicalPageMetaData = GraphBuilder.CreateSRV(PhysicalPageMetaDataRDG);
			PassParameters->OutPhysicalPagePool = GraphBuilder.CreateUAV(PhysicalPagePoolRDG);
			PassParameters->IndirectArgs = InitializePagesIndirectArgsRDG;
			PassParameters->PhysicalPagesToInitialize = GraphBuilder.CreateSRV(PhysicalPagesToInitializeRDG);
			auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FInitializePhysicalPagesIndirectCS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("InitializePhysicalMemoryIndirect"),
				ComputeShader,
				PassParameters,
				PassParameters->IndirectArgs,
				0
			);
		}
	}

	// If present, we always clear the entire third slice of the array as that is used for visualization for the current render
	// TODO: There are potentially interesting cases where we allow the visualization to live along with cached data as well, but
	// for current performance debug purposes this is more directly in line with the cost of that page on a given frame.
	if (PhysicalPagePoolRDG->Desc.ArraySize >= 3)
	{
		// Clear only array slice 2
		FRDGTextureUAVDesc Desc(PhysicalPagePoolRDG, 0 /* MipLevel */, PF_Unknown, 2, 1);
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Desc), 0U);
	}

	UniformParameters.PageTable = PageTableRDG;
	UniformParameters.PageFlags = PageFlagsRDG;
	UniformParameters.PageReceiverMasks = PageReceiverMasksRDG;
	UniformParameters.AllocatedPageRectBounds = GraphBuilder.CreateSRV(AllocatedPageRectBoundsRDG);
	UniformParameters.UncachedPageRectBounds = GraphBuilder.CreateSRV(UncachedPageRectBoundsRDG);

	// Add pass to pipe back important stats
	{
		FVirtualSmFeedbackStatusCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVirtualSmFeedbackStatusCS::FParameters>();
		PassParameters->PhysicalPageLists = GraphBuilder.CreateSRV(PhysicalPageListsRDG);
		PassParameters->GPUMessageParams = GPUMessage::GetShaderParameters(GraphBuilder);
		PassParameters->StatusMessageId = CacheManager->GetStatusFeedbackMessageId();
		PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);

		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FVirtualSmFeedbackStatusCS>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Feedback Status"),
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1)
		);
	}

	// Put any remaining available pages back into the sorted list for next frame
	// NOTE: Must do this *after* feedback status pass
	AppendPhysicalPageList(GraphBuilder, false);

	UpdateCachedUniformBuffers(GraphBuilder);

#if WITH_MGPU
	if (bCopyOtherGPUsPhysicalPageLists)
	{
		//
		// Do this at the end to prevent visual corruption as reported in [UE-295062]
		//
		// The RDG doesn't have any concept of there being a copy of these resources per GPU so when
		// the GPUMask is anything other than All, the access states can diverge on a per-GPU basis.
		// This can then result in incorrect transitions for some of the resource copies. This fix 
		// isn't perfect and does not completely solve the problem, even in this limited case, but does
		// appear to avoid the reported corruption and is light-weight compared to other potential fixes.
		//

		FRHIGPUMask InverseGPUMask;
		Views[0].GPUMask.Invert(InverseGPUMask);
		RDG_GPU_MASK_SCOPE(GraphBuilder, InverseGPUMask);
		AddCopyBufferPass(GraphBuilder, PhysicalPageListsRDG, GraphBuilder.RegisterExternalBuffer(CacheManager->GetPrevBuffers().PhysicalPageLists));
	}
#endif

#if !UE_BUILD_SHIPPING
	// Only dump one frame of light data
	GDumpVSMLightNames = false;
#endif
}

class FDebugVisualizeVirtualSmCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FDebugVisualizeVirtualSmCS);
	SHADER_USE_PARAMETER_STRUCT(FDebugVisualizeVirtualSmCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, ProjectionParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPhysicalPageMetaData>, PhysicalPageMetaData)
		SHADER_PARAMETER(uint32, DebugTargetWidth)
		SHADER_PARAMETER(uint32, DebugTargetHeight)
		SHADER_PARAMETER(uint32, BorderWidth)
		SHADER_PARAMETER(uint32, VisualizeModeId)
		SHADER_PARAMETER(int32, VirtualShadowMapId)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutVisualize)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FDebugVisualizeVirtualSmCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapDebug.usf", "DebugVisualizeVirtualSmCS", SF_Compute);


void FVirtualShadowMapArray::RenderDebugInfo(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views)
{
	check(IsEnabled());
			
	if (Views.Num() > 0)
	{
		LogStats(GraphBuilder, Views[0]);
	}

	if (DebugVisualizationOutput.IsEmpty() || VisualizeLight.IsEmpty())
	{
		return;
	}

	const FVirtualShadowMapVisualizationData& VisualizationData = GetVirtualShadowMapVisualizationData();
	if (VisualizationData.GetActiveModeID() != VIRTUAL_SHADOW_MAP_VISUALIZE_CLIPMAP_VIRTUAL_SPACE)
	{
		return;
	}

	int32 BorderWidth = 2;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (!VisualizeLight[ViewIndex].IsValid())
		{
			continue;
		}

		FViewInfo& View = Views[ViewIndex];

		FIntPoint DebugTargetExtent = DebugVisualizationOutput[ViewIndex]->Desc.Extent;

		FDebugVisualizeVirtualSmCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDebugVisualizeVirtualSmCS::FParameters>();
		PassParameters->ProjectionParameters = GetSamplingParameters(GraphBuilder, ViewIndex);
		PassParameters->PhysicalPageMetaData = GraphBuilder.CreateSRV(PhysicalPageMetaDataRDG);

		PassParameters->DebugTargetWidth = DebugTargetExtent.X;
		PassParameters->DebugTargetHeight = DebugTargetExtent.Y;
		PassParameters->BorderWidth = BorderWidth;
		PassParameters->VisualizeModeId = VisualizationData.GetActiveModeID();
		PassParameters->VirtualShadowMapId = VisualizeLight[ViewIndex].GetVirtualShadowMapId();

		PassParameters->OutVisualize = GraphBuilder.CreateUAV(DebugVisualizationOutput[ViewIndex]);

		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FDebugVisualizeVirtualSmCS>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("DebugVisualizeVirtualShadowMap"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(DebugTargetExtent, FVirtualShadowMapPageManagementShader::DefaultCSGroupXY)
		);
	}
}


class FVirtualSmLogStatsCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FVirtualSmLogStatsCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualSmLogStatsCS, FVirtualShadowMapPageManagementShader)

	class FGenerateStatsDim : SHADER_PERMUTATION_BOOL("VSM_GENERATE_STATS"); 
	using FPermutationDomain = TShaderPermutationDomain<FGenerateStatsDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_STRUCT_INCLUDE(GPUMessage::FParameters, GPUMessageParams)
		SHADER_PARAMETER_STRUCT_INCLUDE( ShaderPrint::FShaderParameters, ShaderPrintStruct )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, InStatsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FNaniteStats>, NaniteStats)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, NanitePerformanceFeedback)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PrevProjectionData)
		SHADER_PARAMETER(uint32, NanitePerformanceFeedbackNumEntries)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ThrottleBuffer)
		SHADER_PARAMETER(uint32, bThrottlingEnabled)
		SHADER_PARAMETER(int, ShowStatsValue)
		SHADER_PARAMETER(uint32, StatusMessageId)
		SHADER_PARAMETER(uint32, StatsMessageId)
	END_SHADER_PARAMETER_STRUCT()


	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FVirtualShadowMapPageManagementShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		// Disable optimizations as shader print causes long compile times
		OutEnvironment.CompilerFlags.Add(CFLAG_SkipOptimizations);
	}
};
IMPLEMENT_GLOBAL_SHADER(FVirtualSmLogStatsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPrintStats.usf", "LogVirtualSmStatsCS", SF_Compute);

void FVirtualShadowMapArray::LogStats(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	check(IsEnabled());
	LLM_SCOPE_BYTAG(Nanite);

	if (!StatsBufferRDG)
	{
		return;
	}

	FVirtualSmLogStatsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVirtualSmLogStatsCS::FParameters>();
	PassParameters->InStatsBuffer = GraphBuilder.CreateSRV(StatsBufferRDG);
	PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
	PassParameters->GPUMessageParams = GPUMessage::GetShaderParameters(GraphBuilder);
	PassParameters->StatusMessageId = CacheManager->GetStatusFeedbackMessageId();

	// If ShouldGenerateStats() is false, the stats buffer will only have data needed for Status messages (e.g. for overflow tracking)
	bool bGenerateStats = ShouldGenerateStats();

	FVirtualSmLogStatsCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVirtualSmLogStatsCS::FGenerateStatsDim>(bGenerateStats);
	
	if (bGenerateStats)
	{
		// Convenience, enable shader print automatically
		ShaderPrint::SetEnabled(true);

		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintStruct);
		ShaderPrint::RequestSpaceForTriangles(8192);

		int ShowStatsValue = (int)FVirtualShadowMap::GetEnabledStatSections();
		PassParameters->ShowStatsValue = ShowStatsValue;

#if !UE_BUILD_SHIPPING
		PassParameters->StatsMessageId = CacheManager->GetStatsFeedbackMessageId();
		bool bBindNaniteStatsBuffer = StatsNaniteBufferRDG != nullptr;
#else
		PassParameters->StatsMessageId = INDEX_NONE;
		constexpr bool bBindNaniteStatsBuffer = false;
#endif

		if (bBindNaniteStatsBuffer)
		{
			PassParameters->NaniteStats = GraphBuilder.CreateSRV(StatsNaniteBufferRDG);
		}
		else
		{
			PassParameters->NaniteStats = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FNaniteStats)));
		}

		PassParameters->PrevProjectionData = GraphBuilder.CreateSRV(PrevProjectionDataRDG);
		
		PassParameters->NanitePerformanceFeedback = GraphBuilder.CreateSRV(NanitePerformanceFeedbackRDG);
		PassParameters->NanitePerformanceFeedbackNumEntries = GetNumShadowMapSlots();
		PassParameters->ThrottleBuffer = GraphBuilder.CreateSRV(ThrottleBufferRDG);
		PassParameters->bThrottlingEnabled = bThrottlingEnabled;
	}
	
	auto ComputeShader = View.ShaderMap->GetShader<FVirtualSmLogStatsCS>(PermutationVector);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("VSM Log Stats And Status"),
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, 1)
	);
}


class FVirtualSmLogPageListStatsCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FVirtualSmLogPageListStatsCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualSmLogPageListStatsCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintStruct)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< int >, OutPhysicalPageLists)
		SHADER_PARAMETER(int, PageListStatsRow)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FVirtualShadowMapPageManagementShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		// Disable optimizations as shader print causes long compile times
		OutEnvironment.CompilerFlags.Add(CFLAG_SkipOptimizations);
	}
};
IMPLEMENT_GLOBAL_SHADER(FVirtualSmLogPageListStatsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "LogPageListStatsCS", SF_Compute);


class FVirtualSmPrintClipmapStatsCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FVirtualSmPrintClipmapStatsCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualSmPrintClipmapStatsCS, FVirtualShadowMapPageManagementShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		//SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintStruct)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FIntVector4 >, AllocatedPageRectBounds)
		SHADER_PARAMETER(uint32, ShadowMapIdRangeStart)
		SHADER_PARAMETER(uint32, ShadowMapIdRangeEnd)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FVirtualSmPrintClipmapStatsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPrintStats.usf", "PrintClipmapStats", SF_Compute);


BEGIN_SHADER_PARAMETER_STRUCT(FVirtualShadowDepthPassParameters,)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FShadowDepthPassUniformParameters, ShadowDepthPass)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPackedView >, InViews)

	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()


class FCullPerPageDrawCommandsCs : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCullPerPageDrawCommandsCs);
	SHADER_USE_PARAMETER_STRUCT(FCullPerPageDrawCommandsCs, FGlobalShader)

	class FUseHzbDim : SHADER_PERMUTATION_BOOL("USE_HZB_OCCLUSION");
	class FGenerateStatsDim : SHADER_PERMUTATION_BOOL("VSM_GENERATE_STATS");
	class FWithShaderPrintDim : SHADER_PERMUTATION_BOOL("WITH_SHADERPRINT");
	class FBatchedDim : SHADER_PERMUTATION_BOOL("ENABLE_BATCH_MODE");
	using FPermutationDomain = TShaderPermutationDomain< FUseHzbDim, FBatchedDim, FGenerateStatsDim, FWithShaderPrintDim >;

	static constexpr uint32 ThreadGroupSize = FInstanceProcessingGPULoadBalancer::ThreadGroupSize;

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	/**
	 */
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		FInstanceProcessingGPULoadBalancer::SetShaderDefines(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("NANITE_MULTI_VIEW"), 1);
		OutEnvironment.SetDefine(TEXT("INDIRECT_ARGS_NUM_WORDS"), FInstanceCullingContext::IndirectArgsNumWords);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);

		DynamicMeshBoundsModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FHZBShaderParameters, )
		SHADER_PARAMETER_RDG_TEXTURE( Texture2D<uint>, HZBPageTable)
		SHADER_PARAMETER_RDG_TEXTURE( Texture2D<uint>, HZBPageFlags)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint4 >, HZBPageRectBounds)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, HZBTextureArray)
		SHADER_PARAMETER_SAMPLER(SamplerState, HZBSampler)
		SHADER_PARAMETER(FVector2f, HZBSize)
	END_SHADER_PARAMETER_STRUCT()



	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDynamicMeshBoundsShaderParameters, DynamicBoundsParameters)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, OutDirtyPageFlags)

		SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceProcessingGPULoadBalancer::FShaderParameters, LoadBalancerParameters)

		SHADER_PARAMETER(int32, FirstPrimaryView)
		SHADER_PARAMETER(int32, NumPrimaryViews)
		SHADER_PARAMETER(uint32, TotalPrimaryViews)
		SHADER_PARAMETER(uint32, VisibleInstancesBufferNum)
		SHADER_PARAMETER(int32, DynamicInstanceIdOffset)
		SHADER_PARAMETER(int32, DynamicInstanceIdMax)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPackedView >, InViews)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FUintVector2 >, DrawCommandDescs)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FContextBatchInfo >, BatchInfos)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FVSMCullingBatchInfo >, VSMCullingBatchInfos)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint32 >, BatchInds)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVSMVisibleInstanceCmd>, VisibleInstancesOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, DrawIndirectArgsBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, VisibleInstanceCountBufferOut)

		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBShaderParameters, HZBShaderParameters)

		SHADER_PARAMETER(uint32, NumPageAreaDiagnosticSlots)
		SHADER_PARAMETER(uint32, LargeInstancePageAreaThreshold)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, OutStatsBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCullPerPageDrawCommandsCs, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapBuildPerPageDrawCommands.usf", "CullPerPageDrawCommandsCs", SF_Compute);



class FAllocateCommandInstanceOutputSpaceCs : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAllocateCommandInstanceOutputSpaceCs);
	SHADER_USE_PARAMETER_STRUCT(FAllocateCommandInstanceOutputSpaceCs, FGlobalShader)
public:
	static constexpr int32 NumThreadsPerGroup = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	/**
	 */
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		FInstanceProcessingGPULoadBalancer::SetShaderDefines(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NumThreadsPerGroup);
		OutEnvironment.SetDefine(TEXT("NANITE_MULTI_VIEW"), 1);
		OutEnvironment.SetDefine(TEXT("INDIRECT_ARGS_NUM_WORDS"), FInstanceCullingContext::IndirectArgsNumWords);
	}


	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumIndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, DrawIndirectArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, InstanceIdOffsetBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutputOffsetBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, TmpInstanceIdOffsetBufferOut)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FAllocateCommandInstanceOutputSpaceCs, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapBuildPerPageDrawCommands.usf", "AllocateCommandInstanceOutputSpaceCs", SF_Compute);


class FOutputCommandInstanceListsCs : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOutputCommandInstanceListsCs);
	SHADER_USE_PARAMETER_STRUCT(FOutputCommandInstanceListsCs, FGlobalShader)
public:
	static constexpr int32 NumThreadsPerGroup = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	/**
	 */
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		FInstanceProcessingGPULoadBalancer::SetShaderDefines(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NumThreadsPerGroup);
		OutEnvironment.SetDefine(TEXT("NANITE_MULTI_VIEW"), 1);
		OutEnvironment.SetDefine(TEXT("INDIRECT_ARGS_NUM_WORDS"), FInstanceCullingContext::IndirectArgsNumWords);
	}


	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FVSMVisibleInstanceCmd >, VisibleInstances)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, InstanceIdsBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, PageInfoBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, TmpInstanceIdOffsetBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, VisibleInstanceCountBuffer)

		// Needed reference for make RDG happy somehow
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FOutputCommandInstanceListsCs, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapBuildPerPageDrawCommands.usf", "OutputCommandInstanceListsCs", SF_Compute);

struct FCullingResult
{
	FRDGBufferRef DrawIndirectArgsRDG;
	FRDGBufferRef InstanceIdOffsetBufferRDG;
	FRDGBufferRef InstanceIdsBuffer;
	FRDGBufferRef PageInfoBuffer;
	uint32 MaxNumInstancesPerPass;
};

template <typename InstanceCullingLoadBalancerType>
static FCullingResult AddCullingPasses(FRDGBuilder& GraphBuilder,
	const TConstArrayView<FRHIDrawIndexedIndirectParameters> &IndirectArgs,
	const TConstArrayView<FUintVector2>& DrawCommandDescs,
	const TConstArrayView<uint32>& InstanceIdOffsets,
	InstanceCullingLoadBalancerType *LoadBalancer,
	const TConstArrayView<FInstanceCullingMergedContext::FContextBatchInfoPacked> BatchInfos,
	const TConstArrayView<FVSMCullingBatchInfo> VSMCullingBatchInfos,
	const TConstArrayView<uint32> BatchInds,
	uint32 TotalInstances,
	uint32 TotalViewScaledInstanceCount,
	uint32 TotalPrimaryViews,
	FRDGBufferRef VirtualShadowViewsRDG,
	const FCullPerPageDrawCommandsCs::FHZBShaderParameters &HZBShaderParameters,
	FVirtualShadowMapArray &VirtualShadowMapArray,
	FSceneUniformBuffer& SceneUniformBuffer,
	ERHIFeatureLevel::Type FeatureLevel)
{
	const bool bUseBatchMode = !BatchInds.IsEmpty();

	int32 NumIndirectArgs = IndirectArgs.Num();

	FRDGBufferRef TmpInstanceIdOffsetBufferRDG = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.TmpInstanceIdOffsetBuffer"), sizeof(uint32), NumIndirectArgs, nullptr, 0);

	FCullingResult CullingResult;
	// TotalViewScaledInstanceCount is conservative since it is the number of instances needed if each instance was drawn into every possible mip-level.
	// This is far more than we'd expect in reasonable circumstances, so we use a scale factor to reduce memory pressure from these passes.
	const uint32 MaxCulledInstanceCount = uint32(CVarNonNaniteMaxCulledInstanceAllocationSize.GetValueOnRenderThread());
	const uint32 ScaledInstanceCount = static_cast<uint32>(static_cast<double>(TotalViewScaledInstanceCount) * CVarNonNaniteCulledInstanceAllocationFactor.GetValueOnRenderThread());
	ensureMsgf(ScaledInstanceCount <= MaxCulledInstanceCount, TEXT("Possible non-nanite VSM Instance culling overflow detected (esitmated required size: %d, if visual artifacts appear either increase the r.Shadow.Virtual.NonNanite.MaxCulledInstanceAllocationSize (%d) or reduce r.Shadow.Virtual.NonNanite.CulledInstanceAllocationFactor (%.2f)"), ScaledInstanceCount, MaxCulledInstanceCount, CVarNonNaniteCulledInstanceAllocationFactor.GetValueOnRenderThread());
	CullingResult.MaxNumInstancesPerPass = FMath::Clamp(ScaledInstanceCount, 1u, MaxCulledInstanceCount);

	FRDGBufferRef VisibleInstancesRdg = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.VisibleInstances"), sizeof(FVSMVisibleInstanceCmd), CullingResult.MaxNumInstancesPerPass, nullptr, 0);

	FRDGBufferRef VisibleInstanceWriteOffsetRDG = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.VisibleInstanceWriteOffset"), sizeof(uint32), 1, nullptr, 0);
	FRDGBufferRef OutputOffsetBufferRDG = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.OutputOffsetBuffer"), sizeof(uint32), 1, nullptr, 0);

	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(VisibleInstanceWriteOffsetRDG), 0);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(OutputOffsetBufferRDG), 0);

	// Create buffer for indirect args and upload draw arg data, also clears the instance to zero
	FRDGBufferDesc IndirectArgsDesc = FRDGBufferDesc::CreateIndirectDesc(FInstanceCullingContext::IndirectArgsNumWords * IndirectArgs.Num());
	IndirectArgsDesc.Usage |= BUF_MultiGPUGraphIgnore;

	CullingResult.DrawIndirectArgsRDG = GraphBuilder.CreateBuffer(IndirectArgsDesc, TEXT("Shadow.Virtual.DrawIndirectArgsBuffer"));
	GraphBuilder.QueueBufferUpload(CullingResult.DrawIndirectArgsRDG, IndirectArgs.GetData(), IndirectArgs.GetTypeSize() * IndirectArgs.Num());

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);

	// Note: we redundantly clear the instance counts here as there is some issue with replays on certain consoles.
	FInstanceCullingContext::AddClearIndirectArgInstanceCountPass(GraphBuilder, ShaderMap, CullingResult.DrawIndirectArgsRDG);

	// not using structured buffer as we have to get at it as a vertex buffer 
	CullingResult.InstanceIdOffsetBufferRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), InstanceIdOffsets.Num()), TEXT("Shadow.Virtual.InstanceIdOffsetBuffer"));

	{
		FCullPerPageDrawCommandsCs::FParameters* PassParameters = GraphBuilder.AllocParameters<FCullPerPageDrawCommandsCs::FParameters>();

		PassParameters->VirtualShadowMap = VirtualShadowMapArray.GetUniformBuffer(0);
		PassParameters->Scene = SceneUniformBuffer.GetBuffer(GraphBuilder);
		ShaderPrint::SetParameters(GraphBuilder, PassParameters->ShaderPrintStruct);

		PassParameters->OutDirtyPageFlags = GraphBuilder.CreateUAV(VirtualShadowMapArray.DirtyPageFlagsRDG, ERDGUnorderedAccessViewFlags::SkipBarrier);
		PassParameters->DynamicInstanceIdOffset = BatchInfos[0].DynamicInstanceIdOffset;
		PassParameters->DynamicInstanceIdMax = BatchInfos[0].DynamicInstanceIdMax;

		auto GPUData = LoadBalancer->Upload(GraphBuilder);
		GPUData.GetShaderParameters(GraphBuilder, PassParameters->LoadBalancerParameters);

		PassParameters->FirstPrimaryView = VSMCullingBatchInfos[0].FirstPrimaryView;
		PassParameters->NumPrimaryViews = VSMCullingBatchInfos[0].NumPrimaryViews;

		PassParameters->TotalPrimaryViews = TotalPrimaryViews;
		PassParameters->VisibleInstancesBufferNum = CullingResult.MaxNumInstancesPerPass;
		PassParameters->InViews = GraphBuilder.CreateSRV(VirtualShadowViewsRDG);
		PassParameters->DrawCommandDescs = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.DrawCommandDescs"), DrawCommandDescs));
		PassParameters->DynamicBoundsParameters = GetDynamicMeshBoundsShaderParameters(GraphBuilder);
		if (bUseBatchMode)
		{
			PassParameters->BatchInfos = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.BatchInfos"), BatchInfos));
			PassParameters->VSMCullingBatchInfos = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.VSMCullingBatchInfos"), VSMCullingBatchInfos));
			PassParameters->BatchInds = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.BatchInds"), BatchInds));
		}

		PassParameters->DrawIndirectArgsBufferOut = GraphBuilder.CreateUAV(CullingResult.DrawIndirectArgsRDG, PF_R32_UINT, ERDGUnorderedAccessViewFlags::SkipBarrier);

		PassParameters->VisibleInstancesOut = GraphBuilder.CreateUAV(VisibleInstancesRdg, ERDGUnorderedAccessViewFlags::SkipBarrier);
		PassParameters->VisibleInstanceCountBufferOut = GraphBuilder.CreateUAV(VisibleInstanceWriteOffsetRDG, ERDGUnorderedAccessViewFlags::SkipBarrier);

		PassParameters->NumPageAreaDiagnosticSlots = 0U;

		PassParameters->HZBShaderParameters = HZBShaderParameters;

		FCullPerPageDrawCommandsCs::FPermutationDomain PermutationVector;

		bool bGenerateStats = SetStatsArgsAndPermutation<FCullPerPageDrawCommandsCs>(
			VirtualShadowMapArray.ShouldGenerateStats(), 
			VirtualShadowMapArray.StatsBufferUAV, 
			PassParameters, 
			PermutationVector);

		bool bVisualizeDynamicMeshBounds = false;
#if !UE_BUILD_SHIPPING
		if (bGenerateStats)
		{
			PassParameters->NumPageAreaDiagnosticSlots = CVarNumPageAreaDiagSlots.GetValueOnRenderThread() < 0 ? FVirtualShadowMapArray::MaxPageAreaDiagnosticSlots : FMath::Min(FVirtualShadowMapArray::MaxPageAreaDiagnosticSlots, uint32(CVarNumPageAreaDiagSlots.GetValueOnRenderThread()));
			PassParameters->LargeInstancePageAreaThreshold = CVarLargeInstancePageAreaThreshold.GetValueOnRenderThread() >= 0 ? CVarLargeInstancePageAreaThreshold.GetValueOnRenderThread() : (VirtualShadowMapArray.GetMaxPhysicalPages() / 8);
		}
#endif
#if VSM_ENABLE_VISUALIZATION
		bVisualizeDynamicMeshBounds = CVarVisualizeDynamicMeshBounds.GetValueOnRenderThread() > 0;
#endif

		PermutationVector.Set< FCullPerPageDrawCommandsCs::FBatchedDim >(bUseBatchMode);
		PermutationVector.Set< FCullPerPageDrawCommandsCs::FUseHzbDim >(HZBShaderParameters.HZBTextureArray != nullptr);
		PermutationVector.Set< FCullPerPageDrawCommandsCs::FWithShaderPrintDim >(bVisualizeDynamicMeshBounds);

		auto ComputeShader = ShaderMap->GetShader<FCullPerPageDrawCommandsCs>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("CullPerPageDrawCommands"),
			ComputeShader,
			PassParameters,
			LoadBalancer->GetWrappedCsGroupCount()
		);
	}
	// 2.2.Allocate space for the final instance ID output and so on.
	{
		FAllocateCommandInstanceOutputSpaceCs::FParameters* PassParameters = GraphBuilder.AllocParameters<FAllocateCommandInstanceOutputSpaceCs::FParameters>();

		FRDGBufferRef InstanceIdOutOffsetBufferRDG = CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCulling.OutputOffsetBufferOut"), sizeof(uint32), 1, nullptr, 0);
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(InstanceIdOutOffsetBufferRDG), 0);

		PassParameters->NumIndirectArgs = NumIndirectArgs;
		PassParameters->InstanceIdOffsetBufferOut = GraphBuilder.CreateUAV(CullingResult.InstanceIdOffsetBufferRDG, PF_R32_UINT);
		PassParameters->OutputOffsetBufferOut = GraphBuilder.CreateUAV(InstanceIdOutOffsetBufferRDG);
		PassParameters->TmpInstanceIdOffsetBufferOut = GraphBuilder.CreateUAV(TmpInstanceIdOffsetBufferRDG);
		PassParameters->DrawIndirectArgsBuffer = GraphBuilder.CreateSRV(CullingResult.DrawIndirectArgsRDG, PF_R32_UINT);

		auto ComputeShader = ShaderMap->GetShader<FAllocateCommandInstanceOutputSpaceCs>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("AllocateCommandInstanceOutputSpaceCs"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(NumIndirectArgs, FAllocateCommandInstanceOutputSpaceCs::NumThreadsPerGroup)
		);
	}
	// 2.3. Perform final pass to re-shuffle the instance ID's to their final resting places
	CullingResult.InstanceIdsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), CullingResult.MaxNumInstancesPerPass), TEXT("Shadow.Virtual.InstanceIdsBuffer"));
	CullingResult.PageInfoBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), CullingResult.MaxNumInstancesPerPass), TEXT("Shadow.Virtual.PageInfoBuffer"));

	FRDGBufferRef OutputPassIndirectArgs = FComputeShaderUtils::AddIndirectArgsSetupCsPass1D(GraphBuilder, FeatureLevel, VisibleInstanceWriteOffsetRDG, TEXT("Shadow.Virtual.IndirectArgs"), FOutputCommandInstanceListsCs::NumThreadsPerGroup);
	{

		FOutputCommandInstanceListsCs::FParameters* PassParameters = GraphBuilder.AllocParameters<FOutputCommandInstanceListsCs::FParameters>();

		PassParameters->VisibleInstances = GraphBuilder.CreateSRV(VisibleInstancesRdg);
		PassParameters->PageInfoBufferOut = GraphBuilder.CreateUAV(CullingResult.PageInfoBuffer);
		PassParameters->InstanceIdsBufferOut = GraphBuilder.CreateUAV(CullingResult.InstanceIdsBuffer);
		PassParameters->TmpInstanceIdOffsetBufferOut = GraphBuilder.CreateUAV(TmpInstanceIdOffsetBufferRDG);
		PassParameters->VisibleInstanceCountBuffer = GraphBuilder.CreateSRV(VisibleInstanceWriteOffsetRDG);
		PassParameters->IndirectArgs = OutputPassIndirectArgs;

		auto ComputeShader = ShaderMap->GetShader<FOutputCommandInstanceListsCs>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("OutputCommandInstanceListsCs"),
			ComputeShader,
			PassParameters,
			OutputPassIndirectArgs,
			0
		);
	}

	return CullingResult;
}

struct FVSMRenderViewCount
{
	uint32 NumPrimaryViews = 0u;
	uint32 NumMipLevels = 0u;
};

FVSMRenderViewCount GetRenderViewCount(const FProjectedShadowInfo* ProjectedShadowInfo)
{
	if (ProjectedShadowInfo->VirtualShadowMapClipmap)
	{
		return { uint32(ProjectedShadowInfo->VirtualShadowMapClipmap->GetLevelCount()), 1u };
	}
	else
	{
		return { ProjectedShadowInfo->bOnePassPointLightShadow ? 6u : 1u, FVirtualShadowMap::MaxMipLevels };
	}
}

static void AddRasterPass(
	FRDGBuilder& GraphBuilder, 
	FRDGEventName&& PassName,
	const FViewInfo* ShadowDepthView, 
	const TRDGUniformBufferRef<FShadowDepthPassUniformParameters>& ShadowDepthPassUniformBuffer,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	FRDGBufferRef VirtualShadowViewsRDG,
	const FCullingResult& CullingResult, 
	FParallelMeshDrawCommandPass& MeshCommandPass,
	FVirtualShadowDepthPassParameters* PassParameters,
	TRDGUniformBufferRef<FInstanceCullingGlobalUniforms> InstanceCullingUniformBuffer,
	TRDGUniformBufferRef<FSceneUniformParameters> SceneUB)
{
	PassParameters->View = ShadowDepthView->ViewUniformBuffer;
	PassParameters->ShadowDepthPass = ShadowDepthPassUniformBuffer;

	PassParameters->VirtualShadowMap = VirtualShadowMapArray.GetUniformBuffer(0);
	PassParameters->InViews = GraphBuilder.CreateSRV(VirtualShadowViewsRDG);
	PassParameters->InstanceCullingDrawParams.DrawIndirectArgsBuffer = CullingResult.DrawIndirectArgsRDG;
	PassParameters->InstanceCullingDrawParams.InstanceIdOffsetBuffer = CullingResult.InstanceIdOffsetBufferRDG;
	PassParameters->InstanceCullingDrawParams.InstanceCulling = InstanceCullingUniformBuffer;
	PassParameters->InstanceCullingDrawParams.Scene = SceneUB;

	FIntRect ViewRect;
	ViewRect.Max = FVirtualShadowMap::VirtualMaxResolutionXY;

	GraphBuilder.AddPass(
		MoveTemp(PassName),
		PassParameters,
		ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass,
		[&MeshCommandPass, PassParameters, ViewRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FRHIRenderPassInfo RPInfo;
			RPInfo.ResolveRect = FResolveRect(ViewRect);
			RHICmdList.BeginRenderPass(RPInfo, TEXT("RasterizeVirtualShadowMaps(Non-Nanite)"));

			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, FMath::Min(ViewRect.Max.X, 32767), FMath::Min(ViewRect.Max.Y, 32767), 1.0f);

			MeshCommandPass.Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
			RHICmdList.EndRenderPass();
		});
}

class FCompactViewsVSM_CS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompactViewsVSM_CS);
	SHADER_USE_PARAMETER_STRUCT(FCompactViewsVSM_CS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPackedView >, InViews)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< FPackedNaniteView >, CompactedViewsOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< FViewDrawGroup >, InOutViewDrawRanges)
		SHADER_PARAMETER(uint32, NumViewRanges)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, CompactedViewsAllocationOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer<FNaniteStats>, OutStatsBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FCompactViewsVSM_CS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapCompactViews.usf", "CompactViewsVSM_CS", SF_Compute);

class FComputeExplicitChunkDrawsViewMask_CS : public FVirtualShadowMapGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComputeExplicitChunkDrawsViewMask_CS);
	SHADER_USE_PARAMETER_STRUCT(FComputeExplicitChunkDrawsViewMask_CS, FVirtualShadowMapGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumWorkGroups)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FViewDrawGroup >, InViewDrawRanges)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< FInstanceCullingGroupWork >, InOutInstanceCullingWorkGroups)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FComputeExplicitChunkDrawsViewMask_CS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapComputeExplicitChunkDrawsViewMask.usf", "ComputeExplicitChunkDrawsViewMask", SF_Compute);

void FVirtualShadowMapArray::RenderVirtualShadowMapsNanite(
	FRDGBuilder& GraphBuilder,
	FSceneRenderer& SceneRenderer,
	bool bUpdateNaniteStreaming,
	const FNaniteVisibilityQuery* VisibilityQuery,
	TConstArrayView<FNaniteVirtualShadowMapRenderPass> VirtualShadowMapPasses)
{
	bool bCsvLogEnabled = IsCsvLogEnabled();

	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualShadowMapArray::RenderVirtualShadowMapsNanite);
	RDG_EVENT_SCOPE(GraphBuilder, "RenderVirtualShadowMaps(Nanite)");
	DynamicRenderScaling::FRDGScope DynamicVSMResolutionScope(GraphBuilder, GDynamicResolutionVSMNaniteBudget); // Only covers nanite for now, because resolution scaling is not very effective for non-nanite.

	const FIntPoint VirtualShadowSize = GetPhysicalPoolSize();
	const FIntRect VirtualShadowViewRect = FIntRect(0, 0, VirtualShadowSize.X, VirtualShadowSize.Y);
	const bool bAllowAnyTessellation = CVarVirtualShadowMapNaniteAllowTessellationDirectional.GetValueOnAnyThread() != 0 ||
		CVarVirtualShadowMapNaniteAllowTessellationLocal.GetValueOnAnyThread() != 0;

	Nanite::FSharedContext SharedContext{};
	SharedContext.FeatureLevel = SceneRenderer.FeatureLevel;
	SharedContext.ShaderMap = GetGlobalShaderMap(SharedContext.FeatureLevel);
	SharedContext.Pipeline = Nanite::ERasterPipeline::Shadows;

	check(PhysicalPagePoolRDG != nullptr);

	const Nanite::FRasterContextInitParams RasterInitParams =
	{
		.TextureSize = VirtualShadowSize,
		.TextureRect = VirtualShadowViewRect,
		.RasterMode = Nanite::EOutputBufferMode::DepthOnly,
		.ExternalDepthBuffer = PhysicalPagePoolRDG,		
		.bClearTarget = false,
		.bVisualize = bEnableNaniteVisualization,
		.bVisualizeOverdraw = bEnableNaniteVisualization, // Overdraw is the only currently supported mode
		.bAllowTessellation = bAllowAnyTessellation
	};
	Nanite::FRasterContext RasterContext = Nanite::InitRasterContext(
		GraphBuilder,
		SharedContext,
		SceneRenderer.ViewFamily,
		RasterInitParams		
	);

	const FViewInfo& SceneView = SceneRenderer.Views[0];

	// TODO: Stats probably doesn't work correctly with multiple passes
	static FString VirtualFilterName = "VirtualShadowMaps";

	for (int32 Index = 0; Index < VirtualShadowMapPasses.Num(); ++Index)
	{
		const FNaniteVirtualShadowMapRenderPass& NaniteRenderPass = VirtualShadowMapPasses[Index];
		const Nanite::FPackedViewArray* RenderViews = NaniteRenderPass.VirtualShadowMapViews;
		FSceneInstanceCullingQuery* SceneInstanceCullingQuery = NaniteRenderPass.SceneInstanceCullingQuery;

		INC_DWORD_STAT_BY(STAT_VSMNaniteViewsPrimary, RenderViews->NumViews);

		// It generates views for any mips that need them and compact away primary views where no views are used.
		// TODO: Nanite however can only ever access up to NANITE_MAX_VIEWS_PER_CULL_RASTERIZE_PASS in a given pass, so we could clamp this
		// to that value, just need to detect overflow in the compaction shader and stop writing.
		FRDGBufferRef CompactedViews = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(Nanite::FPackedView), NaniteRenderPass.MaxCullingViews), TEXT("Shadow.Virtual.CompactedViews"));

		const int32 NumViewDrawRanges = SceneInstanceCullingQuery->GetViewDrawGroups().Num();
		// Future TODO: move the ViewDrawRanges out of the culling query.
		FRDGBufferRef ViewDrawRanges = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.ViewDrawRanges"), SceneInstanceCullingQuery->GetViewDrawGroups());
		
		{
			// Just a pair of atomic counters, zeroed by a clear UAV pass.
			FRDGBufferRef CompactedViewsAllocation = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 2), TEXT("Shadow.Virtual.CompactedViewsAllocation"));
			FRDGBufferUAVRef CompactedViewsAllocationUAV = GraphBuilder.CreateUAV(CompactedViewsAllocation);
			AddClearUAVPass(GraphBuilder, CompactedViewsAllocationUAV, 0);

			const uint32 InputViewsCount = FMath::RoundUpToPowerOfTwo(RenderViews->NumViews);
			FRDGBufferRef InputViews = CreateStructuredBuffer(
				GraphBuilder,
				TEXT("Shadow.Virtual.InputViews"),
				sizeof(Nanite::FPackedView),
				[InputViewsCount] { return InputViewsCount; },
				[RenderViews] { return RenderViews->GetViews().GetData(); },
				[RenderViews] { return RenderViews->GetViews().Num() * sizeof(Nanite::FPackedView); }
			);

			FCompactViewsVSM_CS::FParameters* PassParameters = GraphBuilder.AllocParameters< FCompactViewsVSM_CS::FParameters >();

			PassParameters->VirtualShadowMap			= GetUniformBuffer(0);		// Does not use any per-main-view data (light grid)
			PassParameters->InViews						= GraphBuilder.CreateSRV(InputViews);
			PassParameters->CompactedViewsOut			= GraphBuilder.CreateUAV(CompactedViews);
			PassParameters->CompactedViewsAllocationOut = CompactedViewsAllocationUAV;
			PassParameters->InOutViewDrawRanges			= GraphBuilder.CreateUAV(ViewDrawRanges);

			auto ComputeShader = SharedContext.ShaderMap->GetShader<FCompactViewsVSM_CS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("CompactViewsVSM"),
				ComputeShader,
				PassParameters,
				// One group per primary view range now
				FIntVector(NumViewDrawRanges, 1, 1)
			);
		}

		// Patch up the ActiveViewMask of the explicit chunk draws. This needs to happen after view compaction so it can reference the views in the compacted buffer.
		if (NaniteRenderPass.ExplicitChunkDrawInfo)
		{
			auto ComputeShader = SharedContext.ShaderMap->GetShader<FComputeExplicitChunkDrawsViewMask_CS>();
			FComputeExplicitChunkDrawsViewMask_CS::FParameters* PassParameters = GraphBuilder.AllocParameters< FComputeExplicitChunkDrawsViewMask_CS::FParameters >();
			PassParameters->NumWorkGroups = NaniteRenderPass.ExplicitChunkDrawInfo->NumChunks;
			PassParameters->InViewDrawRanges = GraphBuilder.CreateSRV(ViewDrawRanges);
			PassParameters->InOutInstanceCullingWorkGroups = GraphBuilder.CreateUAV(NaniteRenderPass.ExplicitChunkDrawInfo->ExplicitChunkDraws);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ComputeExplicitChunkDrawsViewMask"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCountWrapped(PassParameters->NumWorkGroups, 64u)
			);
		}

		// Prev HZB requires previous page tables and similar
		bool bPrevHZBValid = HZBPhysicalArrayRDG != nullptr && CacheManager->GetPrevBuffers().PageTable != nullptr;

		Nanite::FConfiguration CullingConfig = { 0 };
		CullingConfig.bIsShadowPass = true;
		CullingConfig.bUpdateStreaming = bUpdateNaniteStreaming;
		CullingConfig.bTwoPassOcclusion = UseHzbOcclusion();
		CullingConfig.bExtractStats = Nanite::IsStatFilterActive(VirtualFilterName);
		CullingConfig.bExtractVSMPerformanceFeedback = true;
		CullingConfig.SetViewFlags(SceneView);

		auto NaniteRenderer = Nanite::IRenderer::Create(
			GraphBuilder,
			Scene,
			SceneView,
			SceneRenderer.GetSceneUniforms(),
			SharedContext,
			RasterContext,
			CullingConfig,
			VirtualShadowViewRect,
			bPrevHZBValid ? HZBPhysicalArrayRDG : nullptr,
			this
		);

		if (bCsvLogEnabled)
		{
			//CullingContext.RenderFlags |= NANITE_RENDER_FLAG_WRITE_STATS;	FIXME
		}

		NaniteRenderer->DrawGeometry(
			Scene.NaniteRasterPipelines[ENaniteMeshPass::BasePass],
			VisibilityQuery,
			CompactedViews,
			ViewDrawRanges,
			0,			// GPU provided view counts (multiview)
			SceneInstanceCullingQuery,
			nullptr,		// OptionalInstanceDraws
			NaniteRenderPass.ExplicitChunkDrawInfo
		);

		if (bCsvLogEnabled)
		{
			//StatsNaniteBufferRDG = CullingContext.StatsBuffer;		FIXME
		}
	}

	if (bUseHzbOcclusion)
	{
		UpdateHZB(GraphBuilder);
	}
}

void FVirtualShadowMapArray::RenderVirtualShadowMapsNonNanite(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer, const TArray<FProjectedShadowInfo *, SceneRenderingAllocator>& VirtualSmMeshCommandPasses, TArrayView<FViewInfo> Views)
{
	if (VirtualSmMeshCommandPasses.Num() == 0)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FVirtualShadowMapArray::RenderVirtualShadowMapsNonNanite);
	RDG_EVENT_SCOPE(GraphBuilder, "RenderVirtualShadowMaps(Non-Nanite)");

	FGPUScene& GPUScene = Scene.GPUScene;
	
	FRDGTextureRef HZBTextureArray = nullptr;
	// When disabling Nanite, there may be stale data in the Nanite-HZB causing incorrect culling.	
	if (bHZBBuiltThisFrame && HZBPhysicalArrayRDG && CVarNonNaniteVsmUseHzb.GetValueOnRenderThread() != 0)
	{
		HZBTextureArray = HZBPhysicalArrayRDG;
	}

	TArray<FVSMCullingBatchInfo, SceneRenderingAllocator> UnBatchedVSMCullingBatchInfo;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> BatchedVirtualSmMeshCommandPasses;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> UnBatchedVirtualSmMeshCommandPasses;
	UnBatchedVSMCullingBatchInfo.Reserve(VirtualSmMeshCommandPasses.Num());
	BatchedVirtualSmMeshCommandPasses.Reserve(VirtualSmMeshCommandPasses.Num());
	UnBatchedVirtualSmMeshCommandPasses.Reserve(VirtualSmMeshCommandPasses.Num());

	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> ShadowsToAddRenderViews;

	TArray<FVSMCullingBatchInfo, SceneRenderingAllocator> VSMCullingBatchInfos;
	VSMCullingBatchInfos.Reserve(VirtualSmMeshCommandPasses.Num());

	TArray<FVirtualShadowDepthPassParameters*, SceneRenderingAllocator> BatchedPassParameters;
	BatchedPassParameters.Reserve(VirtualSmMeshCommandPasses.Num());

	uint32 MaxNumMips = 0;
	uint32 TotalPrimaryViews = 0;
	uint32 TotalViews = 0;

	FInstanceCullingMergedContext InstanceCullingMergedContext(GPUScene.GetShaderPlatform(), true);
	// We don't use the registered culling views (this redundancy should probably be addressed at some point), set the number to disable index range checking
	InstanceCullingMergedContext.NumCullingViews = -1;
	int32 TotalPreCullInstanceCount = 0;
	// Instance count multiplied by the number of (VSM) views, gives a safe maximum number of possible output instances from culling.
	uint32 TotalViewScaledInstanceCount = 0u;
	for (int32 Index = 0; Index < VirtualSmMeshCommandPasses.Num(); ++Index)
	{
		FProjectedShadowInfo* ProjectedShadowInfo = VirtualSmMeshCommandPasses[Index];

		if (!ProjectedShadowInfo->bShouldRenderVSM)
		{
			continue;
		}

		ProjectedShadowInfo->BeginRenderView(GraphBuilder, &Scene);

		FVSMCullingBatchInfo VSMCullingBatchInfo;
		VSMCullingBatchInfo.FirstPrimaryView = TotalPrimaryViews;
		VSMCullingBatchInfo.NumPrimaryViews = 0U;

		const TSharedPtr<FVirtualShadowMapClipmap>& Clipmap = ProjectedShadowInfo->VirtualShadowMapClipmap;
		check(Clipmap.IsValid() || ProjectedShadowInfo->HasVirtualShadowMap());
		{
			FParallelMeshDrawCommandPass& MeshCommandPass = ProjectedShadowInfo->GetShadowDepthPass();
			FInstanceCullingContext* InstanceCullingContext = MeshCommandPass.GetInstanceCullingContext();
			InstanceCullingContext->WaitForSetupTask();

			TotalPreCullInstanceCount += InstanceCullingContext->TotalInstances;

			if (InstanceCullingContext->HasCullingCommands())
			{
				FVSMRenderViewCount VSMRenderViewCount = GetRenderViewCount(ProjectedShadowInfo);
				MaxNumMips = FMath::Max(MaxNumMips, VSMRenderViewCount.NumMipLevels);

				TotalViewScaledInstanceCount += InstanceCullingContext->TotalInstances * VSMRenderViewCount.NumPrimaryViews * VSMRenderViewCount.NumMipLevels;

				VSMCullingBatchInfo.NumPrimaryViews = VSMRenderViewCount.NumPrimaryViews;
				TotalPrimaryViews += VSMRenderViewCount.NumPrimaryViews;
				ShadowsToAddRenderViews.Add(ProjectedShadowInfo);

				if (CVarDoNonNaniteBatching.GetValueOnRenderThread() != 0)
				{
					// NOTE: This array must be 1:1 with the batches inside the InstanceCullingMergedContext, which is guaranteed by checking HasCullingCommands() above (and checked in the merged context)
					//       If we were to defer/async this process, we need to maintain this property or add some remapping.
					VSMCullingBatchInfos.Add(VSMCullingBatchInfo);

					// Note: we have to allocate these up front as the context merging machinery writes the offsets directly to the &PassParameters->InstanceCullingDrawParams, 
					// this is a side-effect from sharing the code with the deferred culling. Should probably be refactored.
					FVirtualShadowDepthPassParameters* PassParameters = GraphBuilder.AllocParameters<FVirtualShadowDepthPassParameters>();
					InstanceCullingMergedContext.AddBatch(GraphBuilder, InstanceCullingContext, &PassParameters->InstanceCullingDrawParams);
					BatchedVirtualSmMeshCommandPasses.Add(ProjectedShadowInfo);
					BatchedPassParameters.Add(PassParameters);
				}
				else
				{
					UnBatchedVSMCullingBatchInfo.Add(VSMCullingBatchInfo);
					UnBatchedVirtualSmMeshCommandPasses.Add(ProjectedShadowInfo);
				}
			}
		}
	}

	FRDGBuffer* VirtualShadowViewsRDG = nullptr;

	if (!ShadowsToAddRenderViews.IsEmpty())
	{
		Nanite::FPackedViewArray* ViewArray = Nanite::FPackedViewArray::CreateWithSetupTask(
			GraphBuilder,
			TotalPrimaryViews * FVirtualShadowMap::MaxMipLevels,
			[this, Views, ShadowsToAddRenderViews = MoveTemp(ShadowsToAddRenderViews), bHasHZBTexture = (HZBTextureArray != nullptr)] (Nanite::FPackedViewArray::ArrayType& OutShadowViews)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AddNonNaniteRenderViews);
			for (FProjectedShadowInfo* ProjectedShadowInfo : ShadowsToAddRenderViews)
			{
				check(ProjectedShadowInfo->ShadowDepthView != nullptr);
				AddRenderViews(ERenderPassType::NonNanite, ProjectedShadowInfo, Views, 1.0f, bHasHZBTexture, false, OutShadowViews);
			}
			CreateMipViews(OutShadowViews);
		});

		VirtualShadowViewsRDG = CreateStructuredBuffer(GraphBuilder, TEXT("Shadow.Virtual.VirtualShadowViews"), [ViewArray]() -> const typename Nanite::FPackedViewArray::ArrayType& { return ViewArray->GetViews(); });
	}

	CSV_CUSTOM_STAT(VSM, NonNanitePreCullInstanceCount, TotalPreCullInstanceCount, ECsvCustomStatOp::Set);

	// Helper function to create raster pass UB - only really need two of these ever
	const FSceneTextures* SceneTextures = &GetViewFamilyInfo(Views).GetSceneTextures();
	auto CreateShadowDepthPassUniformBuffer = [this, &VirtualShadowViewsRDG, &GraphBuilder, SceneTextures](bool bClampToNearPlane)
	{
		FShadowDepthPassUniformParameters* ShadowDepthPassParameters = GraphBuilder.AllocParameters<FShadowDepthPassUniformParameters>();
		check(PhysicalPagePoolRDG != nullptr);
		// TODO: These are not used for this case anyway
		ShadowDepthPassParameters->ProjectionMatrix = FMatrix44f::Identity;
		ShadowDepthPassParameters->ViewMatrix = FMatrix44f::Identity;
		ShadowDepthPassParameters->ShadowParams = FVector4f(0.0f, 0.0f, 0.0f, 1.0f);
		ShadowDepthPassParameters->ShadowDepthType = (uint32)EShadowDepthType::VSM;

		ShadowDepthPassParameters->VirtualSmPageTable = PageTableRDG;
		ShadowDepthPassParameters->PackedNaniteViews = GraphBuilder.CreateSRV(VirtualShadowViewsRDG);
		ShadowDepthPassParameters->AllocatedPageRectBounds = GraphBuilder.CreateSRV(AllocatedPageRectBoundsRDG);
		ShadowDepthPassParameters->UncachedPageRectBounds = GraphBuilder.CreateSRV(UncachedPageRectBoundsRDG);
		ShadowDepthPassParameters->OutDepthBufferArray = GraphBuilder.CreateUAV(PhysicalPagePoolRDG, ERDGUnorderedAccessViewFlags::SkipBarrier);
		SetupSceneTextureUniformParameters(GraphBuilder, SceneTextures, Scene.GetFeatureLevel(), ESceneTextureSetupMode::None, ShadowDepthPassParameters->SceneTextures);
		ShadowDepthPassParameters->bClampToNearPlane = bClampToNearPlane;

		return GraphBuilder.CreateUniformBuffer(ShadowDepthPassParameters);
	};

	FCullPerPageDrawCommandsCs::FHZBShaderParameters HZBShaderParameters;
	if (HZBTextureArray)
	{
		HZBShaderParameters.HZBPageTable = PageTableRDG;
		HZBShaderParameters.HZBPageFlags = PageFlagsRDG;
		HZBShaderParameters.HZBPageRectBounds = GraphBuilder.CreateSRV(AllocatedPageRectBoundsRDG);	// TODO: Uncached?
		check(HZBShaderParameters.HZBPageTable);
		check(HZBShaderParameters.HZBPageFlags);
		check(HZBShaderParameters.HZBPageRectBounds);
				
		HZBShaderParameters.HZBTextureArray = HZBTextureArray;
		HZBShaderParameters.HZBSize = HZBTextureArray->Desc.Extent;
		HZBShaderParameters.HZBSampler = TStaticSamplerState< SF_Point, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI();
	}
	else
	{
		HZBShaderParameters.HZBTextureArray = nullptr;
	}

	TRDGUniformBufferRef<FSceneUniformParameters> SceneUB = SceneUniformBuffer.GetBuffer(GraphBuilder);

	// Process batched passes
	if (!InstanceCullingMergedContext.Batches.IsEmpty())
	{
		RDG_EVENT_SCOPE(GraphBuilder, "Batched");

		InstanceCullingMergedContext.MergeBatches();

		FCullingResult CullingResult;
		{
		    RDG_EVENT_SCOPE(GraphBuilder, "CullingPasses");
		    CullingResult = AddCullingPasses(
			    GraphBuilder,
			    InstanceCullingMergedContext.IndirectArgs,
			    InstanceCullingMergedContext.DrawCommandDescs,
			    InstanceCullingMergedContext.InstanceIdOffsets,
			    &InstanceCullingMergedContext.LoadBalancers[FInstanceCullingMergedContext::FirstGenericBinIndex],
			    InstanceCullingMergedContext.BatchInfos,
			    VSMCullingBatchInfos,
			    InstanceCullingMergedContext.BatchInds[FInstanceCullingMergedContext::FirstGenericBinIndex],
			    InstanceCullingMergedContext.TotalInstances,
			    TotalViewScaledInstanceCount,
			    TotalPrimaryViews,
			    VirtualShadowViewsRDG,
			    HZBShaderParameters,
			    *this,
			    SceneUniformBuffer,
				FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(GPUScene.GetShaderPlatform())
		    );
		}

		TRDGUniformBufferRef<FShadowDepthPassUniformParameters> ShadowDepthPassUniformBuffer = CreateShadowDepthPassUniformBuffer(false);

		FInstanceCullingGlobalUniforms* InstanceCullingGlobalUniforms = GraphBuilder.AllocParameters<FInstanceCullingGlobalUniforms>();
		InstanceCullingGlobalUniforms->InstanceIdsBuffer = GraphBuilder.CreateSRV(CullingResult.InstanceIdsBuffer);
		InstanceCullingGlobalUniforms->PageInfoBuffer = GraphBuilder.CreateSRV(CullingResult.PageInfoBuffer);
		InstanceCullingGlobalUniforms->BufferCapacity = CullingResult.MaxNumInstancesPerPass;
		TRDGUniformBufferRef<FInstanceCullingGlobalUniforms> InstanceCullingUniformBuffer = GraphBuilder.CreateUniformBuffer(InstanceCullingGlobalUniforms);

		if(!BatchedVirtualSmMeshCommandPasses.IsEmpty())
		{
			if (CVarVirtualShadowSinglePassBatched.GetValueOnRenderThread() != 0)
			{
				FVirtualShadowDepthPassParameters* PassParameters = GraphBuilder.AllocParameters<FVirtualShadowDepthPassParameters>();
				FProjectedShadowInfo* ProjectedShadowInfo0 = BatchedVirtualSmMeshCommandPasses[0];
				FViewInfo* ShadowDepthView = ProjectedShadowInfo0->ShadowDepthView;

				PassParameters->View = ShadowDepthView->ViewUniformBuffer;
				PassParameters->ShadowDepthPass = ShadowDepthPassUniformBuffer;

				PassParameters->VirtualShadowMap = GetUniformBuffer(0);
				PassParameters->InViews = GraphBuilder.CreateSRV(VirtualShadowViewsRDG);
				PassParameters->InstanceCullingDrawParams.DrawIndirectArgsBuffer = CullingResult.DrawIndirectArgsRDG;
				PassParameters->InstanceCullingDrawParams.InstanceIdOffsetBuffer = CullingResult.InstanceIdOffsetBufferRDG;
				PassParameters->InstanceCullingDrawParams.InstanceCulling = InstanceCullingUniformBuffer;
				PassParameters->InstanceCullingDrawParams.Scene = SceneUB;
				PassParameters->InstanceCullingDrawParams.IndirectArgsByteOffset = 0U;
				PassParameters->InstanceCullingDrawParams.InstanceDataByteOffset = 0U;


				GraphBuilder.AddPass(
					RDG_EVENT_NAME("RasterPasses"),
					PassParameters,
					ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass,
					[PassParameters, 
					BatchedPassParameters=MoveTemp(BatchedPassParameters), 
					BatchedVirtualSmMeshCommandPasses=MoveTemp(BatchedVirtualSmMeshCommandPasses)](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
						FIntRect ViewRect;
						ViewRect.Min = FIntPoint(0, 0);
						ViewRect.Max = FVirtualShadowMap::VirtualMaxResolutionXY;
						FRHIRenderPassInfo RPInfo;
						RPInfo.ResolveRect = FResolveRect(ViewRect);
						RHICmdList.BeginRenderPass(RPInfo, TEXT("RasterizeVirtualShadowMaps(Non-Nanite)"));

						RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, FMath::Min(ViewRect.Max.X, 32767), FMath::Min(ViewRect.Max.Y, 32767), 1.0f);

						for (int32 Index = 0; Index < BatchedVirtualSmMeshCommandPasses.Num(); ++Index)
						{
							FProjectedShadowInfo* ProjectedShadowInfo = BatchedVirtualSmMeshCommandPasses[Index];
							FParallelMeshDrawCommandPass& MeshCommandPass = ProjectedShadowInfo->GetShadowDepthPass();

							FInstanceCullingDrawParams InstanceCullingDrawParams = PassParameters->InstanceCullingDrawParams;
							InstanceCullingDrawParams.IndirectArgsByteOffset = BatchedPassParameters[Index]->InstanceCullingDrawParams.IndirectArgsByteOffset;
							InstanceCullingDrawParams.InstanceDataByteOffset = BatchedPassParameters[Index]->InstanceCullingDrawParams.InstanceDataByteOffset;
#if WITH_PROFILEGPU
							FString LightNameWithLevel;
							if (GVSMShowLightDrawEvents != 0)
							{
								FSceneRenderer::GetLightNameForDrawEvent(ProjectedShadowInfo->GetLightSceneInfo().Proxy, LightNameWithLevel);
							}
							SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, BatchedNonNanite, GVSMShowLightDrawEvents != 0, TEXT("%s"), LightNameWithLevel);
#endif
							MeshCommandPass.Draw(RHICmdList, &InstanceCullingDrawParams);
						}

						RHICmdList.EndRenderPass();
					});

			}
			else
			{
				RDG_EVENT_SCOPE(GraphBuilder, "RasterPasses");
				for (int32 Index = 0; Index < BatchedVirtualSmMeshCommandPasses.Num(); ++Index)
				{
					FProjectedShadowInfo* ProjectedShadowInfo = BatchedVirtualSmMeshCommandPasses[Index];
					FParallelMeshDrawCommandPass& MeshCommandPass = ProjectedShadowInfo->GetShadowDepthPass();
					FViewInfo* ShadowDepthView = ProjectedShadowInfo->ShadowDepthView;

					FString LightNameWithLevel;
					FSceneRenderer::GetLightNameForDrawEvent(ProjectedShadowInfo->GetLightSceneInfo().Proxy, LightNameWithLevel);
					AddRasterPass(GraphBuilder, RDG_EVENT_NAME("Rasterize[%s]", *LightNameWithLevel), ShadowDepthView, ShadowDepthPassUniformBuffer, *this, VirtualShadowViewsRDG, CullingResult, MeshCommandPass, BatchedPassParameters[Index], InstanceCullingUniformBuffer, SceneUB);
			}
		}
	}
	}

	// Loop over the un batched mesh command passes needed, these are all the clipmaps (but we may change the criteria)
	for (int32 Index = 0; Index < UnBatchedVirtualSmMeshCommandPasses.Num(); ++Index)
	{
		const auto VSMCullingBatchInfo = UnBatchedVSMCullingBatchInfo[Index];
		FProjectedShadowInfo* ProjectedShadowInfo = UnBatchedVirtualSmMeshCommandPasses[Index];
		FInstanceCullingMergedContext::FContextBatchInfoPacked CullingBatchInfo = FInstanceCullingMergedContext::FContextBatchInfoPacked{ 0 };

		FParallelMeshDrawCommandPass& MeshCommandPass = ProjectedShadowInfo->GetShadowDepthPass();
		const TSharedPtr<FVirtualShadowMapClipmap> Clipmap = ProjectedShadowInfo->VirtualShadowMapClipmap;
		FViewInfo* ShadowDepthView = ProjectedShadowInfo->ShadowDepthView;

		MeshCommandPass.WaitForSetupTask();

		FInstanceCullingContext* InstanceCullingContext = MeshCommandPass.GetInstanceCullingContext();

		if (InstanceCullingContext->HasCullingCommands())
		{
			FString LightNameWithLevel;
			FSceneRenderer::GetLightNameForDrawEvent(ProjectedShadowInfo->GetLightSceneInfo().Proxy, LightNameWithLevel);
			RDG_EVENT_SCOPE(GraphBuilder, "%s", *LightNameWithLevel);

			FVSMRenderViewCount VSMRenderViewCount = GetRenderViewCount(ProjectedShadowInfo);
			uint32 ViewScaledInstanceCount = VSMRenderViewCount.NumPrimaryViews * VSMRenderViewCount.NumMipLevels * InstanceCullingContext->TotalInstances;

			CullingBatchInfo.DynamicInstanceIdOffset = ShadowDepthView->DynamicPrimitiveCollector.GetInstanceSceneDataOffset();
			CullingBatchInfo.DynamicInstanceIdMax = CullingBatchInfo.DynamicInstanceIdOffset + ShadowDepthView->DynamicPrimitiveCollector.NumInstances();

			FCullingResult CullingResult = AddCullingPasses(
				GraphBuilder,
				InstanceCullingContext->IndirectArgs, 
				InstanceCullingContext->DrawCommandDescs,
				InstanceCullingContext->InstanceIdOffsets,
				InstanceCullingContext->LoadBalancers[0],
				MakeArrayView(&CullingBatchInfo, 1),
				MakeArrayView(&VSMCullingBatchInfo, 1),
				MakeArrayView<const uint32>(nullptr, 0),
				InstanceCullingContext->TotalInstances,
				ViewScaledInstanceCount,
				TotalPrimaryViews,
				VirtualShadowViewsRDG,
				HZBShaderParameters,
				*this,
				SceneUniformBuffer,
				FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(GPUScene.GetShaderPlatform())
			);

			TRDGUniformBufferRef<FShadowDepthPassUniformParameters> ShadowDepthPassUniformBuffer = CreateShadowDepthPassUniformBuffer(ProjectedShadowInfo->ShouldClampToNearPlane());

			FInstanceCullingGlobalUniforms* InstanceCullingGlobalUniforms = GraphBuilder.AllocParameters<FInstanceCullingGlobalUniforms>();
			InstanceCullingGlobalUniforms->InstanceIdsBuffer = GraphBuilder.CreateSRV(CullingResult.InstanceIdsBuffer);
			InstanceCullingGlobalUniforms->PageInfoBuffer = GraphBuilder.CreateSRV(CullingResult.PageInfoBuffer);
			InstanceCullingGlobalUniforms->BufferCapacity = CullingResult.MaxNumInstancesPerPass;
			TRDGUniformBufferRef<FInstanceCullingGlobalUniforms> InstanceCullingUniformBuffer = GraphBuilder.CreateUniformBuffer(InstanceCullingGlobalUniforms);

			FVirtualShadowDepthPassParameters* DepthPassParams = GraphBuilder.AllocParameters<FVirtualShadowDepthPassParameters>();
			DepthPassParams->InstanceCullingDrawParams.IndirectArgsByteOffset = 0;
			DepthPassParams->InstanceCullingDrawParams.InstanceDataByteOffset = 0;
			AddRasterPass(GraphBuilder, RDG_EVENT_NAME("Rasterize"), ShadowDepthView, ShadowDepthPassUniformBuffer, *this, VirtualShadowViewsRDG, CullingResult, MeshCommandPass, DepthPassParams, InstanceCullingUniformBuffer, SceneUB);
		}


		//
		if (Index == CVarShowClipmapStats.GetValueOnRenderThread())
		{
			// The 'main' view the shadow was created with respect to
			const FViewInfo* ViewUsedToCreateShadow = ProjectedShadowInfo->DependentView;
			const FViewInfo& View = *ViewUsedToCreateShadow;

			FVirtualSmPrintClipmapStatsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVirtualSmPrintClipmapStatsCS::FParameters>();

			ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintStruct);
			//PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
			PassParameters->ShadowMapIdRangeStart = Clipmap->GetVirtualShadowMapId();
			// Note: assumes range!
			PassParameters->ShadowMapIdRangeEnd = Clipmap->GetVirtualShadowMapId() + Clipmap->GetLevelCount();
			PassParameters->AllocatedPageRectBounds = GraphBuilder.CreateSRV(AllocatedPageRectBoundsRDG);

			auto ComputeShader = View.ShaderMap->GetShader<FVirtualSmPrintClipmapStatsCS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("PrintClipmapStats"),
				ComputeShader,
				PassParameters,
				FIntVector(1, 1, 1)
			);
		}
	}
}

class FSelectPagesForHZBAndUpdateDirtyFlagsCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FSelectPagesForHZBAndUpdateDirtyFlagsCS);
	SHADER_USE_PARAMETER_STRUCT(FSelectPagesForHZBAndUpdateDirtyFlagsCS, FVirtualShadowMapPageManagementShader)

	class FGenerateStatsDim : SHADER_PERMUTATION_BOOL("VSM_GENERATE_STATS");
	using FPermutationDomain = TShaderPermutationDomain< FGenerateStatsDim >;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPhysicalPageMetaData>, OutPhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutPagesForHZBIndirectArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, OutPhysicalPagesForHZB)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, DirtyPageFlagsInOut)
		SHADER_PARAMETER(uint32, bFirstBuildThisFrame)
		SHADER_PARAMETER(uint32, bForceFullHZBUpdate)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< uint >, OutStatsBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FSelectPagesForHZBAndUpdateDirtyFlagsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "SelectPagesForHZBAndUpdateDirtyFlagsCS", SF_Compute);

class FVirtualSmBuildHZBPerPageCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FVirtualSmBuildHZBPerPageCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualSmBuildHZBPerPageCS, FVirtualShadowMapPageManagementShader)

	static constexpr uint32 TotalHZBLevels = FVirtualShadowMap::NumHZBLevels;
	static constexpr uint32 HZBLevelsBase = TotalHZBLevels - 2U;

	static_assert(HZBLevelsBase == 5U, "The shader is expecting 5 levels, if the page size is changed, this needs to be massaged");

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPhysicalPageMetaData>, PhysicalPageMetaData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint >, PhysicalPagesForHZB)
		SHADER_PARAMETER_SAMPLER(SamplerState, PhysicalPagePoolSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, PhysicalPagePool)

		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2DArray<float>, FurthestHZBArrayOutput, [HZBLevelsBase])
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FVirtualSmBuildHZBPerPageCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "BuildHZBPerPageCS", SF_Compute);


class FVirtualSmBBuildHZBPerPageTopCS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FVirtualSmBBuildHZBPerPageTopCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualSmBBuildHZBPerPageTopCS, FVirtualShadowMapPageManagementShader)

	// We need one level less as HZB starts at half-size (not really sure if we really need 1x1 and 2x2 sized levels).
	static constexpr uint32 HZBLevelsTop = 2;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint >, PhysicalPagesForHZB)
		SHADER_PARAMETER_SAMPLER(SamplerState, ParentTextureMipSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray, ParentTextureArrayMip)
		SHADER_PARAMETER(FVector2f, InvHzbInputSize)

		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<float>, FurthestHZBArrayOutput, [HZBLevelsTop])
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FVirtualSmBBuildHZBPerPageTopCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapPhysicalPageManagement.usf", "BuildHZBPerPageTopCS", SF_Compute);

void FVirtualShadowMapArray::UpdateHZB(FRDGBuilder& GraphBuilder)
{
	const FIntRect ViewRect(0, 0, GetPhysicalPoolSize().X, GetPhysicalPoolSize().Y);

	// 1. Gather up all physical pages that are allocated
	FRDGBufferRef PagesForHZBIndirectArgsRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(2U * 4U), TEXT("Shadow.Virtual.PagesForHZBIndirectArgs"));
	// NOTE: Total allocated pages since the shader outputs separate entries for static/dynamic pages
	FRDGBufferRef PhysicalPagesForHZBRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(int32), GetTotalAllocatedPhysicalPages() + 1), TEXT("Shadow.Virtual.PhysicalPagesForHZB"));

	// 1. Clear the indirect args buffer (note 2x args)
	AddClearIndirectDispatchArgs1DPass(GraphBuilder, Scene.GetFeatureLevel(), PagesForHZBIndirectArgsRDG, 2u, 4u);

	// 2. Filter the relevant physical pages and set up the indirect args
	{
		FSelectPagesForHZBAndUpdateDirtyFlagsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSelectPagesForHZBAndUpdateDirtyFlagsCS::FParameters>();
		PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
		PassParameters->OutPagesForHZBIndirectArgsBuffer = GraphBuilder.CreateUAV(PagesForHZBIndirectArgsRDG);
		PassParameters->OutPhysicalPagesForHZB = GraphBuilder.CreateUAV(PhysicalPagesForHZBRDG);
		PassParameters->DirtyPageFlagsInOut = GraphBuilder.CreateUAV(DirtyPageFlagsRDG);
		PassParameters->OutPhysicalPageMetaData = GraphBuilder.CreateUAV(PhysicalPageMetaDataRDG);
		PassParameters->bFirstBuildThisFrame = !bHZBBuiltThisFrame;
		PassParameters->bForceFullHZBUpdate = CVarShadowsVirtualForceFullHZBUpdate.GetValueOnRenderThread();
		FSelectPagesForHZBAndUpdateDirtyFlagsCS::FPermutationDomain PermutationVector;
		SetStatsArgsAndPermutation<FSelectPagesForHZBAndUpdateDirtyFlagsCS>(ShouldGenerateStats(), StatsBufferUAV, PassParameters, PermutationVector);
		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FSelectPagesForHZBAndUpdateDirtyFlagsCS>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("SelectPagesForHZB"),
			ComputeShader,
			PassParameters,
			FIntVector(FMath::DivideAndRoundUp(UniformParameters.MaxPhysicalPages, FSelectPagesForHZBAndUpdateDirtyFlagsCS::DefaultCSGroupX), 1, 1)
		);
	}

	bHZBBuiltThisFrame = true;
	const int HZBPoolArraySize = HasSeparateDynamicHZB() ? 2 : 1;
		
	{
		FVirtualSmBuildHZBPerPageCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVirtualSmBuildHZBPerPageCS::FParameters>();

		PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);
		for (int32 DestMip = 0; DestMip < FVirtualSmBuildHZBPerPageCS::HZBLevelsBase; DestMip++)
		{
			PassParameters->FurthestHZBArrayOutput[DestMip] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(HZBPhysicalArrayRDG, DestMip));
		}
		PassParameters->PhysicalPagePool = PhysicalPagePoolRDG;
		PassParameters->PhysicalPagePoolSampler = TStaticSamplerState<SF_Point>::GetRHI();
		PassParameters->PhysicalPageMetaData = GraphBuilder.CreateSRV(PhysicalPageMetaDataRDG);

		PassParameters->IndirectArgs = PagesForHZBIndirectArgsRDG;
		PassParameters->PhysicalPagesForHZB = GraphBuilder.CreateSRV(PhysicalPagesForHZBRDG);
		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FVirtualSmBuildHZBPerPageCS>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("BuildHZBPerPage"),
			ComputeShader,
			PassParameters,
			PassParameters->IndirectArgs,
			0
		);
	}
	{
		FVirtualSmBBuildHZBPerPageTopCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVirtualSmBBuildHZBPerPageTopCS::FParameters>();

		PassParameters->VirtualShadowMap = GetUncachedUniformBuffer(GraphBuilder);

		uint32 StartDestMip = FVirtualSmBuildHZBPerPageCS::HZBLevelsBase;
		for (int32 DestMip = 0; DestMip < FVirtualSmBBuildHZBPerPageTopCS::HZBLevelsTop; DestMip++)
		{
			PassParameters->FurthestHZBArrayOutput[DestMip] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(HZBPhysicalArrayRDG, StartDestMip + DestMip));
		}
		FIntPoint SrcSize = FIntPoint::DivideAndRoundUp(FIntPoint(HZBPhysicalArrayRDG->Desc.GetSize().X, HZBPhysicalArrayRDG->Desc.GetSize().Y), 1 << int32(StartDestMip - 1));
		PassParameters->InvHzbInputSize = FVector2f(1.0f / SrcSize.X, 1.0f / SrcSize.Y);;
		PassParameters->ParentTextureArrayMip = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(HZBPhysicalArrayRDG, StartDestMip - 1));
		PassParameters->ParentTextureMipSampler = TStaticSamplerState<SF_Point>::GetRHI();

		PassParameters->IndirectArgs = PagesForHZBIndirectArgsRDG;
		PassParameters->PhysicalPagesForHZB = GraphBuilder.CreateSRV(PhysicalPagesForHZBRDG);
		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FVirtualSmBBuildHZBPerPageTopCS>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("BuildHZBPerPageTop"),
			ComputeShader,
			PassParameters,
			PassParameters->IndirectArgs,
			// NOTE: offset 4 to get second set of args in the buffer.
			4U * sizeof(uint32)
		);
	}
}

static Nanite::FPackedView CreateNanitePackedView(const Nanite::FPackedViewParams& Params)
{
	Nanite::FPackedView PackedView = Nanite::CreatePackedView(Params);

	// Adjust a few packed view parameters for VSM rendering
	// TODO: Move this stuff into proper packed view creation itself
	static constexpr float ClipSpaceScale = 
		float(FVirtualShadowMap::VirtualMaxResolutionXY) / (FVirtualShadowMap::PageSize * FVirtualShadowMap::RasterWindowPages);

	check(PackedView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.X >= 0);
	check(PackedView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.Y == 0);	// Primary view
	check(PackedView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.Z > 0 && PackedView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.Z <= FVirtualShadowMap::MaxMipLevels);
	check(PackedView.ViewRect.X == 0);
	check(PackedView.ViewRect.Y == 0);
	check(PackedView.ViewRect.Z == FVirtualShadowMap::VirtualMaxResolutionXY);
	check(PackedView.ViewRect.W == FVirtualShadowMap::VirtualMaxResolutionXY);

	// Replace computed clip space offset from the packed nanite view to align with the raster window
	PackedView.ClipSpaceScaleOffset = FVector4f(ClipSpaceScale, ClipSpaceScale, ClipSpaceScale - 1.0f, 1.0f - ClipSpaceScale);
	// Set streaming priority category to zero for some reason
	PackedView.StreamingPriorityCategory_AndFlags &= ~uint32(NANITE_STREAMING_PRIORITY_CATEGORY_MASK);

	return PackedView;
}

void FVirtualShadowMapArray::AddRenderViewsClipmap(
	const TSharedPtr<FVirtualShadowMapClipmap>& Clipmap,
	const FViewInfo& CullingView,
	float NaniteLODScaleFactor,
	bool bSetHZBParams,
	bool bUpdateHZBMetaData,
	TArray<Nanite::FPackedView, SceneRenderingAllocator> &OutVirtualShadowViews) const
{
	Nanite::FPackedViewParams BaseParams;
	BaseParams.ViewRect = FIntRect(0, 0, FVirtualShadowMap::VirtualMaxResolutionXY, FVirtualShadowMap::VirtualMaxResolutionXY);
	BaseParams.HZBTestViewRect = BaseParams.ViewRect;
	BaseParams.RasterContextSize = GetPhysicalPoolSize();
	BaseParams.MaxPixelsPerEdgeMultipler = 1.0f / NaniteLODScaleFactor;
	BaseParams.PrevTargetLayerIndex = INDEX_NONE;
	BaseParams.TargetMipLevel = 0;
	BaseParams.TargetMipCount = 1;	// No mips for clipmaps
	BaseParams.Flags = 0U;
	Nanite::SetViewFrameIndex(&CullingView, BaseParams);

	if (Clipmap->GetLightSceneInfo().Proxy)
	{
		BaseParams.bUseLightingChannelMask = true;
		BaseParams.LightingChannelMask = Clipmap->GetLightSceneInfo().Proxy->GetLightingChannelMask();
	}

	const TSharedPtr<FVirtualShadowMapPerLightCacheEntry>& CacheEntry = Clipmap->GetCacheEntry();
	if (CacheEntry.IsValid())
	{
		CacheEntry->MarkRendered(Scene.GetFrameNumber());

		// TODO: Move this to a "get view flags" type helper?
		if (CacheEntry->IsUncached())
		{
			BaseParams.Flags |= NANITE_VIEW_FLAG_UNCACHED;
		}
		if (CacheEntry->ShouldUseReceiverMask())
		{
			BaseParams.Flags |= NANITE_VIEW_FLAG_USE_RECEIVER_MASK;
		}
	}

	const bool bHasPrefilteredDistant = Clipmap->GetConfig().bHasPrefilteredDistant;
	if (bHasPrefilteredDistant)
	{
		BaseParams.Flags |= NANITE_VIEW_FLAG_PREFILTERED_DISTANT_DEPTH_CULL;
	}

	if (CVarNonNaniteUseRadiusThreshold.GetValueOnAnyThread() != 0 && (!CacheEntry.IsValid() || CacheEntry->IsUncached()))
	{
		BaseParams.Flags |= NANITE_VIEW_MIN_SCREEN_RADIUS_CULL;
		BaseParams.MinBoundsRadius = GMinScreenRadiusForShadowCaster;
	}

	if (CVarVirtualShadowMapNaniteAllowTessellationDirectional.GetValueOnAnyThread() == 0)
	{
		BaseParams.Flags |= NANITE_VIEW_FLAG_FORCE_DISPLACEMENT_FALLBACK;
	}

	Nanite::SetCullingViewOverrides(&CullingView, BaseParams);

	const int32 VirtualShadowMapId = Clipmap->GetVirtualShadowMapId();
	for (int32 ClipmapLevelIndex = 0; ClipmapLevelIndex < Clipmap->GetLevelCount(); ++ClipmapLevelIndex)
	{
		Nanite::FPackedViewParams Params = BaseParams;
		Params.TargetLayerIndex = VirtualShadowMapId + ClipmapLevelIndex;
		Params.ViewMatrices = Clipmap->GetViewMatrices(ClipmapLevelIndex);
		Params.PrevTargetLayerIndex = INDEX_NONE;
		Params.PrevViewMatrices = Params.ViewMatrices;

		Params.DynamicDepthCullRange = Clipmap->GetDynamicDepthCullRange(ClipmapLevelIndex);

		if (CacheEntry)
		{
			FVirtualShadowMapCacheEntry& LevelEntry = CacheEntry->ShadowMapEntries[ClipmapLevelIndex];

			if (bSetHZBParams)
			{
				LevelEntry.SetHZBViewParams(Params);
			}

			// If we're going to generate a new HZB this frame, save the associated metadata
			if (bUpdateHZBMetaData)
			{
				LevelEntry.UpdateHZBMetadata(Params.ViewMatrices, Params.ViewRect, Params.TargetLayerIndex);
			}
		}

		OutVirtualShadowViews.Add(CreateNanitePackedView(Params));
	}
}

void FVirtualShadowMapArray::AddRenderViewsLocal(
	const FProjectedShadowInfo* ProjectedShadowInfo,
	const FViewInfo& CullingView,
	float NaniteLODScaleFactor,
	bool bSetHZBParams,
	bool bUpdateHZBMetaData,
	TArray<Nanite::FPackedView, SceneRenderingAllocator>& OutVirtualShadowViews) const
{
	Nanite::FPackedViewParams BaseParams;
	BaseParams.ViewRect = FIntRect(0, 0, FVirtualShadowMap::VirtualMaxResolutionXY, FVirtualShadowMap::VirtualMaxResolutionXY);
	BaseParams.HZBTestViewRect = BaseParams.ViewRect;
	BaseParams.RasterContextSize = GetPhysicalPoolSize();
	BaseParams.MaxPixelsPerEdgeMultipler = 1.0f / NaniteLODScaleFactor;
	BaseParams.PrevTargetLayerIndex = INDEX_NONE;
	BaseParams.TargetMipLevel = 0;
	BaseParams.TargetMipCount = FVirtualShadowMap::MaxMipLevels;
	// local lights enable distance cull and near clip by default
	BaseParams.Flags = NANITE_VIEW_FLAG_DISTANCE_CULL | NANITE_VIEW_FLAG_NEAR_CLIP;
	if (ProjectedShadowInfo->GetLightSceneInfo().Proxy)
	{
		BaseParams.bUseLightingChannelMask = true;
		BaseParams.LightingChannelMask = ProjectedShadowInfo->GetLightSceneInfo().Proxy->GetLightingChannelMask();
	}

	TSharedPtr<FVirtualShadowMapPerLightCacheEntry> CacheEntry = ProjectedShadowInfo->VirtualShadowMapPerLightCacheEntry;
	check(CacheEntry.IsValid());
	CacheEntry->MarkRendered(Scene.GetFrameNumber());

	if (CVarNonNaniteUseRadiusThreshold.GetValueOnAnyThread() != 0 && CacheEntry->IsUncached())
	{
		BaseParams.Flags |= NANITE_VIEW_MIN_SCREEN_RADIUS_CULL;
		BaseParams.MinBoundsRadius = GMinScreenRadiusForShadowCaster;
	}

	if (CVarVirtualShadowMapNaniteAllowTessellationLocal.GetValueOnAnyThread() == 0)
	{
		BaseParams.Flags |= NANITE_VIEW_FLAG_FORCE_DISPLACEMENT_FALLBACK;
	}

	BaseParams.Flags |= CacheEntry->IsUncached() ? NANITE_VIEW_FLAG_UNCACHED : 0U;
	BaseParams.Flags |= CacheEntry->ShouldUseReceiverMask() ? NANITE_VIEW_FLAG_USE_RECEIVER_MASK : 0U;
	
	Nanite::SetCullingViewOverrides(&CullingView, BaseParams);
	
	for (int32 Index = 0; Index < CacheEntry->ShadowMapEntries.Num(); ++Index)
	{
		FVirtualShadowMapCacheEntry& LevelEntry = CacheEntry->ShadowMapEntries[Index];

		Nanite::FPackedViewParams Params = BaseParams;
		Params.TargetLayerIndex = CacheEntry->GetVirtualShadowMapId() + Index;
		Params.ViewMatrices = ProjectedShadowInfo->GetShadowDepthRenderingViewMatrices(Index, true);
		Params.RangeBasedCullingDistance = ProjectedShadowInfo->GetLightSceneInfo().Proxy->GetRadius();
			
		if (bSetHZBParams)
		{
			LevelEntry.SetHZBViewParams(Params);
		}

		// If we're going to generate a new HZB this frame, save the associated metadata
		if (bUpdateHZBMetaData)
		{
			LevelEntry.UpdateHZBMetadata(Params.ViewMatrices, Params.ViewRect, Params.TargetLayerIndex);
		}

		OutVirtualShadowViews.Add(CreateNanitePackedView(Params));
	}
}

void FVirtualShadowMapArray::AddRenderViews(
	ERenderPassType RenderPassType,
	const FProjectedShadowInfo* ProjectedShadowInfo,
	TConstArrayView<FViewInfo> SceneRendererViews,
	float NaniteLODScaleFactor,
	bool bSetHZBParams,
	bool bUpdateHZBMetaData,
	TArray<Nanite::FPackedView, SceneRenderingAllocator>& OutVirtualShadowViews) const
{
	check(ProjectedShadowInfo->bWholeSceneShadow);
	check(NaniteLODScaleFactor > 0.0f);

	// Use the shadow depth view to inherit the same overrides as the CPU culling logic for non-nanite rendering
	// However, for Nanite we remove the dependency on the ShadowDepthView as that allows skipping the expensive FViewInfo copy for Nanite-only rendering
	if (ProjectedShadowInfo->VirtualShadowMapClipmap)
	{
		const FViewInfo* CullingView = RenderPassType == ERenderPassType::Nanite ? ProjectedShadowInfo->DependentView : ProjectedShadowInfo->ShadowDepthView;
		check(CullingView);
		return AddRenderViewsClipmap(ProjectedShadowInfo->VirtualShadowMapClipmap, *CullingView, NaniteLODScaleFactor, bSetHZBParams, bUpdateHZBMetaData, OutVirtualShadowViews);
	}
	else
	{
		const FViewInfo* CullingView = RenderPassType == ERenderPassType::Nanite ? ProjectedShadowInfo->FindViewForShadow(SceneRendererViews) : ProjectedShadowInfo->ShadowDepthView;
		check(CullingView);
		return AddRenderViewsLocal(ProjectedShadowInfo, *CullingView, NaniteLODScaleFactor, bSetHZBParams, bUpdateHZBMetaData, OutVirtualShadowViews);
	}
}

void FVirtualShadowMapArray::CreateMipViews( TArray<Nanite::FPackedView, SceneRenderingAllocator>& Views ) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateMipViews);
		
	const int32 NumPrimaryViews = Views.Num();

	// 1. create derivative views for each of the Mip levels, 
	Views.AddDefaulted( NumPrimaryViews * ( FVirtualShadowMap::MaxMipLevels - 1) );

	// This is constant based on static defines
	// Replace computed clip space offset from the packed nanite view to align with the raster window
	static constexpr float ClipSpaceScale = 
		float(FVirtualShadowMap::VirtualMaxResolutionXY) / (FVirtualShadowMap::PageSize * FVirtualShadowMap::RasterWindowPages);
	const FVector4f PrimaryClipSpaceScaleOffset = FVector4f(ClipSpaceScale, ClipSpaceScale, ClipSpaceScale - 1.0f, 1.0f - ClipSpaceScale);

	for (int32 ViewIndex = 0; ViewIndex < NumPrimaryViews; ++ViewIndex)
	{
		Nanite::FPackedView& PrimaryView = Views[ViewIndex];
		
		check( PrimaryView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.X >= 0 && PrimaryView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.X < GetNumShadowMapSlots() );
		check( PrimaryView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.Y == 0 );
		check( PrimaryView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.Z > 0 && PrimaryView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.Z <= FVirtualShadowMap::MaxMipLevels );
		check( PrimaryView.ViewRect.X == 0 );
		check( PrimaryView.ViewRect.Y == 0 );
		check( PrimaryView.ViewRect.Z == FVirtualShadowMap::VirtualMaxResolutionXY );
		check( PrimaryView.ViewRect.W == FVirtualShadowMap::VirtualMaxResolutionXY );

		// Replace computed clip space offset from the packed nanite view to align with the raster window
		PrimaryView.ClipSpaceScaleOffset = PrimaryClipSpaceScaleOffset;
		// Set streaming priority category to zero for some reason
		PrimaryView.StreamingPriorityCategory_AndFlags &= ~uint32(NANITE_STREAMING_PRIORITY_CATEGORY_MASK);

		const int32 NumMips = PrimaryView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.Z;
		for (int32 MipLevel = 1; MipLevel < NumMips; ++MipLevel)
		{
			// Primary (Non-Mip views) first followed by derived mip views.
			Nanite::FPackedView& MipView = Views[MipLevel * NumPrimaryViews + ViewIndex];

			MipView = PrimaryView;
			MipView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.Y = MipLevel;

			// Guaranteed by VSM construction - see ensures above
			const int32 MipDim = FVirtualShadowMap::VirtualMaxResolutionXY >> MipLevel;
			MipView.ViewSizeAndInvSize = FVector4f(MipDim, MipDim, 1.0f / MipDim, 1.0f / MipDim);
			MipView.ViewRect = FIntVector4(0, 0, MipDim, MipDim);
			MipView.HZBTestViewRect = MipView.ViewRect;

			// We updated the view scale so need to rebake that into the LODScales
			float ScaleFactor = 1.0f / float(1 << MipLevel);
			MipView.LODScales = ScaleFactor * PrimaryView.LODScales;

			MipView.ClipSpaceScaleOffset.X =  PrimaryView.ClipSpaceScaleOffset.X * ScaleFactor;
			MipView.ClipSpaceScaleOffset.Y =  PrimaryView.ClipSpaceScaleOffset.Y * ScaleFactor;
			MipView.ClipSpaceScaleOffset.Z =  MipView.ClipSpaceScaleOffset.X - 1.0f;
			MipView.ClipSpaceScaleOffset.W = -MipView.ClipSpaceScaleOffset.Y + 1.0f;
		}
	}
}

#if VSM_ENABLE_VISUALIZATION
extern int32 GNaniteVisualizeOverdrawScale;

class FDesaturatePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDesaturatePS);
	SHADER_USE_PARAMETER_STRUCT(FDesaturatePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FDesaturatePS, "/Engine/Private/VirtualShadowMaps/Desaturate.usf", "DesaturatePS", SF_Pixel);

class FTonemapProjectionDebugTexturePS : public FVirtualShadowMapPageManagementShader
{
	DECLARE_GLOBAL_SHADER(FTonemapProjectionDebugTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FTonemapProjectionDebugTexturePS, FVirtualShadowMapPageManagementShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DebugTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DebugTextureSampler)
		SHADER_PARAMETER(uint32, VisualizeModeId)
		SHADER_PARAMETER(int32, VisualizeVirtualShadowMapId)
		SHADER_PARAMETER(float, VisualizeNaniteOverdrawScale)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FTonemapProjectionDebugTexturePS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapDebug.usf", "TonemapProjectionDebugTexturePS", SF_Pixel);

class FDrawShadowCastersFalseColor : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDrawShadowCastersFalseColor);
	SHADER_USE_PARAMETER_STRUCT(FDrawShadowCastersFalseColor, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, DebugAuxTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, DebugAuxTextureViewport)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, SceneColorTextureViewport)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToDebugAuxTextureUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToSceneColorTextureUV)

		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("NANITE_USE_VIEW_UNIFORM_BUFFER"), 1);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FDrawShadowCastersFalseColor, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapShadowCasterColor.usf", "MainPS", SF_Pixel);

class FShadowCasterBoundsVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FShadowCasterBoundsVS);
	SHADER_USE_PARAMETER_STRUCT(FShadowCasterBoundsVS, FGlobalShader);

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return 
			IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			DoesPlatformSupportVirtualShadowMaps(Parameters.Platform) &&
			FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderSRVs(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
	END_SHADER_PARAMETER_STRUCT()
};

class FShadowCasterBoundsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FShadowCasterBoundsPS);
	SHADER_USE_PARAMETER_STRUCT(FShadowCasterBoundsPS, FGlobalShader);

public:
	static const uint32 kMSAASampleCountMax = 8;
	class FSampleCountDimension : SHADER_PERMUTATION_RANGE_INT("MSAA_SAMPLE_COUNT", 1, kMSAASampleCountMax + 1);
	using FPermutationDomain = TShaderPermutationDomain<FSampleCountDimension>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		const int32 SampleCount = PermutationVector.Get<FSampleCountDimension>();

		// Only use permutations with valid MSAA sample counts.
		if (!FMath::IsPowerOfTwo(SampleCount))
		{
			return false;
		}
		if (!RHISupportsMSAA(Parameters.Platform) && SampleCount > 1)
		{
			return false;
		}

		return 
			IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			DoesPlatformSupportVirtualShadowMaps(Parameters.Platform) &&
			FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderSRVs(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SceneDepth)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, DebugDepth)

		SHADER_PARAMETER(int, NumMSAASamples)
		SHADER_PARAMETER_ARRAY(FVector4f, SampleOffsetArray, [kMSAASampleCountMax])
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FShadowCasterBoundsVS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapShadowCasterBounds.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FShadowCasterBoundsPS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapShadowCasterBounds.usf", "MainPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FShadowCasterBoundsParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FShadowCasterBoundsVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FShadowCasterBoundsPS::FParameters, PS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

static void RenderShadowCasterBoundsRHI(
	FRHICommandList& RHICmdList,
	const FViewInfo& View,
	const FIntRect& ViewRect,
	FShadowCasterBoundsParameters* PassParameters,
	int32 NumInstances,
	bool bIsDepthPass)
{
	FShadowCasterBoundsPS::FPermutationDomain PSPermutationVector;
	PSPermutationVector.Set<FShadowCasterBoundsPS::FSampleCountDimension>(PassParameters->PS.NumMSAASamples);

	TShaderMapRef<FShadowCasterBoundsVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FShadowCasterBoundsPS> PixelShader(View.ShaderMap, PSPermutationVector);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

	RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	if (bIsDepthPass)
	{
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();
	}
	else
	{
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	}
	GraphicsPSOInit.DepthStencilAccess = PassParameters->RenderTargets.DepthStencil.GetDepthStencilAccess();
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_One, BF_One>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
	
	ClearUnusedGraphResources(VertexShader, &PassParameters->VS);
	ClearUnusedGraphResources(PixelShader, &PassParameters->PS);
	
	SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);
	
	RHICmdList.SetStreamSource(0, GetUnitCubeVertexBuffer(), 0);
	
	RHICmdList.DrawIndexedPrimitive(GetUnitCubeIndexBuffer(), 0, 0, 8, 0, UE_ARRAY_COUNT(GCubeIndices) / 3, NumInstances);
}
#endif // VSM_ENABLE_VISUALIZATION

void FVirtualShadowMapArray::RenderShadowCasterBounds(FRDGBuilder& GraphBuilder, const FViewInfo& View, int32 ViewIndex, FSceneUniformBuffer& SceneUniformBuffer, const FIntRect& ViewRect, FRDGTextureRef OutputColor, FRDGTextureRef OutputDepth, FRDGTextureRef SceneDepth)
{
#if VSM_ENABLE_VISUALIZATION
	check(OutputColor);
	check(OutputDepth);
	
	FGPUScene& GPUScene = Scene.GPUScene;

	const int32 NumInstances = GPUScene.GetNumInstances();
	const FGPUSceneResourceParameters GPUSceneParameters = GPUScene.GetShaderParameters(GraphBuilder);

	FShadowCasterBoundsParameters* PassParameters = GraphBuilder.AllocParameters<FShadowCasterBoundsParameters>();

	PassParameters->VS.View = View.GetShaderParameters();
	PassParameters->VS.Scene = SceneUniformBuffer.GetBuffer(GraphBuilder);
	PassParameters->VS.VirtualShadowMap = GetUniformBuffer(ViewIndex);

	PassParameters->PS.View = View.GetShaderParameters();
	PassParameters->PS.Scene = SceneUniformBuffer.GetBuffer(GraphBuilder);

	PassParameters->PS.SceneDepth = GraphBuilder.CreateSRV(SceneDepth);

	int32 NumMSAASamples = FMath::Min((uint32)OutputColor->Desc.NumSamples, FShadowCasterBoundsPS::kMSAASampleCountMax);
	PassParameters->PS.NumMSAASamples = NumMSAASamples;
	for (int32 Index = 0; Index < NumMSAASamples; Index++)
	{
		PassParameters->PS.SampleOffsetArray[Index].X = GetMSAASampleOffsets(NumMSAASamples, Index).X;
		PassParameters->PS.SampleOffsetArray[Index].Y = GetMSAASampleOffsets(NumMSAASamples, Index).Y;
	}

	// Depth pre-pass: render debug boxes to depth buffer
	{
		FShadowCasterBoundsParameters* DepthPassParameters = GraphBuilder.AllocParameters<FShadowCasterBoundsParameters>();
		*DepthPassParameters = *PassParameters;

		DepthPassParameters->PS.DebugDepth = GraphBuilder.CreateSRV(GSystemTextures.GetDepthDummy(GraphBuilder)); // Not available, this is what we're rendering
		DepthPassParameters->RenderTargets[0] = FRenderTargetBinding{}; // No color output
		DepthPassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(OutputDepth,
			ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction,
			FExclusiveDepthStencil::DepthWrite_StencilNop);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RenderShadowCasterBounds_Depth"),
			DepthPassParameters, ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
			[DepthPassParameters, NumInstances, &View, ViewRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RenderShadowCasterBoundsRHI(RHICmdList, View, ViewRect, DepthPassParameters, NumInstances, true);
			});
	}

	// Main pass
	{
		PassParameters->PS.DebugDepth = GraphBuilder.CreateSRV(OutputDepth);
		PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputColor, ERenderTargetLoadAction::ELoad);
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding{}; // No depth output

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RenderShadowCasterBounds"),
			PassParameters, ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
			[PassParameters, NumInstances, &View, ViewRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RenderShadowCasterBoundsRHI(RHICmdList, View, ViewRect, PassParameters, NumInstances, false);
			});
	}
#endif // VSM_ENABLE_VISUALIZATION
}

bool FVirtualShadowMapArray::IsVisualizePassEnabled(const FViewInfo& View, int32 ViewIndex, EVSMVisualizationPostPass Pass) const
{
#if VSM_ENABLE_VISUALIZATION
	EDebugViewShaderMode DebugViewShaderMode = View.Family->GetDebugViewShaderMode();
	bool bIsEnabled = View.Family->EngineShowFlags.VisualizeVirtualShadowMap || DebugViewShaderMode == EDebugViewShaderMode::DVSM_ShadowCasters;

	if (CacheManager)
	{
		bIsEnabled = bIsEnabled || CacheManager->IsVisualizePassEnabled(View, ViewIndex, Pass);
	}

	return bIsEnabled;
#else
	return false;
#endif
}

FScreenPassTexture FVirtualShadowMapArray::AddVisualizePass(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View, 
	int32 ViewIndex, 
	EVSMVisualizationPostPass Pass, 
	FSceneUniformBuffer& SceneUniformBuffer, 
	FScreenPassTexture& SceneColor, 
	const FScreenPassTexture& SceneDepth, 
	FScreenPassRenderTarget& OverrideOutput)
{
	FScreenPassTexture Output = MoveTemp(SceneColor);

	auto FinalizeOutput = [](FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& Output, FScreenPassRenderTarget& OverrideOutput) -> FScreenPassTexture&
	{
		if (OverrideOutput.IsValid())
		{
			AddDrawTexturePass(GraphBuilder, View, Output, OverrideOutput);
			return OverrideOutput;
		}

		return Output;
	};

#if VSM_ENABLE_VISUALIZATION

	FScreenPassTextureViewport OutputViewport(Output);
	FScreenPassRenderTarget OutputTarget(Output.Texture, OutputViewport.Rect, ERenderTargetLoadAction::ELoad);

	if (CacheManager)
	{
		Output = CacheManager->AddVisualizePass(GraphBuilder, View, ViewIndex, Pass, SceneColor, OutputTarget);
	}

	const FVirtualShadowMapVisualizationData& VisualizationData = GetVirtualShadowMapVisualizationData();

	FRDGTextureRef DebugAuxTexture = View.GetSceneTextures().DebugAux;

	bool VisualizeShadowCasters = View.Family->GetDebugViewShaderMode() == EDebugViewShaderMode::DVSM_ShadowCasters && HasBeenProduced(DebugAuxTexture);
	bool VisualizeVSM = 
		View.Family->EngineShowFlags.VisualizeVirtualShadowMap
		&& VisualizationData.IsActive()
		&& IsAllocated()
		&& !DebugVisualizationOutput.IsEmpty();

	if (VisualizeShadowCasters)
	{
		if (Pass == EVSMVisualizationPostPass::PreEditorPrimitives)
		{
			FRDGTextureRef SceneColorInput = SceneColor.Texture;
			
			if (SceneColorInput == Output.Texture)
			{
				FRDGTextureRef SceneColorCopy = GraphBuilder.CreateTexture(SceneColorInput->Desc, TEXT("SceneColorCopy"));
				AddCopyTexturePass(GraphBuilder, SceneColorInput, SceneColorCopy);
				SceneColorInput = SceneColorCopy;
			}

			const FScreenPassTextureViewport DebugAuxTextureViewport(DebugAuxTexture);
			const FScreenPassTextureViewport SceneColorTextureViewport(SceneColorInput);

			TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
			TShaderMapRef<FDrawShadowCastersFalseColor> DrawShadowCastersFalseColorPixelShader(View.ShaderMap);

			FDrawShadowCastersFalseColor::FParameters* Parameters = GraphBuilder.AllocParameters<FDrawShadowCastersFalseColor::FParameters>();
			Parameters->View = View.GetShaderParameters();
			Parameters->VirtualShadowMap = GetUniformBuffer(ViewIndex);
			Parameters->Scene = SceneUniformBuffer.GetBuffer(GraphBuilder);

			Parameters->DebugAuxTexture = DebugAuxTexture;
			Parameters->SceneColorTexture = SceneColorInput;

			Parameters->DebugAuxTextureViewport = GetScreenPassTextureViewportParameters(DebugAuxTextureViewport);
			Parameters->SceneColorTextureViewport = GetScreenPassTextureViewportParameters(SceneColorTextureViewport);

			Parameters->ViewportUVToDebugAuxTextureUV = FScreenTransform::ChangeTextureBasisFromTo(
				DebugAuxTextureViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);

			Parameters->ViewportUVToSceneColorTextureUV = FScreenTransform::ChangeTextureBasisFromTo(
				SceneColorTextureViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);

			Parameters->RenderTargets[0] = FRenderTargetBinding(Output.Texture, ERenderTargetLoadAction::ENoAction);
			AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DrawShadowCastersFalseColor"), View, OutputViewport, OutputViewport, VertexShader, DrawShadowCastersFalseColorPixelShader, Parameters, EScreenPassDrawFlags::None);
		}
		else
		{
			FString HeaderLabel = "Shadow Casters";

			FVector2f LegendAnchorPosition = FVector2f(OutputViewport.Rect.Min.X + 8, OutputViewport.Rect.Max.Y - 100);
			FVector2f LegendSize = FVector2f(100.0f, 30.0f);
			TArray<FVisualizationDataLegendEntry, SceneRenderingAllocator> LegendEntries
			{
				FVisualizationDataLegendEntry{LOCTEXT("ShadowCastersStatic","Static, Not Invalidating"), FLinearColor(0, 1, 0)}, 
				FVisualizationDataLegendEntry{LOCTEXT("ShadowCastersStaticInvalidating","Static, Invalidating"), FLinearColor(1, 0, 0)},
				FVisualizationDataLegendEntry{LOCTEXT("ShadowCastersDynamic","Dynamic, Not Invalidating"), FLinearColor(0, 1, 1)}, 
				FVisualizationDataLegendEntry{LOCTEXT("ShadowCastersDynamicInvalidating","Dynamic, Invalidating"), FLinearColor(0.5f, 0, 0.5f)}, 
				FVisualizationDataLegendEntry{LOCTEXT("ShadowCastersDynamicUnknown","Dynamic, Maybe Invalidating"), FLinearColor(0, 0, 1)}, 
				FVisualizationDataLegendEntry{LOCTEXT("ShadowCastersContact","Contact Shadow"), FLinearColor(1, 1, 0)}, 
			};
			AddLegendCanvasPass(GraphBuilder, View, OutputTarget, HeaderLabel, LegendAnchorPosition, LegendSize, LegendEntries);
		}
	}
	else if (VisualizeVSM)
	{
		FScreenPassTextureViewport InputViewport(DebugVisualizationOutput[ViewIndex]->Desc.Extent);

		int ActiveModeId = VisualizationData.GetActiveModeID();
		int VisualizeVirtualShadowMapId = VisualizeLight[ViewIndex].GetVirtualShadowMapId();

		// Resize viewport for layout
		const int32 VisualizeLayout = CVarVisualizeLayout.GetValueOnRenderThread();
		{
			// See CVarVisualizeLayout documentation
			if (VisualizeLayout == 1)		// Thumbnail
			{
				const int32 TileWidth = View.UnscaledViewRect.Width() / 3;
				const int32 TileHeight = View.UnscaledViewRect.Height() / 3;

				OutputViewport.Rect.Max = OutputViewport.Rect.Min + FIntPoint(TileWidth, TileHeight);
			}
			else if (VisualizeLayout == 2)	// Split screen
			{
				InputViewport.Rect.Max.X = InputViewport.Rect.Min.X + (InputViewport.Rect.Width() / 2);
				OutputViewport.Rect.Max.X = OutputViewport.Rect.Min.X + (OutputViewport.Rect.Width() / 2);
			}
		}

		auto DrawDebugVisualizationOutput = [this, &View, ViewIndex, &GraphBuilder, &Output, &OutputViewport, &InputViewport, ActiveModeId, VisualizeVirtualShadowMapId, VisualizeLayout]()
		{
			check(!DebugVisualizationOutput.IsEmpty());

			TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
			TShaderMapRef<FTonemapProjectionDebugTexturePS> PixelShader(View.ShaderMap);

			FTonemapProjectionDebugTexturePS::FParameters* Parameters = GraphBuilder.AllocParameters<FTonemapProjectionDebugTexturePS::FParameters>();
			Parameters->DebugTexture = DebugVisualizationOutput[ViewIndex];
			// Point sampling as DebugTexture could have non-linear data
			Parameters->DebugTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters->VisualizeModeId = ActiveModeId;
			Parameters->VisualizeVirtualShadowMapId = VisualizeVirtualShadowMapId;
			Parameters->VisualizeNaniteOverdrawScale = GNaniteVisualizeOverdrawScale;
			Parameters->RenderTargets[0] = FRenderTargetBinding(Output.Texture, ERenderTargetLoadAction::ENoAction);

			// Blend with scene color if fullscreen, use black background otherwise
			FRHIBlendState* BlendState = 
				VisualizeLayout == 0
				? TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI()
				: TStaticBlendState<>::GetRHI();
			FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

			AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DrawTexture"), View, OutputViewport, InputViewport, VertexShader, PixelShader, BlendState, DepthStencilState, Parameters, EScreenPassDrawFlags::None);
		};

		if (Pass == EVSMVisualizationPostPass::PreEditorPrimitives)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "VirtualShadowMapsVisualization");

			// Desaturate scene color
			{
				TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
				TShaderMapRef<FDesaturatePS> DesaturatePixelShader(View.ShaderMap);

				FRDGTextureRef SceneColorCopy = GraphBuilder.CreateTexture(SceneColor.Texture->Desc, TEXT("SceneColorCopy"));
				AddCopyTexturePass(GraphBuilder, SceneColor.Texture, SceneColorCopy);

				FDesaturatePS::FParameters* Parameters = GraphBuilder.AllocParameters<FDesaturatePS::FParameters>();
				Parameters->InputTexture = SceneColorCopy;
				Parameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				Parameters->RenderTargets[0] = FRenderTargetBinding(Output.Texture, ERenderTargetLoadAction::ENoAction);
				AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Desaturate"), View, OutputViewport, OutputViewport, VertexShader, DesaturatePixelShader, Parameters, EScreenPassDrawFlags::None);
			}

			// Render stuff that blends in with scene
			if (ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_SHADOW_FACTOR
				|| ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_CLIPMAP_OR_MIP
				|| ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_VIRTUAL_PAGE
				|| ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_CACHED_PAGE
				|| ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_SMRT_RAY_COUNT
				|| ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_DIRTY_PAGE
				|| ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_GPU_INVALIDATED_PAGE
				|| ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_MERGED_PAGE
				|| ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_NANITE_OVERDRAW)
			{
				DrawDebugVisualizationOutput();
			}
		}
		else if (Pass == EVSMVisualizationPostPass::PostEditorPrimitives)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "VirtualShadowMapsVisualization");

			// Render stuff that is not part of scene, e.g. UI
			if (ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_CLIPMAP_VIRTUAL_SPACE
				|| ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_GENERAL_DEBUG)
			{
				DrawDebugVisualizationOutput();
			}

			FString ModeName = VisualizationData.GetActiveModeName().ToString();
			FString HeaderLabel = ModeName;
			if (VisualizeLight[ViewIndex].IsValid())
			{
				HeaderLabel = FString::Printf(TEXT("%s (%s)"), *ModeName, *VisualizeLight[ViewIndex].GetLightName());
			}

			FVector2f LegendAnchorPosition = FVector2f(OutputViewport.Rect.Min.X + 8, OutputViewport.Rect.Max.Y - 100);
			FVector2f LegendSize = FVector2f(100.0f, 30.0f);
			TArray<FVisualizationDataLegendEntry, SceneRenderingAllocator> LegendEntries;
			if (ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_SHADOW_FACTOR)
			{
				LegendEntries.Emplace(FVisualizationDataLegendEntry{LOCTEXT("VisualizeShadowFactorLit","Lit"), FLinearColor(1,1,0)});
				LegendEntries.Emplace(FVisualizationDataLegendEntry{LOCTEXT("VisualizeShadowFactorShadow","Shadow"), FLinearColor(0,0,1)});
			}
			else if (ActiveModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_CACHED_PAGE) 
			{
				extern int32 GVisualizeCachedPagesOnly;
				if (VisualizeVirtualShadowMapId != INDEX_NONE || GVisualizeCachedPagesOnly)
				{
					LegendEntries.Emplace(FVisualizationDataLegendEntry{LOCTEXT("VisualizeShadowCacheCached", "Cached"), FLinearColor(0,1,0)});
				}
				if (!GVisualizeCachedPagesOnly)
				{
					LegendEntries.Emplace(FVisualizationDataLegendEntry{LOCTEXT("VisualizeShadowCacheAll", "All Invalidated"), FLinearColor(1,0,0)});
					LegendEntries.Emplace(FVisualizationDataLegendEntry{LOCTEXT("VisualizeShadowCacheDynamic", "Dynamic Invalidated"), FLinearColor(0,0,1)});
					LegendEntries.Emplace(FVisualizationDataLegendEntry{LOCTEXT("VisualizeShadowCacheForce", "Force cached"), FLinearColor(0.75,1,0)});
				}
			}
			AddLegendCanvasPass(GraphBuilder, View, OutputTarget, HeaderLabel, LegendAnchorPosition, LegendSize, LegendEntries);
		}
	}

#endif // VSM_ENABLE_VISUALIZATION

	return MoveTemp(FinalizeOutput(GraphBuilder, View, Output, OverrideOutput));
}

float FVirtualShadowMapArray::InterpolateResolutionBias(float BiasNonMoving, float BiasMoving, float LightMobilityFactor)
{
	return FMath::Lerp(BiasNonMoving, FMath::Max(BiasNonMoving, BiasMoving), LightMobilityFactor);
}

FVirtualShadowMapSamplingParameters FVirtualShadowMapArray::CreateDummySamplingParameters(FRDGBuilder& GraphBuilder)
{
	FVirtualShadowMapSamplingParameters Parameters;
	FVirtualShadowMapUniformParameters* UniformBufferParameters = GraphBuilder.AllocParameters<FVirtualShadowMapUniformParameters>();

	UniformBufferParameters->NumFullShadowMaps						= 0;
	UniformBufferParameters->NumSinglePageShadowMaps				= 0;
	UniformBufferParameters->NumShadowMapSlots						= 0;
	UniformBufferParameters->MaxPhysicalPages						= 0;
	UniformBufferParameters->StaticCachedArrayIndex					= 0;
	UniformBufferParameters->StaticHZBArrayIndex					= 0;
	UniformBufferParameters->bExcludeNonNaniteFromCoarsePages		= 0;
	UniformBufferParameters->CoarsePagePixelThresholdDynamic		= 0.0f;
	UniformBufferParameters->CoarsePagePixelThresholdStatic			= 0.0f;
	UniformBufferParameters->CoarsePagePixelThresholdDynamicNanite	= 0.0f;
	UniformBufferParameters->MipModeLocal							= 0;
	UniformBufferParameters->SceneFrameNumber						= 0;
	
	UniformBufferParameters->ScreenRayLength						= 0.0f;
	UniformBufferParameters->NormalBias								= 0.0f;
	UniformBufferParameters->SMRTAdaptiveRayCount					= 0;
	UniformBufferParameters->SMRTRayCountLocal						= 0;
	UniformBufferParameters->SMRTSamplesPerRayLocal					= 0;
	UniformBufferParameters->SMRTExtrapolateMaxSlopeLocal			= 0.0f;
	UniformBufferParameters->SMRTTexelDitherScaleLocal				= 0.0f;
	UniformBufferParameters->SMRTMaxSlopeBiasLocal					= 0.0f;
	UniformBufferParameters->SMRTCotMaxRayAngleFromLight			= 0.0f;
	
	UniformBufferParameters->SMRTRayCountDirectional				= 0;
	UniformBufferParameters->SMRTSamplesPerRayDirectional			= 0;
	UniformBufferParameters->SMRTExtrapolateMaxSlopeDirectional		= 0.0f;
	UniformBufferParameters->SMRTTexelDitherScaleDirectional		= 0.0f;
	UniformBufferParameters->SMRTRayLengthScale						= 0.0f;

	UniformBufferParameters->SMRTHairRayCount						= 0;

	UniformBufferParameters->PageTableSampler						= TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const uint32 DummyPageTableElement								= 0xFFFFFFFF;
	UniformBufferParameters->PageTable								= GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R32_UINT, DummyPageTableElement);
	UniformBufferParameters->ProjectionData							= GraphBuilder.CreateSRV(GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, sizeof(FVirtualShadowMapProjectionShaderData)));
	UniformBufferParameters->PageFlags								= GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R32_UINT, 0u);
	UniformBufferParameters->PageReceiverMasks						= GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R32_UINT, 0xFFFFFFFFu);

	UniformBufferParameters->UncachedPageRectBounds					= GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FIntVector4)));
	UniformBufferParameters->AllocatedPageRectBounds				= GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FIntVector4)));
	UniformBufferParameters->PerViewData							= MakeEmptyVirtualShadowMapPerViewParameters(GraphBuilder);
	UniformBufferParameters->CachePrimitiveAsDynamic				= GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32)));


	UniformBufferParameters->PhysicalPageRowMask					= 0;
	UniformBufferParameters->PhysicalPageRowShift					= 0;
	UniformBufferParameters->RecPhysicalPoolSize					= FVector4f(1.0f, 1.0f, 1.0f, 1.0f );
	UniformBufferParameters->PhysicalPoolSize						= FIntPoint( 0, 0 );
	UniformBufferParameters->PhysicalPoolSizePages					= FIntPoint( 0, 0 );

	UniformBufferParameters->GlobalResolutionLodBias				= 0.0f;
	UniformBufferParameters->PageTableRowMask						= 0;
	UniformBufferParameters->PageTableRowShift						= 0;
	UniformBufferParameters->PageTableTextureSizeInvSize			= FVector4f(1.0f, 1.0f, 1.0f, 1.0f );
	UniformBufferParameters->PageReceiverMaskTextureSizeInvSize		= FVector4f(1.0f, 1.0f, 1.0f, 1.0f );

	UniformBufferParameters->PackedShadowMaskMaxLightCount			= 0;
	UniformBufferParameters->StaticHZBArrayIndex					= 0;
	UniformBufferParameters->PhysicalPagePool						= GSystemTextures.GetZeroUIntArrayAtomicCompatDummy(GraphBuilder);

	UniformBufferParameters->BilinearClampedSampler = TStaticSamplerState< SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI();
	UniformBufferParameters->RecFilterablePhysicalPoolSize = FVector4f( 0.0f, 0.0f, 1.0f, 1.0f );

	Parameters.VirtualShadowMap = GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
	return Parameters;
}

#undef LOCTEXT_NAMESPACE
