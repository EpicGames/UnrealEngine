// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneManagement.h"
#include "SceneView.h"
#include "VirtualShadowMapDefinitions.h"
#include "ScreenPass.h"
#include "ScenePrivateBase.h"

struct FMinimalSceneTextures;
struct FSortedLightSetSceneInfo;
struct FNaniteVisibilityQuery;
class FViewInfo;
class FProjectedShadowInfo;
class FVisibleLightInfo;
class FVirtualShadowMapCacheEntry;
class FVirtualShadowMapArrayCacheManager;
struct FSortedLightSetSceneInfo;
class FVirtualShadowMapClipmap;
struct FScreenPassTexture;
struct FSingleLayerWaterPrePassResult;
class FNaniteVisibilityResults;
class FSceneRenderer;
class FSceneUniformBuffer;
struct FShaderCompilerEnvironment;
struct FFrontLayerTranslucencyData;
class FSceneInstanceCullingQuery;
class FInstanceCullingManager;
struct FPersistentViewId;
class FVirtualShadowMapSetupContext;

namespace Froxel
{
	class FRenderer;
}

namespace Nanite
{
	struct FPackedView;
	class  FPackedViewArray;
	struct FRasterResults;
	struct FExplicitChunkDrawInfo;
}


// Generally only one pass, but we collect this to handle exceptional cases
struct FNaniteVirtualShadowMapRenderPass
{
	FSceneInstanceCullingQuery *SceneInstanceCullingQuery = nullptr;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> Shadows;
	uint32 TotalPrimaryViews = 0u;
	uint32 MaxCullingViews = 0u;
	Nanite::FPackedViewArray* VirtualShadowMapViews = nullptr;
	Nanite::FExplicitChunkDrawInfo* ExplicitChunkDrawInfo = nullptr;
};

inline bool IsSinglePageVirtualShadowMap(int32 VirtualShadowMapId)
{
	return VirtualShadowMapId < int32(VSM_MAX_SINGLE_PAGE_SHADOW_MAPS);
}

bool DoesVSMWantFroxels(EShaderPlatform ShaderPlatform);

bool IsVirtualShadowMapLocalReceiverMaskEnabled();

class FVirtualShadowMap
{
public:
	// PageSize * Level0DimPagesXY defines the virtual address space, e.g., 128x128 = 16k

	// 128x128 = 16k
	static constexpr uint32 PageSize =  VSM_PAGE_SIZE;
	static constexpr uint32 PageSizeMask =  VSM_PAGE_SIZE_MASK;
	static constexpr uint32 Log2PageSize =  VSM_LOG2_PAGE_SIZE;
	static constexpr uint32 Level0DimPagesXY =  VSM_LEVEL0_DIM_PAGES_XY;
	static constexpr uint32 Log2Level0DimPagesXY =  VSM_LOG2_LEVEL0_DIM_PAGES_XY;
	static constexpr uint32 MaxMipLevels =  VSM_MAX_MIP_LEVELS;
	static constexpr uint32 VirtualMaxResolutionXY =  VSM_VIRTUAL_MAX_RESOLUTION_XY;
	static constexpr uint32 RasterWindowPages = VSM_RASTER_WINDOW_PAGES;

	static constexpr uint32 PhysicalPageAddressBits = 16U;
	static constexpr uint32 MaxPhysicalTextureDimPages = 1U << PhysicalPageAddressBits;
	static constexpr uint32 MaxPhysicalTextureDimTexels = MaxPhysicalTextureDimPages * PageSize;

	static constexpr uint32 NumHZBLevels = Log2PageSize;

	
	static_assert(MaxMipLevels <= 8, ">8 mips requires more PageFlags bits. See VSM_PAGE_FLAGS_BITS_PER_HMIP in PageAccessCommon.ush");

	static EVSMStatSection::Type GetEnabledStatSections();

	// TODO: Currently only used for these constants... probably rename the cache structure to virtual shadow map now instead
private:
	FVirtualShadowMap() {}
};

// Useful data for both the page mapping shader and the projection shader
// as well as cached shadow maps
struct FVirtualShadowMapProjectionShaderData
{
	FMatrix44f ShadowViewToClipMatrix;
	FMatrix44f TranslatedWorldToShadowUVMatrix;
	FMatrix44f ShadowUVToTranslatedWorldMatrix;
	FMatrix44f TranslatedWorldToShadowUVNormalMatrix;

	FVector3f LightDirection;
	uint32 LightType = ELightComponentType::LightType_Directional;

	FVector3f PreViewTranslationHigh;
	float LightRadius;
	
	FVector3f PreViewTranslationLow;
	// Slightly different meaning for clipmaps (includes camera pixel size scaling stuff) and local lights (raw bias)
	float ResolutionLodBias = 0.0f;
	
	// TODO: There are more local lights than directional
	// We should move the directional-specific stuff out to its own structure.
	FVector3f ClipmapWorldOriginOffset;
	float LightSourceRadius;

	FIntPoint ClipmapCornerRelativeOffset = FIntPoint(0, 0);

	// Packed ClipmapLevel, ClipmapLevelCountRemaining and ClipmapLevelIndex, -1 if not a clipmap
	// See PackClipmapLevelAndCount for packing.
	uint32 ClipmapLevel_ClipmapLevelCountRemaining = 0u;

	// See PackCullingViewId for details
	int32 PackedCullingViewId = 0u;

	uint32 Flags = 0U;
	// This clipmap level should allow WPO if this value is less than InstanceWPODisableDistanceSquared
	float ClipmapLevelWPODistanceDisableThresholdSquared = 0.0f;
	float TexelDitherScale;
	
	uint32 MinMipLevel = 0u;

	int32 LightId = -1;
	FVector2f DynamicDepthCullRange = FVector2f(0.0f, FLT_MAX);
	// Note: FMatrix44f forces 16-byte alignment so no need to pad as long as VSM_PSD_STRIDE is kept in sync.

	static uint32 PackCullingViewId(int32 SceneRendererPrimaryViewId, const FPersistentViewId& PersistentViewId);
};
static_assert(sizeof(FVirtualShadowMapProjectionShaderData) == VSM_PSD_STRIDE, "FVirtualShadowMapProjectionShaderData does not match size in shader. See VirtualShadowMapProjectionStructs.ush.");

BEGIN_SHADER_PARAMETER_STRUCT(FVirtualShadowMapPerViewParameters, )
	// Pruned light grid with only the lights that have VSMs present
	// Still references the original indices from the global light grid
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, LightGridData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, NumCulledLightsGrid)
	SHADER_PARAMETER(uint32, MaxLightGridEntryIndex)
	// Directional lights associated with this view
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, DirectionalLightIds)
	SHADER_PARAMETER(uint32, DirectionalLightIdCount)
END_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVirtualShadowMapUniformParameters, )
	SHADER_PARAMETER(uint32, NumFullShadowMaps)
	SHADER_PARAMETER(uint32, NumSinglePageShadowMaps)
	SHADER_PARAMETER(uint32, MaxPhysicalPages)
	SHADER_PARAMETER(uint32, NumShadowMapSlots)
	// Set to 0 if separate static caching is disabled
	SHADER_PARAMETER(uint32, StaticCachedArrayIndex)
	// Set to 0 if separate static caching is disabled OR separate dynamic HZB is disabled
	SHADER_PARAMETER(uint32, StaticHZBArrayIndex)
	
	// use to map linear index to x,y page coord
	SHADER_PARAMETER(uint32, PhysicalPageRowMask)
	SHADER_PARAMETER(uint32, PhysicalPageRowShift)
	SHADER_PARAMETER(uint32, PackedShadowMaskMaxLightCount)
	SHADER_PARAMETER(FVector4f, RecPhysicalPoolSize)
	SHADER_PARAMETER(FIntPoint, PhysicalPoolSize)
	SHADER_PARAMETER(FIntPoint, PhysicalPoolSizePages)
	// TODO: verify whether this turns into 2 loads or one
	SHADER_PARAMETER(uint32, PageTableRowShift)
	SHADER_PARAMETER(uint32, PageTableRowMask)
	// xy == PageTableTextureSize, zw == 1.0f/PageTableTextureSize
	SHADER_PARAMETER(FVector4f, PageTableTextureSizeInvSize) 
	SHADER_PARAMETER(FVector4f, PageReceiverMaskTextureSizeInvSize)

	// Set to 1 if r.Shadow.Virtual.NonNanite.IncludeInCoarsePages is set to 0 in order to signal that we want to use the legacy path for just excluding non-nanite
	SHADER_PARAMETER(uint32, bExcludeNonNaniteFromCoarsePages)
	SHADER_PARAMETER(float, CoarsePagePixelThresholdDynamic)
	SHADER_PARAMETER(float, CoarsePagePixelThresholdStatic)
	SHADER_PARAMETER(float, CoarsePagePixelThresholdDynamicNanite)

	// For shadow page age calculations
	SHADER_PARAMETER(uint32, SceneFrameNumber)

	SHADER_PARAMETER(float, GlobalResolutionLodBias)
	SHADER_PARAMETER(int32, MipModeLocal)

	// SMRT parameters that are sometimes used globally
	SHADER_PARAMETER(float, ScreenRayLength)
	SHADER_PARAMETER(float, NormalBias)
	SHADER_PARAMETER(uint32, SMRTAdaptiveRayCount)
	SHADER_PARAMETER(int32, SMRTRayCountLocal)
	SHADER_PARAMETER(int32, SMRTSamplesPerRayLocal)
	SHADER_PARAMETER(float, SMRTExtrapolateMaxSlopeLocal)
	SHADER_PARAMETER(float, SMRTTexelDitherScaleLocal)
	SHADER_PARAMETER(float, SMRTMaxSlopeBiasLocal)
	SHADER_PARAMETER(float, SMRTCotMaxRayAngleFromLight)

	SHADER_PARAMETER(int32, SMRTRayCountDirectional)
	SHADER_PARAMETER(int32, SMRTSamplesPerRayDirectional)
	SHADER_PARAMETER(float, SMRTExtrapolateMaxSlopeDirectional)
	SHADER_PARAMETER(float, SMRTTexelDitherScaleDirectional)
	SHADER_PARAMETER(float, SMRTRayLengthScale)
		
	SHADER_PARAMETER(uint32, SMRTHairRayCount)

	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, ProjectionData)

	SHADER_PARAMETER_SAMPLER(SamplerState, PageTableSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, PageTable)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, PageFlags)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, PageReceiverMasks)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint4>, AllocatedPageRectBounds)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint4>, UncachedPageRectBounds)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, PhysicalPagePool)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CachePrimitiveAsDynamic)

	SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapPerViewParameters, PerViewData)

	SHADER_PARAMETER_SAMPLER(SamplerState, BilinearClampedSampler)
	SHADER_PARAMETER(FVector4f, RecFilterablePhysicalPoolSize)	
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FVirtualShadowMapSamplingParameters, )
	// NOTE: These parameters must only be uniform buffers/references! Loose parameters do not get bound
	// in some of the forward passes that use this structure.
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FVirtualShadowMapMarkingParameters, )
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutPageRequestFlags)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutPageReceiverMasks)
END_SHADER_PARAMETER_STRUCT()

FMatrix CalcTranslatedWorldToShadowUVMatrix(const FMatrix& TranslatedWorldToShadowView, const FMatrix& ViewToClip);
FMatrix CalcTranslatedWorldToShadowUVNormalMatrix(const FMatrix& TranslatedWorldToShadowView, const FMatrix& ViewToClip);

struct FVirtualShadowMapVisualizeLightSearch
{
public:
	FVirtualShadowMapVisualizeLightSearch()
	{
		Reset();
	}
	
	void Reset()
	{
		FoundKey.Packed = 0;
		FoundProxy = nullptr;
		FoundVirtualShadowMapId = INDEX_NONE;
	}

	void CheckLight(const FLightSceneProxy* CheckProxy, int CheckVirtualShadowMapId);
	void ChooseLight();

	bool IsValid() const { return FoundProxy != nullptr; }

	int GetVirtualShadowMapId() const { return FoundVirtualShadowMapId; }
	const FLightSceneProxy* GetProxy() const { return FoundProxy; }
	const FString GetLightName() const;

private:
	union SortKey
	{
		struct
		{
			// NOTE: Lowest to highest priority
			uint32 bSelected : 1;			// In editor
			uint32 bPartialNameMatch : 1;
			uint32 bExactNameMatch : 1;
		} Fields;
		uint32 Packed;
	};

	SortKey FoundKey;
	const FLightSceneProxy* FoundProxy = nullptr;
	int FoundVirtualShadowMapId = INDEX_NONE;
};

enum class EVSMVisualizationPostPass
{
	PreEditorPrimitives,
	PostEditorPrimitives
};

class FVirtualShadowMapArray
{
public:	
	enum class ERenderPassType
	{
		Nanite,
		NonNanite,
		NUM
	};

	FVirtualShadowMapArray(FScene& InScene);
	~FVirtualShadowMapArray();

	void Initialize(
		FRDGBuilder& GraphBuilder,
		FVirtualShadowMapArrayCacheManager* InCacheManager,
		bool bInEnabled,
		const FEngineShowFlags& EngineShowFlags);

	// Returns true if virtual shadow maps are enabled
	bool IsEnabled() const
	{
		return bEnabled;
	}

	// NOTE: All directional lights must be allocated before all local lights
	// due to receiver mask allocation logic. Unreferenced lights must be allocated last.
	int32 AllocateDirectional(int32 Count);
	int32 AllocateLocal(bool bSinglePageShadowMap, int32 Count);
	int32 AllocateUnreferenced(bool bSinglePageShadowMap, int32 Count);
	
	void UpdateNextData(int32 PrevVirtualShadowMapId, const FNextVirtualShadowMapData& Data);
	
	static bool IsSinglePage(int32 VirtualShadowMapId)
	{
		return (VirtualShadowMapId < VSM_MAX_SINGLE_PAGE_SHADOW_MAPS);
	}

	bool IsDirectional(int32 VirtualShadowMapId)
	{
		int32 FullId = VirtualShadowMapId - int32(VSM_MAX_SINGLE_PAGE_SHADOW_MAPS);
		return FullId >= 0 && FullId < NumDirectionalShadowMaps;
	}

	int32 GetNumShadowMapSlots() const
	{
		return NumShadowMapSlots;
	}

	int32 GetNumFullShadowMaps() const
	{
		return FMath::Max(GetNumShadowMapSlots() - int32(VSM_MAX_SINGLE_PAGE_SHADOW_MAPS), 0);
	}

	int32 GetNumSinglePageShadowMaps() const
	{
		return NumSinglePageShadowMaps;
	}

	/**
	 * Return the total of allocated SMs, both full and single-page SMs
	 */
	int32 GetNumShadowMaps() const
	{
		// If not initialized ShadowMaps is empty, but we want it to return at most 0 anyway
		return GetNumFullShadowMaps() + GetNumSinglePageShadowMaps();
	}

	// Raw size of the physical pool, including both static and dynamic pages (if enabled)
	FIntPoint GetPhysicalPoolSize() const;
	// Size of HZB (level 0)
	FIntPoint GetHZBPhysicalPoolSize() const;

	// Maximum number of physical pages to allocate. This value is NOT doubled when static caching is
	// enabled as we always allocate both as pairs (offset in the page pool).
	uint32 GetMaxPhysicalPages() const { return UniformParameters.MaxPhysicalPages; }
	// Total physical page count that includes separate static pages
	uint32 GetTotalAllocatedPhysicalPages() const;

	EPixelFormat GetPackedShadowMaskFormat() const;

	static void SetShaderDefines(FShaderCompilerEnvironment& OutEnvironment);

	void UpdatePhysicalPageAddresses(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily);

	void BeginMarkPages(
		FRDGBuilder& GraphBuilder,
		const FSceneRenderer& SceneRenderer,
		const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult,
		const FFrontLayerTranslucencyData& FrontLayerTranslucencyData,
		const Froxel::FRenderer& FroxelRenderer,
		bool bAnyLocalLightsWithVSMs);

	void BuildPageAllocations(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		const TConstArrayView<FViewInfo> &Views);

	bool IsAllocated() const
	{
		return PhysicalPagePoolRDG != nullptr && PageTableRDG != nullptr;
	}

	bool HasSeparateDynamicHZB() const
	{
		return UniformParameters.StaticHZBArrayIndex > 0;
	}

	void CreateMipViews(TArray<Nanite::FPackedView, SceneRenderingAllocator>& Views) const;

	/**
	 * Draw Nanite geometry into the VSMs.
	 */
	void RenderVirtualShadowMapsNanite(
		FRDGBuilder& GraphBuilder,
		FSceneRenderer& SceneRenderer,
		bool bUpdateNaniteStreaming,
		const FNaniteVisibilityQuery* VisibilityQuery,
		TConstArrayView<FNaniteVirtualShadowMapRenderPass> VirtualShadowMapPasses);

	/**
	 * Draw Non-Nanite geometry into the VSMs.
	 */
	void RenderVirtualShadowMapsNonNanite(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer, const TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& VirtualSmMeshCommandPasses, TArrayView<FViewInfo> Views);

	void PostRender(FRDGBuilder& GraphBuilder);

	void RenderDebugInfo(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views);
	
	bool ShouldGenerateStats() const;
	bool IsCsvLogEnabled() const;

	void LogStats(FRDGBuilder& GraphBuilder, const FViewInfo& View);

	// Get shader parameters necessary to sample virtual shadow maps
	// It is safe to bind this buffer even if VSMs are disabled, but the sampling should be branched around in the shader.
	// This data becomes valid after the shadow depths pass if VSMs are enabled
	FVirtualShadowMapSamplingParameters GetSamplingParameters(FRDGBuilder& GraphBuilder, int32 ViewIndex) const;

	FVirtualShadowMapMarkingParameters GetMarkingParameters(FRDGBuilder& GraphBuilder, int32 ViewIndex) const;

	TRDGUniformBufferRef<FVirtualShadowMapUniformParameters> GetUniformBuffer(int32 ViewIndex) const
	{
		if (CachedUniformBuffers.IsEmpty())
		{
			return TRDGUniformBufferRef<FVirtualShadowMapUniformParameters>();
		}

		// If the view index is out of range, then it means VSM has not been set up yet, this is not a great time to access the UB but is actully done in skyatmosphere rendering so we need to return a safe default.
		return CachedUniformBuffers[FMath::Min(ViewIndex, CachedUniformBuffers.Num() - 1)];
	}

	bool HasAnyShadowData() const { return PhysicalPagePoolRDG != nullptr;  }

	bool IsPrefilteredDistantEnabled() const { return bEnablePrefilteredDistant; }

	bool ShouldCullBackfacingPixels() const { return bCullBackfacingPixels; }

	void UpdateHZB(FRDGBuilder& GraphBuilder);		

	// Add render views, and mark shadow maps as rendered for a given clipmap or set of VSMs
	void AddRenderViews(
		ERenderPassType RenderPassType,
		const FProjectedShadowInfo* ProjectedShadowInfo,
		TConstArrayView<FViewInfo> SceneRendererViews,
		float NaniteLODScaleFactor,
		bool bSetHzbParams,
		bool bUpdateHZBMetaData,
		TArray<Nanite::FPackedView,	SceneRenderingAllocator>& OutVirtualShadowViews) const;

	// Add visualization composite pass, if enabled
	bool IsVisualizePassEnabled(const FViewInfo& View, int32 ViewIndex, EVSMVisualizationPostPass Pass) const;
	FScreenPassTexture AddVisualizePass(
		FRDGBuilder& GraphBuilder, 
		const FViewInfo& View, 
		int32 ViewIndex, 
		EVSMVisualizationPostPass Pass, 
		FSceneUniformBuffer& SceneUniformBuffer, 
		FScreenPassTexture& SceneColor, 
		const FScreenPassTexture& SceneDepth, 
		FScreenPassRenderTarget& Output);

	void RenderShadowCasterBounds(FRDGBuilder& GraphBuilder, const FViewInfo& View, int32 ViewIndex, FSceneUniformBuffer& SceneUniformBuffer, const FIntRect& ViewRect, FRDGTextureRef OutputColor, FRDGTextureRef OutputDepth, FRDGTextureRef SceneDepth);

	//
	bool UseHzbOcclusion() const { return bUseHzbOcclusion; }

	/**
	 * Helper function to add clamping when interpolating the LOD resolution biases to ensure the bias for moving lights can never be lower than the one for not.
	 * This could occur fairly easily since it is possible to both set the values through console as well as scalability.
	 */
	static float InterpolateResolutionBias(float BiasNonMoving, float BiasMoving, float LightMobilityFactor);

	/**
	 * Creates an uniform buffer filled with dummy values that can be used in scenarios where a shader is used that pulls in the VSM sampling UB but is not actually expected to do anything useful. 
	 */
	static FVirtualShadowMapSamplingParameters CreateDummySamplingParameters(FRDGBuilder& GraphBuilder);

	// We keep a reference to the cache manager that was used to initialize this frame as it owns some of the buffers
	FVirtualShadowMapArrayCacheManager* CacheManager = nullptr;

	FVirtualShadowMapUniformParameters UniformParameters;
	TArray<FVirtualShadowMapPerViewParameters> PerViewParameters;

	// Physical page pool shadow data and associated HZB and metadata
	// NOTE: The underlying textures are owned by FVirtualShadowMapCacheManager.
	// We just import and maintain a copy of the RDG reference for this frame here.
	FRDGTextureRef PhysicalPagePoolRDG = nullptr;
	FRDGTextureRef HZBPhysicalArrayRDG = nullptr;
	FRDGBufferRef PhysicalPageMetaDataRDG = nullptr;

	FRDGTextureRef PrefilteredDistantPhysicalPagePoolSeparateRDG = nullptr;
	FRDGTextureRef PrefilteredDistantPhysicalPagePoolMergedRDG = nullptr;

	// Buffer that serves as the page table for all virtual shadow maps
	FRDGTextureRef PageTableRDG = nullptr;
	// Buffer that holds page requests during marking/page management
	// Later it gets potentially reused to mark invalidations (see VirtualShadowMapArrayCacheManager)
	FRDGTextureRef PageRequestFlagsRDG = nullptr;
	// Buffer that stores flags (uints) marking each page that needs to be rendered and cache status, for all virtual shadow maps.
	// Flag values defined in PageAccessCommon.ush
	FRDGTextureRef PageFlagsRDG = nullptr;

	// Per-page receiver mask 8x8 bits (stored as 2x2 x 4x4 bits - in 32-bits for now because of atomics, later in true 16-bit words hopefully)
	FRDGTextureRef PageReceiverMasksRDG = nullptr;

	// Buffer that receives stats from Nanite about the geometry that was rendered into each VSM
	FRDGBufferRef NanitePerformanceFeedbackRDG = nullptr;

	// Buffer that stores data related to past and current VSM load and performance
	FRDGBufferRef ThrottleBufferRDG = nullptr;
	bool bThrottlingEnabled = false;

	// List(s) of physical pages used during allocation/updates
	// These can be saved frame to frame to allow keeping an LRU-sorted order for cached pages
	FRDGBufferRef PhysicalPageListsRDG = nullptr;

	// Allocation info for each page.
	FRDGBufferRef CachedPageInfosRDG = nullptr;

	// uint4 buffer with one rect for each mip level in all SMs, calculated to bound committed pages
	// Used to clip the rect size of clusters during culling.
	FRDGBufferRef UncachedPageRectBoundsRDG = nullptr;		// For rendering; only includes uncached pages
	FRDGBufferRef AllocatedPageRectBoundsRDG = nullptr;		// For invalidation; includes all mapped/cached pages
	FRDGBufferRef ProjectionDataRDG = nullptr;

	FRDGBufferRef DirtyPageFlagsRDG = nullptr; // Dirty flags that are cleared after render passes
	bool bHZBBuiltThisFrame = false;

	static constexpr uint32 MaxPageAreaDiagnosticSlots = 32;
	static constexpr uint32 MaxNPFDiagnosticSlotsSinglePage = 32;
	static constexpr uint32 MaxNPFDiagnosticSlotsMultiPage = 96;

	FRDGBufferRef StatsBufferRDG = nullptr;
	FRDGBufferUAVRef StatsBufferUAV = nullptr;
	FRDGBufferRef StatsNaniteBufferRDG = nullptr;

	// Debug visualization
	TArray<FRDGTextureRef> DebugVisualizationOutput;
	TArray<FVirtualShadowMapVisualizeLightSearch> VisualizeLight;
	bool bEnableVisualization = false;
	bool bEnableNaniteVisualization = false;

private:
	int32 AllocateShadowMapSlots(bool bSinglePageShadowMap, int32 Count);

	void UpdateVisualizeLight(const TConstArrayView<FViewInfo>& Views, const FVisibleLightInfoArray& VisibleLightInfos);

	void AppendPhysicalPageList(FRDGBuilder& GraphBuilder, bool bEmptyToAvailable);

	void AddRenderViewsLocal(
		const FProjectedShadowInfo* ProjectedShadowInfo,
		const FViewInfo& CullingView,
		float NaniteLODScaleFactor,
		bool bSetHzbParams,
		bool bUpdateHZBMetaData,
		TArray<Nanite::FPackedView,	SceneRenderingAllocator>& OutVirtualShadowViews) const;

	void AddRenderViewsClipmap(
		const TSharedPtr<FVirtualShadowMapClipmap>& Clipmap,
		const FViewInfo& CullingView,
		float NaniteLODScaleFactor,
		bool bSetHzbParams,
		bool bUpdateHZBMetaData,
		TArray<Nanite::FPackedView,
		SceneRenderingAllocator>& OutVirtualShadowViews) const;

	TRDGUniformBufferRef<FVirtualShadowMapUniformParameters> GetUncachedUniformBuffer(FRDGBuilder& GraphBuilder) const;
	void UpdateCachedUniformBuffers(FRDGBuilder& GraphBuilder);
	
	// Track mapping of previous VSM data -> current frame VSM data
	// This is primarily an indirection that allows us to reallocate/repack VirtualShadowMapIds each frame
	TArray<FNextVirtualShadowMapData, SceneRenderingAllocator> NextData;

	FVirtualShadowMapSetupContext* VirtualShadowMapSetupContext = nullptr;

	// Allocation/page table order is:
	// - Single page local light shadow maps (fixed number of slots)
	// - Directional light shadow maps (always full)
	// - Full local light shadow maps
	// - Any unreferenced lights of any type (see FVirtualShadowMapCacheManager::UpdateUnreferencedCacheEntries)

	int32 NumDirectionalShadowMaps = 0;
	int32 NumLocalShadowMaps = 0;			// Includes both single page and full
	int32 NumUnreferencedShadowMaps = 0;	// Includes both single page and full

	int32 NumSinglePageShadowMaps = 0;	// Single page slot allocation counter
	int32 NumShadowMapSlots = 0;		// Total/full allocation counter

	// Gets created in dummy form at initialization time, then updated after VSM data is computed
	TArray<TRDGUniformBufferRef<FVirtualShadowMapUniformParameters>> CachedUniformBuffers;

	FScene &Scene;
	//
	bool bUseHzbOcclusion = true;

	bool bInitialized = false;

	// Are virtual shadow maps enabled? We store this at the start of the frame to centralize the logic.
	bool bEnabled = false;

	// Is backface culling of pixels enabled? We store this here to keep it consistent between projection and generation
	bool bCullBackfacingPixels = false;

	bool bEnablePrefilteredDistant = false;
	FRDGTextureRef PageTableDummyRDG = nullptr;

	FRDGBufferRef PrevProjectionDataRDG = nullptr;
};
