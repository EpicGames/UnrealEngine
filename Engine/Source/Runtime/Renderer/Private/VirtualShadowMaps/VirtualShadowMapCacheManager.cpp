// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualShadowMapCacheManager.h"
#include "CoreMinimal.h"
#include "Containers/AllowShrinking.h"
#include "Containers/ArrayView.h"
#include "Containers/SparseSet.h"
#include "Containers/StringView.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Engine.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "ExtensibleUniformBuffer.h"
#include "GameTime.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "GlobalShader.h"
#include "GPUMessaging.h"
#include "GPUScene.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "InstanceCulling/InstanceCullingLoadBalancer.h"
#include "LightSceneInfo.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "MeshPassProcessor.h"
#include "Misc/CoreDelegates.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Misc/Paths.h"
#include "MultiGPU.h"
#include "Nanite/NaniteShared.h"
#include "NaniteDefinitions.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/CsvProfilerTrace.h"
#include "RendererInterface.h"
#include "LogRenderer.h"
#include "RendererOnScreenNotification.h"
#include "RendererPrivateUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderTargetPool.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIGPUReadback.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneExtensions.h"
#include "SceneInterface.h"
#include "ScenePrimitiveUpdates.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTypes.h"
#include "SceneUniformBuffer.h"
#include "SceneUpdateCommandQueue.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "Shadows/ShadowScene.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "SystemTextures.h"
#include "Trace/Detail/Channel.h"
#include "UnrealEngine.h"
#include "VirtualShadowMapDefinitions.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VirtualShadowMaps/VirtualShadowMapClipmap.h"

#if VSM_ENABLE_VISUALIZATION
#include "Engine/Font.h"
#include "Fonts/SlateFontInfo.h"
#include "Engine/Canvas.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#endif

#define LOCTEXT_NAMESPACE "VirtualShadowMapCacheManager"
CSV_DECLARE_CATEGORY_EXTERN(VSM);

UE_TRACE_CHANNEL_EXTERN(VSMChannel);

static int32 GVSMAccumulateStats = 0;
static FAutoConsoleVariableRef CVarAccumulateStats(
	TEXT("r.Shadow.Virtual.AccumulateStats"),
	GVSMAccumulateStats,
	TEXT("When nonzero, VSM stats will be collected over multiple frames and written to a CSV file output to the Saved/Profiling directory.\n")
	TEXT("  If set to a number N > 0 it will auto disable and write the result after N frames, if < 0 it must be manually turned off by setting back to 0."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarCacheVirtualSMs(
	TEXT("r.Shadow.Virtual.Cache"),
	1,
	TEXT("Turn on to enable caching"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarCacheVsmUseHzb(
	TEXT("r.Shadow.Virtual.Cache.InvalidateUseHZB"),
	1,
	TEXT(" When enabled, instances invalidations are tested against the HZB. Instances that are fully occluded will not cause page invalidations."),
	ECVF_RenderThreadSafe);

static int32 GVSMCacheDeformableMeshesInvalidate = 1;
FAutoConsoleVariableRef CVarCacheDeformableMeshesInvalidate(
	TEXT("r.Shadow.Virtual.Cache.DeformableMeshesInvalidate"),
	GVSMCacheDeformableMeshesInvalidate,
	TEXT("If enabled, Primitive Proxies that are marked as having deformable meshes (HasDeformableMesh() == true) cause invalidations regardless of whether their transforms are updated."),
	ECVF_RenderThreadSafe);

static int32 GVSMCacheDebugSkipRevealedPrimitivesInvalidate = 0;
FAutoConsoleVariableRef CVarCacheDebugSkipRevealedPrimitivesInvalidate(
	TEXT("r.Shadow.Virtual.Cache.DebugSkipRevealedPrimitivesInvalidation"),
	GVSMCacheDebugSkipRevealedPrimitivesInvalidate,
	TEXT("Debug skip invalidation of revealed Non-Nanite primitives, i.e. they go from being culled on the CPU to unculled."),
	ECVF_RenderThreadSafe);

// NOTE: At this point it should be fairly safe and minimal performance impact to have this
// "functionally unlimited", but we'll leave the default somewhat lower as a small mitigation
// for unforeseen issues.
int32 GVSMMaxPageAgeSinceLastRequest = 1000;
FAutoConsoleVariableRef CVarVSMMaxPageAgeSinceLastRequest(
	TEXT("r.Shadow.Virtual.Cache.MaxPageAgeSinceLastRequest"),
	GVSMMaxPageAgeSinceLastRequest,
	TEXT("The maximum number of frames to allow cached pages that aren't requested in the current frame to live. 0=disabled."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarMaxLightAgeSinceLastRequest(
	TEXT("r.Shadow.Virtual.Cache.MaxLightAgeSinceLastRequest"),
	10,
	TEXT("The maximum number of frames to allow lights (and their associated pages) that aren't present in the current frame to live in the cache.\n")
	TEXT("Larger values can allow pages from offscreen local lights to live longer, but can also increase various page table management overheads."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFramesStaticThreshold(
	TEXT("r.Shadow.Virtual.Cache.FramesStaticThreshold"),
	100,
	TEXT("Number of frames without an invalidation before an object will transition to static caching."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVSMReservedResource(
	TEXT("r.Shadow.Virtual.AllocatePagePoolAsReservedResource"),
	1,
	TEXT("Allocate VSM page pool as a reserved/virtual texture, backed by N small physical memory allocations to reduce fragmentation."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarVSMDynamicResolutionMaxLodBias(
	TEXT("r.Shadow.Virtual.DynamicRes.MaxResolutionLodBias"),
	2.0f,
	TEXT("As memory or compute-time cost limits are approached, VSM resolution ramps down by biasing the LOD up, similar to 'ResolutionLodBiasDirectional'.\n")
	TEXT("Memory pressure is based on page allocation compared to pool capacity.\n")
	TEXT("This is the maximum LOD bias to clamp to for global dynamic shadow resolution reduction. 0 = disabled"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarVSMDynamicResolutionMaxLodBiasDirectional(
	TEXT("r.Shadow.Virtual.DynamicRes.MaxComputeResolutionLodBiasDirectional"),
	99999.0f,
	TEXT("As compute-time cost limits are approached, VSM resolution ramps down by biasing the LOD up.\n")
	TEXT("The maximum LOD bias that is applied is min(this variable, MaxResolutionLodBias).\n")
	TEXT("This variable applies to directional lights only."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarVSMDynamicResolutionMaxLodBiasLocal(
	TEXT("r.Shadow.Virtual.DynamicRes.MaxComputeResolutionLodBiasLocal"),
	99999.0f,
	TEXT("As compute-time cost limits are approached, VSM resolution ramps down by biasing the LOD up.\n")
	TEXT("The maximum LOD bias that is applied is min(this variable, MaxResolutionLodBias).\n")
	TEXT("This variable applies to local lights only."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarVSMDynamicResolutionMaxPagePoolLoadFactor(
	TEXT("r.Shadow.Virtual.DynamicRes.MaxPagePoolLoadFactor"),
	0.85f,
	TEXT("If allocation exceeds this factor of total page pool capacity, shadow resolution will be biased downwards. 0 = disabled"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarClipmapWPODisableDistanceInvalidate(
	TEXT("r.Shadow.Virtual.Clipmap.WPODisableDistance.InvalidateOnScaleChange"),
	0,
	TEXT("When enabled, forces an invalidation of clipmap levels when the distance disable clipmap level shifts due to resolution or FOV changes.\n")
	TEXT("This can sometimes make things visually more consistent but can also introduce unpredictable performance spikes, so it is disabled by default."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GVSMLightRadiusInvalidationCulling = 1;
FAutoConsoleVariableRef CVarVSMLightRadiusCulling(
	TEXT("r.Shadow.Virtual.Cache.CPUCullInvalidationsOutsideLightRadius"),
	GVSMLightRadiusInvalidationCulling,
	TEXT("CPU culls invalidations that are outside a local light's radius."),
	ECVF_RenderThreadSafe
);

int32 GVSMAllowScreenOverflowMessages = 1;
FAutoConsoleVariableRef CVarVSMAllowScreenOverflowMessages(
	TEXT("r.Shadow.Virtual.AllowScreenOverflowMessages"),
	GVSMAllowScreenOverflowMessages,
	TEXT("Can be used to disable on-screen VSM-related overflow messages. Even with the messages disabled, overflows will still be logged and can result in visual corruption."),
	ECVF_RenderThreadSafe
);

static bool bDumpLightNaniteStats = false;
static FAutoConsoleCommandWithArgsAndOutputDevice DumpLightNaniteStatsConsoleCommand(
	TEXT("r.Shadow.Virtual.DumpLightNaniteStats"),
	TEXT("Print per-vsm Nanite cluster counts as a performance profiling tool."),
	FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& Output)
		{
			bDumpLightNaniteStats = true;
		}));

static const char *VirtualShadowMap_StatNames[] = 
{
	"REQUESTED_THIS_FRAME_PAGES",
	"STATIC_CACHED_PAGES",
	"STATIC_INVALIDATED_PAGES",
	"DYNAMIC_CACHED_PAGES",
	"DYNAMIC_INVALIDATED_PAGES",
	"EMPTY_PAGES",
	"NON_NANITE_INSTANCES_TOTAL",
	"NON_NANITE_INSTANCES_DRAWN",
	"NON_NANITE_INSTANCES_HZB_CULLED",
	"NON_NANITE_INSTANCES_PAGE_MASK_CULLED",
	"NON_NANITE_INSTANCES_EMPTY_RECT_CULLED",
	"NON_NANITE_INSTANCES_FRUSTUM_CULLED",
	"NUM_PAGES_TO_MERGE",
	"NUM_PAGES_TO_CLEAR",
	"NUM_HZB_PAGES_BUILT",
	"ALLOCATED_NEW",
	"NANITE_CLUSTERS_HW",
	"NANITE_CLUSTERS_SW",
	"NANITE_TRIANGLES",
	"NANITE_INSTANCES_MAIN",
	"NANITE_INSTANCES_POST",
	"WPO_CONSIDERED_PAGES",
	"OVERFLOW_FLAGS",
	"TMP_1",
	"TMP_2",
	"TMP_3",
};
static_assert(UE_ARRAY_COUNT(VirtualShadowMap_StatNames) == VSM_STAT_NUM, "Stat text name array length mismatch!");


namespace Nanite
{
	extern bool IsStatFilterActive(const FString& FilterName);
}

void FVirtualShadowMapCacheEntry::UpdateClipmapLevel(
	const FVirtualShadowMapPerLightCacheEntry& PerLightEntry,
	FInt64Point PageSpaceLocation,
	double LevelRadius,
	double ViewCenterZ,
	double ViewRadiusZ,
	double WPODistanceDisableThresholdSquared)
{
	UpdatePrevHZBMetadata();

	// Not valid if it was never rendered
	bool bCacheValid = (PerLightEntry.RenderedFrameNumber >= 0);
	
	// Not valid if radius has changed
	bCacheValid = bCacheValid && (ViewRadiusZ == Clipmap.ViewRadiusZ);

	// Invalidate if the new Z radius strayed too close/outside the guardband of the cached shadow map
	if (bCacheValid)
	{
		double DeltaZ = FMath::Abs(ViewCenterZ - Clipmap.ViewCenterZ);
		if ((DeltaZ + LevelRadius) > 0.9 * Clipmap.ViewRadiusZ)
		{
			bCacheValid = false;
			//UE_LOGF(LogRenderer, Display, "Invalidated clipmap level (VSM %d) due to depth range movement", VirtualShadowMapId);
		}
	}

	// Not valid if WPO threshold has changed
	if (bCacheValid && CVarClipmapWPODisableDistanceInvalidate.GetValueOnRenderThread() != 0 && (WPODistanceDisableThresholdSquared != Clipmap.WPODistanceDisableThresholdSquared))
	{
		bCacheValid = false;
		// Only warn once per change... when this changes it will hit all of them
		if (&PerLightEntry.ShadowMapEntries[0] == this)
		{
			UE_LOGF(LogRenderer, Display, "Invalidated VSM clipmap due to WPO threshold change. This can occur due to resolution or FOV changes.");
		}
	}

	if (bCacheValid)
	{
		// NOTE: Leave the view center and radius where they were previously for the cached page
		FInt64Point CurrentToPreviousPageOffset(PageSpaceLocation - Clipmap.PageSpaceLocation);
		NextData.Flags = VSM_NEXT_FLAG_KEEP_PAGE;
		NextData.PageAddressOffset = FIntVector2(CurrentToPreviousPageOffset.X, CurrentToPreviousPageOffset.Y);
	}
	else
	{
		NextData.Flags = 0u;
		NextData.PageAddressOffset = FInt32Point(0, 0);
		Clipmap.ViewCenterZ = ViewCenterZ;
		Clipmap.ViewRadiusZ = ViewRadiusZ;
		Clipmap.WPODistanceDisableThresholdSquared = WPODistanceDisableThresholdSquared;		
	}
	
	Clipmap.PageSpaceLocation = PageSpaceLocation;
}

void FVirtualShadowMapCacheEntry::Update(
	const FVirtualShadowMapPerLightCacheEntry& PerLightEntry)
{
	UpdatePrevHZBMetadata();
	
	// Not valid if it was never rendered
	bool bCacheValid = (PerLightEntry.RenderedFrameNumber >= 0);

	if (bCacheValid)
	{
		NextData.Flags = VSM_NEXT_FLAG_KEEP_PAGE;
		NextData.PageAddressOffset = FInt32Point(0, 0);
	}
	else
	{
		NextData.Flags = 0u;
	}

	// Current HZB metadata gets updated during rendering
}

void FVirtualShadowMapCacheEntry::SetHZBViewParams(Nanite::FPackedViewParams& OutParams)
{
	OutParams.PrevTargetLayerIndex = PrevHZBMetadata.TargetLayerIndex;
	OutParams.PrevViewMatrices = PrevHZBMetadata.ViewMatrices;
	OutParams.Flags |= NANITE_VIEW_FLAG_HZBTEST;
}

void FVirtualShadowMapCacheEntry::UpdateHZBMetadata(const FViewMatrices& ViewMatrices, const FIntRect& ViewRect, uint32 TargetLayerIndex)
{
	CurrentHZBMetadata.ViewRect = ViewRect;
	CurrentHZBMetadata.TargetLayerIndex = TargetLayerIndex;

	// As this structure has grown a lot, we try to avoid updating it if we don't have to
	// For shadows, this should be a reasonable test of the relevant parameters that could change
	// TODO: This test is very similar to the tests we do for whether caching is valid; could probably share the two better with some refactoring
	CurrentHZBMetadata.bMatricesDirty =
		ViewMatrices.GetPreViewTranslation() != CurrentHZBMetadata.ViewMatrices.GetPreViewTranslation() ||
		ViewMatrices.GetTranslatedWorldToClip() != CurrentHZBMetadata.ViewMatrices.GetTranslatedWorldToClip();

	if (CurrentHZBMetadata.bMatricesDirty)
	{
		CurrentHZBMetadata.ViewMatrices = ViewMatrices;
	}
}
template <typename CacheKeyType>
void FVirtualShadowMapPerLightCacheEntry::Update(const CacheKeyType& InKey, bool bForceInvalidate, bool bInUseReceiverMask, CacheKeyType& OutCacheKey)
{
	if (bForceInvalidate || InKey != OutCacheKey)
	{
		RenderedFrameNumber = -1;
	}
	OutCacheKey = InKey;
	// If we swap receiver mask on/off we also may need to invalidate due to incomplete pages
	if (bInUseReceiverMask != bUseReceiverMask)
	{
		RenderedFrameNumber = -1;
		bUseReceiverMask = bInUseReceiverMask;
	}

	// If the cache was invalidated for any reason (light movement, etc), we render the next frame as
	// uncached as this is more efficient. Thus continuously moving lights will automatically take the
	// uncached path always without needing to explicitly set ForceInvalidateDirectional. After one static
	// frame though we will swap back so that we can begin establishing static cache data. Thus it is still
	// useful to explicitly set ForceInvalidateDirectional in cases where the light is invalidating frequently
	// but not every single frame to keep the performance consistent.
	bool bNewIsUncached = RenderedFrameNumber < 0;

	// On transition between uncached <-> cached we must invalidate since the static pages may not be initialized
	if (bNewIsUncached != bIsUncached)
	{
		RenderedFrameNumber = -1;
		bIsUncached = bNewIsUncached;
	}

	//// On invalidate, wipe out any existing VSM IDs
	//if (RenderedFrameNumber < 0)
	//{
	//	PrevVirtualShadowMapId = INDEX_NONE;
	//	VirtualShadowMapId = INDEX_NONE;
	//}
}

void FVirtualShadowMapPerLightCacheEntry::UpdateClipmap(const FClipmapCacheKey& InKey, bool bForceInvalidate, bool bInUseReceiverMask)
{
	Update(InKey, bForceInvalidate, bInUseReceiverMask, ClipmapCacheKey);

	// Not used by clipmaps
	LightOrigin = FVector(0, 0, 0);
	LightRadius = -1.0f;
}

void FVirtualShadowMapPerLightCacheEntry::UpdateLocal(
	const FLocalLightCacheKey& InKey,
	const FVector& NewLightOrigin,
	const float NewLightRadius,
	bool bForceInvalidate,
	bool bInUseReceiverMask)
{
	Update(InKey, bForceInvalidate, bInUseReceiverMask, LocalCacheKey);

	LightOrigin = NewLightOrigin;
	LightRadius = NewLightRadius;
}

void FVirtualShadowMapPerLightCacheEntry::UpdateVirtualShadowMapId(int32 NextVirtualShadowMapId)
{
	PrevVirtualShadowMapId = VirtualShadowMapId;
	VirtualShadowMapId = NextVirtualShadowMapId;
}

void FVirtualShadowMapArrayCacheManager::FShadowInvalidatingInstancesImplementation::AddPrimitive(const FPrimitiveSceneInfo* PrimitiveSceneInfo)
{
	AddInstanceRange(PrimitiveSceneInfo->GetPersistentIndex(), PrimitiveSceneInfo->GetInstanceSceneDataOffset(), PrimitiveSceneInfo->GetNumInstanceSceneDataEntries());
}

void FVirtualShadowMapArrayCacheManager::FShadowInvalidatingInstancesImplementation::AddInstanceRange(FPersistentPrimitiveIndex PersistentPrimitiveIndex, uint32 InstanceSceneDataOffset, uint32 NumInstanceSceneDataEntries)
{
	PrimitiveInstancesToInvalidate.Add(FVirtualShadowMapInstanceRange{
		.PersistentPrimitiveIndex = PersistentPrimitiveIndex,
		.InstanceSceneDataOffset = int32(InstanceSceneDataOffset),
		.NumInstanceSceneDataEntries = int32(NumInstanceSceneDataEntries),
		.bMarkAsDynamic = false
	});
}

static uint32 EncodeInstanceInvalidationPayload(int32 VirtualShadowMapId, uint32 Flags = VSM_INVALIDATION_PAYLOAD_FLAG_NONE)
{
	check(VirtualShadowMapId >= 0);		// Should not be INDEX_NONE by this point

	uint32 Payload = Flags;
	Payload = Payload | (uint32(VirtualShadowMapId) << VSM_INVALIDATION_PAYLOAD_FLAG_BITS);
	return Payload;
}

FVirtualShadowMapArrayCacheManager::FInvalidatingPrimitiveCollector::FInvalidatingPrimitiveCollector(
	FVirtualShadowMapArrayCacheManager* InCacheManager)
	: Scene(InCacheManager->Scene)
	, Manager(*InCacheManager)
{
	uint32 Num = Manager.CachePrimitiveAsDynamic.Num();
	InvalidatedPrimitives.SetNum(Num, false);
	RemovedPrimitives.SetNum(Num, false);

	{
		SCOPED_NAMED_EVENT(FScene_VirtualShadowCollectVSMsForInvalidation, FColor::Orange);

		// Make a list of VSM IDs that might need invalidation up front so we can use it
		// for load balancing individual invalidations.
		// NOTE: We still do this twice redundantly (once for pre, once for post), but TBD
		for (auto& It : Manager.CacheEntries)
		{
			const FVirtualShadowMapPerLightCacheEntry& CacheEntry = *It.Value;

			// We don't need explicit invalidations for force invalidated/uncached lights
			// TODO: Should this just be an IsDistantLight check instead? Do we care if it has been rendered yet?
			// TODO: This sort of "is there potentially cached data we need to invalidate" should be
			// wrapped up into a more explicit function.
			if (CacheEntry.IsFullyCached() || CacheEntry.IsUncached() || CacheEntry.IsInvalidated())
			{
				continue;
			}
			int32 VirtualShadowMapId = CacheEntry.GetVirtualShadowMapId();
			if (!ensure(VirtualShadowMapId >= 0))
			{
				continue;
			}

			CacheEntriesForInvalidation.Add(&CacheEntry);
		}
	}
}

void FVirtualShadowMapArrayCacheManager::FInvalidatingPrimitiveCollector::AddPrimitivesToInvalidate()
{	
	const TBitArray<>& ManagerCachePrimitiveAsDynamic = Manager.CachePrimitiveAsDynamic;

	for (auto& CacheEntryIt : Manager.CacheEntries)
	{
		FVirtualShadowMapPerLightCacheEntry& CacheEntry = *CacheEntryIt.Value;

		// Global invalidations
		for (const FVirtualShadowMapInstanceRange& Range : Manager.ShadowInvalidatingInstancesImplementation.PrimitiveInstancesToInvalidate)
		{
			// If something doesn't have a valid persistent primitive index, it is treated as a dynamic primitive
			// See GetCachePrimitiveAsDynamic
			bool bCachePrimativeAsDynamic = true;
			if (Range.PersistentPrimitiveIndex.IsValid() &&
				Range.PersistentPrimitiveIndex.Index < ManagerCachePrimitiveAsDynamic.Num())
			{
				bCachePrimativeAsDynamic = ManagerCachePrimitiveAsDynamic[Range.PersistentPrimitiveIndex.Index];
				if (Range.bMarkAsDynamic)
				{
					InvalidatedPrimitives[Range.PersistentPrimitiveIndex.Index] = true;
				}
			}

			AddInvalidation(
				CacheEntry,
				Range.InstanceSceneDataOffset,
				Range.NumInstanceSceneDataEntries,
				bCachePrimativeAsDynamic);
		}

		// Per-light invalidations
		for (const FVirtualShadowMapInstanceRange& Range : CacheEntry.PrimitiveInstancesToInvalidate)
		{
			bool bCachePrimativeAsDynamic = true;
			if (Range.PersistentPrimitiveIndex.IsValid() &&
				Range.PersistentPrimitiveIndex.Index < ManagerCachePrimitiveAsDynamic.Num())
			{
				bCachePrimativeAsDynamic = ManagerCachePrimitiveAsDynamic[Range.PersistentPrimitiveIndex.Index];
				if (Range.bMarkAsDynamic)
				{
					InvalidatedPrimitives[Range.PersistentPrimitiveIndex.Index] = true;
				}
			}

			AddInvalidation(
				CacheEntry,
				Range.InstanceSceneDataOffset,
				Range.NumInstanceSceneDataEntries,
				bCachePrimativeAsDynamic);
		}

		CacheEntry.PrimitiveInstancesToInvalidate.Reset();
	}
	Manager.ShadowInvalidatingInstancesImplementation.PrimitiveInstancesToInvalidate.Reset();
}

void FVirtualShadowMapArrayCacheManager::FInvalidatingPrimitiveCollector::AddInvalidation(
	FPrimitiveSceneInfo * PrimitiveSceneInfo,
	EInvalidationCause InvalidationCause)
{
	const int32 PrimitiveID = PrimitiveSceneInfo->GetIndex();
	const int32 InstanceSceneDataOffset = PrimitiveSceneInfo->GetInstanceSceneDataOffset();
	if (PrimitiveID < 0 || InstanceSceneDataOffset == INDEX_NONE)
	{
		return;
	}

	const FPrimitiveFlagsCompact PrimitiveFlagsCompact = Scene.PrimitiveFlagsCompact[PrimitiveID];
	if (!PrimitiveFlagsCompact.bCastDynamicShadow || PrimitiveFlagsCompact.bIsForceHidden)
	{
		return;
	}

	const FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitiveSceneInfo->GetPersistentIndex();

	if (InvalidationCause == EInvalidationCause::Removed)
	{
		RemovedPrimitives[PersistentPrimitiveIndex.Index] = true;
		InvalidatedPrimitives[PersistentPrimitiveIndex.Index] = true;
	}
	else if (InvalidationCause == EInvalidationCause::Updated)
	{
		// Suppress invalidations from moved primitives that are marked to behave as if they were static.
		if (PrimitiveSceneInfo->Proxy->GetShadowCacheInvalidationBehavior() == EShadowCacheInvalidationBehavior::Static)
		{
			return;
		}
		InvalidatedPrimitives[PersistentPrimitiveIndex.Index] = true;
	}
	else if (InvalidationCause == EInvalidationCause::Added)
	{
		// Skip marking as dynamic if it is a static mesh (mobility is static & no WPO) or it is forced to behave as static
		// this avoids needing to re-cache all static meshes.
		if (PrimitiveSceneInfo->Proxy->IsMeshShapeOftenMoving() && PrimitiveSceneInfo->Proxy->GetShadowCacheInvalidationBehavior() != EShadowCacheInvalidationBehavior::Static)
		{
			InvalidatedPrimitives[PersistentPrimitiveIndex.Index] = true;
			Manager.CachePrimitiveAsDynamic[PersistentPrimitiveIndex.Index] = true;
		}
	}
	
	const int32 NumInstanceSceneDataEntries = PrimitiveSceneInfo->GetNumInstanceSceneDataEntries();
	const FBoxSphereBounds PrimitiveBounds = PrimitiveSceneInfo->Proxy->GetBounds();
	const bool bCachePrimativeAsDynamic = Manager.CachePrimitiveAsDynamic[PersistentPrimitiveIndex.Index];
	
	for (auto& CacheEntry : CacheEntriesForInvalidation)
	{
		AddInvalidation(
			*CacheEntry,
			InstanceSceneDataOffset,
			NumInstanceSceneDataEntries,
			bCachePrimativeAsDynamic,
			(GVSMLightRadiusInvalidationCulling != 0),
			PrimitiveBounds);
	}
}

void FVirtualShadowMapArrayCacheManager::FInvalidatingPrimitiveCollector::AddInvalidation(
	const FVirtualShadowMapPerLightCacheEntry& CacheEntry,
	int32 InstanceSceneDataOffset,
	int32 NumInstanceSceneDataEntries,
	bool bCachePrimitiveAsDynamic,
	bool bLightRadiusCulling,
	const FBoxSphereBounds& PrimitiveBounds)
{	
	// 
	if (CacheEntry.IsFullyCached() || CacheEntry.IsUncached() || CacheEntry.IsInvalidated())
	{
		return;
	}

	// We don't need explicit dynamic invalidations when using receiver mask
	if (bCachePrimitiveAsDynamic && CacheEntry.ShouldUseReceiverMask())
	{
		return;
	}
	// Quick bounds overlap check to eliminate stuff that is too far away to affect a light
	if (bLightRadiusCulling && !CacheEntry.AffectsBounds(PrimitiveBounds))
	{
		return;
	}
	int32 VirtualShadowMapId = CacheEntry.GetVirtualShadowMapId();
	if (!ensure(VirtualShadowMapId >= 0))
	{
		return;
	}

	// Add item for each shadow map explicitly, inflates host data but improves load balancing
	const int32 NumEntries = CacheEntry.ShadowMapEntries.Num();
	for (int32 Index = 0; Index < NumEntries; ++Index)
	{
		Instances.Add(InstanceSceneDataOffset, NumInstanceSceneDataEntries,
					EncodeInstanceInvalidationPayload(VirtualShadowMapId + Index));
	}
}

IMPLEMENT_SCENE_EXTENSION(FVirtualShadowMapArrayCacheManager);

bool FVirtualShadowMapArrayCacheManager::ShouldCreateExtension(FScene& Scene)
{
	return DoesPlatformSupportVirtualShadowMaps(GetFeatureLevelShaderPlatform(Scene.GetFeatureLevel()));
}

ISceneExtensionUpdater* FVirtualShadowMapArrayCacheManager::CreateUpdater()
{
	// NOTE: We need this check because shader platform can change during scene destruction so we need to ensure we
	// don't try and run shaders on a new platform that doesn't support VSMs...
	if (UseVirtualShadowMaps(Scene.GetShaderPlatform(), Scene.GetFeatureLevel()))
	{
		return new FVirtualShadowMapInvalidationSceneUpdater(*this);
	}

	return nullptr;
}

FVirtualShadowMapArrayCacheManager::FVirtualShadowMapArrayCacheManager(FScene& InScene)
	: ISceneExtension(InScene)
	, ShadowInvalidatingInstancesImplementation(*this)
{
#if !UE_BUILD_SHIPPING
	LastOverflowTimes.Init(-10.0f, VSM_STAT_OVERFLOW_FLAG_NUM);
#endif
}

void FVirtualShadowMapArrayCacheManager::InitExtension(FScene& InScene)
{
	// Handle message with status sent back from GPU
	StatusFeedbackSocket = GPUMessage::RegisterHandler(TEXT("Shadow.Virtual.StatusFeedback"), [this](GPUMessage::FReader Message)
	{
		int32 MessageType = Message.Read<int32>();
		if(MessageType == VSM_STATUS_MSG_PAGE_MANAGEMENT)
		{
			// Goes negative on underflow
			int32 LastFreePhysicalPages = Message.Read<int32>(0);
			const float LastGlobalResolutionLodBias = FMath::AsFloat(Message.Read<uint32>(0U));
		
			CSV_CUSTOM_STAT(VSM, FreePages, LastFreePhysicalPages, ECsvCustomStatOp::Set);

			// Dynamic resolution
			{
				// Could be cvars if needed, but not clearly something that needs to be tweaked currently
				// NOTE: Should react more quickly when reducing resolution than when increasing again
				// TODO: Possibly something smarter/PID-like rather than simple exponential decay
				const float ResolutionDownExpLerpFactor = 0.5f;
				const float ResolutionUpExpLerpFactor = 0.1f;
				const uint32 FramesBeforeResolutionUp = 10;

				const float MaxPageAllocation = CVarVSMDynamicResolutionMaxPagePoolLoadFactor.GetValueOnRenderThread();
				const float MaxLodBias = CVarVSMDynamicResolutionMaxLodBias.GetValueOnRenderThread();
			
				if (MaxPageAllocation > 0.0f)
				{
					const uint32 SceneFrameNumber = Scene.GetFrameNumberRenderThread();

					// Dynamically bias shadow resolution when we get too near the maximum pool capacity
					// NB: In a perfect world each +1 of resolution bias will drop the allocation in half
					float CurrentAllocation = 1.0f - (LastFreePhysicalPages / static_cast<float>(MaxPhysicalPages));
					float AllocationRatio = CurrentAllocation / MaxPageAllocation;
					float TargetLodBias = FMath::Max(0.0f, LastGlobalResolutionLodBias + FMath::Log2(AllocationRatio));

					if (CurrentAllocation <= MaxPageAllocation &&
						(SceneFrameNumber - LastFrameOverPageAllocationBudget) > FramesBeforeResolutionUp)
					{
						GlobalResolutionLodBias = FMath::Lerp(GlobalResolutionLodBias, TargetLodBias, ResolutionUpExpLerpFactor);
					}
					else if (CurrentAllocation > MaxPageAllocation)
					{
						LastFrameOverPageAllocationBudget = SceneFrameNumber;
						GlobalResolutionLodBias = FMath::Lerp(GlobalResolutionLodBias, TargetLodBias, ResolutionDownExpLerpFactor);
					}
				}

				GlobalResolutionLodBias = FMath::Clamp(GlobalResolutionLodBias, 0.0f, MaxLodBias);
			}

#if !UE_BUILD_SHIPPING
			if (LastFreePhysicalPages < 0)
			{
				uint32 PagePoolOverflowTypeIndex = (uint32)FMath::Log2((double)VSM_STAT_OVERFLOW_FLAG_PAGE_POOL);
				LastOverflowTimes[PagePoolOverflowTypeIndex] = float(FPlatformTime::Seconds() - GStartTime);
				if ((LoggedOverflowFlags & VSM_STAT_OVERFLOW_FLAG_PAGE_POOL) == 0)
				{
					static const auto* CVarResolutionLodBiasLocalPtr = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.Shadow.Virtual.ResolutionLodBiasLocal"));
					static const auto* CVarResolutionLodBiasDirectionalPtr = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.Shadow.Virtual.ResolutionLodBiasDirectional"));

					UE_LOGF(LogRenderer, Warning, "Virtual Shadow Map Page Pool overflow (%d page allocations were not served), this will produce visual artifacts (missing shadow), increase the page pool limit or reduce resolution bias to avoid.\n"
						" See r.Shadow.Virtual.MaxPhysicalPages (%d), r.Shadow.Virtual.ResolutionLodBiasLocal (%.2f), r.Shadow.Virtual.ResolutionLodBiasDirectional (%.2f), Global Resolution Lod Bias (%.2f)",
						-LastFreePhysicalPages,
						MaxPhysicalPages,
						CVarResolutionLodBiasLocalPtr->GetValueOnRenderThread(),
						CVarResolutionLodBiasDirectionalPtr->GetValueOnRenderThread(),
						GlobalResolutionLodBias);

					LoggedOverflowFlags |= VSM_STAT_OVERFLOW_FLAG_PAGE_POOL;
				}
			}
			else
			{
				LoggedOverflowFlags &= ~VSM_STAT_OVERFLOW_FLAG_PAGE_POOL;
			}
#endif
		}
		else if(MessageType == VSM_STATUS_MSG_OVERFLOW)
		{
#if !UE_BUILD_SHIPPING
			uint32 OverflowFlags = Message.Read<int32>();
			if (OverflowFlags)
			{
				float CurrentTime = float(FPlatformTime::Seconds() - GStartTime);
				for (uint32 OverflowTypeIndex = 0; OverflowTypeIndex < VSM_STAT_OVERFLOW_FLAG_NUM; OverflowTypeIndex++)
				{
					uint32 OverflowTypeFlag = 1 << OverflowTypeIndex;
					if (OverflowFlags & OverflowTypeFlag)
					{
						LastOverflowTimes[OverflowTypeIndex] = CurrentTime;

						if ((LoggedOverflowFlags & OverflowTypeFlag) == 0)
						{
							const FString& MessageString = GetOverflowMessage(OverflowTypeIndex).ToString();
							UE_LOGF(LogRenderer, Warning, "%ls", *MessageString);
							LoggedOverflowFlags |= OverflowTypeFlag;
						}
					}
				}
			}
#endif
		}
	});

#if !UE_BUILD_SHIPPING
	// Handle message with stats sent back from GPU whenever stats are enabled
	StatsFeedbackSocket = GPUMessage::RegisterHandler(TEXT("Shadow.Virtual.StatsFeedback"), [this](GPUMessage::FReader Message)
	{
		// Culling stats
		int32 NaniteNumTris = Message.Read<int32>(0);
		int32 NanitePostCullNodeCount = Message.Read<int32>(0);

		TConstArrayView<uint32> Stats = Message.ReadCount(VSM_STAT_NUM);

		const bool bInsightsVSMChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(VSMChannel);
		if (bInsightsVSMChannelEnabled)
		{
			// requires 'trace.enable counters,vsm' to see this in ue insights
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.PagesRequested"), Stats[VSM_STAT_REQUESTED_THIS_FRAME_PAGES]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.PagesCachedStatic"), Stats[VSM_STAT_STATIC_CACHED_PAGES]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.PagesInvalidatedStatic"), Stats[VSM_STAT_STATIC_INVALIDATED_PAGES]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.PagesCachedDynamic"), Stats[VSM_STAT_DYNAMIC_CACHED_PAGES]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.PagesInvalidatedDynamic"), Stats[VSM_STAT_DYNAMIC_INVALIDATED_PAGES]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.PagesEmpty"), Stats[VSM_STAT_EMPTY_PAGES]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.NonNanite.InstancesTotal"), Stats[VSM_STAT_NON_NANITE_INSTANCES_TOTAL]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.NonNanite.InstancesDrawn"), Stats[VSM_STAT_NON_NANITE_INSTANCES_DRAWN]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.NonNanite.InstancesHZBCulled"), Stats[VSM_STAT_NON_NANITE_INSTANCES_HZB_CULLED]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.NonNanite.InstancesPageMaskCulled"), Stats[VSM_STAT_NON_NANITE_INSTANCES_PAGE_MASK_CULLED]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.NonNanite.InstancesEmptyRectCulled"), Stats[VSM_STAT_NON_NANITE_INSTANCES_EMPTY_RECT_CULLED]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.NonNanite.InstancesFrustumCulled"), Stats[VSM_STAT_NON_NANITE_INSTANCES_FRUSTUM_CULLED]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.PagesToMerge"), Stats[VSM_STAT_NUM_PAGES_TO_MERGE]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.PagesToClear"), Stats[VSM_STAT_NUM_PAGES_TO_CLEAR]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.HZBPagesBuilt"), Stats[VSM_STAT_NUM_HZB_PAGES_BUILT]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.PagesAllocatedNew"), Stats[VSM_STAT_ALLOCATED_NEW]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.Nanite.ClustersHW"), Stats[VSM_STAT_NANITE_CLUSTERS_HW]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.Nanite.ClustersSW"), Stats[VSM_STAT_NANITE_CLUSTERS_SW]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.Nanite.Triangles"), Stats[VSM_STAT_NANITE_TRIANGLES]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.Nanite.InstancesMain"), Stats[VSM_STAT_NANITE_INSTANCES_MAIN]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.Nanite.InstancesPost"), Stats[VSM_STAT_NANITE_INSTANCES_POST]);
			TRACE_INT_VALUE(TEXT("Shadow.Virtual.PagesWPOConsidered"), Stats[VSM_STAT_WPO_CONSIDERED_PAGES]);
		}

		CSV_CUSTOM_STAT(VSM, NaniteNumTris, NaniteNumTris, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(VSM, NanitePostCullNodeCount, NanitePostCullNodeCount, ECsvCustomStatOp::Set);
#if CSV_PROFILER
		CSV_CUSTOM_STAT(VSM, NonNanitePostCullInstanceCount, int32(Stats[VSM_STAT_NON_NANITE_INSTANCES_DRAWN]), ECsvCustomStatOp::Set);

		if (FCsvProfiler::Get()->IsCapturing_Renderthread())
		{
			static bool bRegisteredInlineStats = false;
			auto StatCatIndex = CSV_CATEGORY_INDEX(VSM);
			if (FCsvProfiler::Get()->IsCategoryEnabled(StatCatIndex))
			{
				for (int32 StatIndex = 0; StatIndex < UE_ARRAY_COUNT(VirtualShadowMap_StatNames); ++StatIndex)
				{
					const char *StatName = VirtualShadowMap_StatNames[StatIndex];
#if CSVPROFILERTRACE_ENABLED
					if (!bRegisteredInlineStats)
					{
						FCsvProfilerTrace::OutputInlineStat(StatName, StatCatIndex);
					}
#endif
					FCsvProfiler::RecordCustomStat(StatName, StatCatIndex, int32(Stats[StatIndex]), ECsvCustomStatOp::Set);
				}
				bRegisteredInlineStats = true;
			}
		}
#endif



		// Large page area items
		LastLoggedPageOverlapAppTime.SetNumZeroed(Scene.GetMaxPersistentPrimitiveIndex());
		float RealTimeSeconds = float(FPlatformTime::Seconds() - GStartTime);

		TConstArrayView<uint32> PageAreaDiags = Message.ReadCount(FVirtualShadowMapArray::MaxPageAreaDiagnosticSlots * 2);
		for (int32 Index = 0; Index < PageAreaDiags.Num(); Index += 2)
		{
			uint32 Overlap = PageAreaDiags[Index];
			uint32 PersistentPrimitiveId = PageAreaDiags[Index + 1];
			int32 PrimitiveIndex = Scene.GetPrimitiveIndex(FPersistentPrimitiveIndex{ int32(PersistentPrimitiveId) });
			if (Overlap > 0 && PrimitiveIndex != INDEX_NONE)
			{
				if (RealTimeSeconds - LastLoggedPageOverlapAppTime[PersistentPrimitiveId] > 5.0f)
				{
					LastLoggedPageOverlapAppTime[PersistentPrimitiveId] = RealTimeSeconds;
					FString ActorName = Scene.Primitives[PrimitiveIndex]->GetOwnerActorNameOrLabelForDebuggingOnly();
					FString PrimitiveName = Scene.Primitives[PrimitiveIndex]->GetFullnameForDebuggingOnly();
					UE_LOGF(LogRenderer, Warning, "Non-Nanite VSM page overlap performance Warning, %d, %ls, %ls", Overlap, *ActorName, *PrimitiveName);
				}
				LargePageAreaItems.Add(PersistentPrimitiveId, FLargePageAreaItem{ Overlap, RealTimeSeconds });
			}
		}

		auto ParseNPFDiagnostic = [&](uint32 VSMId, uint32 LightId, uint32 ClusterCount)
		{
			bool bPrintEntry = false;

			if (ClusterCount > 0)
			{
				bPrintEntry = true;
				NPFDiagnosticTimer.FindOrAdd(VSMId) = 180;
			}
			else
			{
				uint8* FramesRemaining = NPFDiagnosticTimer.Find(VSMId);
				if (FramesRemaining != nullptr)
				{
					bPrintEntry = true;
					if ((*FramesRemaining)-- == 0)
					{
						NPFDiagnosticTimer.Remove(VSMId);
					}
				}
			}

			if (bPrintEntry)
			{
				FString LightNameWithLevel;
				FLightSceneId PersistentLightSceneId{ int32(LightId) };
				if (Scene.Lights.IsValidIndex(PersistentLightSceneId))
				{
					const FLightSceneInfo* LightSceneInfo = Scene.Lights[PersistentLightSceneId].LightSceneInfo;
					if (LightSceneInfo && LightSceneInfo->Proxy)
					{
						FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightNameWithLevel);
					}
				}
				NPFDiagnosticMessages.Add(FString::Printf(TEXT("%6d | %100s | %10d"), VSMId, *LightNameWithLevel, ClusterCount));
			}
		};

		NPFDiagnosticMessages.SetNum(0, EAllowShrinking::No);
		NPFDiagnosticMessages.Add(FString::Printf(TEXT("%6s | %100s | %10s"), TEXT("VSM Id"), TEXT("Light"), TEXT("# Clusters")));
		TConstArrayView<uint32> NPFDiagsSinglePage = Message.ReadCount(FVirtualShadowMapArray::MaxNPFDiagnosticSlotsSinglePage * 2U);
		for (int32 Index = 0; Index < NPFDiagsSinglePage.Num(); Index += 2)
		{
			uint32 VSMId = Index / 2;
			uint32 LightId = NPFDiagsSinglePage[Index];
			uint32 ClusterCount = NPFDiagsSinglePage[Index + 1];
			ParseNPFDiagnostic(VSMId, LightId, ClusterCount);
		}
		TConstArrayView<uint32> NPFDiagsMultiPage = Message.ReadCount(FVirtualShadowMapArray::MaxNPFDiagnosticSlotsMultiPage * 2U);
		for (int32 Index = 0; Index < NPFDiagsMultiPage.Num(); Index += 2)
		{
			uint32 VSMId = VSM_MAX_SINGLE_PAGE_SHADOW_MAPS + (Index / 2);
			uint32 LightId = NPFDiagsMultiPage[Index];
			uint32 ClusterCount = NPFDiagsMultiPage[Index + 1];
			ParseNPFDiagnostic(VSMId, LightId, ClusterCount);
		}
		if (bDumpLightNaniteStats)
		{
			bDumpLightNaniteStats = false;
			for (const FString& Str : NPFDiagnosticMessages)
			{
				if (!Str.IsEmpty())
				{
					UE_LOGF(LogRenderer, Display, "%ls", *Str);
				}
			}
		}
	});
#endif

#if !UE_BUILD_SHIPPING
	ScreenMessageDelegate = FRendererOnScreenNotification::Get().AddLambda([this](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText >& OutMessages)
	{
		float RealTimeSeconds = float(FPlatformTime::Seconds() - GStartTime);

		if (GVSMAllowScreenOverflowMessages != 0)
		{
			for (uint32 OverflowTypeIndex = 0; OverflowTypeIndex < VSM_STAT_OVERFLOW_FLAG_NUM; OverflowTypeIndex++)
			{
				// Show for ~10s after last overflow
				float LastOverflowTime = LastOverflowTimes[OverflowTypeIndex];
				if (LastOverflowTime >= 0.0f && RealTimeSeconds - LastOverflowTime < 10.0f)
				{
					FText OverflowMessage = GetOverflowMessage(OverflowTypeIndex);
					OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("%s (%0.0f seconds ago). See r.Shadow.Virtual.AllowScreenOverflowMessages."), *(OverflowMessage.ToString()), RealTimeSeconds - LastOverflowTime)));
				}
			}
		}

		for (const auto& Item : LargePageAreaItems)
		{
			int32 PrimtiveIndex = Scene.GetPrimitiveIndex(FPersistentPrimitiveIndex{ int32(Item.Key) });
			uint32 Overlap = Item.Value.PageArea;
			if (PrimtiveIndex != INDEX_NONE && RealTimeSeconds - Item.Value.LastTimeSeen < 2.5f)
			{
				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("Non-Nanite VSM page overlap performance Warning: Primitive '%s' overlapped %d Pages"), *Scene.Primitives[PrimtiveIndex]->GetOwnerActorNameOrLabelForDebuggingOnly(), Overlap)));
			}
		}
		TrimLoggingInfo();

		if (GVSMAccumulateStats > 0)
		{
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("Virtual Shadow Map Stats Accumulation (%d frames left)"), GVSMAccumulateStats)));
		}
		else if (GVSMAccumulateStats < 0)
		{
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("Virtual Shadow Map Stats Accumulation Active. Set r.Shadow.Virtual.AccumulateStats to 0 to stop."))));
		}
	});
#endif
}

FVirtualShadowMapArrayCacheManager::~FVirtualShadowMapArrayCacheManager()
{
#if !UE_BUILD_SHIPPING
	FRendererOnScreenNotification::Get().Remove(ScreenMessageDelegate);
#endif
}

bool FVirtualShadowMapArrayCacheManager::IsVisualizePassEnabled(const FViewInfo& View, int32 ViewIndex, EVSMVisualizationPostPass Pass) const
{
#if !UE_BUILD_SHIPPING
	return Pass == EVSMVisualizationPostPass::PostEditorPrimitives && (FVirtualShadowMap::GetEnabledStatSections() & EVSMStatSection::Clusters);
#else
	return false;
#endif
}

FScreenPassTexture FVirtualShadowMapArrayCacheManager::AddVisualizePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, int32 ViewIndex, EVSMVisualizationPostPass Pass, FScreenPassTexture& SceneColor, FScreenPassRenderTarget& Output)
{
#if !UE_BUILD_SHIPPING
	if (Pass == EVSMVisualizationPostPass::PostEditorPrimitives && (FVirtualShadowMap::GetEnabledStatSections() & EVSMStatSection::Clusters))
	{
		AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("NPFDiagnostics"), View, Output,
			[&](FCanvas& Canvas)
			{
				const float DPIScale = Canvas.GetDPIScale();
				Canvas.SetBaseTransform(FMatrix(FScaleMatrix(DPIScale) * Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

				auto DrawColorTile = [&](float X, float Y, float Width, float Height, const FLinearColor& Color)
				{
					Canvas.DrawTile(X / DPIScale, Y / DPIScale, Width / DPIScale, Height / DPIScale, 0, 0, 0, 0, Color);
				};

				auto MeasureString = [&](FStringView Text, int FontSize = -1, const UFont* Font = GetStatsFont())
				{
					FCanvasTextStringViewItem TextItem(FVector2D::ZeroVector, Text, Font, FLinearColor::Transparent);
					
					TextItem.bOutlined = true;
					TextItem.OutlineColor = FLinearColor::Transparent;
					if (FontSize > 0)
					{
						TextItem.SlateFontInfo = Font->GetLegacySlateFontInfo();
						TextItem.SlateFontInfo->Size = FontSize;
					}

					Canvas.DrawItem(TextItem);
					return TextItem.DrawnSize;
				};

				auto DrawString = [&](float X, float Y, FStringView Text, int FontSize = -1, const FLinearColor& Color = FLinearColor::White, const UFont* Font = GetStatsFont())
				{
					FCanvasTextStringViewItem TextItem(FVector2D(X / DPIScale, Y / DPIScale), Text, Font, Color);
					
					TextItem.bOutlined = true;
					TextItem.OutlineColor = FLinearColor::Black;
					if (FontSize > 0)
					{
						TextItem.SlateFontInfo = Font->GetLegacySlateFontInfo();
						TextItem.SlateFontInfo->Size = FontSize;
					}

					Canvas.DrawItem(TextItem);
					return TextItem.DrawnSize;
				};

				FVector2D MessageBoxSize;
				{
					FIntPoint MessageBoxCharacters {0, NPFDiagnosticMessages.Num()};
					for (const FString& Message : NPFDiagnosticMessages)
					{
						MessageBoxCharacters.X = FMath::Max(MessageBoxCharacters.X, Message.Len());
					}

					FVector2D CharSize = MeasureString(TEXT("A"), 8, GEngine->GetMonospaceFont());
					MessageBoxSize = CharSize * MessageBoxCharacters + FVector2D(10, 10);
				}
				
				int BaselineX = 10;
				int BaselineY = Canvas.GetViewRect().Height() - (Canvas.GetViewRect().Height() / 3);
				BaselineY = FMath::Min(BaselineY, Canvas.GetViewRect().Height() - (MessageBoxSize.Y + 10));
				FLinearColor MessageBoxColor(0.1f, 0.1f, 0.1f, 0.4f);

				DrawColorTile(BaselineX, BaselineY, MessageBoxSize.X, MessageBoxSize.Y, MessageBoxColor);
				int Y = 0;
				for (const FString& Message : NPFDiagnosticMessages)
				{
					Y += DrawString(BaselineX, BaselineY + Y, Message, 8, FLinearColor::White, GEngine->GetMonospaceFont()).Y;
				}
			});
		NPFDiagnosticMessages.SetNum(0, EAllowShrinking::No);
	}
#endif

	return Output;
}

#if !UE_BUILD_SHIPPING
FText FVirtualShadowMapArrayCacheManager::GetOverflowMessage(uint32 OverflowTypeIndex) const
{
	uint32 OverflowTypeFlag = 1 << OverflowTypeIndex;
	switch (OverflowTypeFlag)
	{
		case VSM_STAT_OVERFLOW_FLAG_MARKING_JOB_QUEUE: return LOCTEXT("VSM_MarkingJobQueueOverflow", "[VSM] Non-Nanite Marking Job Queue overflow. Performance may be affected. This occurs when many non-nanite meshes cover a large area of the shadow map.");
		case VSM_STAT_OVERFLOW_FLAG_OPP_MAX_LIGHTS: return LOCTEXT("VSM_OPPMaxLightsOverflow", "[VSM] One Pass Projection max lights overflow. If you see shadow artifacts, decrease the amount of local lights per pixel, or increase r.Shadow.Virtual.OnePassProjection.MaxLightsPerPixel.");
		case VSM_STAT_OVERFLOW_FLAG_PAGE_POOL: return LOCTEXT("VSM_PagePoolOverflow", "[VSM] Page Pool overflow detected, this will produce visual artifacts (missing shadow). Increase the page pool limit or reduce resolution bias to avoid.");
		case VSM_STAT_OVERFLOW_FLAG_VISIBLE_INSTANCES: return LOCTEXT("VSM_VisibleInstancesOverflow", "[VSM] Non-Nanite visible instances buffer overflow detected, this will produce visual artifacts (missing shadow).");
		default: return LOCTEXT("VSM_UnknownOverflow", "[VSM] Unknown overflow");
	}
}
#endif

void FVirtualShadowMapArrayCacheManager::SetPhysicalPoolSize(FRDGBuilder& GraphBuilder, FIntPoint RequestedSize, int RequestedArraySize, uint32 RequestedMaxPhysicalPages)
{
	bool bInvalidateCache = false;

	// Using ReservedResource|ImmediateCommit flags hint to the RHI that the resource can be allocated using N small physical memory allocations,
	// instead of a single large contighous allocation. This helps Windows video memory manager page allocations in and out of local memory more efficiently.
	ETextureCreateFlags RequestedCreateFlags = (CVarVSMReservedResource.GetValueOnRenderThread() && GRHIGlobals.ReservedResources.Supported)
		? (TexCreate_ReservedResource | TexCreate_ImmediateCommit)
		: TexCreate_None;

	if (!PhysicalPagePool 
		|| PhysicalPagePool->GetDesc().Extent != RequestedSize 
		|| PhysicalPagePool->GetDesc().ArraySize != RequestedArraySize
		|| RequestedMaxPhysicalPages != MaxPhysicalPages
		|| PhysicalPagePoolCreateFlags != RequestedCreateFlags)
	{
		if (PhysicalPagePool)
		{
			UE_LOGF(LogRenderer, Display, "Recreating Shadow.Virtual.PhysicalPagePool due to size or flags change. This will also drop any cached pages.");
		}

		// Track changes to these ourselves instead of from the GetDesc() since that may get manipulated internally
		PhysicalPagePoolCreateFlags = RequestedCreateFlags;
        
        const ETextureCreateFlags PoolTexCreateFlags = TexCreate_ShaderResource | TexCreate_UAV | TexCreate_AtomicCompatible;
        
		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DArrayDesc(
			RequestedSize,
			PF_R32_UINT,
			FClearValueBinding::None,
			PhysicalPagePoolCreateFlags,
            PoolTexCreateFlags,
			false,
			RequestedArraySize
		);
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, PhysicalPagePool, TEXT("Shadow.Virtual.PhysicalPagePool"));

		MaxPhysicalPages = RequestedMaxPhysicalPages;

		// Allocate page metadata alongside
		FRDGBufferRef PhysicalPageMetaDataRDG = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(UE::HLSL::FPhysicalPageMetaData), MaxPhysicalPages),
			TEXT("Shadow.Virtual.PhysicalPageMetaData"));
		// Persistent, so we extract it immediately
		PhysicalPageMetaData = GraphBuilder.ConvertToExternalBuffer(PhysicalPageMetaDataRDG);

		bInvalidateCache = true;
	}

	if (bInvalidateCache)
	{
		Invalidate(GraphBuilder);
	}
}

void FVirtualShadowMapArrayCacheManager::FreePhysicalPool(FRDGBuilder& GraphBuilder)
{
	if (PhysicalPagePool)
	{
		PhysicalPagePool = nullptr;
		PhysicalPageMetaData = nullptr;
		Invalidate(GraphBuilder);
	}
}

void FVirtualShadowMapArrayCacheManager::SetHZBPhysicalPoolSize(FRDGBuilder& GraphBuilder, FIntPoint RequestedHZBSize, int32 RequestedArraySize, const EPixelFormat Format)
{
	if (!HZBPhysicalPagePoolArray 
		|| HZBPhysicalPagePoolArray->GetDesc().Extent != RequestedHZBSize 
		|| HZBPhysicalPagePoolArray->GetDesc().Format != Format
		|| HZBPhysicalPagePoolArray->GetDesc().ArraySize != RequestedArraySize )
	{
		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DArrayDesc(
			RequestedHZBSize,
			Format,
			FClearValueBinding::None,
			GFastVRamConfig.HZB,
			TexCreate_ShaderResource | TexCreate_UAV,
			false,
			RequestedArraySize,
			FVirtualShadowMap::NumHZBLevels);

		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, HZBPhysicalPagePoolArray, TEXT("Shadow.Virtual.HZBPhysicalPagePool"));

		// TODO: Clear to black?

		Invalidate(GraphBuilder);
	}
}

void FVirtualShadowMapArrayCacheManager::FreeHZBPhysicalPool(FRDGBuilder& GraphBuilder)
{
	if (HZBPhysicalPagePoolArray)
	{
		HZBPhysicalPagePoolArray = nullptr;
		Invalidate(GraphBuilder);
	}
}

void FVirtualShadowMapArrayCacheManager::Invalidate(FRDGBuilder& GraphBuilder)
{
	PrevBuffers = FVirtualShadowMapArrayFrameData();
	//UE_LOGF(LogRenderer, Display, "Virtual shadow map cache invalidated.");
}

bool FVirtualShadowMapArrayCacheManager::IsCacheEnabled()
{
	return CVarCacheVirtualSMs.GetValueOnRenderThread() != 0;
}

bool FVirtualShadowMapArrayCacheManager::IsCacheDataAvailable()
{
	return IsCacheEnabled() &&
		PhysicalPagePool &&
		PhysicalPageMetaData &&
		PrevBuffers.PageTable &&
		PrevBuffers.PageFlags &&
		PrevBuffers.UncachedPageRectBounds &&
		PrevBuffers.AllocatedPageRectBounds &&
		PrevBuffers.ProjectionData &&
		PrevBuffers.PhysicalPageLists &&
		PrevBuffers.PageRequestFlags;
}

bool FVirtualShadowMapArrayCacheManager::IsHZBDataAvailable()
{
	// NOTE: HZB can be used/valid even when physical page caching is disabled
	return HZBPhysicalPagePoolArray && PrevBuffers.PageTable && PrevBuffers.PageFlags;
}

UE::Renderer::Private::IShadowInvalidatingInstances* FVirtualShadowMapArrayCacheManager::GetInvalidatingInstancesInterface()
{ 
	if (IsCacheDataAvailable())
	{
		return &ShadowInvalidatingInstancesImplementation; 
	}
	return nullptr;
}

FRDGBufferRef FVirtualShadowMapArrayCacheManager::UploadCachePrimitiveAsDynamic(FRDGBuilder& GraphBuilder) const
{
	const uint32 NumElements = FMath::Max(1, FMath::DivideAndRoundUp(CachePrimitiveAsDynamic.Num(), 32));
	//GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), BufferElements), 
	
	FRDGBufferRef CachePrimitiveAsDynamicRDG = CreateStructuredBuffer(
		GraphBuilder,
		TEXT("CachePrimitiveAsDynamic"),
		sizeof(uint32),
		NumElements,
		CachePrimitiveAsDynamic.GetData(),
		FMath::DivideAndRoundUp(CachePrimitiveAsDynamic.Num(), 8)	// Size in bytes of initial data
	);

	return CachePrimitiveAsDynamicRDG;
}

TSharedPtr<FVirtualShadowMapPerLightCacheEntry> FVirtualShadowMapArrayCacheManager::FindCreateLightCacheEntry(
	FLightSceneId LightSceneId, uint32 ViewUniqueID, uint32 NumShadowMaps, uint32 TypeIdTag)
{
	const FVirtualShadowMapCacheKey CacheKey = { ViewUniqueID, LightSceneId, TypeIdTag };

	TSharedPtr<FVirtualShadowMapPerLightCacheEntry>* LightEntryKey = CacheEntries.Find(CacheKey);

	if (LightEntryKey)
	{
		TSharedPtr<FVirtualShadowMapPerLightCacheEntry> LightEntry = *LightEntryKey;

		if (LightEntry->ShadowMapEntries.Num() == NumShadowMaps)
		{
			LightEntry->bReferencedThisRender = true;
			LightEntry->LastReferencedFrameNumber = Scene.GetFrameNumberRenderThread();
			return LightEntry;
		}
		else
		{
			// Remove this entry and create a new one below
			// NOTE: This should only happen for clipmaps currently on cvar changes
			UE_LOGF(LogRenderer, Display, "Virtual shadow map cache invalidated for light due to clipmap level count change");
			CacheEntries.Remove(CacheKey);
		}
	}

	// Make new entry for this light
	TSharedPtr<FVirtualShadowMapPerLightCacheEntry> LightEntry = MakeShared<FVirtualShadowMapPerLightCacheEntry>(Scene.GetMaxPersistentPrimitiveIndex(), NumShadowMaps);
	LightEntry->bReferencedThisRender = true;
	LightEntry->LastReferencedFrameNumber = Scene.GetFrameNumberRenderThread();
	CacheEntries.Add(CacheKey, LightEntry);

	return LightEntry;
}

void FVirtualShadowMapPerLightCacheEntry::OnPrimitiveRendered(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bPrimitiveRevealed)
{
	bool bInvalidate = false;
	bool bMarkAsDynamic = true;

	// Deformable mesh primitives need to trigger invalidation (even if they did not move) or we get artifacts, for example skinned meshes that are animating but not currently moving.
	// Skip if the invalidation mode is NOT auto (because Always will do it elsewhere & the others should prevent this).
	if (GVSMCacheDeformableMeshesInvalidate != 0 &&
		PrimitiveSceneInfo->Proxy->HasDeformableMesh() &&
		PrimitiveSceneInfo->Proxy->GetShadowCacheInvalidationBehavior() == EShadowCacheInvalidationBehavior::Auto)
	{
		bInvalidate = true;
	}
	// With new invalidations on, we need to invalidate any time a (non-nanite) primitive is "revealed", i.e. stopped being culled.
	// Note that this invalidation will be a frame late - similar to WPO starting - as it will get picked up by the next scene update.
	else if (bPrimitiveRevealed && GVSMCacheDebugSkipRevealedPrimitivesInvalidate == 0)
	{	
		bInvalidate = true;
		bMarkAsDynamic = false;		// Don't mark primitives as dynamic just because they were revealed
		//UE_LOGF(LogRenderer, Display, "VirtualShadowMapCacheManager: Primitive revealed %d!", PrimitiveSceneInfo->GetPersistentIndex().Index);
	}

	if (bInvalidate)
	{
		PrimitiveInstancesToInvalidate.Add(FVirtualShadowMapInstanceRange{
			PrimitiveSceneInfo->GetPersistentIndex(),
			PrimitiveSceneInfo->GetInstanceSceneDataOffset(),
			PrimitiveSceneInfo->GetNumInstanceSceneDataEntries(),
			bMarkAsDynamic
		});
	}
}

void FVirtualShadowMapArrayCacheManager::UpdateUnreferencedCacheEntries(
	FVirtualShadowMapArray& VirtualShadowMapArray)
{
	if (!VirtualShadowMapArray.IsEnabled())
	{
		return;
	}

	const uint32 SceneFrameNumber = Scene.GetFrameNumberRenderThread();
	const int32 MaxLightAge = CVarMaxLightAgeSinceLastRequest.GetValueOnRenderThread();

	for (FEntryMap::TIterator It = CacheEntries.CreateIterator(); It; ++It)
	{
		bool bKeepEntry = true;

		TSharedPtr<FVirtualShadowMapPerLightCacheEntry> CacheEntry = (*It).Value;
		// For this test we care if it is active *this render*, not just this scene frame number (which can include multiple renders)
		if (CacheEntry->bReferencedThisRender)
		{
			// Active this render, leave it alone
			check((CacheEntry->GetVirtualShadowMapId() + CacheEntry->ShadowMapEntries.Num()) <= VirtualShadowMapArray.GetNumShadowMapSlots());
		}
		else if (int32(SceneFrameNumber - CacheEntry->LastReferencedFrameNumber) <= MaxLightAge)
		{
			// Not active this render, but still recent enough to keep it and its pages alive
			// Reallocate new VSM ID for this frame
			int32 NumMaps = CacheEntry->ShadowMapEntries.Num();
			const int32 NextVirtualShadowMapId = VirtualShadowMapArray.AllocateUnreferenced(CacheEntry->IsDistantLight(), NumMaps);
			CacheEntry->UpdateVirtualShadowMapId(NextVirtualShadowMapId);
			for (int32 Map = 0; Map < NumMaps; ++Map)
			{
				CacheEntry->ShadowMapEntries[Map].Update(*CacheEntry);
				// Mark it as inactive for this frame/render
				// NOTE: We currently recompute/overwrite the whole ProjectionData structure for referenced lights, but if that changes we
				// will need to clear this flag again when they become referenced.
				CacheEntry->ShadowMapEntries[Map].ProjectionData.Flags |= VSM_PROJ_FLAG_UNREFERENCED;
			}
		}
		else
		{
			bKeepEntry = false;
		}

		if (bKeepEntry)
		{
			// Update next data (for each entry)
			const int32 PrevVirtualShadowMapId = CacheEntry->GetPrevVirtualShadowMapId();
			if (PrevVirtualShadowMapId >= 0)
			{
				const int32 BaseId = CacheEntry->GetVirtualShadowMapId();
				if (ensure(BaseId >= 0))
				{
					for (int32 Index = 0; Index < CacheEntry->ShadowMapEntries.Num(); ++Index)
					{
						auto& NextData = CacheEntry->ShadowMapEntries[Index].NextData;
						NextData.NextVirtualShadowMapId = BaseId + Index;
						NextData.Flags |= VSM_NEXT_FLAG_ID_VALID;
						VirtualShadowMapArray.UpdateNextData(PrevVirtualShadowMapId + Index, NextData);
					}
				}
			}
		}
		else
		{
			It.RemoveCurrent();
		}
	}
}

class FVirtualSmCopyStatsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVirtualSmCopyStatsCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualSmCopyStatsCS, FGlobalShader)
public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, InStatsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FNaniteStats>, NaniteStatsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, AccumulatedStatsBufferOut)
	END_SHADER_PARAMETER_STRUCT()
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("MAX_STAT_FRAMES"), FVirtualShadowMapArrayCacheManager::MaxStatFrames);
	}
};
IMPLEMENT_GLOBAL_SHADER(FVirtualSmCopyStatsCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapCopyStats.usf", "CopyStatsCS", SF_Compute);

void FVirtualShadowMapArrayCacheManager::ExtractFrameData(
	FRDGBuilder& GraphBuilder,	
	FVirtualShadowMapArray &VirtualShadowMapArray,
	const FSceneRenderer& SceneRenderer,
	bool bAllowPersistentData)
{
	TrimLoggingInfo();

	const bool bNewShadowData = VirtualShadowMapArray.IsAllocated();
	const bool bDropAll = !bAllowPersistentData;
	const bool bDropPrevBuffers = bDropAll || bNewShadowData;

	if (bDropPrevBuffers)
	{
		PrevBuffers = FVirtualShadowMapArrayFrameData();
		PrevUniformParameters.NumFullShadowMaps = 0;
		PrevUniformParameters.NumSinglePageShadowMaps = 0;
		PrevUniformParameters.NumShadowMapSlots = 0;
	}

	if (bDropAll)
	{
		// We drop the physical page pool here as well to ensure that it disappears in the case where
		// thumbnail rendering or similar creates multiple FSceneRenderers that never get deleted.
		// Caching is disabled on these contexts intentionally to avoid these issues.
		FreePhysicalPool(GraphBuilder);
		FreeHZBPhysicalPool(GraphBuilder);
	}
	else if (bNewShadowData)
	{
		// Page table and associated data are needed by HZB next frame even when VSM physical page caching is disabled
		GraphBuilder.QueueTextureExtraction(VirtualShadowMapArray.PageTableRDG, &PrevBuffers.PageTable);
		GraphBuilder.QueueBufferExtraction(VirtualShadowMapArray.UncachedPageRectBoundsRDG, &PrevBuffers.UncachedPageRectBounds);
		GraphBuilder.QueueBufferExtraction(VirtualShadowMapArray.AllocatedPageRectBoundsRDG, &PrevBuffers.AllocatedPageRectBounds);
		GraphBuilder.QueueTextureExtraction(VirtualShadowMapArray.PageFlagsRDG, &PrevBuffers.PageFlags);
		GraphBuilder.QueueBufferExtraction(VirtualShadowMapArray.NanitePerformanceFeedbackRDG, &PrevBuffers.NanitePerformanceFeedback);
		GraphBuilder.QueueBufferExtraction(VirtualShadowMapArray.ThrottleBufferRDG, &PrevBuffers.ThrottleBuffer);

		bool bPrefilteredDistantEnabled = VirtualShadowMapArray.PrefilteredDistantPhysicalPagePoolSeparateRDG != nullptr;		
		if (bPrefilteredDistantEnabled)
		{
			GraphBuilder.QueueTextureExtraction(VirtualShadowMapArray.PrefilteredDistantPhysicalPagePoolSeparateRDG, &PrevBuffers.PrevPrefilteredDistantPhysicalPagePoolSeparate);
		}

		if (bPrefilteredDistantEnabled && VirtualShadowMapArray.PageReceiverMasksRDG)
		{
			GraphBuilder.QueueTextureExtraction(VirtualShadowMapArray.PageReceiverMasksRDG, &PrevBuffers.PageReceiverMasks);
		}

		if (IsCacheEnabled() || bPrefilteredDistantEnabled)
		{
			GraphBuilder.QueueBufferExtraction(VirtualShadowMapArray.ProjectionDataRDG, &PrevBuffers.ProjectionData);
		}

		if (IsCacheEnabled())
		{
			GraphBuilder.QueueBufferExtraction(VirtualShadowMapArray.PhysicalPageListsRDG, &PrevBuffers.PhysicalPageLists);
			GraphBuilder.QueueTextureExtraction(VirtualShadowMapArray.PageRequestFlagsRDG, &PrevBuffers.PageRequestFlags);
			
			// Store but drop any temp references embedded in the uniform parameters this frame
			PrevUniformParameters = VirtualShadowMapArray.UniformParameters;
			PrevUniformParameters.ProjectionData = nullptr;
			PrevUniformParameters.PageTable = nullptr;
			PrevUniformParameters.UncachedPageRectBounds = nullptr;
			PrevUniformParameters.AllocatedPageRectBounds = nullptr;
			PrevUniformParameters.PageFlags = nullptr;
			PrevUniformParameters.PerViewData.LightGridData = nullptr;
			PrevUniformParameters.PerViewData.NumCulledLightsGrid = nullptr;
			PrevUniformParameters.PerViewData.DirectionalLightIds = nullptr;
			PrevUniformParameters.CachePrimitiveAsDynamic = nullptr;
		}

		// propagate current-frame primitive state to cache entry
		for (const auto& LightInfo : SceneRenderer.VisibleLightInfos)
		{
			for (const TSharedPtr<FVirtualShadowMapClipmap> &Clipmap : LightInfo.VirtualShadowMapClipmaps)
			{
				// Push data to cache entry
				Clipmap->UpdateCachedFrameData();
			}
		}

		ExtractStats(GraphBuilder, VirtualShadowMapArray);
	}
	
	// Clear out the referenced light flags since this render is finishing
	for (auto& LightEntry : CacheEntries)
	{
		LightEntry.Value->bReferencedThisRender = false;
	}
}

void FVirtualShadowMapArrayCacheManager::ExtractStats(FRDGBuilder& GraphBuilder, FVirtualShadowMapArray &VirtualShadowMapArray)
{
	FRDGBufferRef AccumulatedStatsBufferRDG = nullptr;

	// Note: stats accumulation thing is here because it needs to persist over frames.
	if (AccumulatedStatsBuffer.IsValid())
	{
		AccumulatedStatsBufferRDG = GraphBuilder.RegisterExternalBuffer(AccumulatedStatsBuffer, TEXT("Shadow.Virtual.AccumulatedStatsBuffer"));
	}

	// Auto stop at zero, use -1 to record indefinitely
	if (GVSMAccumulateStats > 0)
	{
		--GVSMAccumulateStats;
	}

	if (IsAccumulatingStats())
	{
		if (!AccumulatedStatsBuffer.IsValid())
		{
			FRDGBufferDesc Desc = FRDGBufferDesc::CreateBufferDesc(4, 1 + VSM_STAT_NUM * MaxStatFrames);
			Desc.Usage = EBufferUsageFlags(Desc.Usage | BUF_SourceCopy);

			AccumulatedStatsBufferRDG = GraphBuilder.CreateBuffer(Desc, TEXT("Shadow.Virtual.AccumulatedStatsBuffer"));	// TODO: Can't be a structured buffer as EnqueueCopy is only defined for vertex buffers
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(AccumulatedStatsBufferRDG, PF_R32_UINT), 0);
			AccumulatedStatsBuffer = GraphBuilder.ConvertToExternalBuffer(AccumulatedStatsBufferRDG);
		}

		// Initialize/clear
		if (!bAccumulatingStats)
		{
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(AccumulatedStatsBufferRDG, PF_R32_UINT), 0);
			bAccumulatingStats = true;
		}

		FVirtualSmCopyStatsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVirtualSmCopyStatsCS::FParameters>();

		PassParameters->InStatsBuffer = GraphBuilder.CreateSRV(VirtualShadowMapArray.StatsBufferRDG, PF_R32_UINT);
		PassParameters->AccumulatedStatsBufferOut = GraphBuilder.CreateUAV(AccumulatedStatsBufferRDG, PF_R32_UINT);

		// Dummy data
		PassParameters->NaniteStatsBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer<FNaniteStats>(GraphBuilder));

		// Optionally pull in some nanite stats too
		// NOTE: This only works if nanite is set to gather stats from the VSM pass!
		// i.e. run "NaniteStats VirtualShadowMaps" before starting accumulation
		if (Nanite::IsStatFilterActive(TEXT("VirtualShadowMaps")))
		{
			TRefCountPtr<FRDGPooledBuffer> NaniteStatsBuffer = Nanite::GGlobalResources.GetStatsBufferRef();
			if (NaniteStatsBuffer)
			{
				PassParameters->NaniteStatsBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(NaniteStatsBuffer));
			}
		}

		auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FVirtualSmCopyStatsCS>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Copy Stats"),
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1)
		);
	}
	else if (bAccumulatingStats)
	{
		bAccumulatingStats = false;

		GPUBufferReadback = new FRHIGPUBufferReadback(TEXT("Shadow.Virtual.AccumulatedStatsBufferReadback"));
		AddEnqueueCopyPass(GraphBuilder, GPUBufferReadback, AccumulatedStatsBufferRDG, 0u);
	}
	else if (AccumulatedStatsBuffer.IsValid())
	{
		AccumulatedStatsBuffer.SafeRelease();
	}

	if (GPUBufferReadback && GPUBufferReadback->IsReady())
	{
		TArray<uint32> Tmp;
		Tmp.AddDefaulted(1 + VSM_STAT_NUM * MaxStatFrames);

		{
			const uint32* BufferPtr = (const uint32*)GPUBufferReadback->Lock((1 + VSM_STAT_NUM * MaxStatFrames) * sizeof(uint32));
			FPlatformMemory::Memcpy(Tmp.GetData(), BufferPtr, Tmp.Num() * Tmp.GetTypeSize());
			GPUBufferReadback->Unlock();

			delete GPUBufferReadback;
			GPUBufferReadback = nullptr;
		}

		FString FileName = FPaths::ProfilingDir() + FString::Printf(TEXT("VSMStats(%s).csv"), *FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S")));

		const uint32 NumRows = Tmp[0];

		UE_LOGF(LogRenderer, Log, "Writing VSM accumulated stats (%d frames) to file '%ls'", NumRows, *FileName);

		FArchive * FileToLogTo = IFileManager::Get().CreateFileWriter(*FileName, false);
		ensure(FileToLogTo);
		if (FileToLogTo)
		{
			// Print header
			FString StringToPrint;
			for (int32 Index = 0; Index < VSM_STAT_NUM; ++Index)
			{
				if (!StringToPrint.IsEmpty())
				{
					StringToPrint += TEXT(",");
				}
				StringToPrint.Append(VirtualShadowMap_StatNames[Index]);
			}

			StringToPrint += TEXT("\n");
			FileToLogTo->Serialize(TCHAR_TO_ANSI(*StringToPrint), StringToPrint.Len());

			for (uint32 Ind = 0; Ind < NumRows; ++Ind)
			{
				StringToPrint.Empty();

				for (uint32 StatInd = 0; StatInd < VSM_STAT_NUM; ++StatInd)
				{
					if (!StringToPrint.IsEmpty())
					{
						StringToPrint += TEXT(",");
					}

					StringToPrint += FString::Printf(TEXT("%d"), Tmp[1 + Ind * VSM_STAT_NUM + StatInd]);
				}

				StringToPrint += TEXT("\n");
				FileToLogTo->Serialize(TCHAR_TO_ANSI(*StringToPrint), StringToPrint.Len());
			}


			FileToLogTo->Close();
		}
	}
}

bool FVirtualShadowMapArrayCacheManager::IsAccumulatingStats()
{
	return GVSMAccumulateStats != 0;
}

static uint32 GetPrimFlagsBufferSizeInDwords(int32 MaxPersistentPrimitiveIndex)
{
	return FMath::RoundUpToPowerOfTwo(FMath::DivideAndRoundUp(MaxPersistentPrimitiveIndex, 32));
}

void FVirtualShadowMapArrayCacheManager::ReallocatePersistentPrimitiveIndices()
{
	const int32 MaxPersistentPrimitiveIndex = FMath::Max(1, Scene.GetMaxPersistentPrimitiveIndex());

	for (auto& CacheEntry : CacheEntries)
	{
		CacheEntry.Value->RenderedPrimitives.SetNum(MaxPersistentPrimitiveIndex, false);
	}

	// TODO: Initialize new primitives based on their mobility; need a way to know which ones are newly created though
	CachePrimitiveAsDynamic.SetNum(MaxPersistentPrimitiveIndex, false);	
	if (MaxPersistentPrimitiveIndex > LastPrimitiveInvalidatedFrame.Num())
	{
		const uint32 OldSize = LastPrimitiveInvalidatedFrame.Num();
		LastPrimitiveInvalidatedFrame.SetNumUninitialized(MaxPersistentPrimitiveIndex);
		for (int32 It = OldSize; It < MaxPersistentPrimitiveIndex; ++It)
		{
			// Unknown last invalidation
			LastPrimitiveInvalidatedFrame[It] = 0xFFFFFFFF;
		}
	}

	// Do instance-based GPU allocations here too? For now we do them lazily each frame when the FVirtualShadowMapArray gets constructed
}

uint32 FVirtualShadowMapArrayCacheManager::GetPhysicalMaxWidth()
{
	return GetMax2DTextureDimension();
	//return 2048u;
}

BEGIN_SHADER_PARAMETER_STRUCT(FInvalidatePagesParameters, )
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPhysicalPageMetaData>, PhysicalPageMetaDataOut)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutPageRequestFlags)

	// When USE_HZB_OCCLUSION
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, HZBPageTable)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint4>, HZBPageRectBounds)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, HZBTextureArray)
	SHADER_PARAMETER_SAMPLER(SamplerState, HZBSampler)
	SHADER_PARAMETER(FVector2f, HZBSize)
END_SHADER_PARAMETER_STRUCT()

class FInvalidateInstancePagesLoadBalancerCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInvalidateInstancePagesLoadBalancerCS);
	SHADER_USE_PARAMETER_STRUCT(FInvalidateInstancePagesLoadBalancerCS, FGlobalShader)

	class FUseHzbDim : SHADER_PERMUTATION_BOOL("USE_HZB_OCCLUSION");
	using FPermutationDomain = TShaderPermutationDomain<FUseHzbDim>;

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FInvalidatePagesParameters, InvalidatePagesParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUScene::FInstanceGPULoadBalancer::FShaderParameters, LoadBalancerParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUSceneParameters)
	END_SHADER_PARAMETER_STRUCT()

	// This is probably fine even in instance list mode
	static constexpr int Cs1dGroupSizeX = FVirtualShadowMapArrayCacheManager::FInstanceGPULoadBalancer::ThreadGroupSize;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		OutEnvironment.SetDefine(TEXT("CS_1D_GROUP_SIZE_X"), Cs1dGroupSizeX);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);
		FGPUScene::FInstanceGPULoadBalancer::SetShaderDefines(OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FInvalidateInstancePagesLoadBalancerCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapCacheLoadBalancer.usf", "InvalidateInstancePagesLoadBalancerCS", SF_Compute);

class FProcessInvalidationQueueGPUCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FProcessInvalidationQueueGPUCS);
	SHADER_USE_PARAMETER_STRUCT(FProcessInvalidationQueueGPUCS, FGlobalShader)

	class FUseHzbDim : SHADER_PERMUTATION_BOOL("USE_HZB_OCCLUSION");
	using FPermutationDomain = TShaderPermutationDomain<FUseHzbDim>;

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FInvalidatePagesParameters, InvalidatePagesParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, InvalidationArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, InvalidationQueue)
		SHADER_PARAMETER(int32, InvalidationQueueMax)
		RDG_BUFFER_ACCESS(IndirectArgsBuffer, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

	static constexpr int ThreadGroupSize = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		// OutEnvironment.CompilerFlags.Add(CFLAG_SkipOptimizations);
	}
};
IMPLEMENT_GLOBAL_SHADER(FProcessInvalidationQueueGPUCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapCacheGPUInvalidation.usf", "ProcessInvalidationQueueGPUCS", SF_Compute);

void FVirtualShadowMapArrayCacheManager::UpdateCachePrimitiveAsDynamic(FInvalidatingPrimitiveCollector& InvalidatingPrimitiveCollector)
{
	//UE_LOGF(LogRenderer, Display, "VirtualShadowMapCacheManager: UpdateCachePrimitiveAsDynamic");

	const uint32 SceneFrameNumber = Scene.GetFrameNumberRenderThread();
	const uint32 FramesStaticThreshold = CVarFramesStaticThreshold.GetValueOnRenderThread();

	// Update the cache states of things that are being invalidated
	for (TConstSetBitIterator<SceneRenderingAllocator> BitIt(InvalidatingPrimitiveCollector.InvalidatedPrimitives); BitIt; ++BitIt)
	{
		int32 PersistentPrimitiveIndex = BitIt.GetIndex();
		// Any invalidations mean we set this primitive to dynamic. We already added an invalidation otherwise
		// we wouldn't be here, so no need to add another.
		CachePrimitiveAsDynamic[PersistentPrimitiveIndex] = true;
		LastPrimitiveInvalidatedFrame[PersistentPrimitiveIndex] = SceneFrameNumber;
	}

	// Zero out anything that was being removed
	// NOTE: This will be redundant with the invalidated stuff, but shouldn't be a big deal
	for (TConstSetBitIterator<SceneRenderingAllocator> BitIt(InvalidatingPrimitiveCollector.RemovedPrimitives); BitIt; ++BitIt)
	{
		int32 PersistentPrimitiveIndex = BitIt.GetIndex();
		// TODO: We probably want to start new primitives as dynamic by default instead, but we don't want to have
		// to loop over all of them and try and get their PrimitiveSceneInfo every frame for invalid ones
		CachePrimitiveAsDynamic[PersistentPrimitiveIndex] = false;
		LastPrimitiveInvalidatedFrame[PersistentPrimitiveIndex] = 0xFFFFFFFF;

		//UE_LOGF(LogRenderer, Display, "VirtualShadowMapCacheManager: Removing primitive %d!", PersistentPrimitiveIndex);
	}

	// Finally check anything that is currently dynamic to see if it has not invalidated for long enough that
	// we should move it back to static
	for (TConstSetBitIterator<> BitIt(CachePrimitiveAsDynamic); BitIt; ++BitIt)
	{
		int32 PersistentPrimitiveIndex = BitIt.GetIndex();

		const uint32 LastInvalidationFrame = LastPrimitiveInvalidatedFrame[PersistentPrimitiveIndex];
		// Note: cleared to MAX_uint32; treated as "unknown/no invalidations"
		const uint32 InvalidationAge = 
			SceneFrameNumber >= LastInvalidationFrame ?
			(SceneFrameNumber - LastInvalidationFrame) :
			0xFFFFFFFF;

		const bool bWantStatic = InvalidationAge > FramesStaticThreshold;
		if (bWantStatic)
		{
			// Add invalidation and swap it to static
			FPersistentPrimitiveIndex WrappedIndex;
			WrappedIndex.Index = PersistentPrimitiveIndex;
			FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene.GetPrimitiveSceneInfo(WrappedIndex);
			if (PrimitiveSceneInfo)
			{
				// Add an invalidation for every light
				for (auto& CacheEntry : CacheEntries)
				{
					const int32 VirtualShadowMapId = CacheEntry.Value->GetVirtualShadowMapId();
					check(VirtualShadowMapId >= 0);
					const int32 NumEntries = CacheEntry.Value->ShadowMapEntries.Num();
					for (int32 Index = 0; Index < NumEntries; ++Index)
					{
						const uint32 PayloadForceStatic = EncodeInstanceInvalidationPayload(VirtualShadowMapId + Index, VSM_INVALIDATION_PAYLOAD_FLAG_FORCE_STATIC);
						InvalidatingPrimitiveCollector.Instances.Add(
							PrimitiveSceneInfo->GetInstanceSceneDataOffset(),
							PrimitiveSceneInfo->GetNumInstanceSceneDataEntries(),
							PayloadForceStatic);
					}
				}
			}
			else
			{
				// This seems to still happen very occasionally... presumably a remove gets "missed" somehow and thus we try and transition
				// something that is no longer valid back to static. This could also potentially mean we incorrect transition a new thing that
				// grabbed this slot back to static, but that is less likely as the addition would trigger a separate invalidation.
				// Not much we can do here currently other than ignore it and move on
				
				// Disabling log due to build automation spam
				// UE_LOGF(LogRenderer, Display, "VirtualShadowMapCacheManager: Invalid persistent primitive index %d, age %u!", PersistentPrimitiveIndex, InvalidationAge);
				LastPrimitiveInvalidatedFrame[PersistentPrimitiveIndex] = 0xFFFFFFFF;
			}
			// NOTE: This is safe with the current set bit iterator, but should maybe use a temp array for future safety?
			CachePrimitiveAsDynamic[PersistentPrimitiveIndex] = false;
		}
	}
}

void FVirtualShadowMapArrayCacheManager::ProcessInvalidations(FRDGBuilder& GraphBuilder, FInvalidatingPrimitiveCollector& InvalidatingPrimitiveCollector)
{
	if (IsCacheDataAvailable() && PrevUniformParameters.NumShadowMapSlots > 0)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "Shadow.Virtual.ProcessInvalidations");

		// TODO: Some of this stuff probably needs to move out of this function as we don't need to evaluate it twice
		// (before/after GPUScene update). That said, we clear the lists so in practice we are just going to do it
		// before, just could use some refactoring for clarity.

		// NOTE: Important that we get some of these parameters (ex. CachePrimitiveAsDynamic) before
		// we update them as the shader needs to know the previous cache states for invalidation.
		FInvalidationPassCommon InvalidationPassCommon = GetUniformParametersForInvalidation(GraphBuilder);

		// Add invalidations for skeletal meshes, CPU culling changes, dynamic primitives, etc.
		InvalidatingPrimitiveCollector.AddPrimitivesToInvalidate();

		// Check whether we want to swap any cache states and add any invalidations to that end as well
		UpdateCachePrimitiveAsDynamic(InvalidatingPrimitiveCollector);

		InvalidatingPrimitiveCollector.Instances.FinalizeBatches();

		if (!InvalidatingPrimitiveCollector.Instances.IsEmpty())
		{
			ProcessInvalidations(GraphBuilder, InvalidationPassCommon, InvalidatingPrimitiveCollector.Instances);
		}
	}
	else
	{
		// Clear any queued-up invalidations
		ShadowInvalidatingInstancesImplementation.PrimitiveInstancesToInvalidate.Reset();
		for (auto& CacheEntry : CacheEntries)
		{
			CacheEntry.Value->PrimitiveInstancesToInvalidate.Reset();
		}
	}
}

void FVirtualShadowMapArrayCacheManager::ProcessRemovedLights(const FSceneRendererLightBitArray& RemovedLightMask)
{
	for (auto It = CacheEntries.CreateIterator(); It; ++It)
	{
		FLightSceneId LightSceneId = It->Key.LightSceneId;
		if (RemovedLightMask.IsValidIndex(LightSceneId) && RemovedLightMask[LightSceneId])
		{
			It.RemoveCurrent();
		}
	}
}

FVirtualShadowMapArrayCacheManager::FInvalidationPassCommon FVirtualShadowMapArrayCacheManager::GetUniformParametersForInvalidation(FRDGBuilder& GraphBuilder) const
{
	// Construct a uniform buffer based on the previous frame data, reimported into this graph builder
	FVirtualShadowMapUniformParameters* UniformParameters = GraphBuilder.AllocParameters<FVirtualShadowMapUniformParameters>();
	*UniformParameters = PrevUniformParameters;
	{
		auto RegExtCreateSrv = [&GraphBuilder](const TRefCountPtr<FRDGPooledBuffer>& Buffer, const TCHAR* Name) -> FRDGBufferSRVRef
		{
			return GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(Buffer, Name));
		};

		UniformParameters->PageTableSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		UniformParameters->ProjectionData = RegExtCreateSrv(PrevBuffers.ProjectionData, TEXT("Shadow.Virtual.PrevProjectionData"));
		UniformParameters->PageTable = GraphBuilder.RegisterExternalTexture(PrevBuffers.PageTable, TEXT("Shadow.Virtual.PrevPageTable"));
		UniformParameters->PageFlags = GraphBuilder.RegisterExternalTexture(PrevBuffers.PageFlags, TEXT("Shadow.Virtual.PrevPageFlags"));
		UniformParameters->UncachedPageRectBounds = RegExtCreateSrv(PrevBuffers.UncachedPageRectBounds, TEXT("Shadow.Virtual.PrevUncachedPageRectBounds"));
		UniformParameters->AllocatedPageRectBounds = RegExtCreateSrv(PrevBuffers.AllocatedPageRectBounds, TEXT("Shadow.Virtual.PrevAllocatedPageRectBounds"));
		UniformParameters->CachePrimitiveAsDynamic = GraphBuilder.CreateSRV(UploadCachePrimitiveAsDynamic(GraphBuilder));

		// Unused in this path... may be a better way to handle this
		UniformParameters->PhysicalPagePool = GSystemTextures.GetZeroUIntArrayAtomicCompatDummy(GraphBuilder);
		UniformParameters->PageReceiverMasks = GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R32_UINT, 0xFFFFFFFFu);
		FRDGBufferSRVRef Uint32SRVDummy = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32)));
		UniformParameters->PerViewData.LightGridData = Uint32SRVDummy;
		UniformParameters->PerViewData.NumCulledLightsGrid = Uint32SRVDummy;
		UniformParameters->PerViewData.DirectionalLightIds = Uint32SRVDummy;
		UniformParameters->BilinearClampedSampler = TStaticSamplerState< SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI();
	}
	
	FInvalidationPassCommon Result;
	Result.UniformParameters = UniformParameters;
	Result.VirtualShadowMapUniformBuffer = GraphBuilder.CreateUniformBuffer(UniformParameters);
	return Result;
}

void FVirtualShadowMapArrayCacheManager::SetInvalidateInstancePagesParameters(
	FRDGBuilder& GraphBuilder,
	const FInvalidationPassCommon& InvalidationPassCommon,
	FInvalidatePagesParameters* PassParameters) const
{
	// TODO: We should make this UBO once and reuse it for all the passes
	PassParameters->VirtualShadowMap = InvalidationPassCommon.VirtualShadowMapUniformBuffer;
	PassParameters->PhysicalPageMetaDataOut = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalBuffer(PhysicalPageMetaData));
	PassParameters->OutPageRequestFlags = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(PrevBuffers.PageRequestFlags));

	const bool bUseHZB = (CVarCacheVsmUseHzb.GetValueOnRenderThread() != 0);
	const TRefCountPtr<IPooledRenderTarget> HZBPhysical = (bUseHZB && HZBPhysicalPagePoolArray) ? HZBPhysicalPagePoolArray : nullptr;
	if (HZBPhysical)
	{
		// Same, since we are not producing a new frame just yet
		PassParameters->HZBPageTable = InvalidationPassCommon.UniformParameters->PageTable;
		PassParameters->HZBPageRectBounds = InvalidationPassCommon.UniformParameters->AllocatedPageRectBounds;		// TODO: Uncached?
		PassParameters->HZBTextureArray = GraphBuilder.RegisterExternalTexture(HZBPhysical);
		PassParameters->HZBSize = HZBPhysical->GetDesc().Extent;
		PassParameters->HZBSampler = TStaticSamplerState< SF_Point, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI();
	}
}

void FVirtualShadowMapArrayCacheManager::ProcessInvalidations(
	FRDGBuilder& GraphBuilder,
	const FInvalidationPassCommon& InvalidationPassCommon,
	const FInstanceGPULoadBalancer& Instances) const
{
	RDG_GPU_MASK_SCOPE(GraphBuilder, CacheValidGPUMask);

	check(InvalidationPassCommon.UniformParameters->NumShadowMapSlots > 0);
	check(!Instances.IsEmpty());

	FInvalidateInstancePagesLoadBalancerCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInvalidateInstancePagesLoadBalancerCS::FParameters>();

	SetInvalidateInstancePagesParameters(GraphBuilder, InvalidationPassCommon, &PassParameters->InvalidatePagesParameters);
	PassParameters->GPUSceneParameters = Scene.GPUScene.GetShaderParameters(GraphBuilder);
	Instances.UploadFinalized(GraphBuilder).GetShaderParameters(GraphBuilder, PassParameters->LoadBalancerParameters);

	FInvalidateInstancePagesLoadBalancerCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FInvalidateInstancePagesLoadBalancerCS::FUseHzbDim>(PassParameters->InvalidatePagesParameters.HZBTextureArray != nullptr);
	
	auto ComputeShader = GetGlobalShaderMap(Scene.GetShaderPlatform())->GetShader<FInvalidateInstancePagesLoadBalancerCS>(PermutationVector);

	//UE_LOGF(LogRenderer, Display, "VirtualShadowMapCacheManager: Invalidation %lu instances!", Instances.GetTotalNumInstances());

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("InvalidateInstancePagesLoadBalancerCS (%d batches)", Instances.GetBatches().Num()),
		ComputeShader,
		PassParameters,
		Instances.GetWrappedCsGroupCount()
	);
}

// Remove old info used to track logging.
void FVirtualShadowMapArrayCacheManager::TrimLoggingInfo()
{
#if !UE_BUILD_SHIPPING
	// Remove old items
	float RealTimeSeconds = float(FPlatformTime::Seconds() - GStartTime);
	LargePageAreaItems = LargePageAreaItems.FilterByPredicate([RealTimeSeconds](const TMap<uint32, FLargePageAreaItem>::ElementType& Element)
	{
		return RealTimeSeconds - Element.Value.LastTimeSeen < 5.0f;
	});
#endif
}

FVirtualShadowMapArrayCacheManager::FViewData::FViewData() 
	: InstanceState(1024, TEXT("Shadow.Virtual.CacheManager.ViewState"))
{
}

FVirtualShadowMapInvalidationSceneUpdater::FVirtualShadowMapInvalidationSceneUpdater(FVirtualShadowMapArrayCacheManager& InCacheManager)
	: CacheManager(InCacheManager)
{}

void FVirtualShadowMapInvalidationSceneUpdater::PreLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet)
{
	if (LightSceneChangeSet.RemovedLightIds.Num() > 0)
	{
		CacheManager.ProcessRemovedLights(LightSceneChangeSet.RemovedLightsMask);
	}
}

void FVirtualShadowMapInvalidationSceneUpdater::PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet)
{
	SCOPED_NAMED_EVENT(FScene_VirtualShadowCacheUpdate, FColor::Orange);

	if (ChangeSet.ViewUpdateChangeSet)
	{
		for (FPersistentViewId ViewId : ChangeSet.ViewUpdateChangeSet->RemovedViewIds)
		{
			// If VSM was disabled then the data may not contain the view being removed (if VSM is enabled & the view removed on the same frame).
			if (CacheManager.ViewData.IsValidIndex(ViewId.Index))
			{
				CacheManager.ViewData.RemoveAt(ViewId.Index);
			}
		}
	}

	FShadowScene& ShadowScene = CacheManager.Scene.GetExtension<FShadowScene>();

	// Needs to be called before the first time we start adding invalidations.
	// There may be a way to avoid doing this both in pre and post, but it is pretty light if there is nothing to do anyways.
	CacheManager.ReallocatePersistentPrimitiveIndices();

	if (CacheManager.IsCacheDataAvailable())
	{
		FVirtualShadowMapArrayCacheManager::FInvalidatingPrimitiveCollector InvalidatingPrimitiveCollector(&CacheManager);

		// TODO: Note that there is an ordering dependency here on ShadowScene, which may not have updated the AlwaysInvalidatingPrimitives list at this point. 
		//       Probably harmless (as they will get invalidated next frame etc) but something to think about.
		// Primitives that are tracked as always invalidating shadows, pipe through as transform updates
		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ShadowScene.GetAlwaysInvalidatingPrimitives())
		{
			InvalidatingPrimitiveCollector.UpdatedTransform(PrimitiveSceneInfo);
		}

		// Note: skips added as they are not fully defined at this point (not primitive ID allocated, 
		ChangeSet.PrimitiveUpdates.ForEachUpdateCommand(ESceneUpdateCommandFilter::Updated | ESceneUpdateCommandFilter::Deleted, EPrimitiveUpdateDirtyFlags::AllCulling, [&](const FPrimitiveUpdateCommand& Cmd)
		{
			if (Cmd.IsDelete())
			{
				// All removed primitives must invalidate their footprints in the VSM before leaving.
				InvalidatingPrimitiveCollector.Removed(Cmd.GetSceneInfo());
			}
			else
			{
				InvalidatingPrimitiveCollector.UpdatedTransform(Cmd.GetSceneInfo());
			}
		});

		TRACE_INT_VALUE(TEXT("Shadow.Virtual.Cache.PreInvalidationInstances"), InvalidatingPrimitiveCollector.Instances.GetTotalNumInstances());
		CacheManager.ProcessInvalidations(GraphBuilder, InvalidatingPrimitiveCollector);
	}
}


using FLoadBalancer = TInstanceCullingLoadBalancer<SceneRenderingAllocator>;

class FVSMResetInstanceStateCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVSMResetInstanceStateCS);
	SHADER_USE_PARAMETER_STRUCT(FVSMResetInstanceStateCS, FGlobalShader)

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FLoadBalancer::FShaderParameters, LoadBalancerParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUScene)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, InOutViewInstanceState)
		SHADER_PARAMETER(int32, StateWordStride)
	END_SHADER_PARAMETER_STRUCT()

	// This is probably fine even in instance list mode
	static constexpr int ThreadGroupSize = FLoadBalancer::ThreadGroupSize;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);
		FGPUScene::FInstanceGPULoadBalancer::SetShaderDefines(OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FVSMResetInstanceStateCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapCacheGPUInvalidation.usf", "VSMResetInstanceStateCS", SF_Compute);

void FVirtualShadowMapInvalidationSceneUpdater::PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet)
{
	// Loop over the current state, rather than the change set to handle that VSM can be enabled/disabled
	for (auto It = CacheManager.Scene.PersistentViewStateUniqueIDs.CreateConstIterator(); It; ++It)
	{
		FPersistentViewId ViewId { It.GetIndex() };
		if (!CacheManager.ViewData.IsValidIndex(ViewId.Index))
		{
			CacheManager.ViewData.EmplaceAt(ViewId.Index);
		}
	}

	CacheManager.ReallocatePersistentPrimitiveIndices();

	// Grab a reference, but we currently do all the work in PostGPUSceneUpdate
	PostUpdateChangeSet = &ChangeSet;
}

void FVirtualShadowMapInvalidationSceneUpdater::PostGPUSceneUpdate(FRDGBuilder& GraphBuilder)
{
	// TODO: Separate scope for post-update pass?
	SCOPED_NAMED_EVENT(FScene_VirtualShadowCacheUpdate, FColor::Orange);

	// Update the allocations for storing a bit per view to track state for each view (dynamic/static)
	// This depends on GPU-Scene having correct instance ranges, so must happen post GPU scene update.
	CacheManager.InstanceStateMaskWordStride = FMath::DivideAndRoundUp(CacheManager.Scene.GPUScene.GetNumInstances(), 32);
	const int32 NumInstanceWordsNeeded = CacheManager.InstanceStateMaskWordStride * FVirtualShadowMapArrayCacheManager::FViewData::NumBitsPerInstance;
	if (NumInstanceWordsNeeded > 0 && !CacheManager.ViewData.IsEmpty())
	{
		TArray<FRDGBuffer*, TInlineAllocator<16,  SceneRenderingAllocator> > InstanceStatesRDG;
		InstanceStatesRDG.Reserve(CacheManager.ViewData.Num());
		for (FVirtualShadowMapArrayCacheManager::FViewData& ViewData : CacheManager.ViewData)
		{
			InstanceStatesRDG.Emplace(ViewData.InstanceState.ResizeAndClearBufferIfNeeded(GraphBuilder, NumInstanceWordsNeeded));
		}

		FLoadBalancer LoadBalancer;
		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : PostUpdateChangeSet->AddedPrimitiveSceneInfos)
		{
			int32 WordOffset = PrimitiveSceneInfo->GetInstanceSceneDataOffset() / 32;
			int32 NumWords = FMath::DivideAndRoundUp(PrimitiveSceneInfo->GetNumInstanceSceneDataEntries(), 32);
			LoadBalancer.Add(WordOffset, NumWords, PrimitiveSceneInfo->GetPersistentIndex().Index);
		}

		if (!LoadBalancer.IsEmpty())
		{
			FLoadBalancer::FGPUData LoadBalancerGpuData = LoadBalancer.Upload(GraphBuilder);
			auto ComputeShader = GetGlobalShaderMap(CacheManager.Scene.GetShaderPlatform())->GetShader<FVSMResetInstanceStateCS>();

			FGPUSceneResourceParameters GPUSceneParams = CacheManager.Scene.GPUScene.GetShaderParameters(GraphBuilder);

			for (FRDGBuffer* InstanceStateRDG : InstanceStatesRDG)
			{
				FVSMResetInstanceStateCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVSMResetInstanceStateCS::FParameters>();
				PassParameters->InOutViewInstanceState = GraphBuilder.CreateUAV(InstanceStateRDG);
				PassParameters->GPUScene = GPUSceneParams;
				PassParameters->StateWordStride = CacheManager.InstanceStateMaskWordStride;
				LoadBalancerGpuData.AddPass(GraphBuilder, RDG_EVENT_NAME("ResetInstanceState (%d batches)", LoadBalancerGpuData.NumBatches), ComputeShader, PassParameters);

			}
		}
	}


	if (CacheManager.IsCacheDataAvailable())
	{
		FVirtualShadowMapArrayCacheManager::FInvalidatingPrimitiveCollector InvalidatingPrimitiveCollector(&CacheManager);

		// Filter out all updates that are either "add" or has dirty flags to say they affect the bounds.
		PostUpdateChangeSet->PrimitiveUpdates.ForEachUpdateCommand(ESceneUpdateCommandFilter::AddedUpdated, EPrimitiveUpdateDirtyFlags::AllCulling, [&](const FPrimitiveUpdateCommand& Cmd)
		{
			if (Cmd.IsAdd())
			{
				InvalidatingPrimitiveCollector.Added(Cmd.GetSceneInfo());
			}
			else
			{
				InvalidatingPrimitiveCollector.UpdatedTransform(Cmd.GetSceneInfo());
			}
		});

		TRACE_INT_VALUE(TEXT("Shadow.Virtual.Cache.PostInvalidationInstances"), InvalidatingPrimitiveCollector.Instances.GetTotalNumInstances());
		CacheManager.ProcessInvalidations(GraphBuilder, InvalidatingPrimitiveCollector);
	}
	PostUpdateChangeSet = nullptr;
}

class FVSMUpdateViewInstanceStateCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVSMUpdateViewInstanceStateCS);
	SHADER_USE_PARAMETER_STRUCT(FVSMUpdateViewInstanceStateCS, FGlobalShader)

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutCacheInstanceAsDynamic)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, InOutViewInstanceState)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutInvalidationArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutInvalidationQueue)
		SHADER_PARAMETER(int32, SceneRendererViewId)
		SHADER_PARAMETER(int32, StateWordStride)
		SHADER_PARAMETER(int32, MaxValidInstanceIndex)
		SHADER_PARAMETER(int32, InvalidationQueueMax)
	END_SHADER_PARAMETER_STRUCT()

	static constexpr int ThreadGroupSize = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		//OutEnvironment.CompilerFlags.Add(CFLAG_SkipOptimizations);
	}
};
IMPLEMENT_GLOBAL_SHADER(FVSMUpdateViewInstanceStateCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapCacheGPUInvalidation.usf", "VSMUpdateViewInstanceStateCS", SF_Compute);

BEGIN_SHADER_PARAMETER_STRUCT(FVirtualShadowMapInvalidationSceneUniforms, RENDERER_API)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CacheInstanceAsDynamic)
	SHADER_PARAMETER(uint32, InstanceStateViewWordStride)
END_SHADER_PARAMETER_STRUCT()

DECLARE_SCENE_UB_STRUCT(FVirtualShadowMapInvalidationSceneUniforms, VSMCache, RENDERER_API)

class FVirtualShadowMapInvalidationSceneRenderer : public ISceneExtensionRenderer
{
	DECLARE_SCENE_EXTENSION_RENDERER(FVirtualShadowMapInvalidationSceneRenderer, FVirtualShadowMapArrayCacheManager);
public:
	FVirtualShadowMapInvalidationSceneRenderer(FSceneRendererBase& InSceneRenderer, FVirtualShadowMapArrayCacheManager& InCacheManager)
		: ISceneExtensionRenderer(InSceneRenderer)
		, CacheManager(InCacheManager)
	{
	}
	virtual void UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniforms) override
	{
		FSceneRendererBase& SceneRenderer = GetSceneRenderer();

		const int32 NumInstances = SceneRenderer.Scene->GPUScene.GetNumInstances();
		const int32 NumViews = SceneRenderer.Views.Num();
		int32 CacheInstanceAsDynamicInstanceMaskWordStride = CacheManager.InstanceStateMaskWordStride;
		// Note: we can't check this here because ... dynamic primitives might have allocated more instances. Those need to be handled somehow wrt this. TBD. I guess they need a bit in the mask? Or we just return 0?
		// check(CacheInstanceAsDynamicInstanceMaskWordStride * 32 >= NumInstances);
		// Allocate space for WordStride words for each view, x2 for storing state change mask
		int32 NumCacheInstanceAsDynamicWords = 2 * CacheInstanceAsDynamicInstanceMaskWordStride * NumViews;

		// If there are no instances or views, leave SceneUB with defaults.
		if (NumCacheInstanceAsDynamicWords > 0)
		{
			CacheInstanceAsDynamicRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NumCacheInstanceAsDynamicWords), TEXT("Shadow.Virtual.CacheInstanceAsDynamic"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(CacheInstanceAsDynamicRDG), 0u);

			FVirtualShadowMapInvalidationSceneUniforms VirtualShadowMapInvalidationSceneUniforms;
			VirtualShadowMapInvalidationSceneUniforms.CacheInstanceAsDynamic = GraphBuilder.CreateSRV(CacheInstanceAsDynamicRDG);
			VirtualShadowMapInvalidationSceneUniforms.InstanceStateViewWordStride = CacheInstanceAsDynamicInstanceMaskWordStride;
			SceneUniforms.Set(SceneUB::VSMCache, VirtualShadowMapInvalidationSceneUniforms);
		}
	}

	virtual void PreRender(FRDGBuilder& GraphBuilder) override
	{
		// This happens post visibility update so here we know what instances are deemed deforming.

		// Early out if there are no instances (or no views?)
		if (CacheInstanceAsDynamicRDG == nullptr)
		{
			return;
		}

		FSceneRendererBase& SceneRenderer = GetSceneRenderer();
		ERHIFeatureLevel::Type FeatureLevel = SceneRenderer.Scene->GetFeatureLevel();

		FVSMUpdateViewInstanceStateCS::FParameters SharedParameters;
		SharedParameters.Scene = SceneRenderer.GetSceneUniforms().GetBuffer(GraphBuilder);

		const int32 NumInstances = CacheManager.Scene.GPUScene.GetNumInstances();
		const int32 NumViews = SceneRenderer.Views.Num();
		const int32 InvalidationQueueMax = NumInstances * NumViews;
		GPUInvalidateInstancesArgsRDG = CreateAndClearIndirectDispatchArgs(GraphBuilder, FeatureLevel, TEXT("Shadow.Virtual.GPUInvalidateInstancesArgs"), FIntVector3(0,1,1), 1u, 4u);
	
		// Worst-case allocation of NumInstances X NumView slots, though it's a transient buffer so probably ok
		GPUInvalidationQueueRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), InvalidationQueueMax), TEXT("Shadow.Virtual.GPUInvalidationQueue"));

		SharedParameters.OutCacheInstanceAsDynamic = GraphBuilder.CreateUAV(CacheInstanceAsDynamicRDG, ERDGUnorderedAccessViewFlags::SkipBarrier);
		SharedParameters.MaxValidInstanceIndex = NumInstances;
		SharedParameters.OutInvalidationArgs = GraphBuilder.CreateUAV(GPUInvalidateInstancesArgsRDG, ERDGUnorderedAccessViewFlags::SkipBarrier);
		SharedParameters.OutInvalidationQueue = GraphBuilder.CreateUAV(GPUInvalidationQueueRDG, ERDGUnorderedAccessViewFlags::SkipBarrier);
		SharedParameters.InvalidationQueueMax =  InvalidationQueueMax;
		SharedParameters.StateWordStride = CacheManager.InstanceStateMaskWordStride;

		bool bWasAnyInvalidationsQueued = false;
		for (int32 ViewIndex = 0; ViewIndex < SceneRenderer.Views.Num(); ++ViewIndex)
		{
			FViewInfo& View = SceneRenderer.Views[ViewIndex];
			if (!View.PersistentViewId.IsValid())
			{
				continue;
			}
			FVSMUpdateViewInstanceStateCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVSMUpdateViewInstanceStateCS::FParameters>(&SharedParameters);
			PassParameters->SceneRendererViewId = ViewIndex;
			check(CacheManager.ViewData.IsValidIndex(View.PersistentViewId.Index));
			PassParameters->InOutViewInstanceState = GraphBuilder.CreateUAV(CacheManager.ViewData[View.PersistentViewId.Index].InstanceState.Register(GraphBuilder));

			auto ComputeShader = GetGlobalShaderMap(FeatureLevel)->GetShader<FVSMUpdateViewInstanceStateCS>();

			// Run one thread per word
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("VSMUpdateViewInstanceState"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(CacheManager.InstanceStateMaskWordStride, FVSMUpdateViewInstanceStateCS::ThreadGroupSize)
			);
			bWasAnyInvalidationsQueued = true;
		}

		// Run the resultant invalidations right away, this could be deferred and flushed later instead if desired.
		if (CacheManager.IsCacheDataAvailable() && bWasAnyInvalidationsQueued && CacheManager.PrevUniformParameters.NumFullShadowMaps > 0)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "Shadow.Virtual.ProcessInvalidationsGPU");

			FVirtualShadowMapArrayCacheManager::FInvalidationPassCommon InvalidationPassCommon = CacheManager.GetUniformParametersForInvalidation(GraphBuilder);
		
			FProcessInvalidationQueueGPUCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FProcessInvalidationQueueGPUCS::FParameters>();

			PassParameters->InvalidationArgs = GraphBuilder.CreateSRV(GPUInvalidateInstancesArgsRDG);
			PassParameters->InvalidationQueue = GraphBuilder.CreateSRV(GPUInvalidationQueueRDG);
			PassParameters->InvalidationQueueMax =  InvalidationQueueMax;
			PassParameters->IndirectArgsBuffer = GPUInvalidateInstancesArgsRDG;
			PassParameters->Scene =  SceneRenderer.GetSceneUniforms().GetBuffer(GraphBuilder);
			CacheManager.SetInvalidateInstancePagesParameters(GraphBuilder, InvalidationPassCommon, &PassParameters->InvalidatePagesParameters);

			FProcessInvalidationQueueGPUCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FProcessInvalidationQueueGPUCS::FUseHzbDim>(PassParameters->InvalidatePagesParameters.HZBTextureArray != nullptr);
	
			auto ComputeShader = GetGlobalShaderMap(FeatureLevel)->GetShader<FProcessInvalidationQueueGPUCS>(PermutationVector);

			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("ProcessInvalidationQueueGPU"), ComputeShader, PassParameters, GPUInvalidateInstancesArgsRDG, 0);
		}
	}

private:
	FRDGBuffer* CacheInstanceAsDynamicRDG = nullptr;
	FRDGBuffer* GPUInvalidationQueueRDG = nullptr;
	FRDGBuffer* GPUInvalidateInstancesArgsRDG = nullptr;

	FVirtualShadowMapArrayCacheManager& CacheManager;
};

ISceneExtensionRenderer* FVirtualShadowMapArrayCacheManager::CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags)
{
	// NOTE: We need this check because shader platform can change during scene destruction so we need to ensure we
	// don't try and run shaders on a new platform that doesn't support VSMs...
	if (UseVirtualShadowMaps(Scene.GetShaderPlatform(), Scene.GetFeatureLevel()) 
		&& !EngineShowFlags.HitProxies
		&& EngineShowFlags.VirtualShadowMapPersistentData)
	{
		return new FVirtualShadowMapInvalidationSceneRenderer(InSceneRenderer, *this);
	}

	return nullptr;
}

static void GetSceneUBDefaultParameters(FVirtualShadowMapInvalidationSceneUniforms& OutParameters, FRDGBuilder& GraphBuilder)
{
	OutParameters.CacheInstanceAsDynamic = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 4u));
	OutParameters.InstanceStateViewWordStride = 0u;
}
IMPLEMENT_SCENE_UB_STRUCT(FVirtualShadowMapInvalidationSceneUniforms, VSMCache, GetSceneUBDefaultParameters);

#undef LOCTEXT_NAMESPACE
