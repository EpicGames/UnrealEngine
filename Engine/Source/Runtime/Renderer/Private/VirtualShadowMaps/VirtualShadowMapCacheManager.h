// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VirtualShadowMapArray.h"
#include "SceneManagement.h"
#include "InstanceCulling/InstanceCullingLoadBalancer.h"
#include "GPUScene.h"
#include "GPUMessaging.h"
#include "SceneRendererInterface.h"
#include "SceneExtensions.h"
#include "RendererPrivateUtils.h"
#include "ScenePrivateBase.h"

class FRHIGPUBufferReadback;
class FGPUScene;
class FVirtualShadowMapPerLightCacheEntry;
class FInvalidatePagesParameters;

namespace Nanite { struct FPackedViewParams; }

struct FVirtualShadowMapInstanceRange
{
	FPersistentPrimitiveIndex PersistentPrimitiveIndex;
	int32 InstanceSceneDataOffset;
	int32 NumInstanceSceneDataEntries;
	bool bMarkAsDynamic;					// If true, swaps the primitive/instance to dynamic caching
};

struct FVirtualShadowMapHZBMetadata
{
	// See UpdatePrevHZBMetadata if you modify fields here
	FViewMatrices ViewMatrices;
	FIntRect	  ViewRect;
	uint32		  TargetLayerIndex = INDEX_NONE;
	bool          bMatricesDirty = true;
};

#define VSM_LOG_INVALIDATIONS 0

class FVirtualShadowMapCacheEntry
{
public:
	// Generic version used for local lights but also inactive lights
	void Update(const FVirtualShadowMapPerLightCacheEntry &PerLightEntry);

	// Specific version of the above for clipmap levels, which have additional constraints
	void UpdateClipmapLevel(
		const FVirtualShadowMapPerLightCacheEntry& PerLightEntry,
		FInt64Point PageSpaceLocation,
		double LevelRadius,
		double ViewCenterZ,
		double ViewRadiusZ,
		double WPODistanceDisabledThreshold);

	void SetHZBViewParams(Nanite::FPackedViewParams& OutParams);

	void UpdateHZBMetadata(const FViewMatrices& ViewMatrices, const FIntRect& ViewRect, uint32 TargetLayerIndex);

	void UpdatePrevHZBMetadata()
	{
		PrevHZBMetadata.TargetLayerIndex = CurrentHZBMetadata.TargetLayerIndex;
		PrevHZBMetadata.ViewRect = CurrentHZBMetadata.ViewRect;
		if (CurrentHZBMetadata.bMatricesDirty)
		{
			PrevHZBMetadata.ViewMatrices = CurrentHZBMetadata.ViewMatrices;
		}
	}

	FVirtualShadowMapHZBMetadata PrevHZBMetadata;	
	FVirtualShadowMapHZBMetadata CurrentHZBMetadata;
	
	// Tracks mapping to any previously cached data
	FNextVirtualShadowMapData NextData;

	// Stores the projection shader data. This is needed for cached entries that may be inactive in the current frame/render
	// and also avoids recomputing it every frame.
	FVirtualShadowMapProjectionShaderData ProjectionData;

	// Clipmap-specific information for panning and tracking of cached z-ranges in a given level
	struct FClipmapInfo
	{
		FInt64Point PageSpaceLocation = FInt64Point(0, 0);
		double ViewCenterZ = 0.0;
		double ViewRadiusZ = 0.0;
		double WPODistanceDisableThresholdSquared = 0.0;
	};
	FClipmapInfo Clipmap;
};

class FVirtualShadowMapPerLightCacheEntry
{
public:
	FVirtualShadowMapPerLightCacheEntry(int32 MaxPersistentScenePrimitiveIndex, uint32 NumShadowMaps)
		: RenderedPrimitives(false, MaxPersistentScenePrimitiveIndex)
	{
		ShadowMapEntries.SetNum(NumShadowMaps);
	}

	void OnPrimitiveRendered(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bPrimitiveRevealed);
	
	/**
	 * The (local) VSM is fully cached if it is distant and has been rendered to previously
	 * "Fully" implies that we know all pages are mapped as well as rendered to (ignoring potential CPU-side object culling).
	 */
	bool IsFullyCached() const
	{
		return LocalCacheKey.bIsDistantLight && RenderedFrameNumber >= 0;
	}

	bool IsDistantLight() const
	{
		return LocalCacheKey.bIsDistantLight;
	}

	inline bool IsUncached() const { return bIsUncached; }

	inline bool ShouldUseReceiverMask() const { return bUseReceiverMask; }

	void MarkRendered(int32 FrameIndex) { RenderedFrameNumber = FrameIndex; }
	void Invalidate() { RenderedFrameNumber = -1; }
	bool IsInvalidated() const { return RenderedFrameNumber < 0; }

	void MarkScheduled(int32 FrameIndex) { ScheduledFrameNumber = FrameIndex; }
	int32 GetLastScheduledFrameNumber() const { return ScheduledFrameNumber; }
	
	void UpdateVirtualShadowMapId(int32 NextVirtualShadowMapId);
	int32 GetVirtualShadowMapId() const { return VirtualShadowMapId; }
	int32 GetPrevVirtualShadowMapId() const { return PrevVirtualShadowMapId; }

	/**
	 * Contains attributes which trigger invalidation if any change is detected.
	 */
	struct FClipmapCacheKey
	{
		bool operator==(const FClipmapCacheKey&) const = default;
		bool operator!=(const FClipmapCacheKey&) const = default;

		FVector LightDirection;
		int FirstLevel = 0;
	};

	void UpdateClipmap(const FClipmapCacheKey& InKey, bool bForceInvalidate, bool bInUseReceiverMask);

	struct FLocalLightCacheKey
	{
		bool operator==(const FLocalLightCacheKey&) const = default;
		bool operator!=(const FLocalLightCacheKey&) const = default;

		FMatrix WorldToLight;
		FVector PreShadowTranslation;
		bool bIsDistantLight = false;
	};

	/**
	 */
	void UpdateLocal(
		const FLocalLightCacheKey& InCacheKey,
		const FVector& NewLightOrigin,
		const float NewLightRadius,
		bool bForceInvalidate,
		bool bInUseReceiverMask);

	bool AffectsBounds(const FBoxSphereBounds& Bounds) const
	{
		return (LightRadius <= 0.0f) ||			// Infinite extent light (directional, etc)
			((Bounds.Origin - LightOrigin).SizeSquared() <= FMath::Square(LightRadius + Bounds.SphereRadius));
	}

	// Last frame numbers that we rendered/scheduled this light
	int32 RenderedFrameNumber = -1;
	int32 ScheduledFrameNumber = -1;

	int32 PrevVirtualShadowMapId = INDEX_NONE;
	int32 VirtualShadowMapId = INDEX_NONE;

	bool bIsUncached = false;
	bool bUseReceiverMask = false;

	// Tracks if this cache entry is being used "this render", i.e. "active". Note that there may be multiple renders per frame in the case of
	// scene captures or similar, so unlike the RenderedFrameNumber we don't use the scene frame number, but instead mark this
	// when a light is set up, and clear it when extracting frame data.
	bool bReferencedThisRender = false;

	// This tracks the last "rendered frame" the light was active
	uint32 LastReferencedFrameNumber = 0;

	// Primitives that have been rendered (not culled) the previous frame, when a primitive transitions from being culled to not it must be rendered into the VSM
	// Key culling reasons are small size or distance cutoff.
	TBitArray<> RenderedPrimitives;

	// One entry represents the cached state of a given shadow map in the set of either a clipmap(N), one cube map(6) or a regular VSM (1)
	TArray<FVirtualShadowMapCacheEntry> ShadowMapEntries;

	TArray<FVirtualShadowMapInstanceRange> PrimitiveInstancesToInvalidate;

private:
	template <typename CacheKeyType>
	void Update(const CacheKeyType& InKey, bool bForceInvalidate, bool bInUseReceiverMask, CacheKeyType& OutCacheKey);

	FLocalLightCacheKey LocalCacheKey;
	FClipmapCacheKey ClipmapCacheKey;

	// Rough bounds for invalidation culling
	FVector LightOrigin = FVector(0, 0, 0);
	float LightRadius = -1.0f;		// Negative means infinite
};

// Persistent buffers that we ping pong frame by frame
struct FVirtualShadowMapArrayFrameData
{
	TRefCountPtr<IPooledRenderTarget>			PageTable;
	TRefCountPtr<IPooledRenderTarget>			PageFlags;
	TRefCountPtr<IPooledRenderTarget>			PageReceiverMasks;

	TRefCountPtr<FRDGPooledBuffer>				UncachedPageRectBounds;
	TRefCountPtr<FRDGPooledBuffer>				AllocatedPageRectBounds;
	TRefCountPtr<FRDGPooledBuffer>				ProjectionData;
	TRefCountPtr<FRDGPooledBuffer>				PhysicalPageLists;
	TRefCountPtr<IPooledRenderTarget>			PageRequestFlags;
	TRefCountPtr<FRDGPooledBuffer>				NanitePerformanceFeedback;
	TRefCountPtr<FRDGPooledBuffer>				ThrottleBuffer;
	TRefCountPtr<IPooledRenderTarget>			PrevPrefilteredDistantPhysicalPagePoolSeparate;
	uint64 GetGPUSizeBytes(bool bLogSizes) const;
};

struct FVirtualShadowMapCacheKey
{
	uint32 ViewUniqueID;
	FLightSceneId LightSceneId;
	uint32 ShadowTypeId;

	inline bool operator==(const FVirtualShadowMapCacheKey& Other) const { return ViewUniqueID == Other.ViewUniqueID && Other.LightSceneId == LightSceneId && Other.ShadowTypeId == ShadowTypeId; }
};

inline uint32 GetTypeHash(FVirtualShadowMapCacheKey Key)
{
	return HashCombineFast(GetTypeHash(Key.LightSceneId), HashCombineFast(GetTypeHash(Key.ViewUniqueID), GetTypeHash(Key.ShadowTypeId)));
} 

class FVirtualShadowMapArrayCacheManager : public ISceneExtension
{
	friend class FVirtualShadowMapInvalidationSceneUpdater;
	DECLARE_SCENE_EXTENSION(RENDERER_API, FVirtualShadowMapArrayCacheManager);

public:
	using FEntryMap = TMap< FVirtualShadowMapCacheKey, TSharedPtr<FVirtualShadowMapPerLightCacheEntry> >;

	// Enough for er lots...
	static constexpr uint32 MaxStatFrames = 512 * 1024U;

	FVirtualShadowMapArrayCacheManager(FScene& InScene);
	virtual ~FVirtualShadowMapArrayCacheManager();

	// ISceneExtension
	static bool ShouldCreateExtension(FScene& InScene);
	virtual void InitExtension(FScene& InScene) override;
	virtual ISceneExtensionUpdater* CreateUpdater() override;
	virtual ISceneExtensionRenderer* CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags) override;

	// Called by VirtualShadowMapArray to potentially resize the physical pool
	// If the requested size is not already the size, all cache data is dropped and the pool is resized.
	void SetPhysicalPoolSize(FRDGBuilder& GraphBuilder, FIntPoint RequestedSize, int RequestedArraySize, uint32 MaxPhysicalPages);
	void FreePhysicalPool(FRDGBuilder& GraphBuilder);
	TRefCountPtr<IPooledRenderTarget> GetPhysicalPagePool() const { return PhysicalPagePool; }
	TRefCountPtr<FRDGPooledBuffer> GetPhysicalPageMetaData() const { return PhysicalPageMetaData; }

	// Called by VirtualShadowMapArray to potentially resize the HZB physical pool
	void SetHZBPhysicalPoolSize(FRDGBuilder& GraphBuilder, FIntPoint RequestedSize, int32 RequestedArraySize, const EPixelFormat Format);
	TRefCountPtr<IPooledRenderTarget> GetHZBPhysicalPool() const { return HZBPhysicalPagePoolArray; }
	void FreeHZBPhysicalPool(FRDGBuilder& GraphBuilder);

	// Invalidate the cache for all shadows, causing any pages to be rerendered
	void Invalidate(FRDGBuilder& GraphBuilder);

	/**
	 * Called before VSM builds page allocations to reallocate any lights that may not be visible this frame
	 * but that may still have cached physical pages. We reallocate new VSM each frame for these to allow the associated
	 * physical pages to live through short periods of being offscreen or otherwise culled. This function also removes
	 * entries that are too old.
	 */
	void UpdateUnreferencedCacheEntries(FVirtualShadowMapArray& VirtualShadowMapArray);

	/**
	* Call at end of frame to extract resouces from the virtual SM array to preserve to next frame.
	* 
	* If bAllowPersistentData is false, all previous frame data is dropped and cache (and HZB!) data will not be available for the next frame.
	* This flag is mostly intended for temporary editor resources like thumbnail rendering that will be used infrequently but often not properly destructed.
	* We need to ensure that the VSM data associated with these renderer instances gets dropped.
	*/ 
	void ExtractFrameData(FRDGBuilder& GraphBuilder,
		FVirtualShadowMapArray &VirtualShadowMapArray,
		const FSceneRenderer& SceneRenderer,
		bool bAllowPersistentData);

	/**
	 * Finds an existing cache entry and moves to the active set or creates a fresh one.
	 * TypeIdTag is an arbitrary type ID to make it possible to have more than one shadow map for the same light & view, it is up to the user to make sure there are no collisions.
	 */
	TSharedPtr<FVirtualShadowMapPerLightCacheEntry> FindCreateLightCacheEntry(FLightSceneId LightSceneId, uint32 ViewUniqueID, uint32 NumShadowMaps, uint32 TypeIdTag = 0u);

	bool IsCacheEnabled();
	bool IsCacheDataAvailable();
	bool IsHZBDataAvailable();

	FRHIGPUMask GetCacheValidGPUMask() const
	{
#if WITH_MGPU
		return CacheValidGPUMask;
#else
		return FRHIGPUMask::GPU0();
#endif
	}

	void UpdateCacheValidGPUMask(FRHIGPUMask GPUMask, bool bMergeMask)
	{
#if WITH_MGPU
		if (bMergeMask)
		{
			CacheValidGPUMask |= GPUMask;
		}
		else
		{
			// To handle initialization when first allocating cache resources, we overwrite the mask.  This is necessary because the FRHIGPUMask doesn't
			// support empty masks.  Also, this deals with cases where the cache is cleared -- the cache resources will be missing, and it can use this
			// code path to set the mask to a known state when they get re-created.
			CacheValidGPUMask = GPUMask;
		}
#endif
	}

	bool IsAccumulatingStats();
	bool IsVisualizePassEnabled(const FViewInfo& View, int32 ViewIndex, EVSMVisualizationPostPass Pass) const;
	FScreenPassTexture AddVisualizePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, int32 ViewIndex, EVSMVisualizationPostPass Pass, FScreenPassTexture& SceneColor, FScreenPassRenderTarget& Output);

	using FInstanceGPULoadBalancer = TInstanceCullingLoadBalancer<SceneRenderingAllocator>;

	/**
	 * Helper to collect primitives that need invalidation, filters out redundant adds and also those that are not yet known to the GPU
	 */
	class FInvalidatingPrimitiveCollector
	{
	public:
		FInvalidatingPrimitiveCollector(
			FVirtualShadowMapArrayCacheManager* InCacheManager);

		void AddPrimitivesToInvalidate();

		// Primitive was removed from the scene
		void Removed(FPrimitiveSceneInfo* PrimitiveSceneInfo)
		{
			AddInvalidation(PrimitiveSceneInfo, EInvalidationCause::Removed);
		}

		// Primitive moved/transform was updated
		// NOTE: Cache flags should not be cleared in the pre-pass if there is going to be a post-pass
		void UpdatedTransform(FPrimitiveSceneInfo* PrimitiveSceneInfo)
		{
			AddInvalidation(PrimitiveSceneInfo, EInvalidationCause::Updated);
		}

		void Added(FPrimitiveSceneInfo* PrimitiveSceneInfo)
		{
			AddInvalidation(PrimitiveSceneInfo, EInvalidationCause::Added);
		}

		FInstanceGPULoadBalancer Instances;
		TBitArray<SceneRenderingAllocator> InvalidatedPrimitives;
		TBitArray<SceneRenderingAllocator> RemovedPrimitives;

		// Temp filtered array of any cache entries that may need invalidation
		TArray<const FVirtualShadowMapPerLightCacheEntry*, SceneRenderingAllocator> CacheEntriesForInvalidation;

	private:
		enum class EInvalidationCause
		{
			Added,
			Removed,
			Updated,
		};

		void AddInvalidation(FPrimitiveSceneInfo* PrimitiveSceneInfo, EInvalidationCause InvalidationCause);

		void AddInvalidation(
			const FVirtualShadowMapPerLightCacheEntry& CacheEntry,
			int32 InstanceSceneDataOffset,
			int32 NumInstanceSceneDataEntries,
			bool bCachePrimitiveAsDynamic,
			bool bLightRadiusCulling = false,
			const FBoxSphereBounds& PrimitiveBounds = FBoxSphereBounds());

		FScene& Scene;
		FVirtualShadowMapArrayCacheManager& Manager;
	};

	void ProcessInvalidations(FRDGBuilder& GraphBuilder, FInvalidatingPrimitiveCollector& InvalidatingPrimitiveCollector);

	uint64 GetGPUSizeBytes(bool bLogSizes) const;

	const FVirtualShadowMapArrayFrameData& GetPrevBuffers() const { return PrevBuffers; }

	uint32 GetStatusFeedbackMessageId() const { return StatusFeedbackSocket.GetMessageId().GetIndex(); }

#if !UE_BUILD_SHIPPING
	uint32 GetStatsFeedbackMessageId() const { return StatsFeedbackSocket.GetMessageId().IsValid() ? StatsFeedbackSocket.GetMessageId().GetIndex() : INDEX_NONE; }
#endif

	float GetGlobalResolutionLodBias() const { return GlobalResolutionLodBias; }

	inline FEntryMap::TIterator CreateEntryIterator()
	{
		return CacheEntries.CreateIterator();
	}

	inline FEntryMap::TConstIterator CreateConstEntryIterator() const
	{
		return CacheEntries.CreateConstIterator();
	}

	UE::Renderer::Private::IShadowInvalidatingInstances* GetInvalidatingInstancesInterface();

	FRDGBufferRef UploadCachePrimitiveAsDynamic(FRDGBuilder& GraphBuilder) const;

	// NOTE: Can move to private after we remove old invalidations path
	void ReallocatePersistentPrimitiveIndices();

	// TODO: locked to physical pool size
	FIntPoint GetFilterablePagePoolSizePages() const { return PhysicalPagePool ? (PhysicalPagePool->GetDesc().Extent / FVirtualShadowMap::PageSize) : FIntPoint(0,0); }
	uint32 GetMaxFilterablePages() const { FIntPoint Size = GetFilterablePagePoolSizePages(); return Size.X * Size.Y; }

	uint32 GetPhysicalMaxWidth();
private:

	/**
	 * Handle light removal, need to clear out cache entries as the ID may be reused after this.
	 */
	void ProcessRemovedLights(const FSceneRendererLightBitArray& RemovedLightMask);

	friend class FVirtualShadowMapInvalidationSceneRenderer;
	friend FVirtualShadowMapArray;

	/** 
	 */
	class FShadowInvalidatingInstancesImplementation : public UE::Renderer::Private::IShadowInvalidatingInstances
	{
	public:
		FShadowInvalidatingInstancesImplementation(FVirtualShadowMapArrayCacheManager &InCacheManager) : CacheManager(InCacheManager) {}
		virtual void AddPrimitive(const FPrimitiveSceneInfo *PrimitiveSceneInfo);
		virtual void AddInstanceRange(FPersistentPrimitiveIndex PersistentPrimitiveIndex, uint32 InstanceSceneDataOffset, uint32 NumInstanceSceneDataEntries);

		FVirtualShadowMapArrayCacheManager &CacheManager;
		TArray<FVirtualShadowMapInstanceRange> PrimitiveInstancesToInvalidate;
	};

	struct FInvalidationPassCommon
	{
		FVirtualShadowMapUniformParameters* UniformParameters;
		TRDGUniformBufferRef<FVirtualShadowMapUniformParameters> VirtualShadowMapUniformBuffer;
		FRDGBufferRef AllocatedPageRectBounds;
	};

	FInvalidationPassCommon GetUniformParametersForInvalidation(FRDGBuilder& GraphBuilder) const;

	void SetInvalidateInstancePagesParameters(
		FRDGBuilder& GraphBuilder,
		const FInvalidationPassCommon& InvalidationPassCommon,
		FInvalidatePagesParameters* PassParameters) const;

	void UpdateCachePrimitiveAsDynamic(FInvalidatingPrimitiveCollector& InvalidatingPrimitiveCollector);

	// Invalidate instances based on CPU instance ranges. This is used for CPU-based updates like object transform changes, etc.
	void ProcessInvalidations(FRDGBuilder& GraphBuilder, const FInvalidationPassCommon& InvalidationPassCommon, const FInstanceGPULoadBalancer& Instances) const;
	
	void ExtractStats(FRDGBuilder& GraphBuilder, FVirtualShadowMapArray &VirtualShadowMapArray);

	// Remove old info used to track logging.
	void TrimLoggingInfo();

	FVirtualShadowMapArrayFrameData PrevBuffers;
	FVirtualShadowMapUniformParameters PrevUniformParameters;

	// The actual physical texture data is stored here rather than in VirtualShadowMapArray (which is recreated each frame)
	// This allows us to (optionally) persist cached pages between frames. Regardless of whether caching is enabled,
	// we store the physical pool here.
	TRefCountPtr<IPooledRenderTarget> PhysicalPagePool;
	TRefCountPtr<IPooledRenderTarget> HZBPhysicalPagePoolArray;
	ETextureCreateFlags PhysicalPagePoolCreateFlags = TexCreate_None;
	TRefCountPtr<FRDGPooledBuffer> PhysicalPageMetaData;
	uint32 MaxPhysicalPages = 0;

	// Index the Cache entries by the light ID
	FEntryMap CacheEntries;

	// Store the last time a primitive caused an invalidation for dynamic/static caching purposes
	// NOTE: Set bits as dynamic since the container makes it easier to iterate those
	TBitArray<> CachePrimitiveAsDynamic;
	// Indexed by PersistentPrimitiveIndex
	TArray<uint32> LastPrimitiveInvalidatedFrame;

	// Stores stats over frames when activated.
	TRefCountPtr<FRDGPooledBuffer> AccumulatedStatsBuffer;
	bool bAccumulatingStats = false;
	FRHIGPUBufferReadback* GPUBufferReadback = nullptr;

	GPUMessage::FSocket StatusFeedbackSocket;

	// Current global resolution bias (when enabled) based on feedback from page pressure, etc.
	float GlobalResolutionLodBias = 0.0f;
	uint32 LastFrameOverPageAllocationBudget = 0;
	
	// Debug stuff
#if !UE_BUILD_SHIPPING
	FDelegateHandle ScreenMessageDelegate;
	uint32 LoggedOverflowFlags = 0;
	TArray<float, TInlineAllocator<VSM_STAT_OVERFLOW_FLAG_NUM>> LastOverflowTimes;

	FText GetOverflowMessage(uint32 OverflowTypeIndex) const;
	
	// Socket for optional stats that are only sent back if enabled
	GPUMessage::FSocket StatsFeedbackSocket;

	// Stores the last time (wall-clock seconds since app-start) that an non-nanite page area message was logged,
	TArray<float> LastLoggedPageOverlapAppTime;

	// Map to track non-nanite page area items that are shown on screen
	struct FLargePageAreaItem
	{
		uint32 PageArea;
		float LastTimeSeen;
	};
	TMap<uint32, FLargePageAreaItem> LargePageAreaItems;

	TArray<FString> NPFDiagnosticMessages {};
	TMap<uint32, uint8> NPFDiagnosticTimer {};

#endif // UE_BUILD_SHIPPING

	FShadowInvalidatingInstancesImplementation ShadowInvalidatingInstancesImplementation;

#if WITH_MGPU
	FRHIGPUMask CacheValidGPUMask;
#endif

	struct FViewData
	{
		// For each instance we need to store information over time whether:
		// Bit vector 0: CacheAsDynamic
		// Bit vector 1: IsTracked
		static constexpr int32 NumBitsPerInstance = 2;

		FViewData();

		// Buffer that stores NumBitsPerInstance bits per instance indicating whether it is dynamic of static.
		TPersistentStructuredBuffer<uint32> InstanceState;
	};
	// Indexed by persistent view ID
	TSparseArray<FViewData> ViewData;
	// per instance bit array X NumBitsPerInstance to store the state bits
	int32 InstanceStateMaskWordStride = 0;

	// Retains a reference to a dummy (single page with mips) such that we don't need to re-clear it every frame when the feature is disabled.
	// Used to bind as UAV for passes to not have to use permutations.
	TRefCountPtr<IPooledRenderTarget> PageTableDummy;
};


class FVirtualShadowMapInvalidationSceneUpdater : public ISceneExtensionUpdater
{
	DECLARE_SCENE_EXTENSION_UPDATER(FVirtualShadowMapInvalidationSceneUpdater, FVirtualShadowMapArrayCacheManager);

public:
	FVirtualShadowMapInvalidationSceneUpdater(FVirtualShadowMapArrayCacheManager& InCacheManager);

	virtual void PreLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet);
	virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override;
	virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override;
	virtual void PostGPUSceneUpdate(FRDGBuilder& GraphBuilder) override;

private:
	FVirtualShadowMapArrayCacheManager& CacheManager;

	const FScenePostUpdateChangeSet* PostUpdateChangeSet = nullptr;
};
