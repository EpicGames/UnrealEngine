// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapClipmap.cpp
=============================================================================*/

#include "VirtualShadowMapClipmap.h"
#include "CoreMinimal.h"
#include "CollisionQueryParams.h"
#include "ConvexVolume.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "Logging/LogMacros.h"
#include "Math/DoubleFloat.h"
#include "Math/Plane.h"
#include "Math/Sphere.h"
#include "MeshPassProcessor.h"
#include "Misc/LargeWorldCoordinates.h"
#include "PrimitiveSceneInfo.h"
#include "RenderGraphBuilder.h"
#include "RHI.h"
#include "SceneInterface.h"
#include "SceneRendering.h"
#include "SceneTypes.h"
#include "SceneView.h"
#include "SceneViewOwner.h"
#include "Shader.h"
#include "Stats/Stats.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VirtualShadowMaps/VirtualShadowMapCacheManager.h"
#include "SceneViewState.h"

static TAutoConsoleVariable<int32>  CVarForceInvalidateDirectionalVSM(
	TEXT("r.Shadow.Virtual.Cache.ForceInvalidateDirectional"),
	0,
	TEXT("Forces the clipmap to always invalidate, useful to emulate a moving sun to avoid misrepresenting cache performance."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarVirtualShadowMapResolutionLodBiasDirectional(
	TEXT( "r.Shadow.Virtual.ResolutionLodBiasDirectional" ),
	0.0f,
	TEXT( "Bias applied to LOD calculations for directional lights. -1.0 doubles resolution, 1.0 halves it and so on." ),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarVirtualShadowMapResolutionLodBiasDirectionalMoving(
	TEXT( "r.Shadow.Virtual.ResolutionLodBiasDirectionalMoving" ),
	0.0f,
	TEXT( "Bias applied to LOD calculations for directional lights that are moving. -1.0 doubles resolution, 1.0 halves it and so on.\n" )
	TEXT( "The bias transitions smoothly back to ResolutionLodBiasDirectional as the light transitions to non-moving, see 'r.Shadow.Scene.LightActiveFrameCount'." ),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVirtualShadowMapClipmapFirstLevel(
	TEXT( "r.Shadow.Virtual.Clipmap.FirstLevel" ),
	6,
	TEXT( "First level of the virtual clipmap. Lower values allow higher resolution shadows closer to the camera, but may increase page count." ),
	ECVF_Scalability | ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarVirtualShadowMapClipmapLastLevel(
	TEXT( "r.Shadow.Virtual.Clipmap.LastLevel" ),
	22,
	TEXT( "Last level of the virtual clipmap. Indirectly determines radius the clipmap can cover. Each extra level doubles the maximum range, but may increase page count." ),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarVirtualShadowMapClipmapFirstCoarseLevel(
	TEXT("r.Shadow.Virtual.Clipmap.FirstCoarseLevel"),
	15,
	TEXT("First level of the clipmap to mark coarse pages for. Lower values allow higher resolution coarse pages near the camera but increase total page counts."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarVirtualShadowMapClipmapLastCoarseLevel(
	TEXT("r.Shadow.Virtual.Clipmap.LastCoarseLevel"),
	18,
	TEXT("Last level of the clipmap to mark coarse pages for. Higher values provide dense clipmap data for a longer radius but increase total page counts."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarVirtualShadowMapClipmapZRangeScale(
	TEXT("r.Shadow.Virtual.Clipmap.ZRangeScale"),
	1000.0f,
	TEXT("Scale of the clipmap level depth range relative to the radius. Affects z-near/z-far of the shadow map. Should generally be at least 10 or it will result in excessive cache invalidations. Values that are too large cause depth imprecisions and shadow flickering."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarVirtualShadowMapClipmapMinCameraViewportWidth(
	TEXT("r.Shadow.Virtual.Clipmap.MinCameraViewportWidth"),
	0,
	TEXT("If greater than zero, clamps the camera viewport dimensions used to adjust the clipmap resolution.\n")
	TEXT("This can be useful to avoid dynamic resolution indirectly dropping the shadow resolution far too low."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarClipmapWPODisableDistance(
	TEXT("r.Shadow.Virtual.Clipmap.WPODisableDistance"),
	1,
	TEXT("When enabled, disables WPO animation in clipmap levels based on a primitive's WPO disable distance and r.Shadow.Virtual.Clipmap.WPODisableDistance.LodBias setting."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarClipmapWPODisableDistanceLodBias(
	TEXT("r.Shadow.Virtual.Clipmap.WPODisableDistance.LodBias"),
	3,
	TEXT("The number of clipmap levels further than the distance that an instance would be animated to allow shadow animation.\n")
	TEXT("Typically 2-4 works well but may need to be adjusted for very low light angles with significant WPO movement."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarOrthoVSMEstimateClipmapLevels(
	TEXT("r.Ortho.VSM.EstimateClipmapLevels"),
	true,
	TEXT("Enable/Disable calculating the FirstLevel VSM based on the current camera OrthoWidth"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarOrthoVSMClipmapLODBias(
	TEXT("r.Ortho.VSM.ClipmapLODBias"),
	0,
	TEXT("LOD setting for adjusting the VSM first level from it's OrthoWidth based value."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarOrthoVSMProjectViewOrigin(
	TEXT("r.Ortho.VSM.ProjectViewOrigin"),
	true,
	TEXT("Enable/Disable moving the WorldOrigin of the VSM clipmaps to focus around the ViewTarget (if present)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarOrthoVSMRayCastViewOrigin(
	TEXT("r.Ortho.VSM.RayCastViewOrigin"),
	true,
	TEXT("Enable/Disable whether the ViewOrigin should be estimated with a raycast if the ViewTarget is not present (i.e. standalone camera)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarMarkCoarsePagesDirectional(
	TEXT("r.Shadow.Virtual.MarkCoarsePagesDirectional"),
	2,
	TEXT("Marks coarse pages in local light virtual shadow maps so that low resolution data is available everywhere.\n")
	TEXT(" 0 - Disabled\n")
	TEXT(" 1 - Always mark the last clipmap level and levels indicated by First/LastCoarseLevel.\n")
	TEXT(" 2 - Performance Mode: Suppress dynamic invalidations in coarse pages due to geometry changes (e.g., moving objects or WPO or animation).\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarVSMClipmapCullDynamicTightly(
	TEXT("r.Shadow.Virtual.Clipmap.CullDynamicTightly"),
	true,
	TEXT("When enabled(default) the far culling plane for uncached clipmpap levels is set to the size of the clipmap level.\n")
	TEXT("  Currently, this is only used when r.Shadow.Virtual.Cache.ForceInvalidateDirectional is enabled, as cached levels need a larger range."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarVSMUseReceiverMaskDirectional(
	TEXT("r.Shadow.Virtual.UseReceiverMaskDirectional"),
	true,
	TEXT("Use receiver page masks with directional lights. This enables much more effective culling especially at lower resolutions."),	
	ECVF_RenderThreadSafe
);

bool IsVirtualShadowMapDirectionalReceiverMaskEnabled()
{
	return CVarVSMUseReceiverMaskDirectional.GetValueOnRenderThread() != 0;
}

// "Virtual" clipmap level to clipmap radius
// NOTE: This is the radius of around the clipmap origin that this level must cover
// The actual clipmap dimensions will be larger due to snapping and other accomodations
float FVirtualShadowMapClipmap::GetLevelRadius(float AbsoluteLevel)
{
	// NOTE: Virtual clipmap indices can be negative (although not commonly)
	// Clipmap level rounds *down*, so radius needs to cover out to 2^(Level+1), where it flips
	return FMath::Pow(2.0f, AbsoluteLevel + 1.0f);
}

FVirtualShadowMapClipmapConfig FVirtualShadowMapClipmapConfig::GetGlobal()
{
	bool bMarkCoarsePagesDirectional = CVarMarkCoarsePagesDirectional.GetValueOnRenderThread() != 0;
	return FVirtualShadowMapClipmapConfig 
	{ 
		.FirstLevel = CVarVirtualShadowMapClipmapFirstLevel.GetValueOnRenderThread(),
		.LastLevel = CVarVirtualShadowMapClipmapLastLevel.GetValueOnRenderThread(),
		.FirstCoarseLevel = bMarkCoarsePagesDirectional ? CVarVirtualShadowMapClipmapFirstCoarseLevel.GetValueOnRenderThread() : -1,
		.LastCoarseLevel = bMarkCoarsePagesDirectional ? CVarVirtualShadowMapClipmapLastCoarseLevel.GetValueOnRenderThread() : -1,
		.ResolutionLodBias = CVarVirtualShadowMapResolutionLodBiasDirectional.GetValueOnRenderThread(),
		.ResolutionLodBiasMoving = CVarVirtualShadowMapResolutionLodBiasDirectionalMoving.GetValueOnRenderThread(),
		.bForceInvalidate = CVarForceInvalidateDirectionalVSM.GetValueOnRenderThread() != 0,
		.bCullDynamicTightly = CVarVSMClipmapCullDynamicTightly.GetValueOnRenderThread(),
		.bUseReceiverMask = IsVirtualShadowMapDirectionalReceiverMaskEnabled()
	};
}


FVirtualShadowMapClipmap::FVirtualShadowMapClipmap(
	FVirtualShadowMapArray& VirtualShadowMapArray,
	const FLightSceneInfo& InLightSceneInfo,
	const FViewMatrices& CameraViewMatrices,
	FIntPoint CameraViewRectSize,
	const FViewInfo* InDependentView,
	float LightMobilityFactor,
	const FVirtualShadowMapClipmapConfig& InConfig)
	: Config(InConfig),
	  LightSceneInfo(InLightSceneInfo),
	  DependentView(InDependentView)
{
	FVirtualShadowMapArrayCacheManager* VirtualShadowMapArrayCacheManager = VirtualShadowMapArray.CacheManager;

	LightDirection = LightSceneInfo.Proxy->GetDirection().GetSafeNormal();
	const FMatrix WorldToLightRotationMatrix = FInverseRotationMatrix(LightDirection.Rotation());

	const FMatrix FaceMatrix(
		FPlane( 0, 0, 1, 0 ),
		FPlane( 0, 1, 0, 0 ),
		FPlane(-1, 0, 0, 0 ),
		FPlane( 0, 0, 0, 1 ));

	WorldToLightViewRotationMatrix = WorldToLightRotationMatrix * FaceMatrix;
	// Pure rotation matrix
	FMatrix ViewToWorldRotationMatrix = WorldToLightViewRotationMatrix.GetTransposed();
	
	// Optionally clamp camera viewport to avoid excessively low resolution shadows with dynamic resolution	
	const int32 MinCameraViewportWidth = CVarVirtualShadowMapClipmapMinCameraViewportWidth.GetValueOnRenderThread();	
	bool bIsOrthographicCamera = !CameraViewMatrices.IsPerspectiveProjection();

	int32 CameraViewportWidth = CameraViewRectSize.X;
	if (bIsOrthographicCamera)
	{
		/**
		 * Orthographic cameras have uniform depth, so basing the LodScale on the width alone can cause issues when selecting the clipmap area to resolve
		 * at larger scale views. Instead we use the OrthoWidth. This gives a larger area for the shadows to be drawn to and ensures shadows further away/in 
		 * the corners of the view rect have the correct LOD resolution. We default to the viewport as a minimum.
		 */	
		CameraViewportWidth = FMath::Max(FMath::CeilToInt(CameraViewMatrices.GetClipToView().M[0][0] * 2.0f), CameraViewportWidth);
	}
	CameraViewportWidth = FMath::Max(MinCameraViewportWidth, CameraViewportWidth);

	// NOTE: Rotational (roll) invariance of the directional light depends on square pixels so we just base everything on the camera X scales/resolution
	// NOTE: 0.5 because we double the size of the clipmap region below to handle snapping
	float LodScale = 0.5f / CameraViewMatrices.GetProjectionScale().X;
	LodScale *= float(FVirtualShadowMap::VirtualMaxResolutionXY) / float(CameraViewportWidth);
	
	// For now we adjust resolution by just biasing the page we look up in. This is wasteful in terms of page table vs.
	// just resizing the virtual shadow maps for each clipmap, but convenient for now. This means we need to additionally bias
	// which levels are present.
	ResolutionLodBias = FVirtualShadowMapArray::InterpolateResolutionBias(Config.ResolutionLodBias, Config.ResolutionLodBiasMoving, LightMobilityFactor) + FMath::Log2(LodScale);
	ResolutionLodBias += GetLightSceneInfo().Proxy->GetVSMResolutionLodBias();
	// Clamp negative absolute resolution biases as they would exceed the maximum resolution/ranges allocated
	ResolutionLodBias = FMath::Max(0.0f, ResolutionLodBias);

	WorldOrigin = CameraViewMatrices.GetViewOrigin();
	CameraToViewTarget = FVector::ZeroVector;
	if (bIsOrthographicCamera && CVarOrthoVSMProjectViewOrigin.GetValueOnRenderThread())
	{
		/**
		* If enabled, use the ViewTarget location as the WorldOrigin location, this helps with scaling VSMs in Ortho
		* as the clipmaps emanate more evenly from the focus of the view.
		* A ViewTarget is not always necessarily present, but there isn't really an alternative way to estimate the best
		* WorldOrigin for this effect to work well right now without the ViewTarget set.
		*/
		CameraToViewTarget = CameraViewMatrices.GetCameraToViewTarget();
		if (CameraToViewTarget.Length() == 0.0f && CVarOrthoVSMRayCastViewOrigin.GetValueOnRenderThread()
			&& DependentView && DependentView->Family && DependentView->Family->Scene)
		{
			if (UWorld* World = DependentView->Family->Scene->GetWorld())
			{
				FVector ViewForward = CameraViewMatrices.GetWorldToView().GetColumn(2);
				FCollisionObjectQueryParams ObjectParams = FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllObjects);
				
				FCollisionQueryParams CollisionParams(FName("OrthoCamera_VSMTrace"), true);
				if (DependentView->ViewActor.IsSet())
				{
					CollisionParams.AddIgnoredSourceObject(DependentView->ViewActor.ActorUniqueId);
				}

				FHitResult Hit(ForceInit);

				if (World->LineTraceSingleByObjectType(
					Hit,		//result
					WorldOrigin,	//start
					WorldOrigin + ViewForward * FMath::Abs(CameraViewMatrices.GetClipToView().M[2][2]), //end
					ObjectParams,//collision channel
					CollisionParams
				))
				{
					CameraToViewTarget = ViewForward * Hit.Distance;
				}
			}
		}
		WorldOrigin += CameraToViewTarget;
	}

	FirstLevel = Config.FirstLevel;
	int32 LastLevel = Config.LastLevel;	
	if (bIsOrthographicCamera && CVarOrthoVSMEstimateClipmapLevels.GetValueOnRenderThread())
	{
		/**
		* For Ortho projections, this branch bases the first level VSM on the set OrthoWidth. This reduces the number of clipmaps generated
		* and also scales the precision of the clipmaps depending on the scene.
		* 
		* To be on the safe side, we output -1 FirstLevel compared to what the full OrthoWidth would output. The InvProjectionMatrix outputs
		* half the OrthoWidth in the [0][0] position, and as we are using Log2, we can just use that raw value, rather than multiplying it then
		* subtracting a level.
		*/
		int32 OrthoFirstLevel = FMath::FloorToInt(FMath::Log2(static_cast<float>(CameraViewMatrices.GetClipToView().M[0][0])));
		if (OrthoFirstLevel > FirstLevel)
		{
			//Only apply the ortho level if it above the desired minimum first level.
			FirstLevel = OrthoFirstLevel;
		}
		//Allow manual correction using the Ortho only FirstLevel bias.
		FirstLevel = FMath::Max(FirstLevel + CVarOrthoVSMClipmapLODBias.GetValueOnRenderThread(), 0);
	}

	LastLevel = FMath::Max(FirstLevel, LastLevel);
	int32 LevelCount = LastLevel - FirstLevel + 1;

	// Per-clipmap projection data
	LevelData.Empty();
	LevelData.AddDefaulted(LevelCount);

	// Enable various fast paths if we are not storing cache data
	bool bUncached = Config.bForceInvalidate || !VirtualShadowMapArrayCacheManager->IsCacheEnabled();

	// TODO: We need a light/cache entry for every light/VSM now, but scene captures may not have persistent view state for indexing
	// This is likely to all change as we refactor the multiple cache managers stuff anyways; for now this is hopefully safe since they do have separate copies
	const uint32 UniqueViewKey = InDependentView->ViewState ? InDependentView->ViewState->GetViewKey() : 0U;
	PerLightCacheEntry = VirtualShadowMapArrayCacheManager->FindCreateLightCacheEntry(LightSceneInfo.Id, UniqueViewKey, LevelCount, Config.ShadowTypeId);
	PerLightCacheEntry->UpdateClipmap( { LightDirection, FirstLevel }, bUncached, Config.bUseReceiverMask);

	const int RadiiPerLevel = 4;
	FVector SnappedOriginLn = WorldToLightViewRotationMatrix.TransformPosition(WorldOrigin);
	{
		const int RadiusLn = GetLevelRadius(static_cast<float>(LastLevel));
		FInt64Point OriginSnapUnitsLn(
			FMath::RoundToInt64(SnappedOriginLn.X / RadiusLn),
			FMath::RoundToInt64(SnappedOriginLn.Y / RadiusLn));
		SnappedOriginLn.X = OriginSnapUnitsLn.X * RadiusLn;
		SnappedOriginLn.Y = OriginSnapUnitsLn.Y * RadiusLn;
	}

	// We expand the depth range of the clipmap level to allow for camera movement without having to invalidate cached shadow data
	// (See VirtualShadowMapCacheManager::UpdateClipmap for invalidation logic.)
	// This also better accomodates SMRT where we want to avoid stepping outside of the Z bounds of a given clipmap
	// NOTE: It's tempting to use a single global Z range for the entire clipmap (which avoids some SMRT overhead too)
	// but this can cause precision issues with cached pages very near the camera.
	const double ViewRadiusZScale = CVarVirtualShadowMapClipmapZRangeScale.GetValueOnRenderThread();
	const FVector ViewCenter = WorldToLightViewRotationMatrix.TransformPosition(WorldOrigin);

	float PrevCullDistance = 0.0f;
	for (int32 ClipmapIndex = 0; ClipmapIndex < LevelCount; ++ClipmapIndex)
	{
		FLevelData& Level = LevelData[ClipmapIndex];
		const int32 AbsoluteLevel = ClipmapIndex + FirstLevel;		// Absolute (virtual) level index

		const double RawLevelRadius = GetLevelRadius(static_cast<float>(AbsoluteLevel));

		double HalfLevelDim = 2.0 * RawLevelRadius;
		double SnapSize = RawLevelRadius;

		FVector SnappedViewCenter = ViewCenter;
		FVector2D CenterSnapUnits(
			FMath::RoundToDouble(ViewCenter.X / SnapSize),
			FMath::RoundToDouble(ViewCenter.Y / SnapSize));
		SnappedViewCenter.X = CenterSnapUnits.X * SnapSize;
		SnappedViewCenter.Y = CenterSnapUnits.Y * SnapSize;

		FInt64Point CornerOffset;
		CornerOffset.X = -(int64_t)CenterSnapUnits.X + (RadiiPerLevel/2);
		CornerOffset.Y =  (int64_t)CenterSnapUnits.Y + (RadiiPerLevel/2);

		const FVector SnappedWorldCenter = ViewToWorldRotationMatrix.TransformPosition(SnappedViewCenter);

		Level.WorldCenter = SnappedWorldCenter;
		Level.CornerOffset = CornerOffset;

		// A relative corner offset is used for LWC reasons.
		// The reference point is WorldOrigin snapped to a grid of GetLevelRadius(LastLevel),
		// because points on this grid are guaranteed to also be present on lower levels,
		// therefore allowing the offsets to represented as factors of level radii without precision loss.
		const FInt64Point SnappedPageOriginLi(-SnappedViewCenter.X, SnappedViewCenter.Y);
		const FInt64Point SnappedPageOriginLn(-SnappedOriginLn.X, SnappedOriginLn.Y);
		const FInt64Point RelativeCornerOffset = SnappedPageOriginLi - SnappedPageOriginLn + ((RadiiPerLevel / 2) * (int64_t)SnapSize);
		Level.RelativeCornerOffset = FIntPoint(
			static_cast<int32>(RelativeCornerOffset.X / SnapSize),
			static_cast<int32>(RelativeCornerOffset.Y / SnapSize));

		// Check if we have a cache entry for this level
		// If we do and it covers our required depth range, we can use cached pages. Otherwise we need to invalidate.
		FVirtualShadowMapCacheEntry* ClipmapLevelEntry = &PerLightCacheEntry->ShadowMapEntries[ClipmapIndex];

		double ViewRadiusZ = RawLevelRadius * ViewRadiusZScale;
		double ViewCenterDeltaZ = 0.0f;

		// We snap to half the size of the VSM at each level
		check((FVirtualShadowMap::Level0DimPagesXY & 1) == 0);
		FInt64Point PageOffset(CornerOffset * (FVirtualShadowMap::Level0DimPagesXY >> 2));

		// This is the "WPO distance disable" threshold at which we allow WPO animation into this clipmap
		// See VirtualShadowMapIsWPOAllowed in VirtualShadowMappageCacheCommon.ush
		// NOTE: We use the ResolutionLodBias here because it is desirable for the shadow WPO distance to not
		// vary a ton at different scalability settings. In particular, we do not want it to get *closer* to the
		// caster at higher quality settings, as it otherwise would.
		// We cannot however easily incorportate the *global* GPU resolution bias as this
		// decision needs to be constant, otherwise we'd have to track all these variables for invalidations.
		// As it is, if these variables change we can do full cache invalidation as they are not expected to be
		// changing on the fly in a game.
		// We quantize the result to a powers of two (similar to the clipmaps) to avoid continuous invalidation in
		// cases like window resizes and similar.
		if (CVarClipmapWPODisableDistance.GetValueOnRenderThread() > 0)
		{
			const int32 WPODisableDistanceLodBias = CVarClipmapWPODisableDistanceLodBias.GetValueOnRenderThread();
			double WPOThresholdCombinedLevel = FMath::CeilToDouble(static_cast<double>(AbsoluteLevel - WPODisableDistanceLodBias) - ResolutionLodBias);
			// NOTE: Squared
			Level.WPODistanceDisableThresholdSquared = FMath::Pow(2.0, 2.0 * WPOThresholdCombinedLevel);
		}
		else
		{
			Level.WPODistanceDisableThresholdSquared = 0.0;
		}

		ClipmapLevelEntry->UpdateClipmapLevel(
			*PerLightCacheEntry,
			PageOffset,
			RawLevelRadius,
			ViewCenter.Z,
			ViewRadiusZ,
			Level.WPODistanceDisableThresholdSquared);

		// Update min/max Z based on the cached page (if present and valid)
		// We need to ensure we use a consistent depth range as the camera moves for each level
		ViewCenterDeltaZ = ViewCenter.Z - ClipmapLevelEntry->Clipmap.ViewCenterZ;
		ViewRadiusZ = ClipmapLevelEntry->Clipmap.ViewRadiusZ;

		Level.DynamicDepthCullRange.X = 0.0f;
		Level.DynamicDepthCullRange.Y = FLT_MAX;

		const float CullingEndDistance = GetLevelRadius(static_cast<float>(AbsoluteLevel) - ResolutionLodBias); 
		// TODO: Maybe we remove this path if we end up swapping things to uncached dynamically
		// This path changes the depth range and could have visual effects in some edge cases
		if (Config.bForceInvalidate && Config.bCullDynamicTightly)
		{
			// Far plane in the clipmap should be just the end of the visible range
			const double ViewRangeZ = CullingEndDistance + ViewRadiusZ;
			const double ZScale = 1.0 / ViewRangeZ;
			const double ZOffset = ViewRadiusZ;
			Level.ViewToClip = FReversedZOrthoMatrix(HalfLevelDim, HalfLevelDim, ZScale, ZOffset);
		}
		else
		{
			// NOTE: These values are all in regular ranges after being offset
			const double ZScale = 0.5 / ViewRadiusZ;
			const double ZOffset = ViewRadiusZ + ViewCenterDeltaZ;
			Level.ViewToClip = FReversedZOrthoMatrix(HalfLevelDim, HalfLevelDim, ZScale, ZOffset);

			// With receiver mask enabled, dynamic is uncached and thus can be culled tightly
			if (Config.bUseReceiverMask && Config.bCullDynamicTightly)
			{
				// We subtract the LodBias here because we know that we'll only ever render objects to a fraction of the
				// clipmap range based on the biasing.
				Level.DynamicDepthCullRange.X = Level.ViewToClip.TransformPosition(FVector(0.0, 0.0, CullingEndDistance)).Z;
			}
		}

		if (Config.bHasPrefilteredDistant)
		{
			ensureMsgf(Config.bForceInvalidate && Config.bCullDynamicTightly, TEXT("The prefiltered distant path overloads the DynamicDepthCullRange parameters so it will not work correctly unless bForceInvalidate and bCullDynamicTightly are set."));
			Level.DynamicDepthCullRange.X = Level.ViewToClip.TransformPosition(FVector(0.0, 0.0, -PrevCullDistance)).Z;
			Level.DynamicDepthCullRange.Y = Level.ViewToClip.TransformPosition(FVector(0.0, 0.0, -CullingEndDistance)).Z;
			PrevCullDistance = CullingEndDistance;
		}

		// Update the entry with the new projection data
		ClipmapLevelEntry->ProjectionData = ComputeProjectionShaderData(ClipmapIndex);
	}

	ComputeBoundingVolumes(WorldOrigin);
}

const FVirtualShadowMapProjectionShaderData& FVirtualShadowMapClipmap::GetProjectionShaderData(int32 ClipmapIndex) const
{
	return PerLightCacheEntry->ShadowMapEntries[ClipmapIndex].ProjectionData;
}

void FVirtualShadowMapClipmap::ComputeBoundingVolumes(const FVector ViewOrigin)
{
	// We don't really do much CPU culling with clipmaps. After various testing the fact that we are culling
	// a single frustum that goes out and basically the entire map, and we have to extrude towards (and away!) from
	// the light, and dilate to cover full pages at every clipmap level (to avoid culling something that will go
	// into a page that then gets cached with incomplete geometry), in many situations there is effectively no
	// culling that happens. For instance, as soon as the camera looks vaguely towards or away from the light direction,
	// the extruded frustum effectively covers the whole world.
	// Thus we don't spend a lot of time trying to optimize for the easy cases and instead just pick an extremely
	// conservative frustum.
	ViewFrustumBounds = FConvexVolume();
	BoundingSphere = FSphere(ViewOrigin, GetMaxRadius());
}

float FVirtualShadowMapClipmap::GetMaxRadius() const
{
	return GetLevelRadius(static_cast<float>(GetClipmapLevel(GetLevelCount() - 1)));
}

FViewMatrices FVirtualShadowMapClipmap::GetViewMatrices(int32 ClipmapIndex) const
{
	check(ClipmapIndex >= 0 && ClipmapIndex < LevelData.Num());
	const FLevelData& Level = LevelData[ClipmapIndex];

	FViewMatrices::FMinimalInitializer Initializer;

	// NOTE: Be careful here! There's special logic in FViewMatrices around ViewOrigin for ortho projections we need to bypass...
	// There's also the fact that some of this data is going to be "wrong", due to the "overridden" matrix thing that shadows do
	Initializer.ViewOrigin = Level.WorldCenter;
	Initializer.ViewRotationMatrix = WorldToLightViewRotationMatrix;
	Initializer.ProjectionMatrix = Level.ViewToClip;

	// TODO: This is probably unused in the shadows/nanite path, but coupling here is not ideal
	Initializer.ConstrainedViewRect = FIntRect(0, 0, FVirtualShadowMap::VirtualMaxResolutionXY, FVirtualShadowMap::VirtualMaxResolutionXY);

	return FViewMatrices(Initializer);
}

uint32 PackClipmapLevelAndCount(int32 FirstLevel, int32 ClipmapIndex, int32 ClipmapLevelCountRemaining)
{
	int32 ClipmapLevel = FirstLevel + ClipmapIndex;
	check(ClipmapLevel + VSM_PACKED_CLIP_LEVEL_BIAS >= 0);
	check(ClipmapLevelCountRemaining >= 0 && ClipmapLevelCountRemaining <= 0xFF);
	check(ClipmapIndex >= 0 && ClipmapIndex <= 0xFF);

	return (uint32(ClipmapLevel + VSM_PACKED_CLIP_LEVEL_BIAS) << 16u) 
		| (uint32(ClipmapIndex) << 8u)
		| uint32(ClipmapLevelCountRemaining);
}

FVirtualShadowMapProjectionShaderData FVirtualShadowMapClipmap::ComputeProjectionShaderData(int32 ClipmapIndex) const
{
	check(ClipmapIndex >= 0 && ClipmapIndex < LevelData.Num());
	const FLevelData& Level = LevelData[ClipmapIndex];
	
	const FVector PreViewTranslation = GetPreViewTranslation(ClipmapIndex);
	const FDFVector3 PreViewTranslationDF(PreViewTranslation);

	// WorldOrigin should be near the Level.WorldCenter, so we can make it relative
	// NOTE: We need to negate so that it's not opposite though
	const FVector3f ClipmapWorldOriginOffset(WorldOrigin + PreViewTranslation);

	// NOTE: Some shader logic (projection, etc) assumes some of these parameters are constant across all levels in a clipmap
	FVirtualShadowMapProjectionShaderData Data;
	Data.LightDirection = FVector3f(-LightDirection);		// Negative to be consistent with FLightShaderParameters/GetDeferredLightParameters
	Data.ShadowViewToClipMatrix = FMatrix44f(Level.ViewToClip);
	FMatrix TranslatedWorldToShadowUVMatrix = CalcTranslatedWorldToShadowUVMatrix(WorldToLightViewRotationMatrix, Level.ViewToClip);
	Data.TranslatedWorldToShadowUVMatrix = FMatrix44f(TranslatedWorldToShadowUVMatrix);
	Data.ShadowUVToTranslatedWorldMatrix = FMatrix44f(TranslatedWorldToShadowUVMatrix.Inverse());
	Data.TranslatedWorldToShadowUVNormalMatrix = FMatrix44f(CalcTranslatedWorldToShadowUVNormalMatrix(WorldToLightViewRotationMatrix, Level.ViewToClip));
	Data.PreViewTranslationHigh = PreViewTranslationDF.High;
	Data.PreViewTranslationLow = PreViewTranslationDF.Low;
	Data.LightType = ELightComponentType::LightType_Directional;
	Data.ClipmapWorldOriginOffset = ClipmapWorldOriginOffset;
	int32 ClipmapLevel = FirstLevel + ClipmapIndex;
	int32 ClipmapLevelCountRemaining = LevelData.Num() - ClipmapIndex;
	Data.ClipmapLevel_ClipmapLevelCountRemaining = PackClipmapLevelAndCount(FirstLevel, ClipmapIndex, ClipmapLevelCountRemaining);
	Data.ResolutionLodBias = ResolutionLodBias;
	Data.ClipmapCornerRelativeOffset = Level.RelativeCornerOffset;
	Data.ClipmapLevelWPODistanceDisableThresholdSquared = static_cast<float>(Level.WPODistanceDisableThresholdSquared);
	Data.LightSourceRadius = GetLightSceneInfo().Proxy->GetSourceRadius();
	Data.Flags = PerLightCacheEntry->IsUncached() ? VSM_PROJ_FLAG_UNCACHED : 0U;
	Data.PackedCullingViewId = FVirtualShadowMapProjectionShaderData::PackCullingViewId(DependentView->SceneRendererPrimaryViewId, DependentView->PersistentViewId);
	Data.LightId = GetLightSceneInfo().Id.GetIndex();

	if (Config.bHasPrefilteredDistant)
	{
		Data.DynamicDepthCullRange = Level.DynamicDepthCullRange;
		Data.Flags |= VSM_PROJ_FLAG_HAS_PREFILTERED_DISTANT;
	}

	if (Config.FirstCoarseLevel >= 0)
	{
		if ((ClipmapLevel >= Config.FirstCoarseLevel && ClipmapLevel <= Config.LastCoarseLevel)
			// Always mark coarse pages in the last level for clouds/skyatmosphere
			|| 	ClipmapLevelCountRemaining == 1)
		{
			Data.Flags |= VSM_PROJ_FLAG_IS_COARSE_CLIP_LEVEL;
		}
	}
	if (Config.bIsFirstPersonShadow)
	{
		Data.Flags |= VSM_PROJ_FLAG_IS_FIRST_PERSON_SHADOW;
	}
	if (Config.bUseReceiverMask)
	{
		Data.Flags |= VSM_PROJ_FLAG_USE_RECEIVER_MASK;
	}

	// If the coarse pages mode is 2, we suppress invalidaitons of pages not marked as "detail geometry". 
	// Coarse pages are not marked with this flag so will not be invalidated.
	if (CVarMarkCoarsePagesDirectional.GetValueOnRenderThread() == 2)
	{
		Data.Flags |= VSM_PROJ_FLAG_FORCE_CACHE_DYNAMIC_COARSE;
	}

	Data.TexelDitherScale = GetLightSceneInfo().Proxy->GetVSMTexelDitherScale();
	return Data;
}

static inline void LazyInitAndSetBitArray(TBitArray<>& BitArray, int32 Index, bool Value, int32 MaxNum)
{
	if (BitArray.IsEmpty())
	{
		BitArray.Init(false, MaxNum);
	}
	BitArray[Index] = Value;

}

void FVirtualShadowMapClipmap::OnPrimitiveRendered(const FPrimitiveSceneInfo* PrimitiveSceneInfo)
{
	if (PerLightCacheEntry.IsValid())
	{
		FPersistentPrimitiveIndex PersistentPrimitiveId = PrimitiveSceneInfo->GetPersistentIndex();
		const int32 RenderedPrimitivesMaxNum = PerLightCacheEntry->RenderedPrimitives.Num();
		check(PersistentPrimitiveId.IsValid());
		check(PersistentPrimitiveId.Index < RenderedPrimitivesMaxNum);

		// Check previous-frame state to detect transition from hidden->visible
		const bool bPrimitiveRevealed = !PerLightCacheEntry->RenderedPrimitives[PersistentPrimitiveId.Index];
		
		// update current frame-state.
		LazyInitAndSetBitArray(RenderedPrimitives, PersistentPrimitiveId.Index, true, RenderedPrimitivesMaxNum);

		// update cached state (this is checked & cleared whenever a primitive is invalidating the VSM).
		PerLightCacheEntry->OnPrimitiveRendered(PrimitiveSceneInfo, bPrimitiveRevealed);
	}
}

void FVirtualShadowMapClipmap::UpdateCachedFrameData()
{
	if (PerLightCacheEntry.IsValid() && !RenderedPrimitives.IsEmpty())
	{
		PerLightCacheEntry->RenderedPrimitives = MoveTemp(RenderedPrimitives);
	}
}

