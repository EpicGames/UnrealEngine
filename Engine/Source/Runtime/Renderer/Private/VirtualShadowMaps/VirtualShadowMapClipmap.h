// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapClipmap.h
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ConvexVolume.h"
#include "Templates/RefCounting.h"
#include "VirtualShadowMapProjection.h"
#include "VirtualShadowMapCacheManager.h"

struct FViewMatrices;
struct FVirtualShadowMapProjectionShaderData;
class FPrimitiveSceneInfo;
class FVirtualShadowMap;
class FVirtualShadowMapArray;

bool IsVirtualShadowMapDirectionalReceiverMaskEnabled();

class FVirtualShadowMapClipmapConfig
{
public:
	static FVirtualShadowMapClipmapConfig GetGlobal();

	int32 FirstLevel = 8;
	int32 LastLevel = 18;
	int32 FirstCoarseLevel = -1;
	int32 LastCoarseLevel = -1;
	uint32 ShadowTypeId = 0u;
	float ResolutionLodBias = 0.0f;
	float ResolutionLodBiasMoving = 0.0f;
	bool bForceInvalidate = false;
	bool bIsFirstPersonShadow = false;
	bool bCullDynamicTightly = false;
	bool bUseReceiverMask = false;
	bool bHasPrefilteredDistant = false;
};

class FVirtualShadowMapClipmap : FRefCountedObject
{
public:	
	FVirtualShadowMapClipmap(
		FVirtualShadowMapArray& VirtualShadowMapArray,
		const FLightSceneInfo& InLightSceneInfo,
		const FViewMatrices& CameraViewMatrices,
		FIntPoint CameraViewRectSize,
		const FViewInfo* InDependentView,
		float LightMobilityFactor,
		const FVirtualShadowMapClipmapConfig& Config = FVirtualShadowMapClipmapConfig::GetGlobal()
	);

	FViewMatrices GetViewMatrices(int32 ClipmapIndex) const;

	FVector2f GetDynamicDepthCullRange(int32 ClipmapIndex) const
	{
		check(ClipmapIndex >= 0 && ClipmapIndex < LevelData.Num());
		return LevelData[ClipmapIndex].DynamicDepthCullRange;
	}

	const FVirtualShadowMapClipmapConfig& GetConfig() const { return Config; }

	int32 GetVirtualShadowMapId() const
	{
		return PerLightCacheEntry->GetVirtualShadowMapId();
	}

	int32 GetLevelCount() const
	{
		return LevelData.Num();
	}

	// Get absolute clipmap level from index (0..GetLevelCount())
	int32 GetClipmapLevel(int32 ClipmapIndex) const
	{
		return FirstLevel + ClipmapIndex;
	}

	FVector GetPreViewTranslation(int32 ClipmapIndex) const
	{
		return -LevelData[ClipmapIndex].WorldCenter;
	}

	FMatrix GetViewToClipMatrix(int32 ClipmapIndex) const
	{
		return LevelData[ClipmapIndex].ViewToClip;
	}

	FMatrix GetWorldToLightViewRotationMatrix() const
	{
		return WorldToLightViewRotationMatrix;
	}

	const FLightSceneInfo& GetLightSceneInfo() const
	{
		return LightSceneInfo;
	}

	const FVirtualShadowMapProjectionShaderData& GetProjectionShaderData(int32 ClipmapIndex) const;

	FVector GetWorldOrigin() const
	{
		return WorldOrigin;
	}

	static float GetLevelRadius(float AbsoluteLevel);

	// Returns the max radius the clipmap is guaranteed to cover (i.e. the radius of the last clipmap level)
	// Note that this is not a conservative radius of the level projection, which is snapped
	float GetMaxRadius() const;
	FSphere GetBoundingSphere() const { return BoundingSphere; }
	FConvexVolume GetViewFrustumBounds() const { return ViewFrustumBounds; }

	const FViewInfo* GetDependentView() const { return DependentView; }

	/**
	 * Called when a primitive passes CPU-culling, note that this applies to non-nanite primitives only. Not thread safe in general.
	 */ 
	void OnPrimitiveRendered(const FPrimitiveSceneInfo* PrimitiveSceneInfo);

	/**
	 * Called to push any cache data to cache entry at the end of the frame.
	 */
	void UpdateCachedFrameData();

	TSharedPtr<FVirtualShadowMapPerLightCacheEntry>& GetCacheEntry() { return PerLightCacheEntry; }

	/**
	 * Whether the clipmap is for casting shadow of FirstPersonWorldSpaceRepresentation primitives onto the scene.
	 */
	bool IsFirstPersonShadow() const { return Config.bIsFirstPersonShadow; }

private:
	FVirtualShadowMapProjectionShaderData ComputeProjectionShaderData(int32 ClipmapIndex) const;
	void ComputeBoundingVolumes(const FVector CameraOrigin);

	FVirtualShadowMapClipmapConfig Config;

	const FLightSceneInfo& LightSceneInfo;

	/**
	 * DependentView is the 'main' or visible geometry view that this view-dependent clipmap was created for. Should only be used to 
	 * identify the view during shadow projection (note: this should be refactored to be more explicit instead).
	 */
	const FViewInfo* DependentView;

	/** Origin of the clipmap in world space
	* Usually aligns with the camera position from which it was created.
	* Note that the centers of each of the levels can be different as they are snapped to page alignment at their respective scales
	* */
	FVector WorldOrigin;
	FVector CameraToViewTarget;

	FVector LightDirection;

	/** Directional light rotation matrix (no translation) */
	FMatrix WorldToLightViewRotationMatrix;

	int32 FirstLevel;
	float ResolutionLodBias;

	struct FLevelData
	{
		FMatrix ViewToClip;
		FVector WorldCenter;
		//Offset from (0,0) to clipmap corner, in level radii
		FInt64Point CornerOffset;
		//Offset from LastLevel-snapped WorldCenter to clipmap corner, in level radii
		FIntPoint RelativeCornerOffset;
		double WPODistanceDisableThresholdSquared;
		FVector2f DynamicDepthCullRange;
	};
	TArray< FLevelData, TInlineAllocator<32> > LevelData;

	FSphere BoundingSphere;
	FConvexVolume ViewFrustumBounds;

	TSharedPtr<FVirtualShadowMapPerLightCacheEntry> PerLightCacheEntry;

	// Rendered primitives are marked during culling (through OnPrimitiveRendered being called).
	TBitArray<> RenderedPrimitives;
};
