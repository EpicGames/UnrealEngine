// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapProjection.cpp
=============================================================================*/

#include "VirtualShadowMapProjection.h"
#include "CoreMinimal.h"
#include "BasePassRendering.h"
#include "BlueNoise.h"
#include "CanvasTypes.h"
#include "CommonRenderResources.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"
#include "HairStrands/HairStrandsData.h"
#include "HAL/IConsoleManager.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "PipelineStateCache.h"
#include "PixelShaderUtils.h"
#include "PostProcess/DiaphragmDOF.h"
#include "PSOPrecache.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphParameter.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneManagement.h"
#include "SceneRendering.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneTypes.h"
#include "SceneView.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShadowRendering.h"
#include "Stats/Stats.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "UniformBuffer.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VirtualShadowMaps/VirtualShadowMapClipmap.h"
#include "VirtualShadowMapVisualizationData.h"


#define MAX_TEST_PERMUTATION 0

static TAutoConsoleVariable<int32> CVarForcePerLightShadowMaskClear(
	TEXT( "r.Shadow.Virtual.ForcePerLightShadowMaskClear" ),
	0,
	TEXT( "For debugging purposes. When enabled, the shadow mask texture is cleared before the projection pass writes to it. Projection pass writes all relevant pixels, so clearing should be unnecessary." ),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVSMTranslucentQuality(
	TEXT("r.Shadow.Virtual.TranslucentQuality"),
	0,
	TEXT("Quality of shadow for lit translucent surfaces. This will be applied on all translucent surfaces, and has high-performance impact.\n")
	TEXT("Set to 1 to enable the high-quality mode."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSubsurfaceShadowMinSourceAngle(
	TEXT("r.Shadow.Virtual.SubsurfaceShadowMinSourceAngle"),
	5,
	TEXT("Minimum source angle (in degrees) used for shadow & transmittance of sub-surface materials with directional lights.\n")
	TEXT("To emulate light diffusion with sub-surface materials, VSM can increase the light source radius depending on the material opacity.\n")
	TEXT("The higher this value, the more diffuse the shadowing with these materials will appear."),
	ECVF_RenderThreadSafe
);

extern TAutoConsoleVariable<float> CVarMaxDOFResolutionBias;
extern bool GVirtualShadowMapPrefilteredDistantProjectEnable;

#if MAX_TEST_PERMUTATION > 0
static TAutoConsoleVariable<int32> CVarTestPermutation(
	TEXT( "r.Shadow.Virtual.ProjectionTestPermutation" ),
	0,
	TEXT( "Used for A/B testing projection shader changes. " ),
	ECVF_RenderThreadSafe
);
#endif

int32 GVisualizeCachedPagesOnly = 0;
FAutoConsoleVariableRef CVarVisualizeCachedPagesOnly(
	TEXT("r.Shadow.Virtual.Visualize.ShowCachedPagesOnly"),
	GVisualizeCachedPagesOnly,
	TEXT("When true, shows the cached pages for all lights and hides uncached pages."),
	ECVF_RenderThreadSafe
);

// The tile size in pixels for VSM projection with tile list.
// Is also used as the workgroup size for the CS without tile list.
static constexpr int32 VSMProjectionWorkTileSize = 8;

bool IsVSMTranslucentHighQualityEnabled()
{
	return CVarVSMTranslucentQuality.GetValueOnRenderThread() > 0;
}

const TCHAR* ToString(EVirtualShadowMapProjectionInputType In)
{
	switch (In)
	{
	case EVirtualShadowMapProjectionInputType::HairStrands: return TEXT("HairStrands");
	case EVirtualShadowMapProjectionInputType::GBuffer:     return Substrate::IsSubstrateEnabled() ? TEXT("Substrate") : TEXT("GBuffer");
	}
	return TEXT("Invalid");
}


class FVirtualShadowMapProjectionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVirtualShadowMapProjectionCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualShadowMapProjectionCS, FGlobalShader)
	
	class FDirectionalLightDim			: SHADER_PERMUTATION_BOOL("DIRECTIONAL_LIGHT");
	class FOnePassProjectionDim			: SHADER_PERMUTATION_BOOL("ONE_PASS_PROJECTION");
	class FHairStrandsDim				: SHADER_PERMUTATION_BOOL("HAS_HAIR_STRANDS");
	class FVisualizeOutputDim			: SHADER_PERMUTATION_BOOL("VISUALIZE_OUTPUT");
	class FExtrapolateSlopeDim			: SHADER_PERMUTATION_BOOL("SMRT_EXTRAPOLATE_SLOPE");
	class FUseTileList					: SHADER_PERMUTATION_BOOL("USE_TILE_LIST");
	class FFirstPersonShadow			: SHADER_PERMUTATION_BOOL("FIRST_PERSON_SHADOW");
	class FPrefilteredDistantClipmap	: SHADER_PERMUTATION_BOOL("PREFILTERED_DISTANT_CLIPMAP");
	class FUbershader					: SHADER_PERMUTATION_BOOL("UBERSHADER");
	
	// -1 means dynamic count
	class FSMRTStaticSampleCount	: SHADER_PERMUTATION_SPARSE_INT("SMRT_TEMPLATE_STATIC_SAMPLES_PER_RAY", -1, 2, 4);
	// Used for A/B testing a change that affects reg allocation, etc.
	class FTestDim					: SHADER_PERMUTATION_INT("TEST_PERMUTATION", MAX_TEST_PERMUTATION+1);

	using FPermutationDomain = TShaderPermutationDomain<
		FDirectionalLightDim,
		FOnePassProjectionDim,
		FHairStrandsDim,
		FVisualizeOutputDim,
		FExtrapolateSlopeDim,
		FUseTileList,
		FFirstPersonShadow,
		FPrefilteredDistantClipmap,
		FSMRTStaticSampleCount,
		FUbershader
#if MAX_TEST_PERMUTATION > 0
		, FTestDim
#endif
	>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, SamplingParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualVoxelParameters, HairStrandsVoxel)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)
		SHADER_PARAMETER(FIntVector4, ProjectionRect)
		SHADER_PARAMETER(float, SubsurfaceMinSourceRadius)
		SHADER_PARAMETER(uint32, InputType)
		SHADER_PARAMETER(uint32, bCullBackfacingPixels)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPhysicalPageMetaData >, PhysicalPageMetaData)
		// One pass projection parameters
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutShadowMaskBits)
		// Pass per light parameters
		SHADER_PARAMETER_STRUCT(FLightShaderParameters, Light)
		SHADER_PARAMETER(int32, LightUniformVirtualShadowMapId)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutShadowFactor)
		// First Person shadow parameters
		SHADER_PARAMETER(int32, FirstPersonVirtualShadowMapId)		
		// Visualization output
		SHADER_PARAMETER(int32, VisualizeModeId)
		SHADER_PARAMETER(int32, bVisualizeCachedPagesOnly)
		SHADER_PARAMETER(int32, VisualizeVirtualShadowMapId)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutVisualize)
		// Optional tile list
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, TileListData)
		RDG_BUFFER_ACCESS(IndirectDispatchArgs, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutStatsBuffer)
		// PrefilteredDistant
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, PrefilteredDistantPhysicalPagePool)
		// DoF
		SHADER_PARAMETER(float, DOFBiasStrength)
		SHADER_PARAMETER_STRUCT_INCLUDE(DiaphragmDOF::FDOFCocModelShaderParameters, CocModel)
		SHADER_PARAMETER(int32, SMRTSamplePerRay)
		SHADER_PARAMETER(uint32, bOnePassProjection)
		SHADER_PARAMETER(uint32, bVisualizeOutput)
		SHADER_PARAMETER(uint32, bSMRTExtrapolateSlope)
		SHADER_PARAMETER(uint32, bUseTileList)
		SHADER_PARAMETER(uint32, bFirstPersonShadow)
		SHADER_PARAMETER(uint32, bPrefilteredDistantClipmap)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment( const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment )
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		// TODO: We may no longer need this with SM6 requirement, but shouldn't hurt
		OutEnvironment.CompilerFlags.Add(CFLAG_WaveOperations);

		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);
		if (FDataDrivenShaderPlatformInfo::GetSupportsRealTypes(Parameters.Platform) == ERHIFeatureSupport::RuntimeGuaranteed)
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowRealTypes);
		}

		OutEnvironment.SetDefine(TEXT("WORK_TILE_SIZE"), VSMProjectionWorkTileSize);

		OutEnvironment.SetDefine(TEXT("VSM_WITH_DOF_BIAS"), 1);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (PermutationVector.Get<FUbershader>())
		{
			// Actual permutation 
			// - FDirectionalLightDim
			if (PermutationVector.Get<FOnePassProjectionDim>() &&
				PermutationVector.Get<FHairStrandsDim>() &&
				PermutationVector.Get<FVisualizeOutputDim>() &&
				PermutationVector.Get<FExtrapolateSlopeDim>() &&
				PermutationVector.Get<FUseTileList>() &&
				PermutationVector.Get<FFirstPersonShadow>() &&
				PermutationVector.Get<FPrefilteredDistantClipmap>() &&
				PermutationVector.Get<FSMRTStaticSampleCount>() == 4
				#if MAX_TEST_PERMUTATION > 0
				&& !PermutationVector.Get<FTestDim>(false);
				#endif
			)
			{
				return DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
			}
			else
			{
				return false;
			}
		}

		// Directional lights are always in separate passes as forward light data structure currently
		// only contains a single directional light.
		if( PermutationVector.Get< FDirectionalLightDim >() && PermutationVector.Get< FOnePassProjectionDim >() )
		{
			return false;
		}
		
		// Only need the first person permutation for directional lights.
		if( PermutationVector.Get< FFirstPersonShadow >() && !PermutationVector.Get< FDirectionalLightDim >() )
		{
			return false;
		}
		
		// FOnePassProjectionDim is not used together with the tile list
		if( PermutationVector.Get< FUseTileList >() && PermutationVector.Get< FOnePassProjectionDim >() )
		{
			return false;
		}
		
		// Only need the far Prefiltered Distant Clipmap permutation for directional lights.
		if( PermutationVector.Get< FPrefilteredDistantClipmap >() && (!GVirtualShadowMapPrefilteredDistantProjectEnable || !PermutationVector.Get< FDirectionalLightDim >()) )
		{
			return false;
		}

		return DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{		
		if (PermutationVector.Get< FUbershader>())
		{
			// Actual permutation 
			// - FDirectionalLightDim
			PermutationVector.Set<FOnePassProjectionDim>(true);
			PermutationVector.Set<FHairStrandsDim>(true);
			PermutationVector.Set<FVisualizeOutputDim>(true);
			PermutationVector.Set<FExtrapolateSlopeDim>(true);
			PermutationVector.Set<FUseTileList>(true);
			PermutationVector.Set<FFirstPersonShadow>(true);
			PermutationVector.Set<FPrefilteredDistantClipmap>(true);
			PermutationVector.Set<FSMRTStaticSampleCount>(4);
			#if MAX_TEST_PERMUTATION > 0
			PermutationVector.Set<FTestDim>(false);
			#endif
		}
		return PermutationVector;
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FVisualizeOutputDim>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}

		if (!PermutationVector.Get<FUbershader>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}

		return EShaderPermutationPrecacheRequest::Precached;
	}
};
IMPLEMENT_GLOBAL_SHADER(FVirtualShadowMapProjectionCS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapProjection.usf", "VirtualShadowMapProjection", SF_Compute);

static void RenderVirtualShadowMapProjectionCommon(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View, int32 ViewIndex,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	const FIntRect ProjectionRect,
	EVirtualShadowMapProjectionInputType InputType,
	FRDGTextureRef OutputTexture,
	const FLightSceneProxy* LightProxy,
	int32 VirtualShadowMapId,
	FTiledVSMProjection* TiledVSMProjection,
	int32 FirstPersonVirtualShadowMapId,
	bool bPrefilteredDistantClipmap)
{
	check(GRHISupportsWaveOperations);

	const bool bUseTileList = TiledVSMProjection != nullptr;
	check(!bUseTileList || TiledVSMProjection->TileSize == VSMProjectionWorkTileSize);

	// Use hair strands data (i.e., hair voxel tracing) only for Gbuffer input for casting hair shadow onto opaque geometry.
	const bool bHasHairStrandsData = HairStrands::HasViewHairStrandsData(View);

	FVirtualShadowMapProjectionCS::FParameters* PassParameters = GraphBuilder.AllocParameters< FVirtualShadowMapProjectionCS::FParameters >();
	PassParameters->SamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
	PassParameters->SceneTexturesStruct = SceneTextures.UniformBuffer;
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->ProjectionRect = FIntVector4(ProjectionRect.Min.X, ProjectionRect.Min.Y, ProjectionRect.Max.X, ProjectionRect.Max.Y);
	PassParameters->SubsurfaceMinSourceRadius = FMath::Sin(0.5f * FMath::DegreesToRadians(CVarSubsurfaceShadowMinSourceAngle.GetValueOnRenderThread()));
	PassParameters->InputType = uint32(InputType);
	PassParameters->bCullBackfacingPixels = VirtualShadowMapArray.ShouldCullBackfacingPixels() ? 1 : 0;
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	if (bUseTileList)
	{
		PassParameters->TileListData = TiledVSMProjection->TileListDataBufferSRV;
		PassParameters->IndirectDispatchArgs = TiledVSMProjection->DispatchIndirectParametersBuffer;
	}
	if (bHasHairStrandsData)
	{
		PassParameters->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
		PassParameters->HairStrandsVoxel = HairStrands::BindHairStrandsVoxelUniformParameters(View);
	}

	FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
	PassParameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);
	PassParameters->FirstPersonVirtualShadowMapId = FirstPersonVirtualShadowMapId;
	bool bHasFirstPersonShadow = FirstPersonVirtualShadowMapId != INDEX_NONE;

	bool bDirectionalLight = false;
	bool bOnePassProjection = LightProxy == nullptr;
	PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
	if (bOnePassProjection)
	{
		// These are mutually exclusive in practice, enforce to cut down on permutations
		check(!bUseTileList);

		// One pass projection
		PassParameters->OutShadowMaskBits = GraphBuilder.CreateUAV( OutputTexture );
	}
	else
	{
		// Pass per light
		bDirectionalLight = LightProxy->GetLightType() == LightType_Directional;
		FLightRenderParameters LightParameters;
		LightProxy->GetLightShaderParameters(LightParameters);
		LightParameters.MakeShaderParameters(View.ViewMatrices, View.GetLastEyeAdaptationExposure(), PassParameters->Light);
		PassParameters->LightUniformVirtualShadowMapId = VirtualShadowMapId;
		PassParameters->OutShadowFactor = GraphBuilder.CreateUAV( OutputTexture );
	}

	DiaphragmDOF::FPhysicalCocModel CocModel;
	CocModel.Compile(View);
	DiaphragmDOF::SetCocModelParameters(GraphBuilder, &PassParameters->CocModel, CocModel, View.ViewRect.Size().X);
	PassParameters->DOFBiasStrength = FMath::Max(0.0f, DiaphragmDOF::IsEnabled(View) ? CVarMaxDOFResolutionBias.GetValueOnRenderThread() : 0.0f);
 	
	PassParameters->PhysicalPageMetaData = GraphBuilder.CreateSRV( VirtualShadowMapArray.PhysicalPageMetaDataRDG );

	bool bDebugOutput = false;
#if VSM_ENABLE_VISUALIZATION
	if ( !VirtualShadowMapArray.DebugVisualizationOutput.IsEmpty() && InputType == EVirtualShadowMapProjectionInputType::GBuffer)
	{
		const FVirtualShadowMapVisualizationData& VisualizationData = GetVirtualShadowMapVisualizationData();

		int VisualizeVirtualShadowMapId = VirtualShadowMapArray.VisualizeLight[ViewIndex].GetVirtualShadowMapId();

		bDebugOutput = true;
		PassParameters->VisualizeModeId = VisualizationData.GetActiveModeID();
		PassParameters->bVisualizeCachedPagesOnly = GVisualizeCachedPagesOnly;
		PassParameters->VisualizeVirtualShadowMapId = VisualizeVirtualShadowMapId;
		PassParameters->OutVisualize = GraphBuilder.CreateUAV( VirtualShadowMapArray.DebugVisualizationOutput[ViewIndex] );
	}
#endif

	PassParameters->OutStatsBuffer = VirtualShadowMapArray.StatsBufferUAV;
	PassParameters->PrefilteredDistantPhysicalPagePool = VirtualShadowMapArray.PrefilteredDistantPhysicalPagePoolMergedRDG;

	// If the requested samples per ray matches one of our static permutations, pick that one
	// Otherwise use the dynamic samples per ray permutation (-1).
	const int SamplesPerRay = (bDirectionalLight ? VirtualShadowMapArray.UniformParameters.SMRTSamplesPerRayDirectional : VirtualShadowMapArray.UniformParameters.SMRTSamplesPerRayLocal);
	const int StaticSamplesPerRay = UE::ShaderPermutationUtils::DoesDimensionContainValue<FVirtualShadowMapProjectionCS::FSMRTStaticSampleCount>(SamplesPerRay) ? SamplesPerRay : -1;
	const float ExtrapolateMaxSlope = (bDirectionalLight ? VirtualShadowMapArray.UniformParameters.SMRTExtrapolateMaxSlopeDirectional : VirtualShadowMapArray.UniformParameters.SMRTExtrapolateMaxSlopeLocal);
	const bool bSMRTExtrapolateSlope = ExtrapolateMaxSlope > 0.0f;

	FVirtualShadowMapProjectionCS::FPermutationDomain PermutationVector;
	PermutationVector.Set< FVirtualShadowMapProjectionCS::FDirectionalLightDim >( bDirectionalLight );
	PermutationVector.Set< FVirtualShadowMapProjectionCS::FOnePassProjectionDim >( bOnePassProjection );
	PermutationVector.Set< FVirtualShadowMapProjectionCS::FHairStrandsDim >( bHasHairStrandsData );
	PermutationVector.Set< FVirtualShadowMapProjectionCS::FVisualizeOutputDim >( bDebugOutput );
	PermutationVector.Set< FVirtualShadowMapProjectionCS::FExtrapolateSlopeDim >(bSMRTExtrapolateSlope);
	PermutationVector.Set< FVirtualShadowMapProjectionCS::FUseTileList >(bUseTileList);
	PermutationVector.Set< FVirtualShadowMapProjectionCS::FSMRTStaticSampleCount >(StaticSamplesPerRay);
	
	if (bHasFirstPersonShadow && !ensureMsgf(bDirectionalLight, TEXT("First person shadow can only be used with pass-per light direction shadow projection and will be disabled.")))
	{
		bHasFirstPersonShadow = false;
	}	
	if (bPrefilteredDistantClipmap && !ensureMsgf(bDirectionalLight, TEXT("Prefiltered Distant shadow can only be used with pass-per light directional shadow projection and will be disabled.")))
	{
		bPrefilteredDistantClipmap = false;
	}
	if (bPrefilteredDistantClipmap && !ensureMsgf(VirtualShadowMapArray.PrefilteredDistantPhysicalPagePoolMergedRDG != nullptr, TEXT("Prefiltered Distant page pool not allocated.")))
	{
		bPrefilteredDistantClipmap = false;
	}
	PermutationVector.Set< FVirtualShadowMapProjectionCS::FFirstPersonShadow >(bHasFirstPersonShadow);
	PermutationVector.Set< FVirtualShadowMapProjectionCS::FPrefilteredDistantClipmap >(bPrefilteredDistantClipmap);
#if MAX_TEST_PERMUTATION > 0
	{
		int32 TestPermutation = FMath::Clamp(CVarTestPermutation.GetValueOnRenderThread(), 0, MAX_TEST_PERMUTATION);
		PermutationVector.Set< FVirtualShadowMapProjectionCS::FTestDim >( TestPermutation );
	}
#endif

	PassParameters->SMRTSamplePerRay = StaticSamplesPerRay;
	PassParameters->bOnePassProjection = bOnePassProjection;
	PassParameters->bVisualizeOutput = bDebugOutput;
	PassParameters->bSMRTExtrapolateSlope = bSMRTExtrapolateSlope;
	PassParameters->bUseTileList = bUseTileList;
	PassParameters->bFirstPersonShadow = bHasFirstPersonShadow;
	PassParameters->bPrefilteredDistantClipmap = bPrefilteredDistantClipmap;
	
	PermutationVector = FVirtualShadowMapProjectionCS::RemapPermutation(PermutationVector);
	auto ComputeShader = View.ShaderMap->GetShader< FVirtualShadowMapProjectionCS >( PermutationVector );

	// Ubershader default parameters and feature dynamic branching
	if (!PrecachePSO(FComputePipelineStateInitializer(ComputeShader.GetComputeShader())))
	{
		PermutationVector.Set< FVirtualShadowMapProjectionCS::FUbershader >(true);
		PermutationVector = FVirtualShadowMapProjectionCS::RemapPermutation(PermutationVector);
		ComputeShader = View.ShaderMap->GetShader< FVirtualShadowMapProjectionCS >( PermutationVector );

		// Ubershader default parameters and feature dynamic branching
		GSystemTextures.SetDefaultIfNull(GraphBuilder, PassParameters->PrefilteredDistantPhysicalPagePool);
		GSystemTextures.SetDefaultIfNull(GraphBuilder, PassParameters->OutShadowMaskBits);
		GSystemTextures.SetDefaultIfNull(GraphBuilder, PassParameters->OutShadowFactor);
		GSystemTextures.SetDefaultIfNull(GraphBuilder, PassParameters->TileListData);
		GSystemTextures.SetDefaultIfNull(GraphBuilder, PassParameters->IndirectDispatchArgs);
		GSystemTextures.SetDefaultIfNull(GraphBuilder, PassParameters->OutStatsBuffer);
		GSystemTextures.SetDefaultIfNull(GraphBuilder, PassParameters->OutVisualize);

		if (!bHasHairStrandsData)
		{
			PassParameters->HairStrands = HairStrands::CreateDefaultHairStrandsViewUniformBuffer(GraphBuilder);
			PassParameters->HairStrandsVoxel = HairStrands::CreateDefaultHairStrandsVoxelUniformBuffer(GraphBuilder);
		}
	}
	ClearUnusedGraphResources( ComputeShader, PassParameters );
	ValidateShaderParameters( ComputeShader, *PassParameters );

	if (bUseTileList)
	{
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("VirtualShadowMapProjection(Input:%s%s,TileList)",
				ToString(InputType),
				bDebugOutput ? TEXT(",Debug") : TEXT("")),
			PassParameters,
			ERDGPassFlags::Compute,
			[PassParameters, ComputeShader](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FComputeShaderUtils::DispatchIndirect(RHICmdList, ComputeShader, *PassParameters, PassParameters->IndirectDispatchArgs->GetIndirectRHICallBuffer(), 0);
			});
	}
	else
	{
		const FIntPoint GroupCount = FIntPoint::DivideAndRoundUp(ProjectionRect.Size(), VSMProjectionWorkTileSize);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("VirtualShadowMapProjection(Input:%s%s)",
				ToString(InputType),
				bDebugOutput ? TEXT(",Debug") : TEXT("")),
			ComputeShader,
			PassParameters,
			FIntVector(GroupCount.X, GroupCount.Y, 1)
		);
	}
}

FRDGTextureRef CreateVirtualShadowMapMaskBits(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	const TCHAR* Name)
{
	const FRDGTextureDesc ShadowMaskDesc = FRDGTextureDesc::Create2D(
		SceneTextures.Config.Extent,
		VirtualShadowMapArray.GetPackedShadowMaskFormat(),
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV);

	return GraphBuilder.CreateTexture(ShadowMaskDesc, Name);
}

void RenderVirtualShadowMapProjectionOnePass(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View, int32 ViewIndex,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	EVirtualShadowMapProjectionInputType InputType,
	FRDGTextureRef ShadowMaskBits)
{
	FIntRect ProjectionRect = View.ViewRect;

	RenderVirtualShadowMapProjectionCommon(
		GraphBuilder,
		SceneTextures,
		View, ViewIndex,
		VirtualShadowMapArray,
		ProjectionRect,
		InputType,
		ShadowMaskBits,
		nullptr,
		INDEX_NONE,
		nullptr,
		INDEX_NONE,
		false);
}

static FRDGTextureRef CreateShadowMaskTexture(FRDGBuilder& GraphBuilder, FIntPoint Extent)
{
	const FLinearColor ClearColor(0.0f, 0.0f, 0.0f, 0.0f);

	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
		Extent,
		PF_G16R16,
		FClearValueBinding(ClearColor),
		TexCreate_ShaderResource | TexCreate_UAV);

	FRDGTextureRef Texture = GraphBuilder.CreateTexture(Desc, TEXT("Shadow.Virtual.ShadowMask"));

	// NOTE: Projection pass writes all relevant pixels, so should not need to clear here
	if (CVarForcePerLightShadowMaskClear.GetValueOnRenderThread() != 0)
	{
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Texture), ClearColor);
	}

	return Texture;
}

void RenderVirtualShadowMapProjection(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View, int32 ViewIndex,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	const FIntRect ScissorRect,
	EVirtualShadowMapProjectionInputType InputType,
	const FLightSceneInfo& LightSceneInfo,
	int32 VirtualShadowMapId,
	FRDGTextureRef OutputShadowMaskTexture)
{
	FRDGTextureRef VirtualShadowMaskTexture = CreateShadowMaskTexture(GraphBuilder, View.ViewRect.Max);

	RenderVirtualShadowMapProjectionCommon(
		GraphBuilder,
		SceneTextures,
		View, ViewIndex,
		VirtualShadowMapArray,
		ScissorRect,
		InputType,
		VirtualShadowMaskTexture,
		LightSceneInfo.Proxy,
		VirtualShadowMapId,
		nullptr,
		INDEX_NONE,
		false);

	CompositeVirtualShadowMapMask(
		GraphBuilder,
		View,
		ScissorRect,
		VirtualShadowMaskTexture,
		false,	// bDirectionalLight
		false, // bModulateRGB
		nullptr, //TiledVSMProjection
		OutputShadowMaskTexture);
}

void RenderVirtualShadowMapProjection(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View, int32 ViewIndex,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	const FIntRect ScissorRect,
	EVirtualShadowMapProjectionInputType InputType,
	const TSharedPtr<FVirtualShadowMapClipmap>& Clipmap,
	bool bModulateRGB,
	FTiledVSMProjection* TiledVSMProjection,
	FRDGTextureRef OutputShadowMaskTexture,
	const TSharedPtr<FVirtualShadowMapClipmap>& FirstPersonClipmap)
{
	FRDGTextureRef VirtualShadowMaskTexture = CreateShadowMaskTexture(GraphBuilder, View.ViewRect.Max);

	RenderVirtualShadowMapProjectionCommon(
		GraphBuilder,
		SceneTextures,
		View, ViewIndex,
		VirtualShadowMapArray,		
		ScissorRect,
		InputType,
		VirtualShadowMaskTexture,
		Clipmap->GetLightSceneInfo().Proxy,
		Clipmap->GetVirtualShadowMapId(),
		TiledVSMProjection,
		FirstPersonClipmap.IsValid() ? FirstPersonClipmap->GetVirtualShadowMapId() : INDEX_NONE,
		Clipmap->GetConfig().bHasPrefilteredDistant
	);
	
	CompositeVirtualShadowMapMask(
		GraphBuilder,
		View,
		ScissorRect,
		VirtualShadowMaskTexture,
		true,	// bDirectionalLight
		bModulateRGB,
		TiledVSMProjection,
		OutputShadowMaskTexture);
}

class FVirtualShadowMapProjectionCompositeTileVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVirtualShadowMapProjectionCompositeTileVS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualShadowMapProjectionCompositeTileVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileListData)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Required right now due to where the shader function lives, but not actually used
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		OutEnvironment.SetDefine(TEXT("WORK_TILE_SIZE"), VSMProjectionWorkTileSize);
	}
};
IMPLEMENT_GLOBAL_SHADER(FVirtualShadowMapProjectionCompositeTileVS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapProjectionComposite.usf", "VirtualShadowMapCompositeTileVS", SF_Vertex);

// Composite denoised shadow projection mask onto the light's shadow mask
// Basically just a copy shader with a special blend mode
class FVirtualShadowMapProjectionCompositePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVirtualShadowMapProjectionCompositePS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualShadowMapProjectionCompositePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, InputShadowFactor)
		SHADER_PARAMETER(uint32, bModulateRGB)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Required right now due to where the shader function lives, but not actually used
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		OutEnvironment.SetDefine(TEXT("WORK_TILE_SIZE"), VSMProjectionWorkTileSize);
	}
};
IMPLEMENT_GLOBAL_SHADER(FVirtualShadowMapProjectionCompositePS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapProjectionComposite.usf", "VirtualShadowMapCompositePS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FVirtualShadowMapProjectionCompositeTile, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapProjectionCompositePS::FParameters, PS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapProjectionCompositeTileVS::FParameters, VS)
	RDG_BUFFER_ACCESS(IndirectDrawParameter, ERHIAccess::IndirectArgs)
END_SHADER_PARAMETER_STRUCT()

void CompositeVirtualShadowMapMask(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FIntRect ScissorRect,
	const FRDGTextureRef Input,
	bool bDirectionalLight,
	bool bModulateRGB,
	FTiledVSMProjection* TiledVSMProjection,
	FRDGTextureRef OutputShadowMaskTexture)
{
	const bool bUseTileList = TiledVSMProjection != nullptr;
	check(!bUseTileList || TiledVSMProjection->TileSize == VSMProjectionWorkTileSize);

	auto PixelShader = View.ShaderMap->GetShader<FVirtualShadowMapProjectionCompositePS>();

	FRHIBlendState* BlendState;
	if (bModulateRGB)
	{
		// This has the shadow contribution modulate all the channels, e.g. used for water rendering to apply VSM on the main light RGB luminance for the updated depth buffer with water in it.
		BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_SourceColor, BO_Add, BF_Zero, BF_One>::GetRHI();
	}
	else
	{
		BlendState = FProjectedShadowInfo::GetBlendStateForProjection(
			0,					// ShadowMapChannel
			bDirectionalLight,	// bIsWholeSceneDirectionalShadow,
			false,				// bUseFadePlane
			false,				// bProjectingForForwardShading, 
			false				// bMobileModulatedProjections
		);
	}

	if (bUseTileList)
	{
		FVirtualShadowMapProjectionCompositeTile* PassParameters = GraphBuilder.AllocParameters<FVirtualShadowMapProjectionCompositeTile>();
		PassParameters->VS.ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->VS.TileListData = TiledVSMProjection->TileListDataBufferSRV;
		PassParameters->PS.InputShadowFactor = Input;
		PassParameters->PS.bModulateRGB = bModulateRGB;
		PassParameters->PS.RenderTargets[0] = FRenderTargetBinding(OutputShadowMaskTexture, ERenderTargetLoadAction::ELoad);
		PassParameters->IndirectDrawParameter = TiledVSMProjection->DrawIndirectParametersBuffer;

		auto VertexShader = View.ShaderMap->GetShader<FVirtualShadowMapProjectionCompositeTileVS>();

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("CompositeVirtualShadowMapMask(TileList)"),
			PassParameters,
			ERDGPassFlags::Raster,
			[BlendState, VertexShader, PixelShader, ScissorRect, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(ScissorRect.Min.X, ScissorRect.Min.Y, 0.0f, ScissorRect.Max.X, ScissorRect.Max.Y, 1.0f);
				RHICmdList.SetScissorRect(true, ScissorRect.Min.X, ScissorRect.Min.Y, ScissorRect.Max.X, ScissorRect.Max.Y);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.BlendState = BlendState;
				GraphicsPSOInit.bDepthBounds = false;
				GraphicsPSOInit.PrimitiveType = GRHISupportsRectTopology ? PT_RectList : PT_TriangleList;
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);

				RHICmdList.DrawPrimitiveIndirect(PassParameters->IndirectDrawParameter->GetIndirectRHICallBuffer(), 0);

				RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
			});
	}
	else
	{
		FVirtualShadowMapProjectionCompositePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVirtualShadowMapProjectionCompositePS::FParameters>();
		PassParameters->InputShadowFactor = Input;
		PassParameters->bModulateRGB = bModulateRGB;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputShadowMaskTexture, ERenderTargetLoadAction::ELoad);

		ValidateShaderParameters(PixelShader, *PassParameters);

		FPixelShaderUtils::AddFullscreenPass(GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("CompositeVirtualShadowMapMask"),
			PixelShader,
			PassParameters,
			ScissorRect,
			BlendState);
	}
}

class FVirtualShadowMapProjectionCompositeFromMaskBitsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVirtualShadowMapProjectionCompositeFromMaskBitsPS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualShadowMapProjectionCompositeFromMaskBitsPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, SamplingParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint4>, ShadowMaskBits)
		SHADER_PARAMETER(FIntVector4, ProjectionRect)
		SHADER_PARAMETER(int32, CompositeVirtualShadowMapId)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Required right now due to where the shader function lives, but not actually used
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FVirtualShadowMapProjectionCompositeFromMaskBitsPS, "/Engine/Private/VirtualShadowMaps/VirtualShadowMapProjectionComposite.usf", "VirtualShadowMapCompositeFromMaskBitsPS", SF_Pixel);

void CompositeVirtualShadowMapFromMaskBits(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View, int32 ViewIndex,
	const FIntRect ScissorRect,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	EVirtualShadowMapProjectionInputType InputType,
	int32 VirtualShadowMapId,
	FRDGTextureRef ShadowMaskBits,
	FRDGTextureRef OutputShadowMaskTexture)
{
	FIntRect ProjectionRect = View.ViewRect;

	FVirtualShadowMapProjectionCompositeFromMaskBitsPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVirtualShadowMapProjectionCompositeFromMaskBitsPS::FParameters>();
	PassParameters->SamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
	PassParameters->SceneTexturesStruct = SceneTextures.UniformBuffer;
	PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->ProjectionRect = FIntVector4(ProjectionRect.Min.X, ProjectionRect.Min.Y, ProjectionRect.Max.X, ProjectionRect.Max.Y);
	PassParameters->InputDepthTexture = SceneTextures.UniformBuffer->GetParameters()->SceneDepthTexture;
	if (InputType == EVirtualShadowMapProjectionInputType::HairStrands)
	{
		PassParameters->InputDepthTexture = View.HairStrandsViewData.VisibilityData.HairOnlyDepthTexture;
	}
	PassParameters->ShadowMaskBits = ShadowMaskBits;
	PassParameters->CompositeVirtualShadowMapId = VirtualShadowMapId;

	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputShadowMaskTexture, ERenderTargetLoadAction::ELoad);

	FRHIBlendState* BlendState = FProjectedShadowInfo::GetBlendStateForProjection(
		0,					// ShadowMapChannel
		false,				// bIsWholeSceneDirectionalShadow,
		false,				// bUseFadePlane
		false,				// bProjectingForForwardShading, 
		false				// bMobileModulatedProjections
	);

	auto PixelShader = View.ShaderMap->GetShader<FVirtualShadowMapProjectionCompositeFromMaskBitsPS>();
	ValidateShaderParameters(PixelShader, *PassParameters);
	FPixelShaderUtils::AddFullscreenPass(GraphBuilder,
		View.ShaderMap,
		RDG_EVENT_NAME("CompositeVirtualShadowMapFromMaskBits"),
		PixelShader,
		PassParameters,
		ProjectionRect,
		BlendState);
}
