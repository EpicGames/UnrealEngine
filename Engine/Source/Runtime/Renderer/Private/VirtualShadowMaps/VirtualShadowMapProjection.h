// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapProjection.h
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "RenderGraphFwd.h"
#include "Templates/SharedPointerFwd.h"
#include "ShaderParameterMacros.h"

class FLightSceneInfo;
class FViewInfo;
class FVirtualShadowMapArray;
class FVirtualShadowMapClipmap;
struct FMinimalSceneTextures;

struct FTiledVSMProjection
{
	FRDGBufferRef DrawIndirectParametersBuffer;
	FRDGBufferRef DispatchIndirectParametersBuffer;
	FRDGBufferSRVRef TileListDataBufferSRV;
	uint32 TileSize;
};

// Note: Must match the definitions in VirtualShadowMapPageManagement.usf!
enum class EVirtualShadowMapProjectionInputType
{
	GBuffer = 0,
	HairStrands = 1
};
const TCHAR* ToString(EVirtualShadowMapProjectionInputType In);

void RenderVirtualShadowMapProjection(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View, int32 ViewIndex,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	const FIntRect ScissorRect,
	EVirtualShadowMapProjectionInputType InputType,
	const TSharedPtr<FVirtualShadowMapClipmap>& Clipmap,
	bool bModulateRGB,
	FTiledVSMProjection* TiledVSMProjection,
	FRDGTextureRef OutputShadowMaskTexture,
	const TSharedPtr<FVirtualShadowMapClipmap>& FirstPersonClipmap);

void RenderVirtualShadowMapProjection(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View, int32 ViewIndex,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	const FIntRect ScissorRect,
	EVirtualShadowMapProjectionInputType InputType,
	const FLightSceneInfo& LightSceneInfo,
	int32 VirtualShadowMapId,
	FRDGTextureRef OutputShadowMaskTexture);

FRDGTextureRef CreateVirtualShadowMapMaskBits(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	const TCHAR* Name);

void RenderVirtualShadowMapProjectionOnePass(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View, int32 ViewIndex,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	EVirtualShadowMapProjectionInputType InputType,
	FRDGTextureRef ShadowMaskBits);

void CompositeVirtualShadowMapMask(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FIntRect ScissorRect,
	const FRDGTextureRef Input,
	bool bDirectionalLight,
	bool bModulateRGB,
	FTiledVSMProjection* TiledVSMProjection,
	FRDGTextureRef OutputShadowMaskTexture);

void CompositeVirtualShadowMapFromMaskBits(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View, int32 ViewIndex,
	const FIntRect ScissorRect,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	EVirtualShadowMapProjectionInputType InputType,
	int32 VirtualShadowMapId,
	FRDGTextureRef ShadowMaskBits,
	FRDGTextureRef OutputShadowMaskTexture);

