// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VirtualShadowMapArray.h"
#include "VirtualShadowMapDefinitions.h"
#include "BasePassRendering.h"
#include "RenderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"

class FVirtualShadowMapGlobalShader : public FGlobalShader
{
public:
	FVirtualShadowMapGlobalShader()
	{
	}

	FVirtualShadowMapGlobalShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);

		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
	}
};

class FVirtualShadowMapPageManagementShader : public FVirtualShadowMapGlobalShader
{
public:
	// Kernel launch group sizes
	static constexpr uint32 DefaultCSGroupXY = 8;
	static constexpr uint32 DefaultCSGroupX = 256;

	FVirtualShadowMapPageManagementShader()
	{
	}

	FVirtualShadowMapPageManagementShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FVirtualShadowMapGlobalShader(Initializer)
	{
	}

	/**
	* Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
	*/
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FVirtualShadowMapGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("VSM_DEFAULT_CS_GROUP_X"), DefaultCSGroupX);
		OutEnvironment.SetDefine(TEXT("VSM_DEFAULT_CS_GROUP_XY"), DefaultCSGroupXY);

		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}
};

template <typename ShaderType>
static bool SetStatsArgsAndPermutation(bool bGenerateStats, FRDGBufferUAVRef StatsBufferUAV, typename ShaderType::FParameters *OutPassParameters, typename ShaderType::FPermutationDomain& OutPermutationVector)
{
	OutPassParameters->OutStatsBuffer = StatsBufferUAV;
	OutPermutationVector.template Set<typename ShaderType::FGenerateStatsDim>(bGenerateStats);
	return bGenerateStats;
}
