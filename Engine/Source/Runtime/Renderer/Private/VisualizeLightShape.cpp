// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualizeLightShape.h"
#include "BasePassRendering.h"
#include "SceneViewState.h"
#include "TranslucentPassResource.h"
#include "DeferredShadingRenderer.h"

static TAutoConsoleVariable<bool> CVarVisualizeLightShapeEnable(
	TEXT("r.VisualizeLightShape.Enable"),
	false,
	TEXT("Whether to enable light shape visualization."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarVisualizeLightShapeFadeDuration(
	TEXT("r.VisualizeLightShape.FadeDuration"),
	0.5f,
	TEXT("Fade duration in seconds when visualization is toggled."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarVisualizeLightShapeMaxDelayDistance(
	TEXT("r.VisualizeLightShape.MaxDelayDistance"),
	4000.0f,
	TEXT("Max distance where the fade-in animation can be delayed. ")
	TEXT("Delay time is scaled linearly within this range and closer lights have more delay."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarVisualizeLightShapeMaxDelay(
	TEXT("r.VisualizeLightShape.MaxDelay"),
	2.5f,
	TEXT("Max distance based delay in seconds. Delay is only applied when fading in visualization."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarVisualizeLightShapeMaxOpacity(
	TEXT("r.VisualizeLightShape.MaxOpacity"),
	0.1f,
	TEXT("Max opacity of each visualization mesh."),
	ECVF_RenderThreadSafe);

DECLARE_GPU_STAT(VisualizeLightShape);

namespace VisualizeLightShape
{
	bool GetUserEnable()
	{
		return CVarVisualizeLightShapeEnable.GetValueOnRenderThread();
	}

	float GetMaxDelay()
	{
		return FMath::Max(CVarVisualizeLightShapeMaxDelay.GetValueOnRenderThread(), 0.0f);
	}

	float GetFadeDuration()
	{
		return FMath::Max(CVarVisualizeLightShapeFadeDuration.GetValueOnRenderThread(), 0.0f);
	}

	float GetMaxDelayDistance()
	{
		return FMath::Max(CVarVisualizeLightShapeMaxDelayDistance.GetValueOnRenderThread(), 100.0f);
	}

	float GetMaxOpacity()
	{
		return FMath::Clamp(CVarVisualizeLightShapeMaxOpacity.GetValueOnRenderThread(), 0.0f, 1.0f);
	}

	// Returns whether this view should render
	bool UpdateLastToggleTime(FViewInfo& View)
	{
		if (!View.ViewState)
		{
			return false;
		}

		FViewState& ViewState = View.ViewState->VisualizeLightShape;
		const double CurrentGameTime = View.Family->Time.GetWorldTimeSeconds();

		if (CurrentGameTime < ViewState.LastToggleTime)
		{
			ViewState.LastToggleTime = 0.0;
		}

		const bool bWasEnabled = ViewState.LastToggleTime > 0.0;
		if (GetUserEnable() != bWasEnabled)
		{
			ViewState.LastToggleTime = bWasEnabled ? -CurrentGameTime : CurrentGameTime;
		}

		const double FadeDuration = GetFadeDuration();
		return ViewState.LastToggleTime != 0.0 && (ViewState.LastToggleTime > 0.0 || CurrentGameTime + ViewState.LastToggleTime < FadeDuration);
	}
}

struct FVisualizationMeshBuffers
{
	uint32 NumVertices = 0;
	uint32 NumTriangles = 0;
	FBufferRHIRef VertexBufferRHI;
	FBufferRHIRef IndexBufferRHI;
};

static FVisualizationMeshBuffers CreatePointLightVisualizationMesh(FRDGBuilder& GraphBuilder)
{
	const int32 NumSections = 16;
	const int32 NumSlices = NumSections * 2;
	const int32 NumVertices = (NumSlices + 1) * (NumSections - 1) + NumSlices * 2;
	const int32 NumTriangles = NumSlices * 2 * (NumSections - 2) + NumSlices * 2;
	const float Radius = 1.0f;

	FRHICommandListImmediate& RHICmdList = GraphBuilder.RHICmdList;
	FVisualizationMeshBuffers Out;

	Out.NumVertices = NumVertices;
	Out.NumTriangles = NumTriangles;

	{
		FRHIBufferCreateDesc CreateDesc = FRHIBufferCreateDesc::CreateVertex(TEXT("VisualizeLightShape.Point.VertexBuffer"), sizeof(FVector3f) * NumVertices)
			.AddUsage(EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

		Out.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}

	{
		FRHIBufferCreateDesc CreateDesc = FRHIBufferCreateDesc::CreateIndex(TEXT("VisualizeLightShape.Point.IndexBuffer"), sizeof(uint16) * NumTriangles * 3, sizeof(uint16))
			.AddUsage(EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

		Out.IndexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}

	FVector3f* VertexData = (FVector3f*)RHICmdList.LockBuffer(Out.VertexBufferRHI, 0, sizeof(FVector3f) * NumVertices, RLM_WriteOnly);
	uint16* IndexData = (uint16*)RHICmdList.LockBuffer(Out.IndexBufferRHI, 0, sizeof(uint16) * NumTriangles * 3, RLM_WriteOnly);

	int32 VertexIndex = 0;
	for (int32 X = 0; X <= NumSlices; ++X)
	{
		const float Phi = UE_TWO_PI / NumSlices * X;
		const float SinPhi = FMath::Sin(Phi);
		const float CosPhi = FMath::Cos(Phi);

		const int32 StartY = X == NumSlices ? 1 : 0;
		const int32 EndY = NumSections - (X == NumSlices ? 1 : 0);

		for (int32 Y = StartY; Y <= EndY; ++Y)
		{
			const float Theta = UE_PI / NumSections * Y;
			const float SinTheta = FMath::Sin(Theta);
			const float CosTheta = FMath::Cos(Theta);

			VertexData[VertexIndex] = FVector3f(SinTheta * CosPhi, SinTheta * SinPhi, CosTheta) * Radius;
			++VertexIndex;
		}
	}
	check(VertexIndex == NumVertices);

	int32 TriangleIndex = 0;
	for (int32 X = 0; X < NumSlices; ++X)
	{
		const int32 Offset = (NumSections + 1) * X;

		for (int32 Y = 0; Y < NumSections; ++Y)
		{
			// I3 - I2
			// |  \  |
			// I0 - I1
			const int32 I0 = Offset + Y + 1;
			const int32 I1 = Offset + Y + (NumSections + 1) + (X == NumSlices - 1 ? 0 : 1);
			const int32 I2 = Offset + Y + (NumSections + 1) + (X == NumSlices - 1 ? -1 : 0);
			const int32 I3 = Offset + Y;

			if (Y == 0)
			{
				IndexData[TriangleIndex * 3 + 0] = I0;
				IndexData[TriangleIndex * 3 + 1] = I1;
				IndexData[TriangleIndex * 3 + 2] = I3;
				++TriangleIndex;
			}
			else if (Y == NumSections - 1)
			{
				IndexData[TriangleIndex * 3 + 0] = I0;
				IndexData[TriangleIndex * 3 + 1] = I2;
				IndexData[TriangleIndex * 3 + 2] = I3;
				++TriangleIndex;
			}
			else
			{
				IndexData[TriangleIndex * 3 + 0] = I0;
				IndexData[TriangleIndex * 3 + 1] = I1;
				IndexData[TriangleIndex * 3 + 2] = I3;
				++TriangleIndex;

				IndexData[TriangleIndex * 3 + 0] = I1;
				IndexData[TriangleIndex * 3 + 1] = I2;
				IndexData[TriangleIndex * 3 + 2] = I3;
				++TriangleIndex;
			}
		}
	}
	check(TriangleIndex == NumTriangles);

	RHICmdList.UnlockBuffer(Out.VertexBufferRHI);
	RHICmdList.UnlockBuffer(Out.IndexBufferRHI);

	return Out;
}

static FVisualizationMeshBuffers CreateSpotLightVisualizationMesh(FRDGBuilder& GraphBuilder)
{
	const int32 NumSections = 11;
	const int32 NumSlices = 32;
	const int32 NumVertices = (NumSlices + 1) * (NumSections + 1);
	const int32 NumTriangles = NumSlices * 2 * NumSections;
	const float Radius = 1.0f;
	const float Length = 1.1f;

	FRHICommandListImmediate& RHICmdList = GraphBuilder.RHICmdList;
	FVisualizationMeshBuffers Out;

	Out.NumVertices = NumVertices;
	Out.NumTriangles = NumTriangles;

	{
		FRHIBufferCreateDesc CreateDesc = FRHIBufferCreateDesc::CreateVertex(TEXT("VisualizeLightShape.Spot.VertexBuffer"), sizeof(FVector3f) * NumVertices)
			.AddUsage(EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

		Out.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}

	{
		FRHIBufferCreateDesc CreateDesc = FRHIBufferCreateDesc::CreateIndex(TEXT("VisualizeLightShape.Spot.IndexBuffer"), sizeof(uint16) * NumTriangles * 3, sizeof(uint16))
			.AddUsage(EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

		Out.IndexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}

	FVector3f* VertexData = (FVector3f*)RHICmdList.LockBuffer(Out.VertexBufferRHI, 0, sizeof(FVector3f) * NumVertices, RLM_WriteOnly);
	uint16* IndexData = (uint16*)RHICmdList.LockBuffer(Out.IndexBufferRHI, 0, sizeof(uint16) * NumTriangles * 3, RLM_WriteOnly);

	int32 VertexIndex = 0;
	for (int32 X = 0; X <= NumSlices; ++X)
	{
		const float Phi = UE_TWO_PI / NumSlices * X;
		const float SinPhi = FMath::Sin(Phi);
		const float CosPhi = FMath::Cos(Phi);

		for (int32 Y = 0; Y <= NumSections; ++Y)
		{
			VertexData[VertexIndex] = FVector3f(Length / NumSections * Y, Radius * CosPhi, Radius * SinPhi);
			++VertexIndex;
		}
	}
	check(VertexIndex == NumVertices);

	int32 TriangleIndex = 0;
	for (int32 X = 0; X < NumSlices; ++X)
	{
		const int32 Offset = (NumSections + 1) * X;

		for (int32 Y = 0; Y < NumSections; ++Y)
		{
			// I3 - I2
			// |  \  |
			// I0 - I1
			const int32 I0 = Offset + Y;
			const int32 I1 = Offset + Y + 1;
			const int32 I2 = Offset + Y + (NumSections + 1) + 1;
			const int32 I3 = Offset + Y + (NumSections + 1);

			IndexData[TriangleIndex * 3 + 0] = I0;
			IndexData[TriangleIndex * 3 + 1] = I1;
			IndexData[TriangleIndex * 3 + 2] = I3;
			++TriangleIndex;

			IndexData[TriangleIndex * 3 + 0] = I1;
			IndexData[TriangleIndex * 3 + 1] = I2;
			IndexData[TriangleIndex * 3 + 2] = I3;
			++TriangleIndex;
		}
	}
	check(TriangleIndex == NumTriangles);

	RHICmdList.UnlockBuffer(Out.VertexBufferRHI);
	RHICmdList.UnlockBuffer(Out.IndexBufferRHI);

	return Out;
}

static FVisualizationMeshBuffers CreateRectLightVisualizationMesh(FRDGBuilder& GraphBuilder)
{
	const bool bTrimBehindBarnRect = true;
	const int32 NumSectionsY = 14;
	const int32 NumSectionsX = bTrimBehindBarnRect ? NumSectionsY / 2 + 1 : NumSectionsY;
	const int32 BiasX = NumSectionsY - NumSectionsX;
	const int32 NumVertices = (NumSectionsX + 1) * (NumSectionsY * 4 + 1);
	const int32 NumTriangles = NumSectionsX * NumSectionsY * 4 * 2;
	const float Length = 1.0f;

	FRHICommandListImmediate& RHICmdList = GraphBuilder.RHICmdList;
	FVisualizationMeshBuffers Out;

	Out.NumVertices = NumVertices;
	Out.NumTriangles = NumTriangles;

	{
		FRHIBufferCreateDesc CreateDesc = FRHIBufferCreateDesc::CreateVertex(TEXT("VisualizeLightShape.Rect.VertexBuffer"), sizeof(FVector3f) * NumVertices)
			.AddUsage(EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

		Out.VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}

	{
		FRHIBufferCreateDesc CreateDesc = FRHIBufferCreateDesc::CreateIndex(TEXT("VisualizeLightShape.Rect.IndexBuffer"), sizeof(uint16) * NumTriangles * 3, sizeof(uint16))
			.AddUsage(EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVMask);

		Out.IndexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}

	FVector3f* VertexData = (FVector3f*)RHICmdList.LockBuffer(Out.VertexBufferRHI, 0, sizeof(FVector3f) * NumVertices, RLM_WriteOnly);
	uint16* IndexData = (uint16*)RHICmdList.LockBuffer(Out.IndexBufferRHI, 0, sizeof(uint16) * NumTriangles * 3, RLM_WriteOnly);

	int32 VertexIndex = 0;
	for (int32 Side = 0; Side < 4; ++Side)
	{
		for (int32 Y = 0; Y <= NumSectionsY + (Side == 3 ? 0 : -1); ++Y)
		{
			for (int32 X = 0; X <= NumSectionsX; ++X)
			{
				VertexData[VertexIndex][0] = Length / NumSectionsY * (BiasX + X) - Length * 0.5f;
				VertexData[VertexIndex][Side % 2 + 1] = Length * 0.5f * (Side >= 2 ? -1.0f : 1.0f);
				VertexData[VertexIndex][(Side + 1) % 2 + 1] = (Length / NumSectionsY * Y - Length * 0.5f) * (Side == 1 || Side == 2 ? -1.0f : 1.0f);
				++VertexIndex;
			}
		}
	}
	check(VertexIndex == NumVertices);

	int32 TriangleIndex = 0;
	for (int32 Side = 0; Side < 4; ++Side)
	{
		const int32 SideOffset = (NumSectionsX + 1) * NumSectionsY * Side;

		for (int32 Y = 0; Y < NumSectionsY; ++Y)
		{
			const int32 Offset = SideOffset + (NumSectionsX + 1) * Y;

			for (int32 X = 0; X < NumSectionsX; ++X)
			{
				// I3 - I2
				// |  \  |
				// I0 - I1
				const int32 I0 = Offset + X;
				const int32 I1 = Offset + X + 1;
				const int32 I2 = Offset + X + (NumSectionsX + 1) + 1;
				const int32 I3 = Offset + X + (NumSectionsX + 1);

				IndexData[TriangleIndex * 3 + 0] = I0;
				IndexData[TriangleIndex * 3 + 1] = I1;
				IndexData[TriangleIndex * 3 + 2] = I3;
				++TriangleIndex;

				IndexData[TriangleIndex * 3 + 0] = I1;
				IndexData[TriangleIndex * 3 + 1] = I2;
				IndexData[TriangleIndex * 3 + 2] = I3;
				++TriangleIndex;
			}
		}
	}
	check(TriangleIndex == NumTriangles);

	RHICmdList.UnlockBuffer(Out.VertexBufferRHI);
	RHICmdList.UnlockBuffer(Out.IndexBufferRHI);

	return Out;
}

static constexpr uint32 GNumLocalLightTypes = LIGHT_TYPE_MAX - 1;
static bool bGVisualizationMeshInitialized = false;
static FVisualizationMeshBuffers GVisualizationMeshBuffers[GNumLocalLightTypes];

namespace VisualizeLightShape
{
	void InitVisualizationMeshes(FRDGBuilder& GraphBuilder)
	{
		check(IsInRenderingThread());

		if (!bGVisualizationMeshInitialized)
		{
			bGVisualizationMeshInitialized = true;
			GVisualizationMeshBuffers[LIGHT_TYPE_POINT - 1] = CreatePointLightVisualizationMesh(GraphBuilder);
			GVisualizationMeshBuffers[LIGHT_TYPE_SPOT - 1] = CreateSpotLightVisualizationMesh(GraphBuilder);
			GVisualizationMeshBuffers[LIGHT_TYPE_RECT - 1] = CreateRectLightVisualizationMesh(GraphBuilder);
		}
	}

	void Shutdown()
	{
		if (bGVisualizationMeshInitialized)
		{
			for (int32 Index = 0; Index < GNumLocalLightTypes; ++Index)
			{
				GVisualizationMeshBuffers[Index] = FVisualizationMeshBuffers();
			}
			bGVisualizationMeshInitialized = false;
		}
	}
}

class FVisualizeLightShapeVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeLightShapeVS)
	SHADER_USE_PARAMETER_STRUCT(FVisualizeLightShapeVS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
	END_SHADER_PARAMETER_STRUCT()

	class FLightType : SHADER_PERMUTATION_RANGE_INT("LIGHT_TYPE", LIGHT_TYPE_POINT, LIGHT_TYPE_MAX - LIGHT_TYPE_POINT);
	using FPermutationDomain = TShaderPermutationDomain<FLightType>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		// CFLAG_IndirectDraw is needed for vertex and instance offsets to work on some platforms
		OutEnvironment.CompilerFlags.Add(CFLAG_IndirectDraw);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeLightShapeVS, "/Engine/Private/Tools/VisualizeLightShape.usf", "VisualizeLightShapeVS", SF_Vertex);

class FVisualizeLightShapePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeLightShapePS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeLightShapePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER(float, VisualizationToggleTime)
		SHADER_PARAMETER(float, InvFadeDuration)
		SHADER_PARAMETER(float, InvMaxDelayDistance)
		SHADER_PARAMETER(float, MaxDelay)
		SHADER_PARAMETER(float, MaxOpacity)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeLightShapePS, "/Engine/Private/Tools/VisualizeLightShape.usf", "VisualizeLightShapePS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FVisualizeLightShapeParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FVisualizeLightShapeVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVisualizeLightShapePS::FParameters, PS)
	RDG_BUFFER_ACCESS(LightInfoBuffer, ERHIAccess::VertexOrIndexBuffer)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FVisualizeLightShapeVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, 0, VET_Float3, 0, sizeof(FVector3f)));
		Elements.Add(FVertexElement(1, 0, VET_UShort2, 1, sizeof(VisualizeLightShape::FLightInfo), true));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
	
	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

static TGlobalResource<FVisualizeLightShapeVertexDeclaration> GVisualizeLightShapeVertexDeclaration;

static void RenderLightShapeVisualizationView(
	FRDGBuilder& GraphBuilder,
	int32 ViewIndex,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	const FTranslucencyPassResourcesMap& TranslucencyResourceMap,
	const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions)
{
	const FTranslucencyPassResources& TranslucencyPassResources = TranslucencyResourceMap.Get(ViewIndex, ETranslucencyPass::TPT_TranslucencyAfterDOF);
	const FRDGTextureMSAA& OutColorTexture = TranslucencyPassResources.IsValid() ? TranslucencyPassResources.ColorTexture : SceneTextures.Color;
	const FRDGTextureMSAA& DepthTexture = TranslucencyPassResources.IsValid() ? TranslucencyPassResources.DepthTexture : SceneTextures.Depth;
	const FIntRect ViewRect = TranslucencyPassResources.IsValid() ? TranslucencyPassResources.ViewRect : View.ViewRect;
	const FViewShaderParameters ViewShaderParameters = TranslucencyPassResources.IsValid() ?
		GetSeparateTranslucencyViewParameters(View, SeparateTranslucencyDimensions, ETranslucencyPass::TPT_TranslucencyAfterDOF) : View.GetShaderParameters();
	
	ESceneTextureSetupMode SceneTextureSetupMode = SceneTextures.SetupMode;
	EnumRemoveFlags(SceneTextureSetupMode, ESceneTextureSetupMode::SceneColor);

	FVisualizeLightShapeParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeLightShapeParameters>();
	PassParameters->VS.ViewUniformBuffer = ViewShaderParameters.View;
	PassParameters->VS.ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
	PassParameters->PS.ViewUniformBuffer = ViewShaderParameters.View;
	PassParameters->PS.SceneTextures = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, View.GetFeatureLevel(), SceneTextureSetupMode);
	PassParameters->PS.VisualizationToggleTime = View.ViewState->VisualizeLightShape.LastToggleTime;
	PassParameters->PS.InvFadeDuration = 1.0f / VisualizeLightShape::GetFadeDuration();
	PassParameters->PS.InvMaxDelayDistance = 1.0f / VisualizeLightShape::GetMaxDelayDistance();
	PassParameters->PS.MaxDelay = VisualizeLightShape::GetMaxDelay();
	PassParameters->PS.MaxOpacity = VisualizeLightShape::GetMaxOpacity();
	PassParameters->LightInfoBuffer = View.ForwardLightingResources.VisualizeLightShapeLightInfoBuffer;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutColorTexture.Target, ERenderTargetLoadAction::ELoad);
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DepthTexture.Target, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilNop);

	TShaderRef<FVisualizeLightShapeVS> VertexShaders[GNumLocalLightTypes];
	TShaderRef<FVisualizeLightShapePS> PixelShaders[GNumLocalLightTypes];

	for (uint32 LightTypeMinusOne = 0; LightTypeMinusOne < GNumLocalLightTypes; ++LightTypeMinusOne)
	{
		FVisualizeLightShapeVS::FPermutationDomain VSPermutationVector;
		VSPermutationVector.Set<FVisualizeLightShapeVS::FLightType>(LightTypeMinusOne + 1);

		VertexShaders[LightTypeMinusOne] = View.ShaderMap->GetShader<FVisualizeLightShapeVS>(VSPermutationVector);
		PixelShaders[LightTypeMinusOne] = View.ShaderMap->GetShader<FVisualizeLightShapePS>();
	}

	const TConstArrayView<VisualizeLightShape::FLightInfo> SortedLights = View.ForwardLightingResources.VisualizeLightShapeLightInfos;

	GraphBuilder.AddDispatchPass(
		RDG_EVENT_NAME("VisualizeLightShapeParallel"),
		PassParameters,
		ERDGPassFlags::Raster,
		[ViewRect, VertexShaders, PixelShaders, PassParameters, SortedLights](FRDGDispatchPassBuilder& DispatchPassBuilder)
		{
			extern TAutoConsoleVariable<int32> CVarRHICmdMinDrawsPerParallelCmdList;

			const int32 NumLights = SortedLights.Num();
			const int32 NumThreads = FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), CVarRHICmdWidth.GetValueOnRenderThread());
			const int32 NumTasks = FMath::Min(NumThreads, FMath::DivideAndRoundUp(NumLights, CVarRHICmdMinDrawsPerParallelCmdList.GetValueOnRenderThread()));
			const int32 NumDrawsPerTask = FMath::DivideAndRoundUp(NumLights, NumTasks);

			for (int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
			{
				const int32 StartIndex = TaskIndex * NumDrawsPerTask;
				const int32 NumDraws = FMath::Min(NumDrawsPerTask, NumLights - StartIndex);

				FRHICommandList* RHICmdList = DispatchPassBuilder.CreateCommandList();

				UE::Tasks::FTask Task = UE::Tasks::Launch(UE_SOURCE_LOCATION, [RHICmdList, StartIndex, NumDraws, ViewRect, VertexShaders, PixelShaders, PassParameters, SortedLights]()
				{
					FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
					TRACE_CPUPROFILER_EVENT_SCOPE(RecordVisualizeLightShapeDraws);

					for (int32 Index = 0; Index < NumDraws; ++Index)
					{
						const int32 LightIndex = StartIndex + Index;
						const VisualizeLightShape::FLightInfo& LightInfo = SortedLights[LightIndex];
						check(LightInfo.LightType > 0 && LightInfo.LightType < LIGHT_TYPE_MAX);

						const int32 LightTypeMinusOne = LightInfo.LightType - 1;
						const FVisualizationMeshBuffers& VisualizationMesh = GVisualizationMeshBuffers[LightTypeMinusOne];

						if (Index == 0 || LightInfo.LightType != SortedLights[LightIndex - 1].LightType)
						{
							const TShaderRef<FVisualizeLightShapeVS>& VertexShader = VertexShaders[LightTypeMinusOne];
							const TShaderRef<FVisualizeLightShapePS>& PixelShader = PixelShaders[LightTypeMinusOne];

							FGraphicsPipelineStateInitializer GraphicsPSOInit;
							RHICmdList->ApplyCachedRenderTargets(GraphicsPSOInit);

							GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Wireframe, CM_None>::GetRHI();
							GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
							GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
							GraphicsPSOInit.PrimitiveType = PT_TriangleList;

							GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GVisualizeLightShapeVertexDeclaration.VertexDeclarationRHI;
							GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
							GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

							SetGraphicsPipelineState(*RHICmdList, GraphicsPSOInit, 0);

							SetShaderParameters(*RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
							SetShaderParameters(*RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

							RHICmdList->SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

							RHICmdList->SetStreamSource(0, VisualizationMesh.VertexBufferRHI, 0);
							RHICmdList->SetStreamSource(1, PassParameters->LightInfoBuffer->GetRHI(), 0);
						}
						
						RHICmdList->DrawIndexedPrimitive(VisualizationMesh.IndexBufferRHI, 0, LightIndex, VisualizationMesh.NumVertices, 0, VisualizationMesh.NumTriangles, 1);
					}

					if (!RHICmdList->IsSubCommandList())
					{
						RHICmdList->EndRenderPass();
					}
					RHICmdList->FinishRecording();
				});

				DispatchPassBuilder.AddPrerequisite(Task);
			}
		});
}

void FDeferredShadingSceneRenderer::RenderLightShapeVisualization(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	const FTranslucencyPassResourcesMap& TranslucencyResourceMap)
{
	check(IsInRenderingThread());

	bool bAnyViewShouldDraw = false;
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];
		bAnyViewShouldDraw = bAnyViewShouldDraw || View.ForwardLightingResources.VisualizeLightShapeLightInfoBuffer != nullptr;
	}

	if (!bAnyViewShouldDraw)
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, VisualizeLightShape, "VisualizeLightShape");

	VisualizeLightShape::InitVisualizationMeshes(GraphBuilder);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (View.ForwardLightingResources.VisualizeLightShapeLightInfoBuffer)
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

			RenderLightShapeVisualizationView(
				GraphBuilder,
				ViewIndex,
				View,
				SceneTextures,
				TranslucencyResourceMap,
				SeparateTranslucencyDimensions);
		}
	}
}
