// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FViewInfo;

namespace VisualizeLightShape
{
	struct FLightInfo
	{
		uint16 ForwardLightIndex;
		uint16 LightType;
	};

	struct FSortKey
	{
		union
		{
			struct 
			{
				FLightInfo LightData;
				float Distance;
			};
			uint64 Packed;
		};

		bool operator<(const FSortKey& Other) const
		{
			return Packed < Other.Packed;
		}
	};

	class FViewState
	{
	public:
		double LastToggleTime = 0.0;
	};

	void Shutdown();

	bool UpdateLastToggleTime(FViewInfo& View);
}