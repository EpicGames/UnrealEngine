// Copyright Epic Games, Inc. All Rights Reserved.

#include "VisualizeTexturePresent.h"
#include "VisualizeTexture.h"
#include "ScreenPass.h"
#include "UnrealEngine.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "RenderTargetPool.h"
#include "SceneRendering.h"

void FVisualizeTexturePresent::PresentContent(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassRenderTarget Output)
{
	check(Output.IsValid());

#if SUPPORTS_VISUALIZE_TEXTURE
	FVisualizeTexture::FCaptured& Captured = GVisualizeTexture.Captured;

	if (!Captured.PooledRenderTarget && !Captured.Texture)
	{
		// visualize feature is deactivated
		return;
	}

	// Reset bitmap flags now that we know we've saved out the bitmap we're seeing on screen.
	{
		using EFlags = FVisualizeTexture::EFlags;
		EnumRemoveFlags(GVisualizeTexture.Config.Flags, EFlags::SaveBitmap | EFlags::SaveBitmapAsStencil);
	}

	const FPooledRenderTargetDesc& Desc = Captured.Desc;

	FRDGTextureRef VisualizeTexture2D = Captured.Texture;

	// The RDG version may be stale. The IPooledRenderTarget overrides it.
	if (Captured.PooledRenderTarget)
	{
		Captured.Texture = nullptr;
		VisualizeTexture2D = GraphBuilder.RegisterExternalTexture(Captured.PooledRenderTarget, Desc.DebugName);
	}

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeTexture");

	using EInputUVMapping = FVisualizeTexture::EInputUVMapping;
	const EInputUVMapping InputUVMapping = VisualizeTexture2D->Desc.IsTexture2D() ? GVisualizeTexture.Config.InputUVMapping : EInputUVMapping::Whole;

	using EInputValueMapping = FVisualizeTexture::EInputValueMapping;
	const EInputValueMapping InputValueMapping = Captured.InputValueMapping;

	{
		FScreenPassTexture CopyInput(VisualizeTexture2D);
		FScreenPassRenderTarget CopyOutput = Output;

		switch (InputUVMapping)
		{
		case EInputUVMapping::LeftTop:
			CopyOutput.ViewRect = View.UnconstrainedViewRect;
			break;

		case EInputUVMapping::PixelPerfectCenter:
		{
			FIntPoint SrcSize = CopyInput.ViewRect.Size();
			FIntPoint Center = View.UnconstrainedViewRect.Size() / 2;
			FIntPoint HalfMin = SrcSize / 2;
			FIntPoint HalfMax = SrcSize - HalfMin;

			CopyOutput.ViewRect = FIntRect(Center - HalfMin, Center + HalfMax);
		}
		break;

		case EInputUVMapping::PictureInPicture:
		{
			const FIntPoint CopyInputExtent  = CopyInput.Texture->Desc.Extent;
			const float CopyInputAspectRatio = float(CopyInputExtent.X) / float(CopyInputExtent.Y);

			int32 TargetedHeight = 0.3f * View.UnconstrainedViewRect.Height();
			int32 TargetedWidth = CopyInputAspectRatio * TargetedHeight;
			int32 OffsetFromBorder = 100;

			CopyOutput.ViewRect.Min.X = View.UnconstrainedViewRect.Min.X + OffsetFromBorder;
			CopyOutput.ViewRect.Max.X = CopyOutput.ViewRect.Min.X + TargetedWidth;
			CopyOutput.ViewRect.Max.Y = View.UnconstrainedViewRect.Max.Y - OffsetFromBorder;
			CopyOutput.ViewRect.Min.Y = CopyOutput.ViewRect.Max.Y - TargetedHeight;
		}
		break;
		}

		AddDrawTexturePass(GraphBuilder, View, CopyInput, CopyOutput);
	}

	Output.LoadAction = ERenderTargetLoadAction::ELoad;

	const FIntPoint ViewportSizeXY = Captured.OutputExtent;
	FString ViewDescription = GVisualizeTexture.ViewDescriptionMap[Captured.ViewUniqueId];

	AddDrawCanvasPass(GraphBuilder, {}, View, Output,
		[VisualizeTexture2D, ViewportSizeXY, Desc, &View, InputUVMapping, InputValueMapping, ViewUniqueId = Captured.ViewUniqueId, ViewDescription, ViewRects = CopyTemp(Captured.ViewRects)](FCanvas& Canvas)
	{
		float X = 100 + View.UnconstrainedViewRect.Min.X;
		float Y = 160 + View.UnconstrainedViewRect.Min.Y;
		float YStep = 14;

		{
			const uint32 VersionCount = GVisualizeTexture.GetVersionCount(VisualizeTexture2D->Name);

			FString ExtendedName;
			if (VersionCount > 0)
			{
				const uint32 Version = FMath::Min(GVisualizeTexture.Requested.Version.Get(VersionCount), VersionCount - 1);

				// was reused this frame
				ExtendedName = FString::Printf(TEXT("%s@%d @0..%d"), VisualizeTexture2D->Name, Version, VersionCount - 1);
			}
			else
			{
				// was not reused this frame but can be referenced
				ExtendedName = FString::Printf(TEXT("%s"), VisualizeTexture2D->Name);
			}

			const FVisualizeTexture::FConfig& Config = GVisualizeTexture.Config;

			FString Channels = "RGB";
			switch (Config.SingleChannel)
			{
			case 0: Channels = TEXT("R"); break;
			case 1: Channels = TEXT("G"); break;
			case 2: Channels = TEXT("B"); break;
			case 3: Channels = TEXT("A"); break;
			}
			float Multiplier = (Config.SingleChannel == -1) ? Config.RGBMul : Config.SingleChannelMul;

			FString Line = FString::Printf(TEXT("VisualizeTexture: \"%s\" %s*%g UV%d"),
				*ExtendedName,
				*Channels,
				Multiplier,
				EnumToUnderlyingType(InputUVMapping));

			Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		}
		{
			FString Line = FString::Printf(TEXT("View ID %d: \"%s\""), ViewUniqueId, *ViewDescription);
			Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		}
		{
			FString Line = FString::Printf(TEXT("  TextureInfo: %s"), *(Desc.GenerateInfoString()));
			Canvas.DrawShadowedString(X + 10, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		}
		{
			FString Line = FString::Printf(TEXT("  ViewportSize:(%d,%d)"), ViewportSizeXY.X, ViewportSizeXY.Y);
			Canvas.DrawShadowedString(X + 10, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		}

		const FSceneViewFamily& ViewFamily = *View.Family;

		for (int32 ViewId = 0; ViewId < ViewRects.Num(); ++ViewId)
		{
			const FIntRect& ViewRect = ViewRects[ViewId];
			FString Line = FString::Printf(TEXT("   View #%d: (%d,%d)-(%d,%d)"), ViewId + 1, ViewRect.Min.X, ViewRect.Min.Y, ViewRect.Max.X, ViewRect.Max.Y);
			Canvas.DrawShadowedString(X + 10, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		}

		X += 40;

		if (EnumHasAnyFlags(Desc.Flags, TexCreate_CPUReadback))
		{
			Canvas.DrawShadowedString(X, Y += YStep, TEXT("Content cannot be visualized on the GPU (TexCreate_CPUReadback)"), GetStatsFont(), FLinearColor(1, 1, 0));
		}
		else
		{
			Canvas.DrawShadowedString(X, Y += YStep, TEXT("Blinking Red: <0"), GetStatsFont(), FLinearColor(1, 0, 0));
			Canvas.DrawShadowedString(X, Y += YStep, TEXT("Blinking Blue: NAN or Inf"), GetStatsFont(), FLinearColor(0, 0, 1));

			if (InputValueMapping == EInputValueMapping::Shadow)
			{
				Canvas.DrawShadowedString(X, Y += YStep, TEXT("Color Key: Linear with white near and teal distant"), GetStatsFont(), FLinearColor(54.f / 255.f, 117.f / 255.f, 136.f / 255.f));
			}
			else if (InputValueMapping == EInputValueMapping::Depth)
			{
				Canvas.DrawShadowedString(X, Y += YStep, TEXT("Color Key: Nonlinear with white distant"), GetStatsFont(), FLinearColor(0.5, 0, 0));
			}
		}
	});
#endif
}