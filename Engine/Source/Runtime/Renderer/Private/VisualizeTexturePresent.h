// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

class FVisualizeTexturePresent
{
public:
	/** Present the visualize texture tool on screen. */
	static void PresentContent(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassRenderTarget Output);
};