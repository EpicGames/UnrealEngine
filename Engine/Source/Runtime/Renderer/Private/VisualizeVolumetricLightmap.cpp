// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricLightmap.cpp
=============================================================================*/

#include "Stats/Stats.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "PostProcess/SceneRenderTargets.h"
#include "GlobalShader.h"
#include "SceneRenderTargetParameters.h"
#include "DeferredShadingRenderer.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "ScenePrivate.h"
#include "SpriteIndexBuffer.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PrecomputedVolumetricLightmap.h"

float GVolumetricLightmapVisualizationRadiusScale = .01f;
FAutoConsoleVariableRef CVarVolumetricLightmapVisualizationRadiusScale(
	TEXT("r.VolumetricLightmap.VisualizationRadiusScale"),
	GVolumetricLightmapVisualizationRadiusScale,
	TEXT("Scales the size of the spheres used to visualize volumetric lightmap samples."),
	ECVF_RenderThreadSafe
	);

float GVolumetricLightmapVisualizationMinScreenFraction = .001f;
FAutoConsoleVariableRef CVarVolumetricLightmapVisualizationMinScreenFraction(
	TEXT("r.VolumetricLightmap.VisualizationMinScreenFraction"),
	GVolumetricLightmapVisualizationMinScreenFraction,
	TEXT("Minimum screen size of a volumetric lightmap visualization sphere"),
	ECVF_RenderThreadSafe
	);

// Nvidia has lower vertex throughput when only processing a few verts per instance
const int32 GQuadsPerVisualizeInstance = 8;

TGlobalResource<FSpriteIndexBuffer<GQuadsPerVisualizeInstance>> GVisualizeQuadIndexBuffer;

BEGIN_SHADER_PARAMETER_STRUCT(FVisualizeVolumetricLightmapParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER(FVector3f, DiffuseColor)
	SHADER_PARAMETER(float, VisualizationRadiusScale)
	SHADER_PARAMETER(float, VisualizationMinScreenFraction)
END_SHADER_PARAMETER_STRUCT()

class FVisualizeVolumetricLightmapVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeVolumetricLightmapVS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeVolumetricLightmapVS, FGlobalShader);
	using FParameters = FVisualizeVolumetricLightmapParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("QUADS_PER_INSTANCE"), GQuadsPerVisualizeInstance);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeVolumetricLightmapVS, "/Engine/Private/VisualizeVolumetricLightmap.usf" , "VisualizeVolumetricLightmapVS", SF_Vertex);

class FVisualizeVolumetricLightmapPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeVolumetricLightmapPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeVolumetricLightmapPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FVisualizeVolumetricLightmapParameters, Common)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeVolumetricLightmapPS, "/Engine/Private/VisualizeVolumetricLightmap.usf", "VisualizeVolumetricLightmapPS", SF_Pixel);

void FDeferredShadingSceneRenderer::VisualizeVolumetricLightmap(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures)
{
	if (!ViewFamily.EngineShowFlags.VisualizeVolumetricLightmap)
	{
		return;
	}

	if (!Scene->VolumetricLightmapSceneData.HasData())
	{
		return;
	}

	const FPrecomputedVolumetricLightmapData* VolumetricLightmapData = Scene->VolumetricLightmapSceneData.GetLevelVolumetricLightmap()->Data;
	check(VolumetricLightmapData);

	if (VolumetricLightmapData->IndirectionTextureDimensions.GetMin() <= 0)
	{
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeVolumetricLightmap");

	for (const FViewInfo& View : Views)
	{
		auto* PassParameters = GraphBuilder.AllocParameters<FVisualizeVolumetricLightmapPS::FParameters>();
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);
		PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);

		if (SceneTextures.GBufferB)
		{
			PassParameters->RenderTargets[1] = FRenderTargetBinding(SceneTextures.GBufferB, ERenderTargetLoadAction::ELoad);
		}

		PassParameters->Common.View = View.ViewUniformBuffer;
		PassParameters->Common.VisualizationRadiusScale = GVolumetricLightmapVisualizationRadiusScale;
		PassParameters->Common.VisualizationMinScreenFraction = GVolumetricLightmapVisualizationMinScreenFraction;

		{
			FVector3f DiffuseColorValue(.18f, .18f, .18f);
			if (!ViewFamily.EngineShowFlags.Materials)
			{
				DiffuseColorValue = FVector3f(GEngine->LightingOnlyBrightness);
			}
			PassParameters->Common.DiffuseColor = DiffuseColorValue;
		}

		TShaderMapRef<FVisualizeVolumetricLightmapVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FVisualizeVolumetricLightmapPS> PixelShader(View.ShaderMap);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("VisualizeVolumetricLightmap"),
			PassParameters,
			ERDGPassFlags::Raster,
			[this, VertexShader, PixelShader, &View, VolumetricLightmapData, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGB, CW_RGBA>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->Common);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			const int32 BrickSize = VolumetricLightmapData->BrickSize;
			const uint32 NumQuads = VolumetricLightmapData->IndirectionTextureDimensions.X * VolumetricLightmapData->IndirectionTextureDimensions.Y * VolumetricLightmapData->IndirectionTextureDimensions.Z * BrickSize * BrickSize * BrickSize;

			RHICmdList.SetStreamSource(0, nullptr, 0);
			RHICmdList.DrawIndexedPrimitive(GVisualizeQuadIndexBuffer.IndexBufferRHI, 0, 0, 4 * GQuadsPerVisualizeInstance, 0, 2 * GQuadsPerVisualizeInstance, FMath::DivideAndRoundUp(FMath::Min(NumQuads, 0x7FFFFFFFu / 4), (uint32)GQuadsPerVisualizeInstance));
		});
	}
}
