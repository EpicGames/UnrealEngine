// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightShadowShaderParameters.h
=============================================================================*/

#include "VolumeLighting.h"
#include "SceneView.h"
#include "SceneRendering.h"
#include "LightSceneInfo.h"
#include "ShadowRendering.h"
#include "Components/LightComponent.h"
#include "Engine/MapBuildDataRegistry.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FVolumeShadowingShaderParametersGlobal0, "Light0Shadow");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FVolumeShadowingShaderParametersGlobal1, "Light1Shadow");

const FProjectedShadowInfo* GetFirstWholeSceneShadowMap(const FVisibleLightInfo& VisibleLightInfo)
{
	for (int32 ShadowIndex = 0; ShadowIndex < VisibleLightInfo.ShadowsToProject.Num(); ShadowIndex++)
	{
		const FProjectedShadowInfo* ProjectedShadowInfo = VisibleLightInfo.ShadowsToProject[ShadowIndex];

		if (ProjectedShadowInfo->bAllocated
			&& ProjectedShadowInfo->bWholeSceneShadow
			&& !ProjectedShadowInfo->bRayTracedDistanceField)
		{
			return ProjectedShadowInfo;
		}
	}
	return nullptr;
}

void SetVolumeShadowingDefaultShaderParametersGlobal(FRDGBuilder& GraphBuilder, FVolumeShadowingShaderParameters& ShaderParams)
{
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
	FRDGTextureRef BlackDepthCubeTexture = SystemTextures.BlackDepthCube;

	ShaderParams.TranslatedWorldToShadowMatrix = FMatrix44f::Identity;
	ShaderParams.ShadowmapMinMax = FVector4f(1.0f);
	ShaderParams.DepthBiasParameters = FVector4f(1.0f);
	ShaderParams.ShadowInjectParams = FVector4f(1.0f);
	memset(ShaderParams.ClippingPlanes.GetData(), 0, sizeof(ShaderParams.ClippingPlanes));
	ShaderParams.bStaticallyShadowed = 0;
	ShaderParams.TranslatedWorldToStaticShadowMatrix = FMatrix44f::Identity;
	ShaderParams.StaticShadowBufferSize = FVector4f(1.0f);
	ShaderParams.ShadowDepthTexture = SystemTextures.White;
	ShaderParams.StaticShadowDepthTexture = GWhiteTexture->TextureRHI;
	ShaderParams.ShadowDepthTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	ShaderParams.StaticShadowDepthTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	memset(ShaderParams.OnePassPointShadowProjection.ShadowViewProjectionMatrices.GetData(), 0, sizeof(ShaderParams.OnePassPointShadowProjection.ShadowViewProjectionMatrices));
	ShaderParams.OnePassPointShadowProjection.InvShadowmapResolution = 1.0f;
	ShaderParams.OnePassPointShadowProjection.ShadowDepthCubeTexture = BlackDepthCubeTexture;
	ShaderParams.OnePassPointShadowProjection.ShadowDepthCubeTexture2 = BlackDepthCubeTexture;
	ShaderParams.OnePassPointShadowProjection.ShadowDepthCubeTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, 0, 0, SCF_Less>::GetRHI();
};

void GetVolumeShadowingShaderParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLightSceneInfo* LightSceneInfo,
	const FProjectedShadowInfo* ShadowInfo,
	FVolumeShadowingShaderParameters& OutParameters)
{
	const FMatrix TranslatedWorldToWorld = FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation());

	const bool bDynamicallyShadowed = ShadowInfo != nullptr;
	if (bDynamicallyShadowed)
	{
		const FMatrix WorldToShadowMatrix = ShadowInfo->GetWorldToShadowMatrix(OutParameters.ShadowmapMinMax);
		OutParameters.TranslatedWorldToShadowMatrix = FMatrix44f(TranslatedWorldToWorld * WorldToShadowMatrix);
	}
	else
	{
		OutParameters.TranslatedWorldToShadowMatrix = FMatrix44f::Identity;
		OutParameters.ShadowmapMinMax = FVector4f(1.0f);
	}

	// default to ignore the plane
	FVector4f Planes[2] = { FVector4f(0, 0, 0, -1), FVector4f(0, 0, 0, -1) };
	// .zw:DistanceFadeMAD to use MAD for efficiency in the shader, default to ignore the plane
	FVector4f ShadowInjectParamValue(1, 1, 0, 0);

	int32 InnerSplitIndex = ShadowInfo ? ShadowInfo->CascadeSettings.ShadowSplitIndex : INDEX_NONE;
	if (InnerSplitIndex != INDEX_NONE)
	{
		const FShadowCascadeSettings& ShadowCascadeSettings = ShadowInfo->CascadeSettings;

		// near cascade plane
		{
			ShadowInjectParamValue.X = ShadowCascadeSettings.SplitNearFadeRegion == 0 ? 1.0f : 1.0f / ShadowCascadeSettings.SplitNearFadeRegion;
			const FPlane TranslatedPlane = ShadowCascadeSettings.NearFrustumPlane.TranslateBy(View.ViewMatrices.GetPreViewTranslation());

			Planes[0] = FVector4f(FVector3f(TranslatedPlane), -TranslatedPlane.W);
		}

		uint32 CascadeCount = LightSceneInfo->Proxy->GetNumViewDependentWholeSceneShadows(View, LightSceneInfo->IsPrecomputedLightingValid());

		// far cascade plane
		if (InnerSplitIndex != CascadeCount - 1)
		{
			ShadowInjectParamValue.Y = 1.0f / (ShadowCascadeSettings.SplitFarFadeRegion == 0.0f ? 0.0001f : ShadowCascadeSettings.SplitFarFadeRegion);
			const FPlane TranslatedPlane = ShadowCascadeSettings.FarFrustumPlane.TranslateBy(View.ViewMatrices.GetPreViewTranslation());

			Planes[1] = FVector4f(FVector3f(TranslatedPlane), -TranslatedPlane.W);
		}

		const FVector2D FadeParams = LightSceneInfo->Proxy->GetDirectionalLightDistanceFadeParameters(View.GetFeatureLevel(), LightSceneInfo->IsPrecomputedLightingValid(), View.MaxShadowCascades);

		// setup constants for the MAD in shader
		ShadowInjectParamValue.Z = FadeParams.Y;
		ShadowInjectParamValue.W = -FadeParams.X * FadeParams.Y;
	}
	OutParameters.ShadowInjectParams = ShadowInjectParamValue;
	OutParameters.ClippingPlanes[0] = Planes[0];
	OutParameters.ClippingPlanes[1] = Planes[1];

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	ELightComponentType LightType = (ELightComponentType)LightSceneInfo->Proxy->GetLightType();
	OutParameters.ShadowDepthTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	if (bDynamicallyShadowed)
	{
		OutParameters.DepthBiasParameters = FVector4f(ShadowInfo->GetShaderDepthBias(), ShadowInfo->GetShaderSlopeDepthBias(), ShadowInfo->GetShaderMaxSlopeDepthBias(), 1.0f / (ShadowInfo->MaxSubjectZ - ShadowInfo->MinSubjectZ));

		FRDGTexture* ShadowDepthTextureResource = nullptr;
		if (LightType == LightType_Point || LightType == LightType_Rect)
		{
			ShadowDepthTextureResource = SystemTextures.Black;
		}
		else
		{
			ShadowDepthTextureResource = GraphBuilder.RegisterExternalTexture(ShadowInfo->RenderTargets.DepthTarget);
		}
		OutParameters.ShadowDepthTexture = ShadowDepthTextureResource;
	}
	else
	{
		OutParameters.DepthBiasParameters = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
		OutParameters.ShadowDepthTexture = SystemTextures.Black;
	}
	check(OutParameters.ShadowDepthTexture);

	const FStaticShadowDepthMap* StaticShadowDepthMap = LightSceneInfo->Proxy->GetStaticShadowDepthMap();
	const uint32 bStaticallyShadowedValue = LightSceneInfo->IsPrecomputedLightingValid() 
											&& StaticShadowDepthMap 
											&& StaticShadowDepthMap->Data 
											&& !StaticShadowDepthMap->Data->WorldToLight.ContainsNaN()
											&& StaticShadowDepthMap->TextureRHI ? 1 : 0;
	FRHITexture* StaticShadowDepthMapTexture = bStaticallyShadowedValue ? StaticShadowDepthMap->TextureRHI : GWhiteTexture->TextureRHI;
	const FMatrix WorldToStaticShadow = bStaticallyShadowedValue ? StaticShadowDepthMap->Data->WorldToLight : FMatrix::Identity;
	const FVector4f StaticShadowBufferSizeValue = bStaticallyShadowedValue ? FVector4f(StaticShadowDepthMap->Data->ShadowMapSizeX, StaticShadowDepthMap->Data->ShadowMapSizeY, 1.0f / StaticShadowDepthMap->Data->ShadowMapSizeX, 1.0f / StaticShadowDepthMap->Data->ShadowMapSizeY) : FVector4f(0, 0, 0, 0);

	OutParameters.bStaticallyShadowed = bStaticallyShadowedValue;

	OutParameters.StaticShadowDepthTexture = StaticShadowDepthMapTexture;
	OutParameters.StaticShadowDepthTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	OutParameters.TranslatedWorldToStaticShadowMatrix = FMatrix44f(TranslatedWorldToWorld * WorldToStaticShadow);
	OutParameters.StaticShadowBufferSize = StaticShadowBufferSizeValue;

	GetOnePassPointShadowProjectionParameters(
		GraphBuilder, 
		bDynamicallyShadowed && (LightType == LightType_Point || LightType == LightType_Rect) ? ShadowInfo : NULL, 
		OutParameters.OnePassPointShadowProjection);
}

void SetVolumeShadowingShaderParameters(
	FRDGBuilder& GraphBuilder,
	FVolumeShadowingShaderParametersGlobal0& ShaderParams,
	const FViewInfo& View,
	const FLightSceneInfo* LightSceneInfo,
	const FProjectedShadowInfo* ShadowInfo)
{
	FLightRenderParameters LightParameters;
	LightSceneInfo->Proxy->GetLightShaderParameters(LightParameters);
	ShaderParams.TranslatedWorldPosition = FVector3f(View.ViewMatrices.GetPreViewTranslation() + LightParameters.WorldPosition);
	ShaderParams.InvRadius = LightParameters.InvRadius;

	GetVolumeShadowingShaderParameters(GraphBuilder, View, LightSceneInfo, ShadowInfo, ShaderParams.VolumeShadowingShaderParameters);
}

void SetVolumeShadowingShaderParameters(
	FRDGBuilder& GraphBuilder,
	FVolumeShadowingShaderParametersGlobal1& ShaderParams,
	const FViewInfo& View,
	const FLightSceneInfo* LightSceneInfo,
	const FProjectedShadowInfo* ShadowInfo)
{
	FLightRenderParameters LightParameters;
	LightSceneInfo->Proxy->GetLightShaderParameters(LightParameters);
	ShaderParams.TranslatedWorldPosition = FVector3f(View.ViewMatrices.GetPreViewTranslation() + LightParameters.WorldPosition);
	ShaderParams.InvRadius = LightParameters.InvRadius;

	GetVolumeShadowingShaderParameters(GraphBuilder, View, LightSceneInfo, ShadowInfo, ShaderParams.VolumeShadowingShaderParameters);
}

void SetVolumeShadowingDefaultShaderParameters(FRDGBuilder& GraphBuilder, FVolumeShadowingShaderParametersGlobal0& ShaderParams)
{
	ShaderParams.TranslatedWorldPosition = FVector3f(1.0f);
	ShaderParams.InvRadius = 1.0f;
	SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, ShaderParams.VolumeShadowingShaderParameters);
}

void SetVolumeShadowingDefaultShaderParameters(FRDGBuilder& GraphBuilder, FVolumeShadowingShaderParametersGlobal1& ShaderParams)
{
	ShaderParams.TranslatedWorldPosition = FVector3f(1.0f);
	ShaderParams.InvRadius = 1.0f;
	SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, ShaderParams.VolumeShadowingShaderParameters);
}

