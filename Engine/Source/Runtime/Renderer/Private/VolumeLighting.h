// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumeLighting.h
=============================================================================*/

#pragma once

#include "RHIDefinitions.h"
#include "SceneView.h"
#include "SceneRendering.h"
#include "ShadowRendering.h"
#include "Components/LightComponent.h"
#include "Engine/MapBuildDataRegistry.h"

BEGIN_SHADER_PARAMETER_STRUCT(FVolumeShadowingShaderParameters, )
	SHADER_PARAMETER(FMatrix44f, TranslatedWorldToShadowMatrix)
	SHADER_PARAMETER(FVector4f, ShadowmapMinMax)
	SHADER_PARAMETER(FVector4f, DepthBiasParameters)
	SHADER_PARAMETER(FVector4f, ShadowInjectParams)
	SHADER_PARAMETER_ARRAY(FVector4f, ClippingPlanes, [2])
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowDepthTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ShadowDepthTextureSampler)
	SHADER_PARAMETER_STRUCT_INCLUDE(FOnePassPointShadowProjection, OnePassPointShadowProjection)
	SHADER_PARAMETER(uint32, bStaticallyShadowed)
	SHADER_PARAMETER_TEXTURE(Texture2D, StaticShadowDepthTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, StaticShadowDepthTextureSampler)
	SHADER_PARAMETER(FMatrix44f, TranslatedWorldToStaticShadowMatrix)
	SHADER_PARAMETER(FVector4f, StaticShadowBufferSize)
END_SHADER_PARAMETER_STRUCT()

// InnerSplitIndex: which CSM shadow map level, It should be INDEX_NONE if LightSceneInfo is not a directional light
void GetVolumeShadowingShaderParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLightSceneInfo* LightSceneInfo,
	const FProjectedShadowInfo* ShadowInfo,
	FVolumeShadowingShaderParameters& OutParameters);

void SetVolumeShadowingDefaultShaderParametersGlobal(
	FRDGBuilder& GraphBuilder,
	FVolumeShadowingShaderParameters& ShaderParams);

///
///
///



BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVolumeShadowingShaderParametersGlobal0, )
	SHADER_PARAMETER(FVector3f, TranslatedWorldPosition)
	SHADER_PARAMETER(float, InvRadius)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParameters, VolumeShadowingShaderParameters)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVolumeShadowingShaderParametersGlobal1, )
	SHADER_PARAMETER(FVector3f, TranslatedWorldPosition)
	SHADER_PARAMETER(float, InvRadius)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParameters, VolumeShadowingShaderParameters)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

class FVisibleLightInfo;

void SetVolumeShadowingShaderParameters(
	FRDGBuilder& GraphBuilder,
	FVolumeShadowingShaderParametersGlobal0& ShaderParams,
	const FViewInfo& View,
	const FLightSceneInfo* LightSceneInfo,
	const FProjectedShadowInfo* ShadowInfo);
void SetVolumeShadowingShaderParameters(
	FRDGBuilder& GraphBuilder,
	FVolumeShadowingShaderParametersGlobal1& ShaderParams,
	const FViewInfo& View,
	const FLightSceneInfo* LightSceneInfo,
	const FProjectedShadowInfo* ShadowInfo);

void SetVolumeShadowingDefaultShaderParameters(FRDGBuilder& GraphBuilder, FVolumeShadowingShaderParametersGlobal0& ShaderParams);
void SetVolumeShadowingDefaultShaderParameters(FRDGBuilder& GraphBuilder, FVolumeShadowingShaderParametersGlobal1& ShaderParams);

/**
 * Fetch the first allocated whole scene shadow map for a given light.
 * Since shadows maps for directional lights are sorted far cascades -> near cascades, this will
 * grab the furthest cascade, which is the typical use case.
 */
const FProjectedShadowInfo* GetFirstWholeSceneShadowMap(const FVisibleLightInfo& VisibleLightInfo);


