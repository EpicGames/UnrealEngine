// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricCloudRendering.cpp
=============================================================================*/

#include "VolumetricCloudRendering.h"
#include "Components/VolumetricCloudComponent.h"
#include "VolumetricCloudProxy.h"
#include "DeferredShadingRenderer.h"
#include "MeshPassUtils.h"
#include "PixelShaderUtils.h"
#include "RenderGraphUtils.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyAtmosphereSceneProxy.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "MeshPassProcessor.inl"
#include "StaticMeshResources.h"
#include "DynamicMeshBuilder.h"
#include "SkyAtmosphereRendering.h"
#include "VolumeLighting.h"
#include "DynamicPrimitiveDrawing.h"
#include "ShaderPrint.h"
#include "CanvasTypes.h"
#include "VolumetricRenderTarget.h"
#include "BlueNoise.h"
#include "FogRendering.h"
#include "SkyAtmosphereRendering.h"
#include "BasePassRendering.h"
#include "EnvironmentComponentsFlags.h"
#include "SceneRendererInterface.h"
#include "PostProcess/PostProcessing.h" // IsPostProcessingWithAlphaChannelSupported
#include "SceneViewState.h"

//  If this is enabled, you also need to touch VolumetricCloud.usf for shaders to be recompiled.
#define CLOUD_DEBUG_SAMPLES 0 /*!Never check in enabled!*/

////////////////////////////////////////////////////////////////////////// Cloud rendering and tracing

// The project ON/OFF toggle to prevent compiling materials and shaders when not needed.
static TAutoConsoleVariable<int32> CVarVolumetricCloudSupport(
	TEXT("r.VolumetricCloud.Support"), 1,
	TEXT("Whether Volumetric Clouds are supported at all for the project, can be setup differently per platform. This can be used to avoid compiling shaders and other load time overhead. Note: this is automatically 0 for shader platforms < SM5."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

// The runtime ON/OFF toggle
static TAutoConsoleVariable<int32> CVarVolumetricCloud(
	TEXT("r.VolumetricCloud"), 1,
	TEXT("VolumetricCloud components are rendered when this is not 0, otherwise ignored."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudDistanceToSampleMaxCount(
	TEXT("r.VolumetricCloud.DistanceToSampleMaxCount"), 15.0f,
	TEXT("Distance in kilometers over which the total number of ray samples will be evenly distributed. Before that, the number of ray samples will span 1 to SampleCountMax, for for tracing distance ranging from 0 to DistanceToSampleCountMax (kilometers)."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudSampleMinCount(
	TEXT("r.VolumetricCloud.SampleMinCount"), 2,
	TEXT("The minimum number of samples to take along a ray. This can help with quality for volume close to the camera, e.g. if cloud layer is also used as low altitude fog. SampleMinCount should remain relatively small because it is applied to all tracing process."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudSampleClampCount(
	TEXT("r.VolumetricCloud.SampleClampCount"), 2147483647,
	TEXT("After the sample count for a trace has been evaluated, we can still clamp the sample count. This is useful to avoid long running GPU waves which can increase the cost, especially when started later on the async compute pipe. It should not be set too low otherwise the visual result will look incomplete. It is tied to all the sample count cvar, so it needs to be setup for all quality levels."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudViewRaySampleMaxCount(
	TEXT("r.VolumetricCloud.ViewRaySampleMaxCount"), 768,
	TEXT("The maximum number of samples taken while ray marching view primary rays."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudReflectionRaySampleMaxCount(
	TEXT("r.VolumetricCloud.ReflectionRaySampleMaxCount"), 80,
	TEXT("The maximum number of samples taken while ray marching primary rays in reflections."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudStepSizeOnZeroConservativeDensity(
	TEXT("r.VolumetricCloud.StepSizeOnZeroConservativeDensity"), 1,
	TEXT("Raymarch step size when a sample giving zero conservative density is encountered. If > 1, performance will likely improve but banding artifacts can show up if too large."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudHighQualityAerialPerspective(
	TEXT("r.VolumetricCloud.HighQualityAerialPerspective"), 0,
	TEXT("Enable/disable a second pass to trace the aerial perspective per pixel on clouds instead of using the aerial persepctive texture. Only usable when r.VolumetricCloud.EnableAerialPerspectiveSampling=1 and only needed for extra quality when r.VolumetricRenderTarget=1."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudDisableCompute(
	TEXT("r.VolumetricCloud.DisableCompute"), 0,
	TEXT("Do not use compute shader for cloud tracing."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudApplyFogOnAllPixel(
	TEXT("r.VolumetricCloud.ApplyFogOnAllPixel"), 0,
	TEXT("When true, fog will be applied on cloud tracing buffer on all pixels instead of only where clouds coverage is >0. This is a tiny bit more expenssive but can remove visual discontinuities when cloud coverage is close to or flip to 0."),
	ECVF_RenderThreadSafe);

// This is recommended to be 1 by default so that volumetric fog is correctly applied when lumen is enabled and r.VolumetricRenderTarget.PreferAsyncCompute=1.
// - Order of operations when PreferAsyncCompute=1 are (1) Cloud tracing in async compute pipe => (2) lumen update translucent lighting volume => (3) VFog upsampling (4) Cloud upsampling.
// - But cloud tracing in (1) has bad volumetric since it is generated later, and it cannot be moved up because it depends on lumen generating the translucent lighting volume.
// - So ApplyFogLate=1 allows to to have fog on clouds applied in (4). Which is fine for out single depth operation.
// This is now enabled by default since perfomance ReplayRun on a game has shown no performance regression.
static TAutoConsoleVariable<int32> CVarVolumetricCloudApplyFogLate(
	TEXT("r.VolumetricCloud.ApplyFogLate"), 1,
	TEXT("When true, fog will be applied on cloud later in the frame. This applies fog on cloud without visible latency due to reprojection and reconstruction. It is a bit more expenssive (upsampling pass cost is roughly x2) but the only correct look when Lumen is enabled and tracing is run on async compute using r.VolumetricRenderTarget.PreferAsyncCompute=1."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVolumetricCloudApplyFogLateCloudOnTranslucent(
	TEXT("r.VolumetricCloud.ApplyFogLate.CloudOnTranslucent"), 0,
	TEXT("When r.VolumetricCloud.ApplyFogLate=1, fog is not into the volumetric cloud reconstructed buffer. So when applied on translucent, a visual discrepancy can become visible for translucent behind clouds. This enables the reconstruction step to also output an extra cloud buffer with fog included for application on translucent. It is more expenssive and use more memory so it is disabled by default."),
	ECVF_RenderThreadSafe);

////////////////////////////////////////////////////////////////////////// Shadow tracing

static TAutoConsoleVariable<float> CVarVolumetricCloudShadowViewRaySampleMaxCount(
	TEXT("r.VolumetricCloud.Shadow.ViewRaySampleMaxCount"), 80,
	TEXT("The maximum number of samples taken while ray marching shadow rays."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudShadowReflectionRaySampleMaxCount(
	TEXT("r.VolumetricCloud.Shadow.ReflectionRaySampleMaxCount"), 24,
	TEXT("The maximum number of samples taken while ray marching shadow rays in reflections."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudShadowSampleAtmosphericLightShadowmap(
	TEXT("r.VolumetricCloud.Shadow.SampleAtmosphericLightShadowmap"), 1,
	TEXT("Enable the sampling of atmospheric lights shadow map in order to produce volumetric shadows."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

// The project setting for the cloud shadow to affect all translucenct surface not relying on the translucent lighting volume (enable/disable runtime and shader code).  This is not implemented on mobile as VolumetricClouds are not available on these platforms.
static TAutoConsoleVariable<int32> CVarSupportCloudShadowOnForwardLitTranslucent(
	TEXT("r.SupportCloudShadowOnForwardLitTranslucent"),
	0,
	TEXT("Enables cloud shadow to affect all translucenct surface not relying on the translucent lighting volume."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

////////////////////////////////////////////////////////////////////////// Cloud SKY AO

static TAutoConsoleVariable<int32> CVarVolumetricCloudSkyAO(
	TEXT("r.VolumetricCloud.SkyAO"), 1,
	TEXT("Enable/disable cloud sky ambient occlusion, the scene must contain a Skylight component with Cloud Ambient Occlusion enabled on it."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudSkyAODebug(
	TEXT("r.VolumetricCloud.SkyAO.Debug"), 0,
	TEXT("Print information to debug the cloud sky AO map."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudSkyAOSnapLength(
	TEXT("r.VolumetricCloud.SkyAO.SnapLength"), 20.0f,
	TEXT("Snapping size in kilometers of the cloud sky AO texture position to avoid flickering."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudSkyAOMaxResolution(
	TEXT("r.VolumetricCloud.SkyAO.MaxResolution"), 2048,
	TEXT("The maximum resolution of the texture storing ambient occlusion information for the environment lighting coming from sky light. The active resolution is controlled by the CloudAmbientOcclusionMapResolutionScale property on the Skylight component."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudSkyAOTraceSampleCount(
	TEXT("r.VolumetricCloud.SkyAO.TraceSampleCount"), 10,
	TEXT("The number of samples taken to evaluate ground lighting occlusion."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudSkyAOFiltering(
	TEXT("r.VolumetricCloud.SkyAO.Filtering"), 1,
	TEXT("Enable/disable the sky AO dilation/smoothing filter."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

////////////////////////////////////////////////////////////////////////// Cloud shadow map

static TAutoConsoleVariable<int32> CVarVolumetricCloudShadowMap(
	TEXT("r.VolumetricCloud.ShadowMap"), 1,
	TEXT("Enable/disable the shadow map, only if the scene contains a DirectionalLight component with Cast Cloud Shadows enabled on it."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudShadowMapDebug(
	TEXT("r.VolumetricCloud.ShadowMap.Debug"), 0,
	TEXT("Print information to debug the cloud shadow map."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudShadowMapSnapLength(
	TEXT("r.VolumetricCloud.ShadowMap.SnapLength"), 20.0f,
	TEXT("Snapping size in kilometers of the cloud shadowmap position to avoid flickering."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudShadowMapSnapToPixelGrid(
	TEXT("r.VolumetricCloud.ShadowMap.SnapToPixelGrid"), 1,
	TEXT("Snaps world origin to the shadow map pixel grid. Avoids shimmering with camera movement when r.VolumetricCloud.ShadowMap.SnapLength is small."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudShadowMapRaySampleMaxCount(
	TEXT("r.VolumetricCloud.ShadowMap.RaySampleMaxCount"), 128.0f,
	TEXT("The maximum number of samples taken while ray marching shadow rays to evaluate the cloud shadow map."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudShadowMapLightDistanceOverride(
	TEXT("r.VolumetricCloud.ShadowMap.LightDistanceOverride"), 0.0f,
	TEXT("When > 0, overrides the volumetric cloud shadow map light distance from the ground (km). This would usually be based on the shadow map extent."),
	ECVF_RenderThreadSafe | ECVF_Scalability );

static TAutoConsoleVariable<float> CVarVolumetricCloudShadowMapRaySampleHorizonMultiplier(
	TEXT("r.VolumetricCloud.ShadowMap.RaySampleHorizonMultiplier"), 2.0f,
	TEXT("The multipler on the sample count applied when the atmospheric light reach the horizon. Less pixels in the shadow map need to be traced, but rays need to travel a lot longer."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudShadowMapMaxResolution(
	TEXT("r.VolumetricCloud.ShadowMap.MaxResolution"), 2048,
	TEXT("The maximum resolution of the cloud shadow map. The active resolution is controlled by the CloudShadowMapResolutionScale property on the Directional Light component."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudShadowSpatialFiltering(
	TEXT("r.VolumetricCloud.ShadowMap.SpatialFiltering"), 1,
	TEXT("Enable/disable the shadow map dilation/smoothing spatial filter. Enabled when greater than 0 and it represents the number of blur iterations (constrained to a maximum of 4)."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudShadowTemporalFilteringNewFrameWeight(
	TEXT("r.VolumetricCloud.ShadowMap.TemporalFiltering.NewFrameWeight"), 1.0f,
	TEXT("Experimental and needs more work so disabled by default. Value between [0.0, 1.0] representing the weight of current frame's contribution. Low values can cause precision issues resulting in depth not converging over time. Disabled when set to 1."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudShadowTemporalFilteringLightRotationCutHistory(
	TEXT("r.VolumetricCloud.ShadowMap.TemporalFiltering.LightRotationCutHistory"), 10.0f,
	TEXT("When the atmospheric light rotation in degree is larger than that, the temporal accumulation is restarted."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

////////////////////////////////////////////////////////////////////////// Lighting component controls

static TAutoConsoleVariable<int32> CVarVolumetricCloudEnableAerialPerspectiveSampling(
	TEXT("r.VolumetricCloud.EnableAerialPerspectiveSampling"), 1,
	TEXT("Enable/disable the aerial perspective contribution on clouds."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVolumetricCloudEnableDistantSkyLightSampling(
	TEXT("r.VolumetricCloud.EnableDistantSkyLightSampling"), 1,
	TEXT("Enable/disable the distant sky light contribution on clouds."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVolumetricCloudEnableAtmosphericLightsSampling(
	TEXT("r.VolumetricCloud.EnableAtmosphericLightsSampling"), 1,
	TEXT("Enable/disable atmospheric lights contribution on clouds."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarVolumetricCloudEnableLocalLightsSampling(
	TEXT("r.VolumetricCloud.EnableLocalLightsSampling"), 0,
	TEXT("[EXPERIMENTAL] Enable/disable local lights contribution on clouds. Expenssive! Use for cinematics if needed."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudLocalLightsShadowSampleCount(
	TEXT("r.VolumetricCloud.LocalLights.ShadowSampleCount"), 12,
	TEXT("[EXPERIMENTAL] Set the volumetric shadow sample count when evaluating local lights. Expenssive! Use for cinematics if needed."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

////////////////////////////////////////////////////////////////////////// 

static TAutoConsoleVariable<int32> CVarVolumetricCloudEmptySpaceSkipping(
	TEXT("r.VolumetricCloud.EmptySpaceSkipping"), 0,
	TEXT("Enable/disable empty space skipping to accelerate cloud tracing through emty areas. EXPERIMENTAL"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudEmptySpaceSkippingVolumeDepth(
	TEXT("r.VolumetricCloud.EmptySpaceSkipping.VolumeDepth"), 40.0f,
	TEXT("Set the distance in kilometer over which empty space can be evalauted."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricCloudEmptySpaceSkippingStartTracingSliceBias(
	TEXT("r.VolumetricCloud.EmptySpaceSkipping.StartTracingSliceBias"), 0.0f,
	TEXT("The number of slices to bias the start depth with. A valuie of -1 means a bias of one slice towards the view point."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudEmptySpaceSkippingSampleCorners(
	TEXT("r.VolumetricCloud.EmptySpaceSkipping.SampleCorners"), 1,
	TEXT("0 means center samples only, >0 means corner are also sampled."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricCloudAllowAnalyticDerivatives(
	TEXT("r.VolumetricCloud.AllowAnalyticDerivatives"), 1,
	TEXT("Enables compiling cloud shaders with support for analytical derivatives when needed by the material."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

////////////////////////////////////////////////////////////////////////// 

static bool ProjectSupportVolumetricCloud(const EShaderPlatform ShaderPlatform)
{
	static FShaderPlatformCachedIniValue<int32> CVar(TEXT("r.VolumetricCloud.Support"));
	return CVar.Get(ShaderPlatform) > 0;
}

static bool ShouldUseComputeForCloudTracing(const FStaticFeatureLevel InFeatureLevel)
{
	// When capturing a non-real-time sky light, the reflection cube face is first rendered in the scene texture 
	// before it is copied to the corresponding cubemap face. 
	// We cannot create UAV on a MSAA texture for the compute shader.
	// So in this case, when capturing such a sky light, we must disabled the compute path.
	const bool bMSAAEnabled = GetDefaultMSAACount(InFeatureLevel) > 1;

	return !CVarVolumetricCloudDisableCompute.GetValueOnRenderThread()
		&& UE::PixelFormat::HasCapabilities(PF_FloatRGBA, EPixelFormatCapabilities::TypedUAVLoad)
		&& UE::PixelFormat::HasCapabilities(PF_G16R16F, EPixelFormatCapabilities::TypedUAVLoad)
		&& !bMSAAEnabled;
}

static bool ShouldUseComputeCloudEmptySpaceSkipping()
{
	return !CVarVolumetricCloudDisableCompute.GetValueOnRenderThread() && RHIIsTypedUAVLoadSupported(PF_R16F) && CVarVolumetricCloudEmptySpaceSkipping.GetValueOnRenderThread() > 0;
}

bool ShouldRenderVolumetricCloud(const FScene* Scene, const FEngineShowFlags& EngineShowFlags)
{
	if (Scene && Scene->HasVolumetricCloud() && EngineShowFlags.Atmosphere && EngineShowFlags.Cloud &&
		ProjectSupportVolumetricCloud(Scene->GetShaderPlatform()) && CVarVolumetricCloud.GetValueOnRenderThread() > 0)
	{
		const FVolumetricCloudRenderSceneInfo* VolumetricCloud = Scene->GetVolumetricCloudSceneInfo();
		check(VolumetricCloud);

		bool bCloudMaterialValid = false;
		if (VolumetricCloud->GetVolumetricCloudSceneProxy().GetCloudVolumeMaterial())
		{
			FMaterialRenderProxy* CloudVolumeMaterialProxy = VolumetricCloud->GetVolumetricCloudSceneProxy().GetCloudVolumeMaterial()->GetRenderProxy();
			const FMaterialRenderProxy* FallbackMaterialRenderProxyPtr = nullptr;
			const FMaterial& MaterialTest = CloudVolumeMaterialProxy->GetMaterialWithFallback(Scene->GetFeatureLevel(), FallbackMaterialRenderProxyPtr);
			bCloudMaterialValid = MaterialTest.GetMaterialDomain() == MD_Volume;
		}

		return bCloudMaterialValid;
	}
	return false;
}

bool ShouldRenderVolumetricCloudWithBlueNoise_GameThread(const FScene* Scene, const FSceneView& View)
{
	// We cannot use Scene->HasVolumetricCloud() here because the proxy is set form the render thread.
	// We cannot use View.Family->EngineShowFlags.Atmosphere && View.Family->EngineShowFlags.Cloud because those could cause sync loading texture during gameplay when toggled.
	return CVarVolumetricCloud.GetValueOnGameThread() > 0;
}

bool ShouldViewVisualizeVolumetricCloudConservativeDensity(const FViewInfo& ViewInfo, const FEngineShowFlags& EngineShowFlags)
{
	return (!!EngineShowFlags.VisualizeVolumetricCloudConservativeDensity || !!EngineShowFlags.VisualizeVolumetricCloudEmptySpaceSkipping)
		&& !ViewInfo.bIsReflectionCapture
		&& !ViewInfo.bIsSceneCapture
		&& GIsEditor;
}

static bool ShouldRenderCloudShadowmap(const FLightSceneProxy* AtmosphericLight)
{
	return CVarVolumetricCloudShadowMap.GetValueOnRenderThread() > 0 && AtmosphericLight && AtmosphericLight->GetCastCloudShadows() && AtmosphericLight->GetCloudShadowStrength() > 0;
}

bool ShouldVolumetricCloudTraceWithMinMaxDepth(const FViewInfo& ViewInfo)
{
	if (ViewInfo.ViewState)
	{
		FVolumetricRenderTargetViewStateData& VRT = ViewInfo.ViewState->VolumetricCloudRenderTarget;
		return ShouldViewRenderVolumetricCloudRenderTarget(ViewInfo) && ShouldUseComputeForCloudTracing(ViewInfo.FeatureLevel) && VRT.GetMode() == 0;
	}
	return false;
}

bool ShouldVolumetricCloudTraceWithMinMaxDepth(const TArray<FViewInfo>& Views)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (ShouldVolumetricCloudTraceWithMinMaxDepth(Views[ViewIndex]))
		{
			return true;
		}
	}
	return false;
}

bool VolumetricCloudWantsSeparatedAtmosphereMieRayLeigh(const FScene* Scene)
{
	const FVolumetricCloudRenderSceneInfo* VCloud = Scene->GetVolumetricCloudSceneInfo();
	if (VCloud && CVarVolumetricCloud.GetValueOnRenderThread() > 0 && ProjectSupportVolumetricCloud(Scene->GetShaderPlatform()))
	{
		const FVolumetricCloudSceneProxy& VCloudProxy = VCloud->GetVolumetricCloudSceneProxy();
		return VCloudProxy.AerialPespectiveMieScatteringStartDistance > 0.0f || VCloudProxy.AerialPespectiveRayleighScatteringStartDistance > 0.0f;
	}
	return false;
}

bool ShouldVolumetricCloudsApplyFogDuringUpsampling(const FViewInfo& ViewInfo)
{
	if (ViewInfo.ViewState)
	{
		FVolumetricRenderTargetViewStateData& VRT = ViewInfo.ViewState->VolumetricCloudRenderTarget;
		int32 VRTMode = VRT.GetMode();
		return CVarVolumetricCloudApplyFogLate.GetValueOnRenderThread() > 0 && ShouldViewRenderVolumetricCloudRenderTarget(ViewInfo) && (VRTMode == 0 || VRTMode == 2); // Only restricting to a few mode which can exhibit update delay due to reconstruction/reprojection).
	}
	return false;
}

bool ShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent(const FViewInfo& ViewInfo)
{
	extern bool ShouldRenderFog(const FSceneViewFamily & Family);

	if (ViewInfo.ViewState)
	{
		return ShouldVolumetricCloudsApplyFogDuringUpsampling(ViewInfo) && CVarVolumetricCloudApplyFogLateCloudOnTranslucent.GetValueOnRenderThread() > 0 
			&& ViewInfo.Family && (ViewInfo.Family->Scene->HasAnyExponentialHeightFog() || ViewInfo.LocalFogVolumeViewData.GPUInstanceCount > 0)
			&& ShouldRenderFog(*ViewInfo.Family);
	}
	return false;
}

//////////////////////////////////////////////////////////////////////////

static float GetVolumetricCloudShadowmapStrength(const FLightSceneProxy* AtmosphericLight)
{
	if (AtmosphericLight)
	{
		return AtmosphericLight->GetCloudShadowStrength();
	}
	return 1.0f;
}

static float GetVolumetricCloudShadowmapDepthBias(const FLightSceneProxy* AtmosphericLight)
{
	if (AtmosphericLight)
	{
		return AtmosphericLight->GetCloudShadowDepthBias();
	}
	return 0.0f;
}

static int32 GetVolumetricCloudShadowMapResolution(const FLightSceneProxy* AtmosphericLight)
{
	if (AtmosphericLight)
	{
		return FMath::Min( int32(512.0f * float(AtmosphericLight->GetCloudShadowMapResolutionScale())), CVarVolumetricCloudShadowMapMaxResolution.GetValueOnAnyThread());
	}
	return 32;
}

static float GetVolumetricCloudShadowRaySampleCountScale(const FLightSceneProxy* AtmosphericLight)
{
	if (AtmosphericLight)
	{
		return AtmosphericLight->GetCloudShadowRaySampleCountScale();
	}
	return 4.0f;
}

static float GetVolumetricCloudShadowMapExtentKm(const FLightSceneProxy* AtmosphericLight)
{
	if (AtmosphericLight)
	{
		return AtmosphericLight->GetCloudShadowExtent();
	}
	return 1.0f;
}

static bool GetVolumetricCloudReceiveAtmosphericLightShadowmap(const FLightSceneProxy* AtmosphericLight)
{
	if (AtmosphericLight)
	{
		return AtmosphericLight->GetCastShadowsOnClouds();
	}
	return false;
}

static FLinearColor GetVolumetricCloudScatteredLuminanceScale(const FLightSceneProxy* AtmosphericLight)
{
	if (AtmosphericLight)
	{
		return AtmosphericLight->GetCloudScatteredLuminanceScale();
	}
	return FLinearColor::White;
}

static bool ShouldRenderCloudSkyAO(const FSkyLightSceneProxy* SkyLight)
{
	return CVarVolumetricCloudSkyAO.GetValueOnRenderThread() > 0 && SkyLight && SkyLight->bCloudAmbientOcclusion;
}

static float GetVolumetricCloudSkyAOStrength(const FSkyLightSceneProxy* SkyLight)
{
	if (SkyLight)
	{
		return SkyLight->CloudAmbientOcclusionStrength;
	}
	return 1.0f;
}

static int32 GetVolumetricCloudSkyAOResolution(const FSkyLightSceneProxy* SkyLight)
{
	if (SkyLight)
	{
		return FMath::Min(int32(512.0f * float(SkyLight->CloudAmbientOcclusionMapResolutionScale)), CVarVolumetricCloudSkyAOMaxResolution.GetValueOnAnyThread());
	}
	return 32;
}

FRDGTextureDesc GetVolumetricCloudSkyAORDGTextureDesc(const FIntPoint& Size)
{
	return FRDGTextureDesc::Create2D(
		Size,
		PF_FloatR11G11B10,
		FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_RenderTargetable);
}

static float GetVolumetricCloudSkyAOExtentKm(const FSkyLightSceneProxy* SkyLight)
{
	if (SkyLight)
	{
		return SkyLight->CloudAmbientOcclusionExtent;
	}
	return 1.0f;
}

static float GetVolumetricCloudSkyAOApertureScale(const FSkyLightSceneProxy* SkyLight)
{
	if (SkyLight)
	{
		return SkyLight->CloudAmbientOcclusionApertureScale;
	}
	return 1.0f;
}

static bool ShouldUsePerSampleAtmosphereTransmittance(const FScene* Scene, const FViewInfo* InViewIfDynamicMeshCommand)
{
	return Scene->VolumetricCloud && Scene->VolumetricCloud->GetVolumetricCloudSceneProxy().bUsePerSampleAtmosphericLightTransmittance &&
		Scene->HasSkyAtmosphere() && ShouldRenderSkyAtmosphere(Scene, InViewIfDynamicMeshCommand->Family->EngineShowFlags);
}


//////////////////////////////////////////////////////////////////////////

FVolumetricCloudShadowAOParameters GetCloudShadowAOParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVolumetricCloudRenderSceneInfo* CloudInfo)
{
	FVolumetricCloudShadowAOParameters Out;
	if (CloudInfo)
	{
		Out.ShadowMap0 = View.VolumetricCloudShadowRenderTarget[0];
		Out.ShadowMap1 = View.VolumetricCloudShadowRenderTarget[1];
		Out.SkyAO = View.VolumetricCloudSkyAO;
	}
	else
	{
		FRDGTextureRef DummyWhite = GSystemTextures.GetWhiteDummy(GraphBuilder);
		FRDGTextureRef DummyDepth = GSystemTextures.GetDepthDummy(GraphBuilder);
		Out.ShadowMap0 = DummyDepth;
		Out.ShadowMap1 = DummyDepth;
		Out.SkyAO = DummyWhite;
	}
	return Out;
}

void GetCloudShadowAOData(const FVolumetricCloudRenderSceneInfo* CloudInfo, const FViewInfo& View, FRDGBuilder& GraphBuilder, FCloudShadowAOData& OutData)
{
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	// We pick up the texture if they exists, the decision has been mande to render them before already.
	const bool VolumetricCloudShadowMap0Valid = View.VolumetricCloudShadowRenderTarget[0] != nullptr;
	const bool VolumetricCloudShadowMap1Valid = View.VolumetricCloudShadowRenderTarget[1] != nullptr;
	OutData.bShouldSampleCloudShadow = CloudInfo && (VolumetricCloudShadowMap0Valid || VolumetricCloudShadowMap1Valid);
	OutData.VolumetricCloudShadowMap[0] = OutData.bShouldSampleCloudShadow && VolumetricCloudShadowMap0Valid ? View.VolumetricCloudShadowRenderTarget[0] : SystemTextures.Black;
	OutData.VolumetricCloudShadowMap[1] = OutData.bShouldSampleCloudShadow && VolumetricCloudShadowMap1Valid ? View.VolumetricCloudShadowRenderTarget[1] : SystemTextures.Black;

	OutData.bShouldSampleCloudSkyAO = CloudInfo && View.VolumetricCloudSkyAO != nullptr;
	OutData.VolumetricCloudSkyAO = OutData.bShouldSampleCloudSkyAO ? View.VolumetricCloudSkyAO : SystemTextures.Black;

	// We also force disable shadows if the VolumetricCloud component itself has not initialised due to extra reasons in ShouldRenderVolumetricCloud().
	// The below test is a simple way to check the fact that if ShouldRenderVolumetricCloud is false, then InitVolumetricCloudsForViews is not run
	// and in this case the GetVolumetricCloudCommonShaderParametersUB has not been created, preventing the rendering of cloud shadow on SkyAtmosphere.
	const bool bCloudComponentValid = CloudInfo && CloudInfo->GetVolumetricCloudCommonShaderParametersUB().IsValid();
	OutData.bShouldSampleCloudShadow = OutData.bShouldSampleCloudShadow && bCloudComponentValid;
	OutData.bShouldSampleCloudSkyAO  = OutData.bShouldSampleCloudSkyAO  && bCloudComponentValid;
}

/*=============================================================================
	FVolumetricCloudRenderSceneInfo implementation.
=============================================================================*/


FVolumetricCloudRenderSceneInfo::FVolumetricCloudRenderSceneInfo(FVolumetricCloudSceneProxy& VolumetricCloudSceneProxyIn)
	:VolumetricCloudSceneProxy(VolumetricCloudSceneProxyIn)
{
}

FVolumetricCloudRenderSceneInfo::~FVolumetricCloudRenderSceneInfo()
{
}



/*=============================================================================
	FScene functions
=============================================================================*/



void FScene::AddVolumetricCloud(FVolumetricCloudSceneProxy* VolumetricCloudSceneProxy)
{
	check(VolumetricCloudSceneProxy);
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FAddVolumetricCloudCommand)(
		[Scene, VolumetricCloudSceneProxy](FRHICommandListImmediate& RHICmdList)
		{
			check(!Scene->VolumetricCloudStack.Contains(VolumetricCloudSceneProxy));
			Scene->VolumetricCloudStack.Push(VolumetricCloudSceneProxy);

			VolumetricCloudSceneProxy->RenderSceneInfo = new FVolumetricCloudRenderSceneInfo(*VolumetricCloudSceneProxy);

			// Use the most recently enabled VolumetricCloud
			Scene->VolumetricCloud = VolumetricCloudSceneProxy->RenderSceneInfo;
			Scene->InvalidatePathTracedOutput();
		} );
}

void FScene::RemoveVolumetricCloud(FVolumetricCloudSceneProxy* VolumetricCloudSceneProxy)
{
	check(VolumetricCloudSceneProxy);
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FRemoveVolumetricCloudCommand)(
		[Scene, VolumetricCloudSceneProxy](FRHICommandListImmediate& RHICmdList)
		{
			delete VolumetricCloudSceneProxy->RenderSceneInfo;
			Scene->VolumetricCloudStack.RemoveSingle(VolumetricCloudSceneProxy);

			if (Scene->VolumetricCloudStack.Num() > 0)
			{
				// Use the most recently enabled VolumetricCloud
				Scene->VolumetricCloud = Scene->VolumetricCloudStack.Last()->RenderSceneInfo;
			}
			else
			{
				Scene->VolumetricCloud = nullptr;
			}
			Scene->InvalidatePathTracedOutput();
		} );
}



/*=============================================================================
	VolumetricCloud rendering functions
=============================================================================*/



DECLARE_GPU_STAT(VolumetricCloud);
DECLARE_GPU_STAT(VolumetricCloudShadow);



FORCEINLINE bool IsVolumetricCloudMaterialSupported(const EShaderPlatform ShaderPlatform)
{
	return IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM5) && ProjectSupportVolumetricCloud(ShaderPlatform);
}


FORCEINLINE bool IsMaterialCompatibleWithVolumetricCloud(const FMaterialShaderParameters& Material, const EShaderPlatform Platform)
{
	return IsVolumetricCloudMaterialSupported(Platform) && Material.bIsUsedWithVolumetricCloud && Material.MaterialDomain == MD_Volume;
}



//////////////////////////////////////////////////////////////////////////

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FRenderVolumetricCloudGlobalParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FVolumetricCloudCommonShaderParameters, VolumetricCloud)
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, SceneDepthMinAndMaxTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, CloudShadowTexture0)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, CloudShadowTexture1)
	SHADER_PARAMETER_SAMPLER(SamplerState, CloudBilinearTextureSampler)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParametersGlobal0, Light0Shadow)
	SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
	SHADER_PARAMETER(int32, VirtualShadowMapId0)
	SHADER_PARAMETER(FUintVector4, TracingCoordToZbufferCoordScaleBias)
	SHADER_PARAMETER(FUintVector4, TracingCoordToFullResPixelCoordScaleBias)
	SHADER_PARAMETER(FUintVector4, SceneDepthTextureMinMaxCoord)
	SHADER_PARAMETER(uint32, NoiseFrameIndexModPattern)
	SHADER_PARAMETER(int32, OpaqueIntersectionMode)
	SHADER_PARAMETER(uint32, VolumetricRenderTargetMode)
	SHADER_PARAMETER(uint32, CloudDebugViewMode)
	SHADER_PARAMETER(uint32, IsReflectionRendering)
	SHADER_PARAMETER(uint32, TraceShadowmapMode)
	SHADER_PARAMETER(float, LocalLightsShadowSampleCount)
	SHADER_PARAMETER(FVector4f, OutputSizeInvSize)
	SHADER_PARAMETER(int32, StepSizeOnZeroConservativeDensity)
	SHADER_PARAMETER(int32, EnableAerialPerspectiveSampling)
	SHADER_PARAMETER(int32, EnableDistantSkyLightSampling)
	SHADER_PARAMETER(int32, EnableAtmosphericLightsSampling)
	SHADER_PARAMETER(int32, EnableHeightFog)
	SHADER_PARAMETER(float, EmptySpaceSkippingSliceDepth)
	SHADER_PARAMETER(float, AerialPerspectiveRayOnlyStartDistanceKm)
	SHADER_PARAMETER(float, AerialPerspectiveRayOnlyFadeDistanceKmInv)
	SHADER_PARAMETER(float, AerialPerspectiveMieOnlyStartDistanceKm)
	SHADER_PARAMETER(float, AerialPerspectiveMieOnlyFadeDistanceKmInv)
	SHADER_PARAMETER_STRUCT_INCLUDE(FFogUniformParameters, FogStruct)
	SHADER_PARAMETER_STRUCT(FForwardLightUniformParameters, Forward)
	SHADER_PARAMETER_STRUCT(FBlueNoise, BlueNoise)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FRenderVolumetricCloudGlobalParameters, "RenderVolumetricCloudParameters", SceneTextures);

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FVolumetricCloudCommonGlobalShaderParameters, "VolumetricCloudCommonParameters");

uint32 GetVolumetricCloudDebugViewMode(const FEngineShowFlags& ShowFlags)
{
	if (ShowFlags.VisualizeVolumetricCloudConservativeDensity)
	{
		return 1;
	}
	if (ShowFlags.VisualizeVolumetricCloudEmptySpaceSkipping)
	{
		return 2;
	}
	return 0;
}

static float GetEmptySpaceSkippingSliceDepth();

// When calling this, you still need to setup Light0Shadow yourself.
void SetupDefaultRenderVolumetricCloudGlobalParameters(FRDGBuilder& GraphBuilder, FRenderVolumetricCloudGlobalParameters& VolumetricCloudParams, FVolumetricCloudRenderSceneInfo& CloudInfo, FViewInfo& ViewInfo, bool bForceNonLateFogOnCloud = false)
{
	FRDGTextureRef BlackDummy = GSystemTextures.GetBlackDummy(GraphBuilder);
	VolumetricCloudParams.VolumetricCloud = CloudInfo.GetVolumetricCloudCommonShaderParameters();
	VolumetricCloudParams.SceneDepthTexture = BlackDummy;
	VolumetricCloudParams.SceneDepthMinAndMaxTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
	VolumetricCloudParams.CloudShadowTexture0 = BlackDummy;
	VolumetricCloudParams.CloudShadowTexture1 = BlackDummy;
	VolumetricCloudParams.CloudBilinearTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	// Light0Shadow
	VolumetricCloudParams.VirtualShadowMapId0 = INDEX_NONE;
	VolumetricCloudParams.TracingCoordToZbufferCoordScaleBias = FUintVector4(1, 1, 0, 0);
	VolumetricCloudParams.TracingCoordToFullResPixelCoordScaleBias = FUintVector4(1, 1, 0, 0);
	VolumetricCloudParams.NoiseFrameIndexModPattern = 0;
	VolumetricCloudParams.VolumetricRenderTargetMode = ViewInfo.ViewState ? ViewInfo.ViewState->VolumetricCloudRenderTarget.GetMode() : 0;
	VolumetricCloudParams.CloudDebugViewMode = GetVolumetricCloudDebugViewMode(ViewInfo.Family->EngineShowFlags);

	VolumetricCloudParams.StepSizeOnZeroConservativeDensity = FMath::Max(CVarVolumetricCloudStepSizeOnZeroConservativeDensity.GetValueOnRenderThread(), 1);

	VolumetricCloudParams.EmptySpaceSkippingSliceDepth = GetEmptySpaceSkippingSliceDepth();

	VolumetricCloudParams.BlueNoise = GetBlueNoiseGlobalParameters();

	VolumetricCloudParams.EnableHeightFog = 0;
	if (ViewInfo.Family->Scene->HasAnyExponentialHeightFog() && ShouldRenderFog(*ViewInfo.Family))
	{
		VolumetricCloudParams.EnableHeightFog = CVarVolumetricCloudApplyFogOnAllPixel.GetValueOnRenderThread() > 0 ? 2 : 1;
	}
	SetupFogUniformParameters(GraphBuilder, ViewInfo, VolumetricCloudParams.FogStruct);
	if (ShouldVolumetricCloudsApplyFogDuringUpsampling(ViewInfo) && !bForceNonLateFogOnCloud)
	{
		VolumetricCloudParams.FogStruct.ApplyVolumetricFog = 0;
		VolumetricCloudParams.EnableHeightFog = 0;
	}

	VolumetricCloudParams.LFV = ViewInfo.LocalFogVolumeViewData.UniformParametersStruct;

	VolumetricCloudParams.Forward = *ViewInfo.ForwardLightingResources.ForwardLightUniformParameters;
	VolumetricCloudParams.LocalLightsShadowSampleCount = FMath::Clamp(CVarVolumetricCloudLocalLightsShadowSampleCount.GetValueOnRenderThread(), 0.0f, 128.0f);

	ESceneTextureSetupMode SceneTextureSetupMode = ESceneTextureSetupMode::All;
	EnumRemoveFlags(SceneTextureSetupMode, ESceneTextureSetupMode::SceneColor);
	EnumRemoveFlags(SceneTextureSetupMode, ESceneTextureSetupMode::SceneVelocity);
	EnumRemoveFlags(SceneTextureSetupMode, ESceneTextureSetupMode::GBuffers);
	EnumRemoveFlags(SceneTextureSetupMode, ESceneTextureSetupMode::SSAO);
	SetupSceneTextureUniformParameters(GraphBuilder, ViewInfo.GetSceneTexturesChecked(), ViewInfo.FeatureLevel, ESceneTextureSetupMode::CustomDepth, VolumetricCloudParams.SceneTextures);
}

//////////////////////////////////////////////////////////////////////////

class FRenderVolumetricCloudVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRenderVolumetricCloudVS, MeshMaterial);

public:

	FRenderVolumetricCloudVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}

	FRenderVolumetricCloudVS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		const bool bIsCompatible = IsMaterialCompatibleWithVolumetricCloud(Parameters.MaterialParameters, Parameters.Platform);
		return bIsCompatible;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_MAINVS"), TEXT("1"));
	}

private:
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FRenderVolumetricCloudVS, TEXT("/Engine/Private/VolumetricCloud.usf"), TEXT("MainVS"), SF_Vertex);

//////////////////////////////////////////////////////////////////////////

enum EVolumetricCloudRenderViewPsPermutations
{
	CloudPs_AtmoTrans0_SampleShad0_2ndLight0,
	CloudPs_AtmoTrans1_SampleShad0_2ndLight0,
	CloudPs_AtmoTrans0_SampleShad1_2ndLight0,
	CloudPs_AtmoTrans1_SampleShad1_2ndLight0,
	CloudPs_AtmoTrans0_SampleShad0_2ndLight1,
	CloudPs_AtmoTrans1_SampleShad0_2ndLight1,
	CloudPs_AtmoTrans0_SampleShad1_2ndLight1,
	CloudPs_AtmoTrans1_SampleShad1_2ndLight1,
	CloudPsCount
};

BEGIN_SHADER_PARAMETER_STRUCT(FRenderVolumetricCloudRenderViewParametersPS, )
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRenderVolumetricCloudGlobalParameters, VolumetricCloudRenderViewParamsUB)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CloudShadowTexture0)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CloudShadowTexture1)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMap)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FInstanceCullingGlobalUniforms, InstanceCulling)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

template<EVolumetricCloudRenderViewPsPermutations Permutation>
class FRenderVolumetricCloudRenderViewPs : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRenderVolumetricCloudRenderViewPs, MeshMaterial);

public:

	FRenderVolumetricCloudRenderViewPs(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		PassUniformBuffer.Bind(Initializer.ParameterMap, FRenderVolumetricCloudGlobalParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
	}

	FRenderVolumetricCloudRenderViewPs() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		const bool bIsCompatible = IsMaterialCompatibleWithVolumetricCloud(Parameters.MaterialParameters, Parameters.Platform);

		return bIsCompatible;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RENDERVIEW_PS"), TEXT("1"));
		OutEnvironment.SetDefine(TEXT("CLOUD_LAYER_PIXEL_SHADER"), TEXT("1"));

		// Force texture fetches to not use automatic mip generation because the pixel shader is using a dynamic loop to evaluate the material multiple times.
		OutEnvironment.SetDefine(TEXT("USE_FORCE_TEXTURE_MIP"), TEXT("1"));
		
		const bool bUseAtmosphereTransmittance =Permutation == CloudPs_AtmoTrans1_SampleShad0_2ndLight0 || Permutation == CloudPs_AtmoTrans1_SampleShad0_2ndLight1 ||
												Permutation == CloudPs_AtmoTrans1_SampleShad1_2ndLight1 || Permutation == CloudPs_AtmoTrans1_SampleShad1_2ndLight0;
		OutEnvironment.SetDefine(TEXT("CLOUD_PER_SAMPLE_ATMOSPHERE_TRANSMITTANCE"), bUseAtmosphereTransmittance ? TEXT("1") : TEXT("0"));

		const bool bSampleLightShadowmap =	Permutation == CloudPs_AtmoTrans0_SampleShad1_2ndLight0 || Permutation == CloudPs_AtmoTrans0_SampleShad1_2ndLight1 ||
											Permutation == CloudPs_AtmoTrans1_SampleShad1_2ndLight1 || Permutation == CloudPs_AtmoTrans1_SampleShad1_2ndLight0;
		OutEnvironment.SetDefine(TEXT("CLOUD_SAMPLE_ATMOSPHERIC_LIGHT_SHADOWMAP"), bSampleLightShadowmap ? TEXT("1") : TEXT("0"));

		if (bSampleLightShadowmap)
		{
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), TEXT("1"));
		}

		const bool bSampleSecondLight =	Permutation == CloudPs_AtmoTrans0_SampleShad0_2ndLight1 || Permutation == CloudPs_AtmoTrans0_SampleShad1_2ndLight1 ||
										Permutation == CloudPs_AtmoTrans1_SampleShad1_2ndLight1 || Permutation == CloudPs_AtmoTrans1_SampleShad0_2ndLight1;
		OutEnvironment.SetDefine(TEXT("CLOUD_SAMPLE_SECOND_LIGHT"), bSampleSecondLight ? TEXT("1") : TEXT("0"));

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
	}

private:
};

#define IMPLEMENT_CLOUD_RENDERVIEW_PS(A, B, C) \
	typedef FRenderVolumetricCloudRenderViewPs<CloudPs_AtmoTrans##A##_SampleShad##B##_2ndLight##C> FCloudRenderViewPS##A##B##C; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FCloudRenderViewPS##A##B##C, TEXT("/Engine/Private/VolumetricCloud.usf"), TEXT("MainPS"), SF_Pixel)

IMPLEMENT_CLOUD_RENDERVIEW_PS(0, 0, 0);
IMPLEMENT_CLOUD_RENDERVIEW_PS(1, 0, 0);
IMPLEMENT_CLOUD_RENDERVIEW_PS(0, 1, 0);
IMPLEMENT_CLOUD_RENDERVIEW_PS(1, 1, 0);
IMPLEMENT_CLOUD_RENDERVIEW_PS(0, 0, 1);
IMPLEMENT_CLOUD_RENDERVIEW_PS(1, 0, 1);
IMPLEMENT_CLOUD_RENDERVIEW_PS(0, 1, 1);
IMPLEMENT_CLOUD_RENDERVIEW_PS(1, 1, 1);

#undef IMPLEMENT_CLOUD_RENDERVIEW_PS

//////////////////////////////////////////////////////////////////////////

class FRenderVolumetricCloudRenderViewCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRenderVolumetricCloudRenderViewCS, MeshMaterial)
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FRenderVolumetricCloudRenderViewCS, FMeshMaterialShader)

	class FCloudPerSampleAtmosphereTransmittance : SHADER_PERMUTATION_BOOL("CLOUD_PER_SAMPLE_ATMOSPHERE_TRANSMITTANCE");
	class FCloudSampleAtmosphericLightShadowmap : SHADER_PERMUTATION_BOOL("CLOUD_SAMPLE_ATMOSPHERIC_LIGHT_SHADOWMAP");
	class FCloudSampleSecondLight : SHADER_PERMUTATION_BOOL("CLOUD_SAMPLE_SECOND_LIGHT");
	class FCloudSampleLocalLights : SHADER_PERMUTATION_BOOL("CLOUD_SAMPLE_LOCAL_LIGHTS");
	class FCloudMinAndMaxDepth : SHADER_PERMUTATION_BOOL("CLOUD_MIN_AND_MAX_DEPTH");
	class FCloudLocalFogVolume : SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME");	// Only for the CS since platform not supporting the CS to run async likely won't render cloud
	class FCloudDebugViewMode : SHADER_PERMUTATION_BOOL("CLOUD_DEBUG_VIEW_MODE");					// Only the CS feature this because CS is typically used in editor.

	using FPermutationDomain = TShaderPermutationDomain<
		FCloudPerSampleAtmosphereTransmittance,
		FCloudSampleAtmosphericLightShadowmap,
		FCloudSampleSecondLight,
		FCloudSampleLocalLights,
		FCloudMinAndMaxDepth,
		FCloudLocalFogVolume,
		FCloudDebugViewMode>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, OutputViewRect)
		SHADER_PARAMETER(FVector2f, StartTracingDistanceTextureResolution)
		SHADER_PARAMETER(float, StartTracingSampleVolumeDepth)
		SHADER_PARAMETER(int32, bAccumulateAlphaHoldOut)
		SHADER_PARAMETER(int32, bBlendCloudColor)
		SHADER_PARAMETER(int32, TargetCubeFace)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRenderVolumetricCloudGlobalParameters, VolumetricCloudRenderViewParamsUB)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMap)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCloudColor0)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCloudColor1)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCloudDepth)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCloudColorCube)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCloudAlphaHoldout)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, StartTracingDistanceTexture)
#if CLOUD_DEBUG_SAMPLES
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
#endif
	END_SHADER_PARAMETER_STRUCT()

	static const int32 ThreadGroupSizeX = 8;
	static const int32 ThreadGroupSizeY = 8;

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const bool bIsCompatible = IsMaterialCompatibleWithVolumetricCloud(Parameters.MaterialParameters, Parameters.Platform);

		if (PermutationVector.Get<FCloudDebugViewMode>())
		{
			// Only compile visualization shaders for editor builds
			return bIsCompatible
				&& EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData)
				&& !PermutationVector.Get<FCloudPerSampleAtmosphereTransmittance>()
				&& !PermutationVector.Get<FCloudSampleAtmosphericLightShadowmap>()
				&& !PermutationVector.Get<FCloudSampleSecondLight>()
				&& !PermutationVector.Get<FCloudSampleLocalLights>();
		}

		return bIsCompatible;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		const FPermutationDomain PermutationVector(Parameters.PermutationId);

		OutEnvironment.SetDefine(TEXT("SHADER_RENDERVIEW_CS"), TEXT("1"));
		OutEnvironment.SetDefine(TEXT("CLOUD_LAYER_PIXEL_SHADER"), TEXT("1"));

		// Force texture fetches to not use automatic mip generation because the shader is compute and using a dynamic loop to evaluate the material multiple times.
		OutEnvironment.SetDefine(TEXT("USE_FORCE_TEXTURE_MIP"), TEXT("1"));

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), ThreadGroupSizeY);

		if (PermutationVector.Get<FCloudSampleAtmosphericLightShadowmap>())
		{
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), TEXT("1"));
		}

		// If this is enabled
		OutEnvironment.SetDefine(TEXT("CLOUD_DEBUG_SAMPLES"), CLOUD_DEBUG_SAMPLES);

		if (CVarVolumetricCloudAllowAnalyticDerivatives.GetValueOnAnyThread() != 0)
		{
			OutEnvironment.SetDefine(TEXT("ALLOW_ANALYTIC_DERIVATIVES"), 1);
		}

		// This shader must support typed UAV load and we are testing if it is supported at runtime using RHIIsTypedUAVLoadSupported
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);

		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);

		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FRenderVolumetricCloudRenderViewCS, TEXT("/Engine/Private/VolumetricCloud.usf"), TEXT("MainCS"), SF_Compute);

//////////////////////////////////////////////////////////////////////////

class FRenderVolumetricCloudEmptySpaceSkippingCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRenderVolumetricCloudEmptySpaceSkippingCS, MeshMaterial)
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FRenderVolumetricCloudEmptySpaceSkippingCS, FMeshMaterialShader)

	class FCloudEmptySpaceSkippingDebug : SHADER_PERMUTATION_BOOL("EMPTY_SPACE_SKIPPING_DEBUG");
	class FCloudEmptySpaceSkippingSampleCorners : SHADER_PERMUTATION_BOOL("EMPTY_SPACE_SKIPPING_SAMPLE_CORNERS");

	using FPermutationDomain = TShaderPermutationDomain<FCloudEmptySpaceSkippingDebug, FCloudEmptySpaceSkippingSampleCorners>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, StartTracingDistanceTextureResolution)
		SHADER_PARAMETER(float, StartTracingSampleVolumeDepth)
		SHADER_PARAMETER(float, StartTracingSliceBias)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRenderVolumetricCloudGlobalParameters, VolumetricCloudRenderViewParamsUB)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutStartTracingDistanceTexture)
//		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()

	static const int32 ThreadGroupSizeX = 1;
	static const int32 ThreadGroupSizeY = 1;
	static const int32 ThreadGroupSizeZ = 64;

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const bool bIsCompatible = IsMaterialCompatibleWithVolumetricCloud(Parameters.MaterialParameters, Parameters.Platform);

		if (PermutationVector.Get<FCloudEmptySpaceSkippingDebug>())
		{
			return false;	// TODO implement the debug version using ShaderPrint::FShaderParameters
		}

		return bIsCompatible;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		const FPermutationDomain PermutationVector(Parameters.PermutationId);

		OutEnvironment.SetDefine(TEXT("SHADER_EMPTY_SPACE_SKIPPING_CS"), TEXT("1"));
		OutEnvironment.SetDefine(TEXT("CLOUD_LAYER_PIXEL_SHADER"), TEXT("1"));

		// Force texture fetches to not use automatic mip generation because the shader is compute and using a dynamic loop to evaluate the material multiple times.
		OutEnvironment.SetDefine(TEXT("USE_FORCE_TEXTURE_MIP"), TEXT("1"));

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), ThreadGroupSizeY);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEZ"), ThreadGroupSizeZ);

		if (CVarVolumetricCloudAllowAnalyticDerivatives.GetValueOnAnyThread() != 0)
		{
			OutEnvironment.SetDefine(TEXT("ALLOW_ANALYTIC_DERIVATIVES"), 1);
		}

		// This shader must support typed UAV load and we are testing if it is supported at runtime using RHIIsTypedUAVLoadSupported
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
	}
};

static float GetEmptySpaceSkippingVolumeDepth()
{
	const float KilometersToCentimeters = 100000.0f;
	return FMath::Max(1.0f, CVarVolumetricCloudEmptySpaceSkippingVolumeDepth.GetValueOnRenderThread()) * KilometersToCentimeters;
}

static float GetEmptySpaceSkippingSliceDepth()
{
	const float KilometersToCentimeters = 100000.0f;
	float EmptySpaceSkippingVolumeDepth = GetEmptySpaceSkippingVolumeDepth();
	return EmptySpaceSkippingVolumeDepth * (1.0f / FRenderVolumetricCloudEmptySpaceSkippingCS::ThreadGroupSizeZ);
}

IMPLEMENT_MATERIAL_SHADER_TYPE(, FRenderVolumetricCloudEmptySpaceSkippingCS, TEXT("/Engine/Private/VolumetricCloud.usf"), TEXT("MainCS"), SF_Compute);

//////////////////////////////////////////////////////////////////////////

class FSingleTriangleMeshVertexBuffer : public FRenderResource
{
public:
	FStaticMeshVertexBuffers Buffers;

	FSingleTriangleMeshVertexBuffer()
	{
		TArray<FDynamicMeshVertex> Vertices;

		// Vertex position constructed in the shader
		Vertices.Add(FDynamicMeshVertex(FVector3f(-3.0f,  1.0f, 0.5f)));
		Vertices.Add(FDynamicMeshVertex(FVector3f( 1.0f, -3.0f, 0.5f)));
		Vertices.Add(FDynamicMeshVertex(FVector3f( 1.0f,  1.0f, 0.5f)));

		Buffers.PositionVertexBuffer.Init(Vertices.Num());
		Buffers.StaticMeshVertexBuffer.Init(Vertices.Num(), 1);

		for (int32 i = 0; i < Vertices.Num(); i++)
		{
			const FDynamicMeshVertex& Vertex = Vertices[i];

			Buffers.PositionVertexBuffer.VertexPosition(i) = Vertex.Position;
			Buffers.StaticMeshVertexBuffer.SetVertexTangents(i, Vertex.TangentX.ToFVector3f(), Vertex.GetTangentY(), Vertex.TangentZ.ToFVector3f());
			Buffers.StaticMeshVertexBuffer.SetVertexUV(i, 0, Vertex.TextureCoordinate[0]);
		}
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		Buffers.PositionVertexBuffer.InitResource(RHICmdList);
		Buffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
	}

	virtual void ReleaseRHI() override
	{
		Buffers.PositionVertexBuffer.ReleaseResource();
		Buffers.StaticMeshVertexBuffer.ReleaseResource();
	}
};

static TGlobalResource<FSingleTriangleMeshVertexBuffer> GSingleTriangleMeshVertexBuffer;

class FSingleTriangleMeshVertexFactory final : public FLocalVertexFactory
{
public:
	FSingleTriangleMeshVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FLocalVertexFactory(InFeatureLevel, "FSingleTriangleMeshVertexFactory")
	{}

	~FSingleTriangleMeshVertexFactory()
	{
		ReleaseResource();
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FSingleTriangleMeshVertexBuffer* VertexBuffer = &GSingleTriangleMeshVertexBuffer;
		FLocalVertexFactory::FDataType NewData;
		VertexBuffer->Buffers.PositionVertexBuffer.BindPositionVertexBuffer(this, NewData);
		VertexBuffer->Buffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(this, NewData);
		VertexBuffer->Buffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(this, NewData);
		VertexBuffer->Buffers.StaticMeshVertexBuffer.BindLightMapVertexBuffer(this, NewData, 0);
		FColorVertexBuffer::BindDefaultColorVertexBuffer(this, NewData, FColorVertexBuffer::NullBindStride::ZeroForDefaultBufferBind);
		// Don't call SetData(), because that ends up calling UpdateRHI(), and if the resource has already been initialized
		// (e.g. when switching the feature level in the editor), that calls InitRHI(), resulting in an infinite loop.
		Data = NewData;
		FLocalVertexFactory::InitRHI(RHICmdList);
	}

	bool HasIncompatibleFeatureLevel(ERHIFeatureLevel::Type InFeatureLevel)
	{
		return InFeatureLevel != GetFeatureLevel();
	}
};

static FSingleTriangleMeshVertexFactory* GSingleTriangleMeshVertexFactory = NULL;

static void GetSingleTriangleMeshBatch(FMeshBatch& LocalSingleTriangleMesh, const FMaterialRenderProxy* CloudVolumeMaterialProxy, const ERHIFeatureLevel::Type FeatureLevel)
{
	check(GSingleTriangleMeshVertexFactory && !GSingleTriangleMeshVertexFactory->HasIncompatibleFeatureLevel(FeatureLevel));

	LocalSingleTriangleMesh.VertexFactory = GSingleTriangleMeshVertexFactory;
	LocalSingleTriangleMesh.MaterialRenderProxy = CloudVolumeMaterialProxy;
	LocalSingleTriangleMesh.Elements[0].IndexBuffer = &GScreenRectangleIndexBuffer;
	LocalSingleTriangleMesh.Elements[0].FirstIndex = 0;
	LocalSingleTriangleMesh.Elements[0].NumPrimitives = 1;
	LocalSingleTriangleMesh.Elements[0].MinVertexIndex = 0;
	LocalSingleTriangleMesh.Elements[0].MaxVertexIndex = 2;

	LocalSingleTriangleMesh.Elements[0].PrimitiveUniformBufferResource = &GIdentityPrimitiveUniformBuffer;
	LocalSingleTriangleMesh.Elements[0].PrimitiveIdMode = PrimID_ForceZero;
}

//////////////////////////////////////////////////////////////////////////

class FVolumetricCloudRenderViewMeshProcessor : public FMeshPassProcessor
{
public:
	FVolumetricCloudRenderViewMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FViewInfo* InViewIfDynamicMeshCommand,
		bool bShouldViewRenderVolumetricRenderTarget, bool bSkipAtmosphericLightShadowmap, bool bSecondAtmosphereLightEnabled,
		bool bCloudDebugViewModeEnabledIn, FMeshPassDrawListContext* InDrawListContext)
		: FMeshPassProcessor(EMeshPass::Num, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
		, bVolumetricCloudPerSampleAtmosphereTransmittance(ShouldUsePerSampleAtmosphereTransmittance(Scene, InViewIfDynamicMeshCommand))
		, bVolumetricCloudSampleLightShadowmap(!bSkipAtmosphericLightShadowmap && CVarVolumetricCloudShadowSampleAtmosphericLightShadowmap.GetValueOnAnyThread() > 0)
		, bVolumetricCloudSecondLight(bSecondAtmosphereLightEnabled)
		, bCloudDebugViewModeEnabled(bCloudDebugViewModeEnabledIn)
	{
		PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

		if (bShouldViewRenderVolumetricRenderTarget || bCloudDebugViewModeEnabled)
		{
			// No blending as we only render clouds in that render target today. Avoids clearing for now.
			PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
		}
		else
		{
			// When volumetric render target is not enabled globally or for some views, e.g. reflection captures.
			PassDrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI());
		}
	}

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material)
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}

	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material)
	{
		if (Material.GetMaterialDomain() != MD_Volume)
		{
			// Skip in this case. This can happens when the material is compiled and a fallback is provided.
			return true;
		}

		// Determine the mesh's material and blend mode.
		const ERasterizerFillMode MeshFillMode = FM_Solid;
		const ERasterizerCullMode MeshCullMode = CM_None;

		if (bVolumetricCloudSecondLight)
		{
			if (bVolumetricCloudSampleLightShadowmap)
			{
				if (bVolumetricCloudPerSampleAtmosphereTransmittance)
				{
					return TemplatedProcess<FRenderVolumetricCloudRenderViewPs<CloudPs_AtmoTrans1_SampleShad1_2ndLight1>>(
						MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material, StaticMeshId, MeshFillMode, MeshCullMode);
				}
				else
				{
					return TemplatedProcess<FRenderVolumetricCloudRenderViewPs<CloudPs_AtmoTrans0_SampleShad1_2ndLight1>>(
						MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material, StaticMeshId, MeshFillMode, MeshCullMode);
				}
			}
			else
			{
				if (bVolumetricCloudPerSampleAtmosphereTransmittance)
				{
					return TemplatedProcess<FRenderVolumetricCloudRenderViewPs<CloudPs_AtmoTrans1_SampleShad0_2ndLight1>>(
						MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material, StaticMeshId, MeshFillMode, MeshCullMode);
				}
				else
				{
					return TemplatedProcess<FRenderVolumetricCloudRenderViewPs<CloudPs_AtmoTrans0_SampleShad0_2ndLight1>>(
						MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material, StaticMeshId, MeshFillMode, MeshCullMode);
				}
			}
		}
		else
		{
			if (bVolumetricCloudSampleLightShadowmap)
			{
				if (bVolumetricCloudPerSampleAtmosphereTransmittance)
				{
					return TemplatedProcess<FRenderVolumetricCloudRenderViewPs<CloudPs_AtmoTrans1_SampleShad1_2ndLight0>>(
						MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material, StaticMeshId, MeshFillMode, MeshCullMode);
				}
				else
				{
					return TemplatedProcess<FRenderVolumetricCloudRenderViewPs<CloudPs_AtmoTrans0_SampleShad1_2ndLight0>>(
						MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material, StaticMeshId, MeshFillMode, MeshCullMode);
				}
			}
			else
			{
				if (bVolumetricCloudPerSampleAtmosphereTransmittance)
				{
					return TemplatedProcess<FRenderVolumetricCloudRenderViewPs<CloudPs_AtmoTrans1_SampleShad0_2ndLight0>>(
						MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material, StaticMeshId, MeshFillMode, MeshCullMode);
				}
				else
				{
					return TemplatedProcess<FRenderVolumetricCloudRenderViewPs<CloudPs_AtmoTrans0_SampleShad0_2ndLight0>>(
						MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material, StaticMeshId, MeshFillMode, MeshCullMode);
				}
			}
		}
	}

private:

	template<class RenderVolumetricCloudRenderViewPsType>
	bool TemplatedProcess(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		int32 StaticMeshId,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode)
	{
		FMeshMaterialShaderElementData EmptyShaderElementData;
		EmptyShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

		const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<RenderVolumetricCloudRenderViewPsType>();
		ShaderTypes.AddShaderType<FRenderVolumetricCloudVS>();

		FMaterialShaders Shaders;
		if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
		{
			return false;
		}

		TMeshProcessorShaders< FRenderVolumetricCloudVS, RenderVolumetricCloudRenderViewPsType> PassShaders;
		Shaders.TryGetVertexShader(PassShaders.VertexShader);
		Shaders.TryGetPixelShader(PassShaders.PixelShader);

		const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);
		BuildMeshDrawCommands(
			MeshBatch,
			BatchElementMask,
			PrimitiveSceneProxy,
			MaterialRenderProxy,
			MaterialResource,
			PassDrawRenderState,
			PassShaders,
			MeshFillMode,
			MeshCullMode,
			SortKey,
			EMeshPassFeatures::Default,
			EmptyShaderElementData);

		return true;
	}

	FMeshPassProcessorRenderState PassDrawRenderState;
	bool bVolumetricCloudPerSampleAtmosphereTransmittance;
	bool bVolumetricCloudSampleLightShadowmap;
	bool bVolumetricCloudSecondLight;
	bool bCloudDebugViewModeEnabled;
};



//////////////////////////////////////////////////////////////////////////

BEGIN_SHADER_PARAMETER_STRUCT(FVolumetricCloudShadowParametersPS, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRenderVolumetricCloudGlobalParameters, TraceVolumetricCloudParamsUB)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FInstanceCullingGlobalUniforms, InstanceCulling)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FVolumetricCloudShadowPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FVolumetricCloudShadowPS, MeshMaterial);

public:

	FVolumetricCloudShadowPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}

	FVolumetricCloudShadowPS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		const bool bIsCompatible = IsMaterialCompatibleWithVolumetricCloud(Parameters.MaterialParameters, Parameters.Platform);
		return bIsCompatible;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_SHADOW_PS"), TEXT("1"));
		OutEnvironment.SetDefine(TEXT("CLOUD_LAYER_PIXEL_SHADER"), TEXT("1"));

		// Force texture fetches to not use automatic mip generation because the pixel shader is using a dynamic loop to evaluate the material multiple times.
		OutEnvironment.SetDefine(TEXT("USE_FORCE_TEXTURE_MIP"), TEXT("1"));

		// Make the Shadow Pass Switch node work for volumetric cloud shadow tracing.
		OutEnvironment.SetDefine(TEXT("SHADOW_DEPTH_SHADER"), TEXT("1"));
	}

private:
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FVolumetricCloudShadowPS, TEXT("/Engine/Private/VolumetricCloud.usf"), TEXT("MainPS"), SF_Pixel);

static const TCHAR* VolumetricCloudRenderShadowPassName = TEXT("VolumetricCloudRenderShadow");

class FVolumetricCloudRenderShadowMeshProcessor : public FMeshPassProcessor
{
public:
	FVolumetricCloudRenderShadowMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FViewInfo* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
		: FMeshPassProcessor(VolumetricCloudRenderShadowPassName, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	{
		PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
		PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
	}

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material)
			{
				check(Material->GetMaterialDomain() == MD_Volume);
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}

private:

	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material)
	{
		// Determine the mesh's material and blend mode.
		const ERasterizerFillMode MeshFillMode = FM_Solid;
		const ERasterizerCullMode MeshCullMode = CM_None;
		return Process(MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material, StaticMeshId, MeshFillMode, MeshCullMode);
	}

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		int32 StaticMeshId,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode)
	{
		FMeshMaterialShaderElementData EmptyShaderElementData;
		EmptyShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

		const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

		TMeshProcessorShaders<
			FRenderVolumetricCloudVS,
			FVolumetricCloudShadowPS> PassShaders;
		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FRenderVolumetricCloudVS>();
		ShaderTypes.AddShaderType<FVolumetricCloudShadowPS>();
		FMaterialShaders Shaders;
		if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
		{
			return false;
		}
		Shaders.TryGetVertexShader(PassShaders.VertexShader);
		Shaders.TryGetPixelShader(PassShaders.PixelShader);
		const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);
		BuildMeshDrawCommands(
			MeshBatch,
			BatchElementMask,
			PrimitiveSceneProxy,
			MaterialRenderProxy,
			MaterialResource,
			PassDrawRenderState,
			PassShaders,
			MeshFillMode,
			MeshCullMode,
			SortKey,
			EMeshPassFeatures::Default,
			EmptyShaderElementData);

		return true;
	}
	
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override
	{
		if (Material.GetMaterialDomain() != MD_Volume)
		{
			return;
		}

		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
		ERasterizerFillMode MeshFillMode = FM_Solid;
		ERasterizerCullMode MeshCullMode = CM_None;

		TMeshProcessorShaders<
			FRenderVolumetricCloudVS,
		FVolumetricCloudShadowPS> PassShaders;
		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FRenderVolumetricCloudVS>();
		ShaderTypes.AddShaderType<FVolumetricCloudShadowPS>();
				
		FMaterialShaders Shaders;
		if (!Material.TryGetShaders(ShaderTypes, VertexFactoryData.VertexFactoryType, Shaders))
		{
			return;
		}
		Shaders.TryGetVertexShader(PassShaders.VertexShader);
		Shaders.TryGetPixelShader(PassShaders.PixelShader);

		FRDGTextureDesc CloudSkyOARDGTextureDesc = GetVolumetricCloudSkyAORDGTextureDesc(FIntPoint(0));

		FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
		RenderTargetsInfo.NumSamples = 1;
		AddRenderTargetInfo(CloudSkyOARDGTextureDesc.Format, CloudSkyOARDGTextureDesc.Flags, RenderTargetsInfo);
		
		AddGraphicsPipelineStateInitializer(
			VertexFactoryData,
			Material,
			PassDrawRenderState,
			RenderTargetsInfo,
			PassShaders,
			MeshFillMode,
			MeshCullMode,
			(EPrimitiveType)PreCacheParams.PrimitiveType,
			EMeshPassFeatures::Default,
			true /*bRequired*/,
			PSOInitializers);
	}

	FMeshPassProcessorRenderState PassDrawRenderState;
};

IPSOCollector* CreatePSOCollectorVolumetricCloudRenderShadow(ERHIFeatureLevel::Type FeatureLevel)
{
	return new FVolumetricCloudRenderShadowMeshProcessor(nullptr, FeatureLevel, nullptr, nullptr);
}
FRegisterPSOCollectorCreateFunction RegisterPSOCollectorVolumetricCloudRenderShadow(&CreatePSOCollectorVolumetricCloudRenderShadow, EShadingPath::Deferred, VolumetricCloudRenderShadowPassName);


//////////////////////////////////////////////////////////////////////////

class FDrawDebugCloudShadowCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDrawDebugCloudShadowCS);
	SHADER_USE_PARAMETER_STRUCT(FDrawDebugCloudShadowCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CloudTracedTexture)
		SHADER_PARAMETER(FVector4f, CloudTextureSizeInvSize)
		SHADER_PARAMETER(FVector3f, CloudTraceDirection)
		SHADER_PARAMETER(FMatrix44f, CloudTranslatedWorldToLightClipMatrixInv)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsVolumetricCloudMaterialSupported(Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SHADER_DEBUG_SHADOW_CS"), TEXT("1"));
	}
};

IMPLEMENT_GLOBAL_SHADER(FDrawDebugCloudShadowCS, "/Engine/Private/VolumetricCloud.usf", "MainDrawDebugShadowCS", SF_Compute);



//////////////////////////////////////////////////////////////////////////

class FCloudShadowFilterCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCloudShadowFilterCS);
	SHADER_USE_PARAMETER_STRUCT(FCloudShadowFilterCS, FGlobalShader);

	class FFilterSkyAO : SHADER_PERMUTATION_BOOL("PERMUTATION_SKYAO");
	using FPermutationDomain = TShaderPermutationDomain<FFilterSkyAO>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SAMPLER(SamplerState, BilinearSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CloudShadowTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCloudShadowTexture)
		SHADER_PARAMETER(FVector4f, CloudTextureSizeInvSize)
		SHADER_PARAMETER(FVector4f, CloudTextureTexelWorldSizeInvSize)
		SHADER_PARAMETER(float, CloudLayerStartHeight)
		SHADER_PARAMETER(float, CloudSkyAOApertureScaleAdd)
		SHADER_PARAMETER(float, CloudSkyAOApertureScaleMul)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsVolumetricCloudMaterialSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_SHADOW_FILTER_CS"), TEXT("1"));
	}
};

IMPLEMENT_GLOBAL_SHADER(FCloudShadowFilterCS, "/Engine/Private/VolumetricCloud.usf", "MainShadowFilterCS", SF_Compute);



//////////////////////////////////////////////////////////////////////////

class FCloudShadowTemporalProcessCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCloudShadowTemporalProcessCS);
	SHADER_USE_PARAMETER_STRUCT(FCloudShadowTemporalProcessCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SAMPLER(SamplerState, BilinearSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CurrCloudShadowTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PrevCloudShadowTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCloudShadowTexture)
		SHADER_PARAMETER(FMatrix44f, CurrFrameCloudShadowmapTranslatedWorldToLightClipMatrixInv)
		SHADER_PARAMETER(FMatrix44f, PrevFrameCloudShadowmapTranslatedWorldToLightClipMatrix)
		SHADER_PARAMETER(FVector3f, CurrFrameLightPos)
		SHADER_PARAMETER(FVector3f, PrevFrameLightPos)
		SHADER_PARAMETER(FVector3f, CurrFrameLightDir)
		SHADER_PARAMETER(FVector3f, PrevFrameLightDir)
		SHADER_PARAMETER(FVector4f, CloudTextureSizeInvSize)
		SHADER_PARAMETER(FVector4f, CloudTextureTracingSizeInvSize)
		SHADER_PARAMETER(FVector4f, CloudTextureTracingPixelScaleOffset)
		SHADER_PARAMETER(float, TemporalFactor)
		SHADER_PARAMETER(uint32, PreviousDataIsValid)
		SHADER_PARAMETER(uint32, CloudShadowMapAnchorPointMoved)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsVolumetricCloudMaterialSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_SHADOW_TEMPORAL_PROCESS_CS"), TEXT("1"));
	}
};

IMPLEMENT_GLOBAL_SHADER(FCloudShadowTemporalProcessCS, "/Engine/Private/VolumetricCloud.usf", "MainShadowTemporalProcessCS", SF_Compute);



//////////////////////////////////////////////////////////////////////////



void CleanUpCloudDataFunction(TArray<FViewInfo>& Views)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& ViewInfo = Views[ViewIndex];
		ViewInfo.VolumetricCloudSkyAO = nullptr;
		for (int LightIndex = 0; LightIndex < NUM_ATMOSPHERE_LIGHTS; ++LightIndex)
		{
			ViewInfo.VolumetricCloudShadowRenderTarget[LightIndex] = nullptr;
			ViewInfo.VolumetricCloudShadowExtractedRenderTarget[LightIndex] = nullptr;
			if (ViewInfo.ViewState != nullptr)
			{
				ViewInfo.ViewState->VolumetricCloudShadowRenderTarget[LightIndex].Reset();
			}
		}
	}
};

void FSceneRenderer::InitVolumetricCloudsForViews(FRDGBuilder& GraphBuilder, bool bShouldRenderVolumetricCloud, FInstanceCullingManager& InstanceCullingManager)
{
	SCOPED_NAMED_EVENT(InitVolumetricCloudsForViews, FColor::Emerald);

	auto CleanUpCloudDataPass = [&Views = Views](FRDGBuilder& GraphBuilder)
	{
		GraphBuilder.AddPostExecuteCallback([&Views]
		{
			// Make sure we always clear the texture on views to make sure no dangling texture pointers are ever used
			CleanUpCloudDataFunction(Views);
		});
	};

	if (!bShouldRenderVolumetricCloud)
	{
		return;
	}

	if (!GSingleTriangleMeshVertexFactory || GSingleTriangleMeshVertexFactory->HasIncompatibleFeatureLevel(ViewFamily.GetFeatureLevel()))
	{
		if (GSingleTriangleMeshVertexFactory)
		{
			GSingleTriangleMeshVertexFactory->ReleaseResource();
			delete GSingleTriangleMeshVertexFactory;
		}
		GSingleTriangleMeshVertexFactory = new FSingleTriangleMeshVertexFactory(ViewFamily.GetFeatureLevel());
		GSingleTriangleMeshVertexBuffer.UpdateRHI(GraphBuilder.RHICmdList);
		GSingleTriangleMeshVertexFactory->InitResource(GraphBuilder.RHICmdList);
	}

	if (Scene)
	{
		FRDGExternalAccessQueue ExternalAccessQueue;

		check(ShouldRenderVolumetricCloud(Scene, ViewFamily.EngineShowFlags)); // This should not be called if we should not render SkyAtmosphere

		check(Scene->GetVolumetricCloudSceneInfo());
		const FSkyAtmosphereRenderSceneInfo* SkyInfo = Scene->GetSkyAtmosphereSceneInfo();
		FVolumetricCloudRenderSceneInfo& CloudInfo = *Scene->GetVolumetricCloudSceneInfo();
		const FVolumetricCloudSceneProxy& CloudProxy = CloudInfo.GetVolumetricCloudSceneProxy();
		FLightSceneProxy* AtmosphericLight0 = Scene->AtmosphereLights[0] ? Scene->AtmosphereLights[0]->Proxy : nullptr;
		FLightSceneProxy* AtmosphericLight1 = Scene->AtmosphereLights[1] ? Scene->AtmosphereLights[1]->Proxy : nullptr;
		FSkyLightSceneProxy* SkyLight = Scene->SkyLight;
		const float KilometersToCentimeters = 100000.0f;
		const float CentimetersToKilometers = 1.0f / KilometersToCentimeters;
		const float KilometersToMeters = 1000.0f;
		const float MetersToKilometers = 1.0f / KilometersToMeters;

		const float CloudShadowTemporalWeight = FMath::Min(FMath::Max(CVarVolumetricCloudShadowTemporalFilteringNewFrameWeight.GetValueOnRenderThread(), 0.0f), 1.0f);
		const bool CloudShadowTemporalEnabled = CloudShadowTemporalWeight < 1.0f;

		// Initialise the cloud common parameters
		{
			FVolumetricCloudCommonShaderParameters& CloudGlobalShaderParams = CloudInfo.GetVolumetricCloudCommonShaderParameters();
			float PlanetRadiusKm = CloudProxy.PlanetRadiusKm;
			if (SkyInfo)
			{
				const FAtmosphereSetup& AtmosphereSetup = SkyInfo->GetSkyAtmosphereSceneProxy().GetAtmosphereSetup();
				PlanetRadiusKm = AtmosphereSetup.BottomRadiusKm;
				CloudGlobalShaderParams.CloudLayerCenterKm = (FVector3f)AtmosphereSetup.PlanetCenterKm;	// LWC_TODO: Precision Loss
			}
			else
			{
				CloudGlobalShaderParams.CloudLayerCenterKm = FVector3f(0.0f, 0.0f, -PlanetRadiusKm);
			}
			CloudGlobalShaderParams.PlanetRadiusKm						= PlanetRadiusKm;
			CloudGlobalShaderParams.BottomRadiusKm						= PlanetRadiusKm + CloudProxy.LayerBottomAltitudeKm;
			CloudGlobalShaderParams.TopRadiusKm							= CloudGlobalShaderParams.BottomRadiusKm + CloudProxy.LayerHeightKm;
			CloudGlobalShaderParams.GroundAlbedo						= FLinearColor(CloudProxy.GroundAlbedo);
			CloudGlobalShaderParams.SkyLightCloudBottomVisibility		= 1.0f - CloudProxy.SkyLightCloudBottomOcclusion;

			CloudGlobalShaderParams.TracingStartMaxDistance				= FMath::Max(0.0f, KilometersToCentimeters * CloudProxy.TracingStartMaxDistance);
			CloudGlobalShaderParams.TracingStartDistanceFromCamera		= FMath::Max(0.0f, KilometersToCentimeters * CloudProxy.TracingStartDistanceFromCamera);
			CloudGlobalShaderParams.TracingMaxDistance					= FMath::Max(0.0f, KilometersToCentimeters * CloudProxy.TracingMaxDistance);
			CloudGlobalShaderParams.TracingMaxDistanceMode				= CloudProxy.TracingMaxDistanceMode;
			
			CloudGlobalShaderParams.SampleCountClamp					= (uint32) FMath::Max(0, CVarVolumetricCloudSampleClampCount.GetValueOnAnyThread());
			CloudGlobalShaderParams.SampleCountMin						= FMath::Max(0, CVarVolumetricCloudSampleMinCount.GetValueOnAnyThread());
			CloudGlobalShaderParams.SampleCountMax						= FMath::Max(2.0f, FMath::Min(UVolumetricCloudComponent::BaseViewRaySampleCount   * CloudProxy.ViewSampleCountScale,       CVarVolumetricCloudViewRaySampleMaxCount.GetValueOnAnyThread()));
			CloudGlobalShaderParams.ShadowSampleCountMax				= FMath::Max(2.0f, FMath::Min(UVolumetricCloudComponent::BaseShadowRaySampleCount * CloudProxy.ShadowViewSampleCountScale, CVarVolumetricCloudShadowViewRaySampleMaxCount.GetValueOnAnyThread()));
			CloudGlobalShaderParams.ShadowTracingMaxDistance			= KilometersToCentimeters * FMath::Max(0.1f, CloudProxy.ShadowTracingDistance);
			CloudGlobalShaderParams.InvDistanceToSampleCountMax			= 1.0f / FMath::Max(1.0f, KilometersToCentimeters * CVarVolumetricCloudDistanceToSampleMaxCount.GetValueOnAnyThread());
			CloudGlobalShaderParams.StopTracingTransmittanceThreshold	= FMath::Clamp(CloudProxy.StopTracingTransmittanceThreshold, 0.0f, 1.0f);

			auto PrepareCloudShadowMapLightData = [&](FLightSceneProxy* AtmosphericLight, int LightIndex)
			{
				const float CloudShadowmapResolution = float(GetVolumetricCloudShadowMapResolution(AtmosphericLight));
				const float CloudShadowmapResolutionInv = 1.0f / CloudShadowmapResolution;
				CloudGlobalShaderParams.CloudShadowmapSizeInvSize[LightIndex] = FVector4f(CloudShadowmapResolution, CloudShadowmapResolution, CloudShadowmapResolutionInv, CloudShadowmapResolutionInv);
				CloudGlobalShaderParams.CloudShadowmapStrength[LightIndex] = FVector4f(GetVolumetricCloudShadowmapStrength(AtmosphericLight), 0.0f, 0.0f, 0.0f);
				CloudGlobalShaderParams.CloudShadowmapDepthBias[LightIndex] = FVector4f(GetVolumetricCloudShadowmapDepthBias(AtmosphericLight), 0.0f, 0.0f, 0.0f);
				CloudGlobalShaderParams.AtmosphericLightCloudScatteredLuminanceScale[LightIndex] = GetVolumetricCloudScatteredLuminanceScale(AtmosphericLight);

				if (CloudShadowTemporalEnabled)
				{
					const float CloudShadowmapHalfResolution = CloudShadowmapResolution * 0.5f;
					const float CloudShadowmapHalfResolutionInv = 1.0f / CloudShadowmapHalfResolution;
					CloudGlobalShaderParams.CloudShadowmapTracingSizeInvSize[LightIndex] = FVector4f(CloudShadowmapHalfResolution, CloudShadowmapHalfResolution, CloudShadowmapHalfResolutionInv, CloudShadowmapHalfResolutionInv);

					uint32 Status = 0;
					if (Views.Num() > 0 && Views[0].ViewState)
					{
						Status = Views[0].ViewState->GetFrameIndex() % 4;
					}
					const float PixelOffsetX = Status == 1 || Status == 2 ? 1.0f : 0.0f;
					const float PixelOffsetY = Status == 1 || Status == 3 ? 1.0f : 0.0f;

					const float HalfResolutionFactor = 2.0f;
					CloudGlobalShaderParams.CloudShadowmapTracingPixelScaleOffset[LightIndex] = FVector4f(HalfResolutionFactor, HalfResolutionFactor, PixelOffsetX, PixelOffsetY);
				}
				else
				{
					CloudGlobalShaderParams.CloudShadowmapTracingSizeInvSize[LightIndex] = CloudGlobalShaderParams.CloudShadowmapSizeInvSize[LightIndex];
					CloudGlobalShaderParams.CloudShadowmapTracingPixelScaleOffset[LightIndex] = FVector4f(1.0f, 1.0f, 0.0f, 0.0f);
				}

				// Setup cloud shadow constants
				if (AtmosphericLight)
				{
					const FVector3f AtmopshericLightDirection = (FVector3f)AtmosphericLight->GetDirection();
					const FVector3f UpVector = FMath::Abs(FVector3f::DotProduct(AtmopshericLightDirection, FVector3f::UpVector)) > 0.99f ? FVector3f::ForwardVector : FVector3f::UpVector;

					const float SphereRadius = GetVolumetricCloudShadowMapExtentKm(AtmosphericLight) * KilometersToCentimeters;
					const float LightDistanceOverride = CVarVolumetricCloudShadowMapLightDistanceOverride.GetValueOnRenderThread();
					const float LightDistance = LightDistanceOverride > 0.0f ? (LightDistanceOverride * KilometersToCentimeters) : SphereRadius * 2.0f;
					const float SphereDiameter = SphereRadius * 2.0f;
					const float NearPlane = 0.0f;
					const float FarPlane = LightDistance * 2.0f;
					const float ZScale = 1.0f / (FarPlane - NearPlane);
					const float ZOffset = -NearPlane;
					FMatrix TranslatedWorldToWorld = FMatrix::Identity;

					// TODO Make it work for all views
					FVector3f LookAtPosition = FVector3f::ZeroVector;
					FVector3f PlanetToCameraNormUp = FVector3f::UpVector;
					if (Views.Num() > 0)
					{
						FViewInfo& View = Views[0];

						// Look at position is positioned on the planet surface under the camera.
						LookAtPosition = ((FVector3f)View.ViewMatrices.GetViewOrigin() - (CloudGlobalShaderParams.CloudLayerCenterKm * KilometersToCentimeters));
						LookAtPosition.Normalize();
						PlanetToCameraNormUp = LookAtPosition;
						LookAtPosition = (CloudGlobalShaderParams.CloudLayerCenterKm + LookAtPosition * PlanetRadiusKm) * KilometersToCentimeters;
						// Light position is positioned away from the look at position in the light direction according to the shadowmap radius.
						const FVector3f LightPosition = LookAtPosition - AtmopshericLightDirection * LightDistance;

						float WorldSizeSnap = CVarVolumetricCloudShadowMapSnapLength.GetValueOnAnyThread() * KilometersToCentimeters;
						LookAtPosition.X = (FMath::FloorToFloat((LookAtPosition.X + 0.5f * WorldSizeSnap) / WorldSizeSnap)) * WorldSizeSnap; // offset by 0.5 to not snap around origin
						LookAtPosition.Y = (FMath::FloorToFloat((LookAtPosition.Y + 0.5f * WorldSizeSnap) / WorldSizeSnap)) * WorldSizeSnap;
						LookAtPosition.Z = (FMath::FloorToFloat((LookAtPosition.Z + 0.5f * WorldSizeSnap) / WorldSizeSnap)) * WorldSizeSnap;

						TranslatedWorldToWorld = FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation());
					}

					const FVector3f LightPosition = LookAtPosition - AtmopshericLightDirection * LightDistance;
					FReversedZOrthoMatrix ShadowProjectionMatrix(SphereDiameter, SphereDiameter, ZScale, ZOffset);
					FLookAtMatrix ShadowViewMatrix((FVector)LightPosition, (FVector)LookAtPosition, (FVector)UpVector);
					FMatrix44f ShadowViewProjectionMatrix = FMatrix44f( ShadowViewMatrix * ShadowProjectionMatrix );
					if (CVarVolumetricCloudShadowMapSnapToPixelGrid.GetValueOnRenderThread() > 0)
					{
						// Make a point in the world (here the origin) always snap to pixel grid exactly, to prevent shimmering.
						FVector4f WorldOrigin = FVector4f::Zero();
						WorldOrigin.W = 1.0f;
						FVector4f WorldOriginInClipSpace = ShadowViewProjectionMatrix.TransformPosition(WorldOrigin);

						// Snap to align with pixel grid in clip (-1 to 1) space.
						const auto SnapComponentToPixelGrid = [](float Component, float Resolution) -> float
						{
							float HalfResolution = Resolution * 0.5f;
							HalfResolution *= 0.5f; // needed to avoid offsetting at half-pixel increments (perhaps RT isn't traced at 1:1 ratio?)
							return FMath::RoundToFloat(Component * HalfResolution) / HalfResolution;
						};

						FVector4f SnappedWorldOriginInClipSpace = FVector4f::Zero();
						SnappedWorldOriginInClipSpace.X = SnapComponentToPixelGrid(WorldOriginInClipSpace.X, CloudShadowmapResolution);
						SnappedWorldOriginInClipSpace.Y = SnapComponentToPixelGrid(WorldOriginInClipSpace.Y, CloudShadowmapResolution);
						SnappedWorldOriginInClipSpace.Z = WorldOriginInClipSpace.Z;
						SnappedWorldOriginInClipSpace.W = 1.0f;

						FVector4f SnapOffset = SnappedWorldOriginInClipSpace - WorldOriginInClipSpace;
						FTranslationMatrix44f SnapToPixelGridMatrix(SnapOffset);

						// Apply the snap offset
						ShadowViewProjectionMatrix = ShadowViewProjectionMatrix * SnapToPixelGridMatrix;
					}
					CloudGlobalShaderParams.CloudShadowmapTranslatedWorldToLightClipMatrix[LightIndex] = FMatrix44f((FMatrix44f)TranslatedWorldToWorld * ShadowViewProjectionMatrix);

					CloudGlobalShaderParams.CloudShadowmapTranslatedWorldToLightClipMatrixInv[LightIndex] = CloudGlobalShaderParams.CloudShadowmapTranslatedWorldToLightClipMatrix[LightIndex].Inverse();
					CloudGlobalShaderParams.CloudShadowmapLightDir[LightIndex] = AtmopshericLightDirection;
					CloudGlobalShaderParams.CloudShadowmapLightPos[LightIndex] = LightPosition;
					CloudGlobalShaderParams.CloudShadowmapLightAnchorPos[LightIndex] = LookAtPosition;
					CloudGlobalShaderParams.CloudShadowmapFarDepthKm[LightIndex] = FVector4f(FarPlane * CentimetersToKilometers, 0.0f, 0.0f, 0.0f);

					// More samples when the sun is at the horizon: a lot more distance to travel and less pixel covered so trying to keep the same cost and quality.
					const float CloudShadowRaySampleCountScale = GetVolumetricCloudShadowRaySampleCountScale(AtmosphericLight);
					const float CloudShadowRaySampleBaseCount = 16.0f;
					const float CloudShadowRayMapSampleCount = FMath::Max(4.0f, FMath::Min(CloudShadowRaySampleBaseCount * CloudShadowRaySampleCountScale, CVarVolumetricCloudShadowMapRaySampleMaxCount.GetValueOnAnyThread()));
					const float RaySampleHorizonFactor = FMath::Max(0.0f, CVarVolumetricCloudShadowMapRaySampleHorizonMultiplier.GetValueOnAnyThread()-1.0f);
					const float HorizonFactor = FMath::Clamp(0.2f / FMath::Abs(FVector3f::DotProduct(PlanetToCameraNormUp, -AtmopshericLightDirection)), 0.0f, 1.0f);
					CloudGlobalShaderParams.CloudShadowmapSampleCount[LightIndex] = FVector4f(CloudShadowRayMapSampleCount + RaySampleHorizonFactor * CloudShadowRayMapSampleCount * HorizonFactor, 0.0f, 0.0f, 0.0f);
				}
				else
				{
					const FVector4f UpVector = FVector4f(0.0f, 0.0f, 1.0f, 0.0f);
					const FVector4f ZeroVector = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
					const FVector4f OneVector = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
					CloudGlobalShaderParams.CloudShadowmapTranslatedWorldToLightClipMatrix[LightIndex] = FMatrix44f::Identity;
					CloudGlobalShaderParams.CloudShadowmapTranslatedWorldToLightClipMatrixInv[LightIndex] = FMatrix44f::Identity;
					CloudGlobalShaderParams.CloudShadowmapLightDir[LightIndex] = UpVector;
					CloudGlobalShaderParams.CloudShadowmapLightPos[LightIndex] = ZeroVector;
					CloudGlobalShaderParams.CloudShadowmapLightAnchorPos[LightIndex] = ZeroVector;
					CloudGlobalShaderParams.CloudShadowmapFarDepthKm[LightIndex] = OneVector;
					CloudGlobalShaderParams.CloudShadowmapSampleCount[LightIndex] = ZeroVector;
				}
			};
			PrepareCloudShadowMapLightData(AtmosphericLight0, 0);
			PrepareCloudShadowMapLightData(AtmosphericLight1, 1);

			// Setup cloud SkyAO constants
			{
				const float CloudSkyAOResolution = float(GetVolumetricCloudSkyAOResolution(SkyLight));
				const float CloudSkyAOResolutionInv = 1.0f / CloudSkyAOResolution;
				CloudGlobalShaderParams.CloudSkyAOSizeInvSize = FVector4f(CloudSkyAOResolution, CloudSkyAOResolution, CloudSkyAOResolutionInv, CloudSkyAOResolutionInv);
				CloudGlobalShaderParams.CloudSkyAOStrength = GetVolumetricCloudSkyAOStrength(SkyLight);

				const float WorldSizeSnap = CVarVolumetricCloudSkyAOSnapLength.GetValueOnAnyThread() * KilometersToCentimeters;
				const float SphereDiameter = GetVolumetricCloudSkyAOExtentKm(SkyLight) * KilometersToCentimeters * 2.0f;
				const float VolumeDepthRange = (CloudProxy.LayerBottomAltitudeKm + CloudProxy.LayerHeightKm) * KilometersToCentimeters + WorldSizeSnap;
				const float NearPlane = 0.0f;
				const float FarPlane = 2.0f * VolumeDepthRange;
				const float ZScale = 1.0f / (FarPlane - NearPlane);
				const float ZOffset = -NearPlane;
				FMatrix TranslatedWorldToWorld = FMatrix::Identity;

				// TODO Make it work for all views
				FVector3f LookAtPosition = FVector3f::ZeroVector;
				if (Views.Num() > 0)
				{
					FViewInfo& View = Views[0];

					// Look at position is positioned on the planet surface under the camera.
					LookAtPosition = ((FVector3f)View.ViewMatrices.GetViewOrigin() - (CloudGlobalShaderParams.CloudLayerCenterKm * KilometersToCentimeters));
					LookAtPosition.Normalize();
					LookAtPosition = (CloudGlobalShaderParams.CloudLayerCenterKm + LookAtPosition * PlanetRadiusKm) * KilometersToCentimeters;

					// Snap the texture projection
					LookAtPosition.X = (FMath::FloorToFloat((LookAtPosition.X + 0.5f * WorldSizeSnap) / WorldSizeSnap)) * WorldSizeSnap; // offset by 0.5 to not snap around origin
					LookAtPosition.Y = (FMath::FloorToFloat((LookAtPosition.Y + 0.5f * WorldSizeSnap) / WorldSizeSnap)) * WorldSizeSnap;
					LookAtPosition.Z = (FMath::FloorToFloat((LookAtPosition.Z + 0.5f * WorldSizeSnap) / WorldSizeSnap)) * WorldSizeSnap;

					TranslatedWorldToWorld = FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation());
				}

				// Trace direction is towards the ground
				FVector3f TraceDirection =  CloudGlobalShaderParams.CloudLayerCenterKm * KilometersToCentimeters - LookAtPosition;
				TraceDirection.Normalize();

				const FVector3f UpVector = FMath::Abs(FVector::DotProduct((FVector)TraceDirection, FVector::UpVector)) > 0.99f ? FVector3f::ForwardVector : FVector3f::UpVector;
				const FVector3f LightPosition = LookAtPosition - TraceDirection * VolumeDepthRange;
				FReversedZOrthoMatrix ShadowProjectionMatrix(SphereDiameter, SphereDiameter, ZScale, ZOffset);
				FLookAtMatrix ShadowViewMatrix((FVector)LightPosition, (FVector)LookAtPosition, (FVector)UpVector);
				CloudGlobalShaderParams.CloudSkyAOTranslatedWorldToLightClipMatrix = FMatrix44f((TranslatedWorldToWorld * ShadowViewMatrix) * ShadowProjectionMatrix);
				CloudGlobalShaderParams.CloudSkyAOTranslatedWorldToLightClipMatrixInv = CloudGlobalShaderParams.CloudSkyAOTranslatedWorldToLightClipMatrix.Inverse();
				CloudGlobalShaderParams.CloudSkyAOTraceDir = TraceDirection;
				CloudGlobalShaderParams.CloudSkyAOFarDepthKm = FarPlane * CentimetersToKilometers;

				// More samples when the sun is at the horizon: a lot more distance to travel and less pixel covered so trying to keep the same cost and quality.
				CloudGlobalShaderParams.CloudSkyAOSampleCount = CVarVolumetricCloudSkyAOTraceSampleCount.GetValueOnAnyThread();
			}

			FVolumetricCloudCommonGlobalShaderParameters CloudGlobalShaderParamsUB;
			CloudGlobalShaderParamsUB.VolumetricCloudCommonParams = CloudGlobalShaderParams;
			CloudInfo.GetVolumetricCloudCommonShaderParametersUB() = TUniformBufferRef<FVolumetricCloudCommonGlobalShaderParameters>::CreateUniformBufferImmediate(CloudGlobalShaderParamsUB, UniformBuffer_SingleFrame);
		}

		if (CloudProxy.GetCloudVolumeMaterial())
		{
			FMaterialRenderProxy* CloudVolumeMaterialProxy = CloudProxy.GetCloudVolumeMaterial()->GetRenderProxy();
			if (CloudVolumeMaterialProxy->GetIncompleteMaterialWithFallback(ViewFamily.GetFeatureLevel()).GetMaterialDomain() == MD_Volume && !ViewFamily.EngineShowFlags.PathTracing)
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, VolumetricCloudShadow, "VolumetricCloudShadow");

				TRefCountPtr<IPooledRenderTarget> BlackDummy = GSystemTextures.BlackDummy;
				FRDGTextureRef BlackDummyRDG = GraphBuilder.RegisterExternalTexture(BlackDummy);

				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

					FViewInfo& ViewInfo = Views[ViewIndex];
					FVector3f ViewOrigin(ViewInfo.ViewMatrices.GetViewOrigin());

					FVolumeShadowingShaderParametersGlobal0 LightShadowShaderParams0;
					SetVolumeShadowingDefaultShaderParameters(GraphBuilder, LightShadowShaderParams0);

					FRenderVolumetricCloudGlobalParameters& VolumetricCloudParams = *GraphBuilder.AllocParameters<FRenderVolumetricCloudGlobalParameters>();
					VolumetricCloudParams.Light0Shadow = LightShadowShaderParams0;
					SetupDefaultRenderVolumetricCloudGlobalParameters(GraphBuilder, VolumetricCloudParams, CloudInfo, ViewInfo);
					
					auto TraceCloudTexture = [&](FRDGTextureRef CloudTextureTracedOutput, bool bSkyAOPass,
						TRDGUniformBufferRef<FRenderVolumetricCloudGlobalParameters> TraceVolumetricCloudParamsUB)
					{
						FVolumetricCloudShadowParametersPS* CloudShadowParameters = GraphBuilder.AllocParameters<FVolumetricCloudShadowParametersPS>();
						CloudShadowParameters->View = ViewInfo.GetShaderParameters();
						CloudShadowParameters->Scene = GetSceneUniformBufferRef(GraphBuilder);
						CloudShadowParameters->TraceVolumetricCloudParamsUB = TraceVolumetricCloudParamsUB;
						CloudShadowParameters->InstanceCulling = InstanceCullingManager.GetDummyInstanceCullingUniformBuffer();
						CloudShadowParameters->RenderTargets[0] = FRenderTargetBinding(CloudTextureTracedOutput, ERenderTargetLoadAction::ENoAction);

						GraphBuilder.AddPass(
							bSkyAOPass ? RDG_EVENT_NAME("CloudSkyAO") : RDG_EVENT_NAME("CloudShadow"),
							CloudShadowParameters,
							ERDGPassFlags::Raster,
							[CloudShadowParameters, &ViewInfo, CloudVolumeMaterialProxy](FRHICommandListImmediate& RHICmdList)
							{
								DrawDynamicMeshPass(ViewInfo, RHICmdList,
									[&ViewInfo, CloudVolumeMaterialProxy](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
									{
										FVolumetricCloudRenderShadowMeshProcessor PassMeshProcessor(
											ViewInfo.Family->Scene->GetRenderScene(), ViewInfo.GetFeatureLevel(), &ViewInfo,
											DynamicMeshPassContext);

										FMeshBatch LocalSingleTriangleMesh;
										GetSingleTriangleMeshBatch(LocalSingleTriangleMesh, CloudVolumeMaterialProxy, ViewInfo.GetFeatureLevel());

										const FPrimitiveSceneProxy* PrimitiveSceneProxy = nullptr;
										const uint64 DefaultBatchElementMask = ~0ull;
										PassMeshProcessor.AddMeshBatch(LocalSingleTriangleMesh, DefaultBatchElementMask, PrimitiveSceneProxy);
									});
							});
					};

					const float CloudLayerStartHeight = CloudProxy.LayerBottomAltitudeKm * KilometersToCentimeters;

					auto FilterTracedCloudTexture = [&](FRDGTextureRef* TracedCloudTextureOutput, FVector4f TracedTextureSizeInvSize, FVector4f CloudAOTextureTexelWorldSizeInvSize, bool bSkyAOPass)
					{
						const float DownscaleFactor = bSkyAOPass ? 1.0f : 0.5f; // No downscale for AO
						const FIntPoint DownscaledResolution = FIntPoint((*TracedCloudTextureOutput)->Desc.Extent.X * DownscaleFactor, (*TracedCloudTextureOutput)->Desc.Extent.Y * DownscaleFactor);
						FRDGTextureRef CloudShadowTexture2 = GraphBuilder.CreateTexture(
							FRDGTextureDesc::Create2D(DownscaledResolution, PF_FloatR11G11B10,
								FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV), bSkyAOPass ? TEXT("Cloud.SkyAOTexture2") : TEXT("Cloud.ShadowTexture2"));

						FCloudShadowFilterCS::FPermutationDomain Permutation;
						Permutation.Set<FCloudShadowFilterCS::FFilterSkyAO>(bSkyAOPass);
						TShaderMapRef<FCloudShadowFilterCS> ComputeShader(GetGlobalShaderMap(ViewInfo.GetFeatureLevel()), Permutation);

						FCloudShadowFilterCS::FParameters* Parameters = GraphBuilder.AllocParameters<FCloudShadowFilterCS::FParameters>();
						Parameters->BilinearSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
						Parameters->CloudShadowTexture = *TracedCloudTextureOutput;
						Parameters->CloudTextureSizeInvSize = TracedTextureSizeInvSize;
						Parameters->CloudTextureTexelWorldSizeInvSize = CloudAOTextureTexelWorldSizeInvSize;
						Parameters->CloudLayerStartHeight = CloudLayerStartHeight;
						Parameters->CloudSkyAOApertureScaleMul = GetVolumetricCloudSkyAOApertureScale(SkyLight);
						Parameters->CloudSkyAOApertureScaleAdd = 1.0f - Parameters->CloudSkyAOApertureScaleMul;
						Parameters->OutCloudShadowTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CloudShadowTexture2));

						const FIntVector DispatchCount = DispatchCount.DivideAndRoundUp(FIntVector(DownscaledResolution.X, DownscaledResolution.Y, 1), FIntVector(8, 8, 1));
						FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("CloudDataSpatialFilter"), ComputeShader, Parameters, DispatchCount);

						*TracedCloudTextureOutput = CloudShadowTexture2;
					};

					// Render Cloud SKY AO
					if (ShouldRenderCloudSkyAO(SkyLight))
					{
						const uint32 VolumetricCloudSkyAOResolution = GetVolumetricCloudSkyAOResolution(SkyLight);
						const FRDGTextureDesc CloudSkyAOTextureDesc = GetVolumetricCloudSkyAORDGTextureDesc(FIntPoint(VolumetricCloudSkyAOResolution,VolumetricCloudSkyAOResolution));
						FRDGTextureRef CloudSkyAOTexture = GraphBuilder.CreateTexture(CloudSkyAOTextureDesc, TEXT("Cloud.SkyAOTexture"));

						// We need to make a copy of the parameters on CPU to morph them because the creation is deferred.
						FRenderVolumetricCloudGlobalParameters& VolumetricCloudParamsAO = *GraphBuilder.AllocParameters<FRenderVolumetricCloudGlobalParameters>();
						VolumetricCloudParamsAO = VolumetricCloudParams;	// Use the same parameter as for the directional light shadow
						VolumetricCloudParamsAO.TraceShadowmapMode = 0;		// Notify that this pass is for SkyAO (avoid to use another shader permutation)
						TRDGUniformBufferRef<FRenderVolumetricCloudGlobalParameters> TraceVolumetricCloudSkyAOParamsUB = GraphBuilder.CreateUniformBuffer(&VolumetricCloudParamsAO);
						TraceCloudTexture(CloudSkyAOTexture, true, TraceVolumetricCloudSkyAOParamsUB);

						if (CVarVolumetricCloudSkyAOFiltering.GetValueOnAnyThread() > 0)
						{
							const float CloudAOTextureTexelWorldSize = GetVolumetricCloudSkyAOExtentKm(SkyLight) * KilometersToCentimeters * VolumetricCloudParamsAO.VolumetricCloud.CloudSkyAOSizeInvSize.Z;
							const FVector4f CloudAOTextureTexelWorldSizeInvSize = FVector4f(CloudAOTextureTexelWorldSize, CloudAOTextureTexelWorldSize, 1.0f / CloudAOTextureTexelWorldSize, 1.0f / CloudAOTextureTexelWorldSize);

							FilterTracedCloudTexture(&CloudSkyAOTexture, VolumetricCloudParamsAO.VolumetricCloud.CloudSkyAOSizeInvSize, CloudAOTextureTexelWorldSizeInvSize, true);
						}

						ViewInfo.VolumetricCloudSkyAO = CloudSkyAOTexture;
					}


					// Render atmospheric lights shadow maps
					auto GenerateCloudTexture = [&](FLightSceneProxy* AtmosphericLight, int LightIndex)
					{
						FVolumetricCloudCommonShaderParameters& CloudGlobalShaderParams = CloudInfo.GetVolumetricCloudCommonShaderParameters();
						if (ShouldRenderCloudShadowmap(AtmosphericLight))
						{
							const int32 CloudShadowSpatialFiltering = FMath::Clamp(CVarVolumetricCloudShadowSpatialFiltering.GetValueOnAnyThread(), 0, 4);
							const uint32 Resolution1D = GetVolumetricCloudShadowMapResolution(AtmosphericLight);
							const FIntPoint Resolution2D = FIntPoint(Resolution1D, Resolution1D);
							const FIntPoint TracingResolution2D = CloudShadowTemporalEnabled ? FIntPoint(Resolution1D>>1, Resolution1D>>1) : FIntPoint(Resolution1D, Resolution1D);
							const EPixelFormat CloudShadowPixelFormat = PF_FloatR11G11B10;

							if (ViewInfo.ViewState)
							{
								FTemporalRenderTargetState& CloudShadowTemporalRT = ViewInfo.ViewState->VolumetricCloudShadowRenderTarget[LightIndex];
								CloudShadowTemporalRT.Initialise(Resolution2D, CloudShadowPixelFormat);
							}

							FRDGTextureRef NewCloudShadowTexture = GraphBuilder.CreateTexture(
								FRDGTextureDesc::Create2D(TracingResolution2D, CloudShadowPixelFormat,
									FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable), LightIndex == 0 ? TEXT("Cloud.ShadowTexture0") : TEXT("Cloud.ShadowTexture1"));

							FRenderVolumetricCloudGlobalParameters& VolumetricCloudParamsShadow = *GraphBuilder.AllocParameters<FRenderVolumetricCloudGlobalParameters>();
							VolumetricCloudParamsShadow = VolumetricCloudParams;			// Use the same parameters generated above for the directional light shadow
							VolumetricCloudParamsShadow.TraceShadowmapMode = 1 + LightIndex;// But change the shadow mode
							TRDGUniformBufferRef<FRenderVolumetricCloudGlobalParameters> TraceVolumetricCloudShadowParamsUB = GraphBuilder.CreateUniformBuffer(&VolumetricCloudParamsShadow);
							TraceCloudTexture(NewCloudShadowTexture, false, TraceVolumetricCloudShadowParamsUB);

							// Directional light shadow temporal filter only if the view has a ViewState (not a sky light capture view for instance)
							if(CloudShadowTemporalEnabled && ViewInfo.ViewState)
							{
								const float LightRotationCutCosAngle = FMath::Cos(FMath::DegreesToRadians(CVarVolumetricCloudShadowTemporalFilteringLightRotationCutHistory.GetValueOnAnyThread()));

								FCloudShadowTemporalProcessCS::FPermutationDomain Permutation;
								TShaderMapRef<FCloudShadowTemporalProcessCS> ComputeShader(GetGlobalShaderMap(ViewInfo.GetFeatureLevel()), Permutation);

								FTemporalRenderTargetState& CloudShadowTemporalRT = ViewInfo.ViewState->VolumetricCloudShadowRenderTarget[LightIndex];
								FRDGTextureRef CurrentShadowTexture = CloudShadowTemporalRT.GetOrCreateCurrentRT(GraphBuilder);

								const bool bLightRotationCutHistory = FVector3f::DotProduct((FVector3f)ViewInfo.ViewState->VolumetricCloudShadowmapPreviousAtmosphericLightDir[LightIndex], (FVector3f)AtmosphericLight->GetDirection()) < LightRotationCutCosAngle;
								const bool bHistoryValid = CloudShadowTemporalRT.GetHistoryValid() && !bLightRotationCutHistory;

								FCloudShadowTemporalProcessCS::FParameters* Parameters = GraphBuilder.AllocParameters<FCloudShadowTemporalProcessCS::FParameters>();
								Parameters->BilinearSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
								Parameters->CurrCloudShadowTexture = NewCloudShadowTexture;
								Parameters->PrevCloudShadowTexture = bHistoryValid ? CloudShadowTemporalRT.GetOrCreatePreviousRT(GraphBuilder) : BlackDummyRDG;
								Parameters->OutCloudShadowTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(CurrentShadowTexture));

								if (bHistoryValid)
								{
									Parameters->PrevCloudShadowTexture = CloudShadowTemporalRT.GetOrCreatePreviousRT(GraphBuilder);
									Parameters->CurrFrameCloudShadowmapTranslatedWorldToLightClipMatrixInv = CloudGlobalShaderParams.CloudShadowmapTranslatedWorldToLightClipMatrixInv[LightIndex];
									Parameters->PrevFrameCloudShadowmapTranslatedWorldToLightClipMatrix = FMatrix44f(ViewInfo.ViewState->VolumetricCloudShadowmapPreviousTranslatedWorldToLightClipMatrix[LightIndex]);
								}
								else
								{
									Parameters->PrevCloudShadowTexture = BlackDummyRDG;
									Parameters->CurrFrameCloudShadowmapTranslatedWorldToLightClipMatrixInv = FMatrix44f::Identity;
									Parameters->PrevFrameCloudShadowmapTranslatedWorldToLightClipMatrix = FMatrix44f::Identity;
								}

								Parameters->CurrFrameLightPos = CloudGlobalShaderParams.CloudShadowmapLightPos[LightIndex];
								Parameters->CurrFrameLightDir = CloudGlobalShaderParams.CloudShadowmapLightDir[LightIndex];

								Parameters->PrevFrameLightPos = (FVector3f)ViewInfo.ViewState->VolumetricCloudShadowmapPreviousAtmosphericLightPos[LightIndex];
								Parameters->PrevFrameLightDir = (FVector3f)ViewInfo.ViewState->VolumetricCloudShadowmapPreviousAtmosphericLightDir[LightIndex];
								Parameters->CloudShadowMapAnchorPointMoved = (ViewInfo.ViewState->VolumetricCloudShadowmapPreviousAnchorPoint[LightIndex] - FVector4(CloudGlobalShaderParams.CloudShadowmapLightAnchorPos[LightIndex])).SizeSquared() < KINDA_SMALL_NUMBER ? 0 : 1;

								Parameters->CloudTextureSizeInvSize = VolumetricCloudParams.VolumetricCloud.CloudShadowmapSizeInvSize[LightIndex];
								Parameters->CloudTextureTracingSizeInvSize = VolumetricCloudParams.VolumetricCloud.CloudShadowmapTracingSizeInvSize[LightIndex];
								Parameters->CloudTextureTracingPixelScaleOffset = VolumetricCloudParams.VolumetricCloud.CloudShadowmapTracingPixelScaleOffset[LightIndex];
								Parameters->PreviousDataIsValid = bHistoryValid ? 1 : 0;
								Parameters->TemporalFactor = CloudShadowTemporalWeight;

								const FIntVector CloudShadowTextureSize = FIntVector(Parameters->CloudTextureSizeInvSize.X, Parameters->CloudTextureSizeInvSize.Y, 1);
								const FIntVector DispatchCount = DispatchCount.DivideAndRoundUp(FIntVector(CloudShadowTextureSize.X, CloudShadowTextureSize.Y, 1), FIntVector(8, 8, 1));
								FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("CloudDataTemporalFilter"), ComputeShader, Parameters, DispatchCount);
								CloudShadowTemporalRT.ExtractCurrentRT(GraphBuilder, CurrentShadowTexture);

								// Spatial filtering after temporal
								if (CloudShadowSpatialFiltering > 0)
								{
									FRDGTextureRef SpatiallyFilteredShadowTexture = CurrentShadowTexture;
									const float CloudShadowTextureTexelWorldSize = GetVolumetricCloudShadowMapExtentKm(AtmosphericLight) * KilometersToCentimeters * VolumetricCloudParams.VolumetricCloud.CloudShadowmapSizeInvSize[LightIndex].Z;
									const FVector4f CloudShadowTextureTexelWorldSizeInvSize = FVector4f(CloudShadowTextureTexelWorldSize, CloudShadowTextureTexelWorldSize, 1.0f / CloudShadowTextureTexelWorldSize, 1.0f / CloudShadowTextureTexelWorldSize);
									for (int i = 0; i < CloudShadowSpatialFiltering; ++i)
									{
										FilterTracedCloudTexture(&SpatiallyFilteredShadowTexture, VolumetricCloudParams.VolumetricCloud.CloudShadowmapSizeInvSize[LightIndex], CloudShadowTextureTexelWorldSizeInvSize, false);
									}
									ViewInfo.VolumetricCloudShadowRenderTarget[LightIndex] = SpatiallyFilteredShadowTexture;
								}
								else
								{
									ViewInfo.VolumetricCloudShadowRenderTarget[LightIndex] = GraphBuilder.RegisterExternalTexture(CloudShadowTemporalRT.CurrentRenderTarget());
								}
								ViewInfo.VolumetricCloudShadowExtractedRenderTarget[LightIndex] = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, ViewInfo.VolumetricCloudShadowRenderTarget[LightIndex], ERHIAccess::SRVMask, ERHIPipeline::All);

							}
							else
							{
								if (CloudShadowSpatialFiltering > 0)
								{
									FRDGTextureRef SpatiallyFilteredShadowTexture = NewCloudShadowTexture;
									const float CloudShadowTextureTexelWorldSize = GetVolumetricCloudShadowMapExtentKm(AtmosphericLight) * KilometersToCentimeters * VolumetricCloudParams.VolumetricCloud.CloudShadowmapSizeInvSize[LightIndex].Z;
									const FVector4f CloudShadowTextureTexelWorldSizeInvSize = FVector4f(CloudShadowTextureTexelWorldSize, CloudShadowTextureTexelWorldSize, 1.0f / CloudShadowTextureTexelWorldSize, 1.0f / CloudShadowTextureTexelWorldSize);
									for (int i = 0; i < CloudShadowSpatialFiltering; ++i)
									{
										FilterTracedCloudTexture(&SpatiallyFilteredShadowTexture, VolumetricCloudParams.VolumetricCloud.CloudShadowmapSizeInvSize[LightIndex], CloudShadowTextureTexelWorldSizeInvSize, false);
									}
									ViewInfo.VolumetricCloudShadowRenderTarget[LightIndex] = SpatiallyFilteredShadowTexture;
								}
								else
								{
									ViewInfo.VolumetricCloudShadowRenderTarget[LightIndex] = NewCloudShadowTexture;
								}
								ViewInfo.VolumetricCloudShadowExtractedRenderTarget[LightIndex] = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, ViewInfo.VolumetricCloudShadowRenderTarget[LightIndex], ERHIAccess::SRVMask, ERHIPipeline::All);
							}
						}
						else
						{
							ViewInfo.VolumetricCloudShadowRenderTarget[LightIndex] = nullptr;
							if (ViewInfo.ViewState)
							{
								ViewInfo.ViewState->VolumetricCloudShadowRenderTarget[LightIndex].Reset();
							}
						}

						if (ViewInfo.ViewState)
						{
							// Update the view previous cloud shadow matrix for next frame
							ViewInfo.ViewState->VolumetricCloudShadowmapPreviousTranslatedWorldToLightClipMatrix[LightIndex] = FMatrix(CloudGlobalShaderParams.CloudShadowmapTranslatedWorldToLightClipMatrix[LightIndex]);
							ViewInfo.ViewState->VolumetricCloudShadowmapPreviousAtmosphericLightDir[LightIndex] = AtmosphericLight ? AtmosphericLight->GetDirection() : FVector::ZeroVector;
							ViewInfo.ViewState->VolumetricCloudShadowmapPreviousAtmosphericLightPos[LightIndex] = FVector4(CloudGlobalShaderParams.CloudShadowmapLightPos[LightIndex]);
							ViewInfo.ViewState->VolumetricCloudShadowmapPreviousAnchorPoint[LightIndex] = FVector4(CloudGlobalShaderParams.CloudShadowmapLightAnchorPos[LightIndex]);
						}
					};
					GenerateCloudTexture(AtmosphericLight0, 0);
					GenerateCloudTexture(AtmosphericLight1, 1);
				}
			}
			else
			{
				CleanUpCloudDataPass(GraphBuilder);
			}
		}
		else
		{
			CleanUpCloudDataPass(GraphBuilder);
		}

		ExternalAccessQueue.Submit(GraphBuilder);
	}
	else
	{
		CleanUpCloudDataPass(GraphBuilder);
	}
}

FCloudRenderContext::FCloudRenderContext()
{
	TracingCoordToZbufferCoordScaleBias = FUintVector4(1, 1, 0, 0);
	TracingCoordToFullResPixelCoordScaleBias = FUintVector4(1, 1, 0, 0);
	NoiseFrameIndexModPattern = 0;

	bDisableCloudBlending = false;
	bIsReflectionRendering = false;
	bIsSkyRealTimeReflectionRendering = false;
	bSkipAtmosphericLightShadowmap = false;

	bSkipAerialPerspective = false;
	bSkipHeightFog = false;

	bAsyncCompute = false;
	bCloudDebugViewModeEnabled = false;
	bAccumulateAlphaHoldOut = false;
}

static TAutoConsoleVariable<int32> CVarCloudDefaultTexturesNoFastClear(
	TEXT("r.CloudDefaultTexturesNoFastClear"),
	1,
	TEXT("Remove fast clear on default cloud textures"),
	ECVF_RenderThreadSafe);

void FCloudRenderContext::CreateDefaultTexturesIfNeeded(FRDGBuilder& GraphBuilder)
{
	if (DefaultCloudColorCubeTexture == nullptr)
	{
		const ETextureCreateFlags NoFastClearFlags = (CVarCloudDefaultTexturesNoFastClear.GetValueOnAnyThread() != 0) ? TexCreate_NoFastClear : TexCreate_None;

		DefaultCloudColorCubeTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::CreateCube(1, PF_FloatRGBA, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV | NoFastClearFlags),
			TEXT("Cloud.ColorCubeDummy"));

		DefaultCloudColor02DTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_FloatRGBA, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV | NoFastClearFlags),
			TEXT("Cloud.ColorDummy"));

		DefaultCloudColor12DTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_FloatRGBA, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV | NoFastClearFlags),
			TEXT("Cloud.ColorDummy"));

		DefaultCloudDepthTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_G16R16F, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV | NoFastClearFlags),
			TEXT("Cloud.DepthDummy"));

		DefaultCloudAlphaHoldout = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_R16F, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV | NoFastClearFlags),
			TEXT("Cloud.AlphaHoldoutDummy"));

		DefaultCloudColorCubeTextureUAV = GraphBuilder.CreateUAV(DefaultCloudColorCubeTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		DefaultCloudColor02DTextureUAV = GraphBuilder.CreateUAV(DefaultCloudColor12DTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		DefaultCloudColor12DTextureUAV = GraphBuilder.CreateUAV(DefaultCloudColor02DTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		DefaultCloudDepthTextureUAV = GraphBuilder.CreateUAV(DefaultCloudDepthTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		DefaultCloudAlphaHoldoutUAV = GraphBuilder.CreateUAV(DefaultCloudAlphaHoldout, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}
}

static TRDGUniformBufferRef<FRenderVolumetricCloudGlobalParameters> CreateCloudPassUniformBuffer(FRDGBuilder& GraphBuilder, FCloudRenderContext& CloudRC)
{
	FViewInfo& MainView = *CloudRC.MainView;
	FVolumetricCloudRenderSceneInfo& CloudInfo = *CloudRC.CloudInfo;

	FRenderVolumetricCloudGlobalParameters& VolumetricCloudParams = *GraphBuilder.AllocParameters<FRenderVolumetricCloudGlobalParameters>();
	const bool bForceNonLateFogOnCloud = CloudRC.bIsSkyRealTimeReflectionRendering;
	SetupDefaultRenderVolumetricCloudGlobalParameters(GraphBuilder, VolumetricCloudParams, CloudInfo, MainView, bForceNonLateFogOnCloud);

	VolumetricCloudParams.SceneDepthTexture = CloudRC.SceneDepthZ;
	VolumetricCloudParams.SceneDepthMinAndMaxTexture = CloudRC.SceneDepthMinAndMax ? CloudRC.SceneDepthMinAndMax : VolumetricCloudParams.SceneDepthMinAndMaxTexture;
	if (CloudRC.bShouldViewRenderVolumetricRenderTarget && MainView.ViewState)
	{
		FVolumetricRenderTargetViewStateData& VRT = MainView.ViewState->VolumetricCloudRenderTarget;
		const uint32 VolumetricReconstructRTDownsampleFactor = VRT.GetVolumetricTracingRTDownsampleFactor();

		// Use the main view to get the target rect. The clouds are reconstructed at the same resolution as the depth buffer read when tracing.
		VolumetricCloudParams.SceneDepthTextureMinMaxCoord = FUintVector4(
			MainView.ViewRect.Min.X / VolumetricReconstructRTDownsampleFactor,
			MainView.ViewRect.Min.Y / VolumetricReconstructRTDownsampleFactor,
			(MainView.ViewRect.Max.X - 1) / VolumetricReconstructRTDownsampleFactor,
			(MainView.ViewRect.Max.Y - 1) / VolumetricReconstructRTDownsampleFactor);
	}
	else
	{
		VolumetricCloudParams.SceneDepthTextureMinMaxCoord = FUintVector4(
			MainView.ViewRect.Min.X,
			MainView.ViewRect.Min.Y,
			(MainView.ViewRect.Max.X - 1),
			(MainView.ViewRect.Max.Y - 1));
	}

	VolumetricCloudParams.Light0Shadow = CloudRC.LightShadowShaderParams0;
	VolumetricCloudParams.VirtualShadowMapId0 = CloudRC.VirtualShadowMapId0;
	VolumetricCloudParams.CloudShadowTexture0 = CloudRC.VolumetricCloudShadowTexture[0];
	VolumetricCloudParams.CloudShadowTexture1 = CloudRC.VolumetricCloudShadowTexture[1];
	VolumetricCloudParams.TracingCoordToZbufferCoordScaleBias = CloudRC.TracingCoordToZbufferCoordScaleBias;
	VolumetricCloudParams.TracingCoordToFullResPixelCoordScaleBias = CloudRC.TracingCoordToFullResPixelCoordScaleBias;
	VolumetricCloudParams.NoiseFrameIndexModPattern = CloudRC.NoiseFrameIndexModPattern;
	VolumetricCloudParams.IsReflectionRendering = CloudRC.bIsReflectionRendering ? 1 : 0;

	if (CloudRC.bShouldViewRenderVolumetricRenderTarget && MainView.ViewState)
	{
		FVolumetricRenderTargetViewStateData& VRT = MainView.ViewState->VolumetricCloudRenderTarget;
		VolumetricCloudParams.OpaqueIntersectionMode = VRT.GetMode()== 2 ? 0 : 2;	// intersect with opaque only if not using mode 2 (full res distant cloud updating 1 out 4x4 pixels)
	}
	else
	{
		VolumetricCloudParams.OpaqueIntersectionMode = 2;	// always intersect with opaque
	}

	const bool bIsPathTracing = MainView.Family && MainView.Family->EngineShowFlags.PathTracing;
	if (CloudRC.bIsReflectionRendering 
		&& !(bIsPathTracing && CloudRC.bIsSkyRealTimeReflectionRendering))	// When using path tracing and real time sky capture, the captured sky cubemap is used to render the sky in the main view. As such, we always use the view settings.
	{
		VolumetricCloudParams.VolumetricCloud.SampleCountMax = FMath::Max(2.0f, FMath::Min(
			UVolumetricCloudComponent::BaseViewRaySampleCount * CloudInfo.GetVolumetricCloudSceneProxy().ReflectionViewSampleCountScale,
			CVarVolumetricCloudReflectionRaySampleMaxCount.GetValueOnAnyThread()));
		VolumetricCloudParams.VolumetricCloud.ShadowSampleCountMax = FMath::Max(2.0f, FMath::Min(
			UVolumetricCloudComponent::BaseShadowRaySampleCount * CloudInfo.GetVolumetricCloudSceneProxy().ShadowReflectionViewSampleCountScale,
			CVarVolumetricCloudShadowReflectionRaySampleMaxCount.GetValueOnAnyThread()));
	}

	VolumetricCloudParams.EnableAerialPerspectiveSampling = CloudRC.bSkipAerialPerspective ? 0 : 1;
	VolumetricCloudParams.EnableHeightFog = VolumetricCloudParams.EnableHeightFog > 0 && !CloudRC.bSkipHeightFog ? VolumetricCloudParams.EnableHeightFog : 0;
	VolumetricCloudParams.EnableDistantSkyLightSampling = CVarVolumetricCloudEnableDistantSkyLightSampling.GetValueOnAnyThread() > 0 ? 1 : 0;
	VolumetricCloudParams.EnableAtmosphericLightsSampling = CVarVolumetricCloudEnableAtmosphericLightsSampling.GetValueOnAnyThread() > 0 ? 1 : 0;

	VolumetricCloudParams.AerialPerspectiveRayOnlyStartDistanceKm = CloudInfo.GetVolumetricCloudSceneProxy().AerialPespectiveRayleighScatteringStartDistance;
	VolumetricCloudParams.AerialPerspectiveRayOnlyFadeDistanceKmInv = 1.0f / FMath::Max(CloudInfo.GetVolumetricCloudSceneProxy().AerialPespectiveRayleighScatteringFadeDistance, 0.00001f);// One centimeter minimum
	VolumetricCloudParams.AerialPerspectiveMieOnlyStartDistanceKm = CloudInfo.GetVolumetricCloudSceneProxy().AerialPespectiveMieScatteringStartDistance;
	VolumetricCloudParams.AerialPerspectiveMieOnlyFadeDistanceKmInv = 1.0f / FMath::Max(CloudInfo.GetVolumetricCloudSceneProxy().AerialPespectiveMieScatteringFadeDistance, 0.00001f);// One centimeter minimum

	const FRDGTextureDesc& Desc = CloudRC.RenderTargets.Output[0].GetTexture()->Desc;
	VolumetricCloudParams.OutputSizeInvSize = FVector4f(Desc.Extent.X, Desc.Extent.Y, 1.0f / Desc.Extent.X, 1.0f / Desc.Extent.Y);

	if (CloudRC.bIsSkyRealTimeReflectionRendering)
	{
		VolumetricCloudParams.FogStruct.ApplyVolumetricFog = 0;		// No valid camera froxel volume available.
		VolumetricCloudParams.OpaqueIntersectionMode = 0;			// No depth buffer is available
	}
	else
	{
		if (ShouldVolumetricCloudsApplyFogDuringUpsampling(MainView))
		{
			VolumetricCloudParams.FogStruct.ApplyVolumetricFog = 0;
			VolumetricCloudParams.EnableHeightFog = 0;
		}
	}

	return GraphBuilder.CreateUniformBuffer(&VolumetricCloudParams);
}

static void GetOutputTexturesWithFallback(
	FRDGBuilder& GraphBuilder, FCloudRenderContext& CloudRC, 
	FRDGTextureUAVRef& CloudColorCubeTextureUAV, 
	FRDGTextureUAVRef& CloudColor0TextureUAV,
	FRDGTextureUAVRef& CloudColor1TextureUAV,
	FRDGTextureUAVRef& CloudDepthTextureUAV,
	FRDGTextureUAVRef& CloudAlphaHoldoutUAV)
{
	CloudRC.CreateDefaultTexturesIfNeeded(GraphBuilder);

	FRDGTextureRef CloudColorCubeTexture = CloudRC.RenderTargets[0].GetTexture();
	if (!CloudColorCubeTexture || !CloudColorCubeTexture->Desc.IsTextureCube())
	{
		CloudColorCubeTextureUAV = CloudRC.DefaultCloudColorCubeTextureUAV;
	}
	else
	{
		if (CloudRC.ComputeOverlapCloudColorCubeTextureUAVWithoutBarrier)
		{
			// If specified, this is to make sure a single UAV without barrier so that compute work items can overlap.
			CloudColorCubeTextureUAV = CloudRC.ComputeOverlapCloudColorCubeTextureUAVWithoutBarrier;
		}
		else
		{
			CloudColorCubeTextureUAV = GraphBuilder.CreateUAV(CloudColorCubeTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		}
	}

	FRDGTextureRef CloudColor0Texture = CloudRC.RenderTargets[0].GetTexture();
	if (!CloudColor0Texture || CloudColor0Texture->Desc.IsTextureCube())
	{
		CloudColor0TextureUAV = CloudRC.DefaultCloudColor02DTextureUAV;
	}
	else
	{
		CloudColor0TextureUAV = GraphBuilder.CreateUAV(CloudColor0Texture, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}

	FRDGTextureRef CloudColor1Texture = CloudRC.SecondaryCloudTracingDataTexture;
	if (!CloudColor1Texture || CloudColor1Texture->Desc.IsTextureCube())
	{
		CloudColor1TextureUAV = CloudRC.DefaultCloudColor12DTextureUAV;
	}
	else
	{
		CloudColor1TextureUAV = GraphBuilder.CreateUAV(CloudColor1Texture, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}

	FRDGTextureRef CloudDepthTexture = CloudRC.RenderTargets[1].GetTexture();
	if (!CloudDepthTexture)
	{
		CloudDepthTextureUAV = CloudRC.DefaultCloudDepthTextureUAV;
	}
	else
	{
		CloudDepthTextureUAV = GraphBuilder.CreateUAV(CloudDepthTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}

	FRDGTextureRef CloudAlphaHoldoutTexture = CloudRC.CloudAlphaHoldoutTexture;
	if (!CloudAlphaHoldoutTexture)
	{
		CloudAlphaHoldoutUAV = CloudRC.DefaultCloudAlphaHoldoutUAV;
	}
	else
	{
		CloudAlphaHoldoutUAV = GraphBuilder.CreateUAV(CloudAlphaHoldoutTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}
}

void FSceneRenderer::RenderVolumetricCloudsInternal(FRDGBuilder& GraphBuilder, FCloudRenderContext& CloudRC, FInstanceCullingManager& InstanceCullingManager, const FIntPoint& CloudViewportSize)
{
	check(CloudRC.MainView);
	check(CloudRC.CloudInfo);
	check(CloudRC.CloudVolumeMaterialProxy);

	FMaterialRenderProxy* CloudVolumeMaterialProxy = CloudRC.CloudVolumeMaterialProxy;

#if !UE_BUILD_SHIPPING
	{
		const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
		const FMaterial* MaterialResource = &CloudVolumeMaterialProxy->GetMaterialWithFallback(Scene->GetFeatureLevel(), MaterialRenderProxy);
		MaterialRenderProxy = MaterialRenderProxy ? MaterialRenderProxy : CloudVolumeMaterialProxy;

		if (!MaterialResource->IsUsedWithVolumetricCloud())
		{
			// This is not a material designed to run with volumetric cloud.
			OnGetOnScreenMessages.AddLambda([](FScreenMessageWriter& ScreenMessageWriter)->void
				{
					static const FText Message = NSLOCTEXT("Renderer", "MaterialNotVolumetricCloud", "A material assigned on a volumetric cloud component does not have the UsedWithVolumetricCloud flag set.");
					ScreenMessageWriter.DrawLine(Message);
				});
			return;
		}
		if (MaterialResource->GetMaterialDomain() != MD_Volume)
		{
			// This is not a material designed to run with volumetric cloud.
			OnGetOnScreenMessages.AddLambda([](FScreenMessageWriter& ScreenMessageWriter)->void
				{
					static const FText Message = NSLOCTEXT("Renderer", "MaterialNotVolumetricDomain", "A material assigned on a volumetric cloud component does not have the Volumetric domain set.");
					ScreenMessageWriter.DrawLine(Message);
				});
			return;
		}
	}
#endif

	// Copy parameters to lambda
	const bool bShouldViewRenderVolumetricRenderTarget = CloudRC.bShouldViewRenderVolumetricRenderTarget;
	const bool bSkipAtmosphericLightShadowmap = CloudRC.bSkipAtmosphericLightShadowmap;
	const bool bSecondAtmosphereLightEnabled = CloudRC.bSecondAtmosphereLightEnabled;
	const bool bCloudDebugViewModeEnabled = CloudRC.bCloudDebugViewModeEnabled;
	FViewInfo& MainView = *CloudRC.MainView;
	TUniformBufferRef<FViewUniformShaderParameters> ViewUniformBuffer = CloudRC.ViewUniformBuffer;
	TRDGUniformBufferRef<FRenderVolumetricCloudGlobalParameters> CloudPassUniformBuffer = CreateCloudPassUniformBuffer(GraphBuilder, CloudRC);

	const bool bCloudEnableLocalLightSampling = CVarVolumetricCloudEnableLocalLightsSampling.GetValueOnRenderThread() > 0;
	if (ShouldUseComputeForCloudTracing(Scene->GetFeatureLevel()))
	{
		//
		// Cloud empty space skipping
		// TODO: add support for reflection rendering without breaking compute work item overlap for each faces.
		//

		FRDGTextureRef EmptySpaceSkippingTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		if(ShouldUseComputeCloudEmptySpaceSkipping() && !CloudRC.bIsReflectionRendering)
		{
			const FIntPoint EmptySpaceSkippingTextureResolution = FIntPoint(32, 32);
			EmptySpaceSkippingTexture = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(EmptySpaceSkippingTextureResolution, PF_R16F, FClearValueBinding(EClearBinding::ENoneBound),
					TexCreate_ShaderResource | TexCreate_UAV), TEXT("Cloud.EmptySpaceSkippingTexture"));
			
			FRenderVolumetricCloudEmptySpaceSkippingCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderVolumetricCloudEmptySpaceSkippingCS::FParameters>();
			PassParameters->VolumetricCloudRenderViewParamsUB = CloudPassUniformBuffer;
			PassParameters->View = ViewUniformBuffer;
			PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder);
			PassParameters->OutStartTracingDistanceTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(EmptySpaceSkippingTexture));
			PassParameters->StartTracingDistanceTextureResolution = FVector2f(EmptySpaceSkippingTextureResolution.X, EmptySpaceSkippingTextureResolution.Y);
			PassParameters->StartTracingSampleVolumeDepth = GetEmptySpaceSkippingVolumeDepth();
			PassParameters->StartTracingSliceBias = CVarVolumetricCloudEmptySpaceSkippingStartTracingSliceBias.GetValueOnRenderThread();

			const bool bCloudEmptySpaceSkippingDebug = false;
			//if (bCloudEmptySpaceSkippingDebug)
			//{
			//	ShaderPrint::SetEnabled(true);
			//	ShaderPrint::RequestSpaceForLines(65536);
			//	ShaderPrint::SetParameters(GraphBuilder, MainView.ShaderPrintData, PassParameters->ShaderPrintParameters);
			//}
			
			const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
			const FMaterial* MaterialResource = &CloudVolumeMaterialProxy->GetMaterialWithFallback(Scene->GetFeatureLevel(), MaterialRenderProxy);
			MaterialRenderProxy = MaterialRenderProxy ? MaterialRenderProxy : CloudVolumeMaterialProxy;

			const bool bSampleCorners = CVarVolumetricCloudEmptySpaceSkippingSampleCorners.GetValueOnRenderThread() > 0;

			typename FRenderVolumetricCloudEmptySpaceSkippingCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<typename FRenderVolumetricCloudEmptySpaceSkippingCS::FCloudEmptySpaceSkippingDebug>(bCloudEmptySpaceSkippingDebug);
			PermutationVector.Set<typename FRenderVolumetricCloudEmptySpaceSkippingCS::FCloudEmptySpaceSkippingSampleCorners>(bSampleCorners);

			TShaderRef<FRenderVolumetricCloudEmptySpaceSkippingCS> ComputeShader = MaterialResource->GetShader<FRenderVolumetricCloudEmptySpaceSkippingCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);

			const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(EmptySpaceSkippingTextureResolution, FIntPoint(FRenderVolumetricCloudEmptySpaceSkippingCS::ThreadGroupSizeX, FRenderVolumetricCloudEmptySpaceSkippingCS::ThreadGroupSizeY));

			if (ComputeShader.IsValid())
			{
				ClearUnusedGraphResources(ComputeShader, PassParameters);
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("CloudViewEmptySpaceSkipping (CS) %dx%d", EmptySpaceSkippingTextureResolution.X, EmptySpaceSkippingTextureResolution.Y),
				PassParameters,
				CloudRC.bAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
				[Scene = Scene, MaterialRenderProxy, MaterialResource, PassParameters, ComputeShader, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
				{
					if (MaterialResource->GetMaterialDomain() != MD_Volume)
					{
						return;
					}

					FMeshDrawShaderBindings ShaderBindings;
					UE::MeshPassUtils::SetupComputeBindings(ComputeShader, Scene, Scene->GetFeatureLevel(), nullptr, *MaterialRenderProxy, *MaterialResource, ShaderBindings);

					UE::MeshPassUtils::Dispatch(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, GroupCount);
				});
		}
		
		//
		// Cloud view tracing
		//

		FRDGTextureUAVRef CloudColorCubeTextureUAV;
		FRDGTextureUAVRef CloudColor0TextureUAV;
		FRDGTextureUAVRef CloudColor1TextureUAV;
		FRDGTextureUAVRef CloudDepthTextureUAV;
		FRDGTextureUAVRef CloudAlphaHoldoutUAV;
		GetOutputTexturesWithFallback(GraphBuilder, CloudRC, CloudColorCubeTextureUAV, CloudColor0TextureUAV, CloudColor1TextureUAV, CloudDepthTextureUAV, CloudAlphaHoldoutUAV);

		bool bSampleVirtualShadowMap = (!bCloudDebugViewModeEnabled && !bSkipAtmosphericLightShadowmap && CVarVolumetricCloudShadowSampleAtmosphericLightShadowmap.GetValueOnRenderThread() > 0);

		FRenderVolumetricCloudRenderViewCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderVolumetricCloudRenderViewCS::FParameters>();
		PassParameters->OutputViewRect = FVector4f(0.f, 0.f, CloudViewportSize.X, CloudViewportSize.Y);
		PassParameters->bBlendCloudColor = !bShouldViewRenderVolumetricRenderTarget && !bCloudDebugViewModeEnabled && !CloudRC.bDisableCloudBlending;
		PassParameters->TargetCubeFace = CloudRC.RenderTargets.Output[0].GetArraySlice();
		PassParameters->VolumetricCloudRenderViewParamsUB = CloudPassUniformBuffer;
		PassParameters->View = ViewUniformBuffer;
		PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder);
		if (bSampleVirtualShadowMap)
		{
			PassParameters->VirtualShadowMap = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, 0);
		}
		PassParameters->OutCloudColor0 = CloudColor0TextureUAV;
		PassParameters->OutCloudColor1 = CloudColor1TextureUAV;
		PassParameters->OutCloudDepth = CloudDepthTextureUAV;
		PassParameters->OutCloudColorCube = CloudColorCubeTextureUAV;
		PassParameters->OutCloudAlphaHoldout = CloudAlphaHoldoutUAV;

		PassParameters->StartTracingDistanceTextureResolution = FVector2f(EmptySpaceSkippingTexture->Desc.Extent.X, EmptySpaceSkippingTexture->Desc.Extent.Y);
		PassParameters->StartTracingDistanceTexture = EmptySpaceSkippingTexture;
		PassParameters->StartTracingSampleVolumeDepth = GetEmptySpaceSkippingVolumeDepth();
		PassParameters->bAccumulateAlphaHoldOut = CloudRC.bAccumulateAlphaHoldOut ? 1 : 0;

#if CLOUD_DEBUG_SAMPLES
		ShaderPrint::SetEnabled(true);
		ShaderPrint::RequestSpaceForLines(65536);
		ShaderPrint::SetParameters(GraphBuilder, MainView.ShaderPrintData, PassParameters->ShaderPrintParameters);
#endif

		const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
		const FMaterial* MaterialResource = &CloudVolumeMaterialProxy->GetMaterialWithFallback(Scene->GetFeatureLevel(), MaterialRenderProxy);
		MaterialRenderProxy = MaterialRenderProxy ? MaterialRenderProxy : CloudVolumeMaterialProxy;


		typename FRenderVolumetricCloudRenderViewCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<typename FRenderVolumetricCloudRenderViewCS::FCloudPerSampleAtmosphereTransmittance>(!bCloudDebugViewModeEnabled && ShouldUsePerSampleAtmosphereTransmittance(Scene, &MainView));
		PermutationVector.Set<typename FRenderVolumetricCloudRenderViewCS::FCloudSampleAtmosphericLightShadowmap>(bSampleVirtualShadowMap);
		PermutationVector.Set<typename FRenderVolumetricCloudRenderViewCS::FCloudSampleSecondLight>(!bCloudDebugViewModeEnabled && bSecondAtmosphereLightEnabled);
		PermutationVector.Set<typename FRenderVolumetricCloudRenderViewCS::FCloudSampleLocalLights>(!bCloudDebugViewModeEnabled && bCloudEnableLocalLightSampling && !CloudRC.bIsSkyRealTimeReflectionRendering);
		PermutationVector.Set<typename FRenderVolumetricCloudRenderViewCS::FCloudMinAndMaxDepth>(ShouldVolumetricCloudTraceWithMinMaxDepth(MainView) && CloudRC.SecondaryCloudTracingDataTexture!=nullptr && !CloudRC.bIsReflectionRendering);// Only for mode 0 in non reflection
		PermutationVector.Set<typename FRenderVolumetricCloudRenderViewCS::FCloudLocalFogVolume>(MainView.LocalFogVolumeViewData.GPUInstanceCount > 0 && !CloudRC.bIsSkyRealTimeReflectionRendering && !ShouldVolumetricCloudsApplyFogDuringUpsampling(MainView)); // LFVs are not culled for real time capture views.
		PermutationVector.Set<typename FRenderVolumetricCloudRenderViewCS::FCloudDebugViewMode>(bCloudDebugViewModeEnabled);

		TShaderRef<FRenderVolumetricCloudRenderViewCS> ComputeShader = MaterialResource->GetShader<FRenderVolumetricCloudRenderViewCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);

		const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(CloudViewportSize, FIntPoint(FRenderVolumetricCloudRenderViewCS::ThreadGroupSizeX, FRenderVolumetricCloudRenderViewCS::ThreadGroupSizeY));

		if (ComputeShader.IsValid())
		{
			ClearUnusedGraphResources(ComputeShader, PassParameters);
		}

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("CloudView (CS) %dx%d", CloudViewportSize.X, CloudViewportSize.Y),
			PassParameters,
			CloudRC.bAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
			[Scene = Scene, MaterialRenderProxy, MaterialResource, PassParameters, ComputeShader, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
			{
				if (MaterialResource->GetMaterialDomain() != MD_Volume)
				{
					return;
				}

				FMeshDrawShaderBindings ShaderBindings;
				UE::MeshPassUtils::SetupComputeBindings(ComputeShader, Scene, Scene->GetFeatureLevel(), nullptr, *MaterialRenderProxy, *MaterialResource, ShaderBindings);

				UE::MeshPassUtils::Dispatch(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, GroupCount);
			});
	}
	else
	{
		ensureMsgf(!bCloudEnableLocalLightSampling, TEXT("We do not support lighting clouds with local light through the pixel shader in order to reduce permutations count."));

		FRenderVolumetricCloudRenderViewParametersPS* RenderViewPassParameters = GraphBuilder.AllocParameters<FRenderVolumetricCloudRenderViewParametersPS>();
		RenderViewPassParameters->VolumetricCloudRenderViewParamsUB = CloudPassUniformBuffer;
		RenderViewPassParameters->CloudShadowTexture0 = CloudRC.VolumetricCloudShadowTexture[0];
		RenderViewPassParameters->CloudShadowTexture1 = CloudRC.VolumetricCloudShadowTexture[1];
		RenderViewPassParameters->View = ViewUniformBuffer;
		RenderViewPassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder);
		RenderViewPassParameters->VirtualShadowMap = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, 0);
		RenderViewPassParameters->InstanceCulling = InstanceCullingManager.GetDummyInstanceCullingUniformBuffer();
		RenderViewPassParameters->RenderTargets = CloudRC.RenderTargets;
		
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("CloudView (PS) %dx%d", CloudViewportSize.X, CloudViewportSize.Y),
			RenderViewPassParameters,
			ERDGPassFlags::Raster,
			[RenderViewPassParameters, Scene = Scene, &MainView, bCloudDebugViewModeEnabled,
			bShouldViewRenderVolumetricRenderTarget, CloudVolumeMaterialProxy, bSkipAtmosphericLightShadowmap, bSecondAtmosphereLightEnabled, CloudViewportSize](FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, (float)CloudViewportSize.X, (float)CloudViewportSize.Y, 1.0f);

				DrawDynamicMeshPass(MainView, RHICmdList,
					[&MainView, bShouldViewRenderVolumetricRenderTarget, bSkipAtmosphericLightShadowmap, bSecondAtmosphereLightEnabled,
					bCloudDebugViewModeEnabled, CloudVolumeMaterialProxy](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
					{
						FVolumetricCloudRenderViewMeshProcessor PassMeshProcessor(
							MainView.Family->Scene->GetRenderScene(), MainView.GetFeatureLevel(), & MainView, bShouldViewRenderVolumetricRenderTarget,
							bSkipAtmosphericLightShadowmap, bSecondAtmosphereLightEnabled, bCloudDebugViewModeEnabled, DynamicMeshPassContext);

						FMeshBatch LocalSingleTriangleMesh;
						GetSingleTriangleMeshBatch(LocalSingleTriangleMesh, CloudVolumeMaterialProxy, MainView.GetFeatureLevel());

						const FPrimitiveSceneProxy* PrimitiveSceneProxy = nullptr;
						const uint64 DefaultBatchElementMask = ~0ull;
						PassMeshProcessor.AddMeshBatch(LocalSingleTriangleMesh, DefaultBatchElementMask, PrimitiveSceneProxy);
					});
			});
	}
}

bool FSceneRenderer::RenderVolumetricCloud(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	bool bSkipVolumetricRenderTarget,
	bool bSkipPerPixelTracing,
	FRDGTextureRef HalfResolutionDepthCheckerboardMinMaxTexture,
	FRDGTextureRef QuarterResolutionDepthMinMaxTexture,
	bool bAsyncCompute,
	FInstanceCullingManager& InstanceCullingManager)
{
	check(ShouldRenderVolumetricCloud(Scene, ViewFamily.EngineShowFlags)); // This should not be called if we should not render SkyAtmosphere

	FVolumetricCloudRenderSceneInfo& CloudInfo = *Scene->GetVolumetricCloudSceneInfo();
	FVolumetricCloudSceneProxy& CloudSceneProxy = CloudInfo.GetVolumetricCloudSceneProxy();

	FLightSceneInfo* AtmosphericLight0Info = Scene->AtmosphereLights[0];
	FLightSceneProxy* AtmosphericLight0 = AtmosphericLight0Info ? AtmosphericLight0Info->Proxy : nullptr;
	FSkyLightSceneProxy* SkyLight = Scene->SkyLight;
	bool bAsyncComputeUsed = false;

	if (CloudSceneProxy.GetCloudVolumeMaterial())
	{
		FMaterialRenderProxy* CloudVolumeMaterialProxy = CloudSceneProxy.GetCloudVolumeMaterial()->GetRenderProxy();
		if (CloudVolumeMaterialProxy->GetIncompleteMaterialWithFallback(ViewFamily.GetFeatureLevel()).GetMaterialDomain() == MD_Volume)
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, VolumetricCloud, "VolumetricCloud"); 
			SCOPED_NAMED_EVENT(VolumetricCloud, FColor::Emerald);

			FRDGTextureRef SceneDepthZ = SceneTextures.Depth.Resolve;

			FCloudRenderContext CloudRC;
			CloudRC.CloudInfo = &CloudInfo;
			CloudRC.CloudVolumeMaterialProxy= CloudVolumeMaterialProxy;
			CloudRC.bSkipAtmosphericLightShadowmap = !GetVolumetricCloudReceiveAtmosphericLightShadowmap(AtmosphericLight0);
			CloudRC.bSecondAtmosphereLightEnabled = Scene->IsSecondAtmosphereLightEnabledForCloud();

			const bool DebugCloudShadowMap = CVarVolumetricCloudShadowMapDebug.GetValueOnRenderThread() && ShouldRenderCloudShadowmap(AtmosphericLight0);
			const bool DebugCloudSkyAO = CVarVolumetricCloudSkyAODebug.GetValueOnRenderThread() && ShouldRenderCloudSkyAO(SkyLight);

			struct FLocalCloudView
			{
				FViewInfo* ViewInfo;
				bool bShouldViewRenderVolumetricCloudRenderTarget;
				bool bEnableAerialPerspectiveSampling;
				bool bShouldUseHighQualityAerialPerspective;
				bool bCloudDebugViewModeEnabled;
			};
			TArray<FLocalCloudView, TInlineAllocator<2>> ViewsToProcess;
			
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				FViewInfo& ViewInfo = Views[ViewIndex];

				// RenderTarget are not used by reflection captures for instance.
				bool bShouldViewRenderVolumetricCloudRenderTarget = ShouldViewRenderVolumetricCloudRenderTarget(ViewInfo);

				if ((bShouldViewRenderVolumetricCloudRenderTarget && bSkipVolumetricRenderTarget) || (!bShouldViewRenderVolumetricCloudRenderTarget && bSkipPerPixelTracing))
				{
					continue;
				}

				const bool bCloudDebugViewModeEnabled = ShouldViewVisualizeVolumetricCloudConservativeDensity(ViewInfo, ViewFamily.EngineShowFlags);
				const bool bEnableAerialPerspectiveSampling = CVarVolumetricCloudEnableAerialPerspectiveSampling.GetValueOnAnyThread() > 0;
				const bool bShouldUseHighQualityAerialPerspective =
					bEnableAerialPerspectiveSampling
					&& Scene->HasSkyAtmosphere()
					&& CVarVolumetricCloudHighQualityAerialPerspective.GetValueOnAnyThread() > 0
					&& !ViewInfo.bIsReflectionCapture
					&& !bCloudDebugViewModeEnabled;

				if (bAsyncCompute
					&& bShouldViewRenderVolumetricCloudRenderTarget
					&& !bShouldUseHighQualityAerialPerspective
					&& !DebugCloudShadowMap
					&& !DebugCloudSkyAO)
				{
					// TODO: see whether high quality AP rendering can use async compute
					bAsyncComputeUsed = true;
				}
				else if (bAsyncCompute)
				{
					// Skip if async compute is requested but cannot fulfill
					continue;
				}

				const bool bVRTValid = ViewInfo.ViewState != nullptr && ViewInfo.ViewState->VolumetricCloudRenderTarget.IsValid();
				if (!bShouldViewRenderVolumetricCloudRenderTarget || bVRTValid)
				{
					FLocalCloudView& ViewToProcess = ViewsToProcess[ViewsToProcess.AddUninitialized()];
					ViewToProcess.ViewInfo = &ViewInfo;
					ViewToProcess.bShouldViewRenderVolumetricCloudRenderTarget = bVRTValid && bShouldViewRenderVolumetricCloudRenderTarget;
					ViewToProcess.bEnableAerialPerspectiveSampling = bEnableAerialPerspectiveSampling;
					ViewToProcess.bShouldUseHighQualityAerialPerspective = bShouldUseHighQualityAerialPerspective;
					ViewToProcess.bCloudDebugViewModeEnabled = bCloudDebugViewModeEnabled;
				}
			}

			const bool bHeightFogInScene = Scene->ExponentialFogs.Num() > 0;
			for (int32 ViewIndex = 0; ViewIndex < ViewsToProcess.Num(); ViewIndex++)
			{
				RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, ViewsToProcess.Num() > 1, "View%d", ViewIndex);

				FLocalCloudView& ViewToProcess = ViewsToProcess[ViewIndex];
				FViewInfo& ViewInfo = *ViewToProcess.ViewInfo;

				CloudRC.MainView = ViewToProcess.ViewInfo;
				CloudRC.bAsyncCompute = bAsyncCompute;
				CloudRC.bCloudDebugViewModeEnabled = ViewToProcess.bCloudDebugViewModeEnabled;

				const bool bShouldViewRenderVolumetricCloudRenderTarget = ViewToProcess.bShouldViewRenderVolumetricCloudRenderTarget;
				const bool bEnableAerialPerspectiveSampling = ViewToProcess.bEnableAerialPerspectiveSampling;
				const bool bShouldUseHighQualityAerialPerspective = ViewToProcess.bShouldUseHighQualityAerialPerspective;

				CloudRC.bDisableCloudBlending = !bShouldViewRenderVolumetricCloudRenderTarget && bShouldUseHighQualityAerialPerspective;
				CloudRC.bShouldViewRenderVolumetricRenderTarget = bShouldViewRenderVolumetricCloudRenderTarget;
				CloudRC.ViewUniformBuffer = bShouldViewRenderVolumetricCloudRenderTarget ? ViewInfo.VolumetricRenderTargetViewUniformBuffer : ViewInfo.ViewUniformBuffer;

				CloudRC.bSkipAerialPerspective = !bEnableAerialPerspectiveSampling || bShouldUseHighQualityAerialPerspective; // Skip AP on clouds if we are going to trace it separately in a second pass
				CloudRC.bSkipHeightFog = bShouldUseHighQualityAerialPerspective;
				CloudRC.bIsReflectionRendering = ViewInfo.bIsReflectionCapture;

				FRDGTextureRef IntermediateRT = nullptr;
				FRDGTextureRef IntermediateSecondaryRT = nullptr;
				FRDGTextureRef IntermediateRT2 = nullptr;
				FRDGTextureRef IntermediateSecondaryRT2 = nullptr;
				FRDGTextureRef DestinationRT = nullptr;
				FRDGTextureRef DestinationSecondaryRT = nullptr;
				FRDGTextureRef DestinationRTDepth = nullptr;
				CloudRC.TracingCoordToZbufferCoordScaleBias = FUintVector4(1, 1, 0, 0);
				CloudRC.TracingCoordToFullResPixelCoordScaleBias = FUintVector4(1, 1, 0, 0);
				CloudRC.NoiseFrameIndexModPattern = ViewInfo.CachedViewUniformShaderParameters->StateFrameIndexMod8;

				FIntPoint CloudDefaultRtSize(0, 0);
				bool bAccumulateAlphaHoldOut = false;

				if (bShouldViewRenderVolumetricCloudRenderTarget)
				{
					FVolumetricRenderTargetViewStateData& VRT = ViewInfo.ViewState->VolumetricCloudRenderTarget;
					VRT.SetStartTracingDistance(CloudSceneProxy.TracingStartDistanceFromCamera);
					DestinationRT = VRT.GetOrCreateVolumetricTracingRT(GraphBuilder);
					DestinationRTDepth = VRT.GetOrCreateVolumetricTracingRTDepth(GraphBuilder);

					// Only account for alphaholdout if any important primitive is holdout, and we are in mode 3
					if (ViewInfo.CachedViewUniformShaderParameters->bPrimitiveAlphaHoldoutEnabled)
					{
						const FIntVector4& EnvironmentComponentFlags = ViewInfo.CachedViewUniformShaderParameters->EnvironmentComponentsFlags;
						bAccumulateAlphaHoldOut = VRT.GetMode() == 3 &&  (IsSkyAtmosphereHoldout(EnvironmentComponentFlags) || IsVolumetricCloudHoldout(EnvironmentComponentFlags) || IsExponentialFogHoldout(EnvironmentComponentFlags));
					}

					CloudDefaultRtSize = DestinationRT->Desc.Extent;

					if (bShouldUseHighQualityAerialPerspective)
					{
						FIntPoint IntermadiateTargetResolution = FIntPoint(DestinationRT->Desc.GetSize().X, DestinationRT->Desc.GetSize().Y);
						IntermediateRT = GraphBuilder.CreateTexture(
							FRDGTextureDesc::Create2D(IntermadiateTargetResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(63000.0f, 63000.0f, 63000.0f, 63000.0f)),
								TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV), TEXT("Cloud.HighQualityAPIntermediate"));
						IntermediateSecondaryRT = GraphBuilder.CreateTexture(
							FRDGTextureDesc::Create2D(IntermadiateTargetResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(63000.0f, 63000.0f, 63000.0f, 63000.0f)),
								TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV), TEXT("Cloud.HighQualityAPIntermediateSecondary"));

						if (bHeightFogInScene)
						{
							IntermediateRT2 = GraphBuilder.CreateTexture(
								FRDGTextureDesc::Create2D(IntermadiateTargetResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(63000.0f, 63000.0f, 63000.0f, 63000.0f)),
									TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV), TEXT("Cloud.HighQualityAPIntermediate2"));
							IntermediateSecondaryRT2 = GraphBuilder.CreateTexture(
								FRDGTextureDesc::Create2D(IntermadiateTargetResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(63000.0f, 63000.0f, 63000.0f, 63000.0f)),
									TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV), TEXT("Cloud.HighQualityAPIntermediateSecondary2"));
						}
					}

					// No action because we only need to render volumetric clouds so we do not blend in that render target.
					// When we have more elements rendered in that target later, we can clear it to default and blend.
					CloudRC.RenderTargets[0] = FRenderTargetBinding(bShouldUseHighQualityAerialPerspective ? IntermediateRT : DestinationRT, ERenderTargetLoadAction::ENoAction);
					CloudRC.RenderTargets[1] = FRenderTargetBinding(DestinationRTDepth, ERenderTargetLoadAction::ENoAction);
					CloudRC.TracingCoordToZbufferCoordScaleBias = VRT.GetTracingCoordToZbufferCoordScaleBias();
					// Also take into account the view rect min to be able to read correct depth
					CloudRC.TracingCoordToZbufferCoordScaleBias.Z += ViewInfo.CachedViewUniformShaderParameters->ViewRectMin.X / ((VRT.GetMode() == 0 || VRT.GetMode() == 1) ? 2 : 1);
					CloudRC.TracingCoordToZbufferCoordScaleBias.W += ViewInfo.CachedViewUniformShaderParameters->ViewRectMin.Y / ((VRT.GetMode() == 0 || VRT.GetMode() == 1) ? 2 : 1);
					CloudRC.TracingCoordToFullResPixelCoordScaleBias = VRT.GetTracingCoordToFullResPixelCoordScaleBias();

					if (bAccumulateAlphaHoldOut)
					{
						// Requesting the holdout texture will automatically trigger the composition of holdout into alpha during the composition stage.
						CloudRC.CloudAlphaHoldoutTexture = VRT.GetOrCreateVolumetricTracingRTHoldout(GraphBuilder);
					}

					const bool bShouldVolumetricCloudTraceWithMinMaxDepth = ShouldVolumetricCloudTraceWithMinMaxDepth(ViewInfo);
					if (bShouldVolumetricCloudTraceWithMinMaxDepth)
					{
						DestinationSecondaryRT = VRT.GetOrCreateVolumetricSecondaryTracingRT(GraphBuilder);
						CloudRC.SecondaryCloudTracingDataTexture = bShouldUseHighQualityAerialPerspective ? IntermediateSecondaryRT : DestinationSecondaryRT;
					}

					CloudRC.NoiseFrameIndexModPattern = VRT.GetNoiseFrameIndexModPattern();

					check(VRT.GetMode() != 0 || CloudRC.bCloudDebugViewModeEnabled || (bShouldVolumetricCloudTraceWithMinMaxDepth || (!bShouldVolumetricCloudTraceWithMinMaxDepth && HalfResolutionDepthCheckerboardMinMaxTexture)));
					CloudRC.SceneDepthZ = ((VRT.GetMode() == 0 && !bShouldVolumetricCloudTraceWithMinMaxDepth) || VRT.GetMode() == 1) ? HalfResolutionDepthCheckerboardMinMaxTexture : SceneDepthZ;
					CloudRC.SceneDepthMinAndMax = bShouldVolumetricCloudTraceWithMinMaxDepth ? QuarterResolutionDepthMinMaxTexture : SceneDepthZ;
				}
				else
				{
					if (ViewInfo.CachedViewUniformShaderParameters->RenderingReflectionCaptureMask == 0 && !IsVolumetricCloudRenderedInMain(ViewInfo.CachedViewUniformShaderParameters->EnvironmentComponentsFlags))
					{
						continue;
					}

					DestinationRT = SceneTextures.Color.Target;
					const FIntPoint RtSize = SceneTextures.Config.Extent;

					CloudDefaultRtSize = RtSize;

					if (bShouldUseHighQualityAerialPerspective)
					{
						FIntPoint IntermadiateTargetResolution = FIntPoint(RtSize.X, RtSize.Y);
						IntermediateRT = GraphBuilder.CreateTexture(
							FRDGTextureDesc::Create2D(IntermadiateTargetResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f)),
								TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV), TEXT("Cloud.HighQualityAPIntermediate"));

						if (bHeightFogInScene)
						{
							IntermediateRT2 = GraphBuilder.CreateTexture(
								FRDGTextureDesc::Create2D(IntermadiateTargetResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(63000.0f, 63000.0f, 63000.0f, 63000.0f)),
									TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV), TEXT("Cloud.HighQualityAPIntermediate2"));
						}
					}

					// Create a texture for cloud depth data that will be dropped out just after.
					// This texture must also match the MSAA sample count of the color buffer in case this direct to SceneColor rendering happens 
					// while capturing a non-real-time sky light. This is because, in this case, the capture scene rendering happens in the scene color which can have MSAA NumSamples > 1.
					uint8 DepthNumMips = 1;
					uint8 DepthNumSamples = bShouldUseHighQualityAerialPerspective ? IntermediateRT->Desc.NumSamples : DestinationRT->Desc.NumSamples;
					ETextureCreateFlags DepthTexCreateFlags = ETextureCreateFlags(TexCreate_ShaderResource | TexCreate_RenderTargetable | (DepthNumSamples > 1 ? TexCreate_None : TexCreate_UAV));
					DestinationRTDepth = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(RtSize.X, RtSize.Y), PF_G16R16F, FClearValueBinding::Black,
						DepthTexCreateFlags, DepthNumMips, DepthNumSamples), TEXT("Cloud.DummyDepth"));

					if (bShouldUseHighQualityAerialPerspective && ShouldUseComputeForCloudTracing(Scene->GetFeatureLevel()))
					{
						// If using the compute path, we then need to clear the intermediate render target manually as RDG won't do it for us in this case.
						AddClearRenderTargetPass(GraphBuilder, IntermediateRT);
						AddClearRenderTargetPass(GraphBuilder, DestinationRTDepth);
					}

					CloudRC.RenderTargets[0] = FRenderTargetBinding(bShouldUseHighQualityAerialPerspective ? IntermediateRT : DestinationRT, bShouldUseHighQualityAerialPerspective ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad);
					CloudRC.RenderTargets[1] = FRenderTargetBinding(DestinationRTDepth, bShouldUseHighQualityAerialPerspective ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ENoAction);

					CloudRC.SceneDepthZ = SceneDepthZ;
					CloudRC.SceneDepthMinAndMax = SceneDepthZ;
				}

				if (CloudRC.bCloudDebugViewModeEnabled)
				{
					CloudRC.SceneDepthZ = GSystemTextures.GetBlackDummy(GraphBuilder);
					CloudRC.SceneDepthMinAndMax = CloudRC.SceneDepthZ;
				}

				const FProjectedShadowInfo* ProjectedShadowInfo0 = nullptr;
				if (AtmosphericLight0Info)
				{
					ProjectedShadowInfo0 = GetFirstWholeSceneShadowMap(VisibleLightInfos[AtmosphericLight0Info->Id]);
				}
				if (!CloudRC.bSkipAtmosphericLightShadowmap && AtmosphericLight0 && ProjectedShadowInfo0)
				{
					SetVolumeShadowingShaderParameters(GraphBuilder, CloudRC.LightShadowShaderParams0, ViewInfo, AtmosphericLight0Info, ProjectedShadowInfo0);
				}
				else
				{
					SetVolumeShadowingDefaultShaderParameters(GraphBuilder, CloudRC.LightShadowShaderParams0);
				}
				if (!CloudRC.bSkipAtmosphericLightShadowmap && AtmosphericLight0)
				{
					CloudRC.VirtualShadowMapId0 = VisibleLightInfos[AtmosphericLight0Info->Id].GetVirtualShadowMapId(&ViewInfo);
				}
				else
				{
					CloudRC.VirtualShadowMapId0 = INDEX_NONE;
				}

				CloudRC.bAccumulateAlphaHoldOut = bAccumulateAlphaHoldOut;

				// Cannot nest a global buffer into another one and we are limited to only one PassUniformBuffer on PassDrawRenderState.
				//TUniformBufferRef<FVolumeShadowingShaderParametersGlobal0> LightShadowShaderParams0UniformBuffer = TUniformBufferRef<FVolumeShadowingShaderParametersGlobal0>::CreateUniformBufferImmediate(LightShadowShaderParams0, UniformBuffer_SingleFrame);

				FCloudShadowAOData CloudShadowAOData;
				GetCloudShadowAOData(&CloudInfo, ViewInfo, GraphBuilder, CloudShadowAOData);
				CloudRC.VolumetricCloudShadowTexture[0] = CloudShadowAOData.VolumetricCloudShadowMap[0];
				CloudRC.VolumetricCloudShadowTexture[1] = CloudShadowAOData.VolumetricCloudShadowMap[1];

				FIntPoint ViewRectSize = ViewInfo.ViewState && bShouldViewRenderVolumetricCloudRenderTarget ? ViewInfo.ViewState->VolumetricCloudRenderTarget.GetCurrentVolumetricTracingViewRect() : CloudDefaultRtSize;
				RenderVolumetricCloudsInternal(GraphBuilder, CloudRC, InstanceCullingManager, ViewRectSize);

				// Render high quality sky light shaft on clouds.
				if (bShouldUseHighQualityAerialPerspective)
				{
					RDG_EVENT_SCOPE(GraphBuilder, "HighQualityAerialPerspectiveOnCloud");

					FSkyAtmosphereRenderSceneInfo& SkyInfo = *Scene->GetSkyAtmosphereSceneInfo();
					const FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy = SkyInfo.GetSkyAtmosphereSceneProxy();
					const FAtmosphereSetup& AtmosphereSetup = SkyAtmosphereSceneProxy.GetAtmosphereSetup();

					FSkyAtmosphereRenderContext SkyRC;
					SkyRC.bFastSky = false;
					SkyRC.bFastAerialPerspective = false;
					SkyRC.bFastAerialPerspectiveDepthTest = false;
					SkyRC.bSecondAtmosphereLightEnabled = Scene->IsSecondAtmosphereLightEnabled();

					SkyAtmosphereLightShadowData LightShadowData;
					SkyRC.bShouldSampleOpaqueShadow = ShouldSkySampleAtmosphereLightsOpaqueShadow(*Scene, VisibleLightInfos, LightShadowData);
					SkyRC.bUseDepthBoundTestIfPossible = false;
					SkyRC.bForceRayMarching = true;				// We do not have any valid view LUT
					SkyRC.bDepthReadDisabled = true;

					// Disable blending if volumetric render target is used, or if height fog is present (meaning we are going to render in a separate render target before blending over the scene)
					SkyRC.bDisableBlending = (bShouldViewRenderVolumetricCloudRenderTarget || bHeightFogInScene) ? true : false;

					SkyRC.TransmittanceLut = SkyInfo.GetTransmittanceLutTexture(GraphBuilder);
					SkyRC.MultiScatteredLuminanceLut = GraphBuilder.RegisterExternalTexture(SkyInfo.GetMultiScatteredLuminanceLutTexture());

					// Select the AerialPersepctiveOnCloud mode and set required parameters.
					SkyRC.bAPOnCloudMode = true;
					SkyRC.VolumetricCloudDepthTexture = DestinationRTDepth;
					SkyRC.InputCloudLuminanceTransmittanceTexture = IntermediateRT;
					SkyRC.RenderTargets[0] = FRenderTargetBinding(bHeightFogInScene ? IntermediateRT2 : DestinationRT, ERenderTargetLoadAction::ENoAction);

					SkyRC.ViewMatrices = &ViewInfo.ViewMatrices;
					SkyRC.ViewUniformBuffer = bShouldViewRenderVolumetricCloudRenderTarget ? ViewInfo.VolumetricRenderTargetViewUniformBuffer : ViewInfo.ViewUniformBuffer;

					SkyRC.SceneUniformBuffer = GetSceneUniforms().GetBuffer(GraphBuilder);

					SkyRC.Viewport = bShouldViewRenderVolumetricCloudRenderTarget ?
						FIntRect(FIntPoint(0, 0), FIntPoint(DestinationRT->Desc.GetSize().X, DestinationRT->Desc.GetSize().Y)) // this texture is per view, so we don't need to use viewrect inside it
						: ViewInfo.ViewRect;
					SkyRC.bIsReflectionCapture = ViewInfo.bIsReflectionCapture;
					SkyRC.AerialPerspectiveStartDepthInCm = GetValidAerialPerspectiveStartDepthInCm(ViewInfo, SkyAtmosphereSceneProxy);
					SkyRC.NearClippingDistance = ViewInfo.NearClippingDistance;
					SkyRC.FeatureLevel = ViewInfo.FeatureLevel;

					SkyRC.bRenderSkyPixel = false;
					SkyRC.bSceneHasSkyMaterial = false;

					if (ViewInfo.SkyAtmosphereViewLutTexture && ViewInfo.SkyAtmosphereCameraAerialPerspectiveVolume)
					{
						SkyRC.SkyAtmosphereViewLutTexture = GraphBuilder.RegisterExternalTexture(ViewInfo.SkyAtmosphereViewLutTexture);
						SkyRC.SkyAtmosphereCameraAerialPerspectiveVolume = GraphBuilder.RegisterExternalTexture(ViewInfo.SkyAtmosphereCameraAerialPerspectiveVolume);
						SkyRC.SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly = GraphBuilder.RegisterExternalTexture(ViewInfo.SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly.IsValid() ? ViewInfo.SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly : GSystemTextures.VolumetricBlackAlphaOneDummy);
						SkyRC.SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly = GraphBuilder.RegisterExternalTexture(ViewInfo.SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly.IsValid() ? ViewInfo.SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly : GSystemTextures.VolumetricBlackAlphaOneDummy);
					}
					else
					{
						SkyRC.SkyAtmosphereViewLutTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
						SkyRC.SkyAtmosphereCameraAerialPerspectiveVolume = GSystemTextures.GetVolumetricBlackDummy(GraphBuilder);
						SkyRC.SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly = GSystemTextures.GetVolumetricBlackDummy(GraphBuilder);
						SkyRC.SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly = GSystemTextures.GetVolumetricBlackDummy(GraphBuilder);
					}

					GetSkyAtmosphereLightsUniformBuffers(GraphBuilder, SkyRC.LightShadowShaderParams0UniformBuffer, SkyRC.LightShadowShaderParams1UniformBuffer,
						LightShadowData, ViewInfo, SkyRC.bShouldSampleOpaqueShadow, UniformBuffer_SingleDraw);

					SkyRC.bShouldSampleCloudShadow = CloudShadowAOData.bShouldSampleCloudShadow;
					SkyRC.VolumetricCloudShadowMap[0] = CloudShadowAOData.VolumetricCloudShadowMap[0];
					SkyRC.VolumetricCloudShadowMap[1] = CloudShadowAOData.VolumetricCloudShadowMap[1];
					SkyRC.bShouldSampleCloudSkyAO = CloudShadowAOData.bShouldSampleCloudSkyAO;
					SkyRC.VolumetricCloudSkyAO = CloudShadowAOData.VolumetricCloudSkyAO;

					RenderSkyAtmosphereInternal(GraphBuilder, GetSceneTextureShaderParameters(SceneTextures.UniformBuffer), SkyRC);

					if (DestinationSecondaryRT)
					{
						// We need to execute a second tracing for the far depth cloud result in order to correctly fix up edges.
						SkyRC.VolumetricCloudDepthTexture = DestinationRTDepth;
						SkyRC.InputCloudLuminanceTransmittanceTexture = IntermediateSecondaryRT;
						SkyRC.RenderTargets[0] = FRenderTargetBinding(bHeightFogInScene ? IntermediateSecondaryRT2 : DestinationSecondaryRT, ERenderTargetLoadAction::ENoAction);
						RenderSkyAtmosphereInternal(GraphBuilder, GetSceneTextureShaderParameters(SceneTextures.UniformBuffer), SkyRC);
					}

					// Also render fog on top of AP
					if (bHeightFogInScene)
					{
						const bool bShouldRenderVolumetricFog = ShouldRenderVolumetricFog();

						FRDGTextureRef SrcCloudDepth = DestinationRTDepth;
						FRDGTextureRef SrcCloudView  = IntermediateRT2;
						FRDGTextureRef DstCloudView  = DestinationRT;
						RenderFogOnClouds(
							GraphBuilder,
							Scene,
							ViewInfo, 
							SrcCloudDepth,
							SrcCloudView,
							DstCloudView,
							bShouldRenderVolumetricFog,
							bShouldViewRenderVolumetricCloudRenderTarget);

						if (DestinationSecondaryRT)
						{
							// We need to execute a second tracing for the far depth cloud result in order to correctly fix up edges.
							SrcCloudDepth = DestinationRTDepth;
							SrcCloudView = IntermediateSecondaryRT2;
							DstCloudView = DestinationSecondaryRT;
							RenderFogOnClouds(
								GraphBuilder,
								Scene,
								ViewInfo,
								SrcCloudDepth,
								SrcCloudView,
								DstCloudView,
								bShouldRenderVolumetricFog,
								bShouldViewRenderVolumetricCloudRenderTarget);
						}
					}
				}

				if (DebugCloudShadowMap || DebugCloudSkyAO)
				{
					FViewElementPDI ShadowFrustumPDI(&ViewInfo, nullptr, nullptr);

					FRenderVolumetricCloudGlobalParameters VolumetricCloudParams;
					SetupDefaultRenderVolumetricCloudGlobalParameters(GraphBuilder, VolumetricCloudParams, CloudInfo, ViewInfo);

					auto DebugCloudTexture = [&](FDrawDebugCloudShadowCS::FParameters* Parameters)
					{
						if (ShaderPrint::IsEnabled(ViewInfo.ShaderPrintData))
						{
							FDrawDebugCloudShadowCS::FPermutationDomain Permutation;
							TShaderMapRef<FDrawDebugCloudShadowCS> ComputeShader(GetGlobalShaderMap(ViewInfo.GetFeatureLevel()), Permutation);

							ShaderPrint::SetParameters(GraphBuilder, ViewInfo.ShaderPrintData, Parameters->ShaderPrintParameters);

							const FIntVector CloudShadowTextureSize = Parameters->CloudTracedTexture->Desc.GetSize();
							const FIntVector DispatchCount = DispatchCount.DivideAndRoundUp(FIntVector(CloudShadowTextureSize.X, CloudShadowTextureSize.Y, 1), FIntVector(8, 8, 1));
							FComputeShaderUtils::AddPass( GraphBuilder, RDG_EVENT_NAME("DrawDebugCloudShadow"), ComputeShader, Parameters, DispatchCount);
						}
					};

					FMatrix44f TranslatedWorldToWorld = FMatrix44f(FTranslationMatrix(-ViewInfo.ViewMatrices.GetPreViewTranslation()));

					if (DebugCloudShadowMap)
					{
						const int DebugLightIndex = 0;	// only debug atmospheric light 0 for now
						const int32 CloudShadowmapSampleCount = VolumetricCloudParams.VolumetricCloud.CloudShadowmapSampleCount[DebugLightIndex].X;

						AddDrawCanvasPass(GraphBuilder, {}, ViewInfo, FScreenPassRenderTarget(SceneTextures.Color.Target, ViewInfo.ViewRect, ERenderTargetLoadAction::ELoad), [CloudShadowmapSampleCount, &ViewInfo](FCanvas& Canvas)
						{
							const float ViewPortWidth = float(ViewInfo.ViewRect.Width());
							const float ViewPortHeight = float(ViewInfo.ViewRect.Height());
							FLinearColor TextColor(1.0f, 0.5f, 0.0f);
							FString Text = FString::Printf(TEXT("Shadow Sample Count = %.1f"), float(CloudShadowmapSampleCount));
							Canvas.DrawShadowedString(0.05f, ViewPortHeight * 0.4f, *Text, GetStatsFont(), TextColor);
						});

						DrawFrustumWireframe(&ShadowFrustumPDI, FMatrix(VolumetricCloudParams.VolumetricCloud.CloudShadowmapTranslatedWorldToLightClipMatrixInv[DebugLightIndex] * TranslatedWorldToWorld), FColor::Orange, 0);
						FDrawDebugCloudShadowCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDrawDebugCloudShadowCS::FParameters>();
						Parameters->CloudTracedTexture = CloudRC.VolumetricCloudShadowTexture[DebugLightIndex];
						Parameters->CloudTraceDirection = VolumetricCloudParams.VolumetricCloud.CloudShadowmapLightDir[DebugLightIndex];

						uint32 CloudShadowmapResolution = CloudRC.VolumetricCloudShadowTexture[DebugLightIndex]->Desc.Extent.X;
						const float CloudShadowmapResolutionInv = 1.0f / CloudShadowmapResolution;
						Parameters->CloudTextureSizeInvSize = FVector4f(CloudShadowmapResolution, CloudShadowmapResolution, CloudShadowmapResolutionInv, CloudShadowmapResolutionInv);

						Parameters->CloudTranslatedWorldToLightClipMatrixInv = VolumetricCloudParams.VolumetricCloud.CloudShadowmapTranslatedWorldToLightClipMatrixInv[DebugLightIndex];
						DebugCloudTexture(Parameters);
					}

					if (DebugCloudSkyAO && ViewInfo.VolumetricCloudSkyAO != nullptr)
					{
						DrawFrustumWireframe(&ShadowFrustumPDI, FMatrix(VolumetricCloudParams.VolumetricCloud.CloudSkyAOTranslatedWorldToLightClipMatrixInv * TranslatedWorldToWorld), FColor::Blue, 0);
						FDrawDebugCloudShadowCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDrawDebugCloudShadowCS::FParameters>();
						Parameters->CloudTracedTexture = ViewInfo.VolumetricCloudSkyAO;
						Parameters->CloudTextureSizeInvSize = VolumetricCloudParams.VolumetricCloud.CloudSkyAOSizeInvSize;
						Parameters->CloudTraceDirection = VolumetricCloudParams.VolumetricCloud.CloudSkyAOTraceDir;
						Parameters->CloudTranslatedWorldToLightClipMatrixInv = VolumetricCloudParams.VolumetricCloud.CloudSkyAOTranslatedWorldToLightClipMatrixInv;
						DebugCloudTexture(Parameters);
					}
				}
			}
		}
	}

	return bAsyncComputeUsed;
}

bool SetupLightCloudTransmittanceParameters(FRDGBuilder& GraphBuilder, const FScene* Scene, const FViewInfo& View, const FLightSceneInfo* LightSceneInfo, FLightCloudTransmittanceParameters& OutParameters)
{
	auto DefaultLightCloudTransmittanceParameters = [](FRDGBuilder& GraphBuilder, FLightCloudTransmittanceParameters& OutParam)
	{
		OutParam.CloudShadowmapTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		OutParam.CloudShadowmapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		OutParam.CloudShadowmapFarDepthKm = 1.0f;
		OutParam.CloudShadowmapTranslatedWorldToLightClipMatrix = FMatrix44f::Identity;
		OutParam.CloudShadowmapStrength = 0.0f;
	};

	if (!Scene || !LightSceneInfo)
	{
		DefaultLightCloudTransmittanceParameters(GraphBuilder, OutParameters);
		return false;
	}

	FLightSceneProxy* AtmosphereLight0Proxy = Scene->AtmosphereLights[0] ? Scene->AtmosphereLights[0]->Proxy : nullptr;
	FLightSceneProxy* AtmosphereLight1Proxy = Scene->AtmosphereLights[1] ? Scene->AtmosphereLights[1]->Proxy : nullptr;
	const FVolumetricCloudRenderSceneInfo* CloudInfo = Scene->GetVolumetricCloudSceneInfo();
	const bool VolumetricCloudShadowMap0Valid = View.VolumetricCloudShadowRenderTarget[0] != nullptr;
	const bool VolumetricCloudShadowMap1Valid = View.VolumetricCloudShadowRenderTarget[1] != nullptr;
	const bool bLight0CloudShadowEnabled = CloudInfo && LightSceneInfo && VolumetricCloudShadowMap0Valid && AtmosphereLight0Proxy == LightSceneInfo->Proxy && AtmosphereLight0Proxy && AtmosphereLight0Proxy->GetCloudShadowOnSurfaceStrength() > 0.0f;
	const bool bLight1CloudShadowEnabled = CloudInfo && LightSceneInfo && VolumetricCloudShadowMap1Valid && AtmosphereLight1Proxy == LightSceneInfo->Proxy && AtmosphereLight1Proxy && AtmosphereLight1Proxy->GetCloudShadowOnSurfaceStrength() > 0.0f;
	if (bLight0CloudShadowEnabled)
	{
		OutParameters.CloudShadowmapTexture = View.VolumetricCloudShadowRenderTarget[0];
		OutParameters.CloudShadowmapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		OutParameters.CloudShadowmapFarDepthKm = CloudInfo->GetVolumetricCloudCommonShaderParameters().CloudShadowmapFarDepthKm[0].X;
		OutParameters.CloudShadowmapTranslatedWorldToLightClipMatrix = CloudInfo->GetVolumetricCloudCommonShaderParameters().CloudShadowmapTranslatedWorldToLightClipMatrix[0];
		OutParameters.CloudShadowmapStrength = AtmosphereLight0Proxy->GetCloudShadowOnSurfaceStrength();
	}
	else if (bLight1CloudShadowEnabled)
	{
		OutParameters.CloudShadowmapTexture = View.VolumetricCloudShadowRenderTarget[1];
		OutParameters.CloudShadowmapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		OutParameters.CloudShadowmapFarDepthKm = CloudInfo->GetVolumetricCloudCommonShaderParameters().CloudShadowmapFarDepthKm[1].X;
		OutParameters.CloudShadowmapTranslatedWorldToLightClipMatrix = CloudInfo->GetVolumetricCloudCommonShaderParameters().CloudShadowmapTranslatedWorldToLightClipMatrix[1];
		OutParameters.CloudShadowmapStrength = AtmosphereLight1Proxy->GetCloudShadowOnSurfaceStrength();
	}
	else
	{
		DefaultLightCloudTransmittanceParameters(GraphBuilder, OutParameters);
	}

	return bLight0CloudShadowEnabled || bLight1CloudShadowEnabled;
}

bool LightMayCastCloudShadow(const FScene* Scene, const FViewInfo& View, const FLightSceneInfo* LightSceneInfo)
{
	const FLightSceneInfo* Light0 = Scene->AtmosphereLights[0];
	const FLightSceneInfo* Light1 = Scene->AtmosphereLights[1];

	if (Scene->GetVolumetricCloudSceneInfo()
		//&& (View.VolumetricCloudShadowRenderTarget[0] || View.VolumetricCloudShadowRenderTarget[1]) // We cannot check for RTs when this function might be called earlier in the frame (e.g. Lumen), before the RTs have been generated.
		&& ((Light0 && Light0->Proxy == LightSceneInfo->Proxy && Light0->Proxy->GetCastCloudShadows() && Light0->Proxy->GetCloudShadowOnSurfaceStrength() > 0.f && Light0->Proxy->GetCloudShadowStrength() > 0.f)
			|| (Light1 && Light1->Proxy == LightSceneInfo->Proxy && Light1->Proxy->GetCastCloudShadows() && Light1->Proxy->GetCloudShadowOnSurfaceStrength() > 0.f && Light1->Proxy->GetCloudShadowStrength() > 0.f)))
	{
		return true;
	}

	return false;
}
