// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricCloudRendering.h
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "RendererInterface.h"
#include "RenderResource.h"
#include "VolumeLighting.h"


class FScene;
class FViewInfo;
class FLightSceneInfo;
class UVolumetricCloudComponent;
class FVolumetricCloudSceneProxy;
class FLightSceneProxy;

struct FEngineShowFlags;



BEGIN_SHADER_PARAMETER_STRUCT(FVolumetricCloudCommonShaderParameters, )
	SHADER_PARAMETER(FLinearColor, GroundAlbedo)
	SHADER_PARAMETER(FVector3f, CloudLayerCenterKm)
	SHADER_PARAMETER(float, PlanetRadiusKm)
	SHADER_PARAMETER(float, BottomRadiusKm)
	SHADER_PARAMETER(float, TopRadiusKm)
	SHADER_PARAMETER(float, TracingStartDistanceFromCamera)
	SHADER_PARAMETER(float, TracingStartMaxDistance)
	SHADER_PARAMETER(int32, TracingMaxDistanceMode)
	SHADER_PARAMETER(float, TracingMaxDistance)
	SHADER_PARAMETER(uint32, SampleCountClamp)
	SHADER_PARAMETER(int32, SampleCountMin)
	SHADER_PARAMETER(int32, SampleCountMax)
	SHADER_PARAMETER(float, InvDistanceToSampleCountMax)
	SHADER_PARAMETER(int32, ShadowSampleCountMax)
	SHADER_PARAMETER(float, ShadowTracingMaxDistance)
	SHADER_PARAMETER(float, StopTracingTransmittanceThreshold)
	SHADER_PARAMETER(float, SkyLightCloudBottomVisibility)
	SHADER_PARAMETER_ARRAY(FLinearColor, AtmosphericLightCloudScatteredLuminanceScale, [2])
	SHADER_PARAMETER_ARRAY(FVector4f,	CloudShadowmapFarDepthKm, [2]) // SHADER_PARAMETER_ARRAY of float is always a vector behind the scene and we must comply with SPIRV-Cross alignment requirement so using an array Float4 for now for easy indexing from code.
	SHADER_PARAMETER_ARRAY(FVector4f,	CloudShadowmapStrength, [2])
	SHADER_PARAMETER_ARRAY(FVector4f,	CloudShadowmapDepthBias, [2])
	SHADER_PARAMETER_ARRAY(FVector4f,	CloudShadowmapSampleCount, [2])
	SHADER_PARAMETER_ARRAY(FVector4f,CloudShadowmapSizeInvSize, [2])
	SHADER_PARAMETER_ARRAY(FVector4f,CloudShadowmapTracingSizeInvSize, [2])
	SHADER_PARAMETER_ARRAY(FMatrix44f,	CloudShadowmapTranslatedWorldToLightClipMatrix, [2])
	SHADER_PARAMETER_ARRAY(FMatrix44f,	CloudShadowmapTranslatedWorldToLightClipMatrixInv, [2])
	SHADER_PARAMETER_ARRAY(FVector4f, CloudShadowmapTracingPixelScaleOffset, [2])
	SHADER_PARAMETER_ARRAY(FVector4f, CloudShadowmapLightDir, [2])
	SHADER_PARAMETER_ARRAY(FVector4f, CloudShadowmapLightPos, [2])
	SHADER_PARAMETER_ARRAY(FVector4f, CloudShadowmapLightAnchorPos, [2])	// Snapped position on the planet the shadow map rotate around 
	SHADER_PARAMETER(float,		CloudSkyAOFarDepthKm)
	SHADER_PARAMETER(float,		CloudSkyAOStrength)
	SHADER_PARAMETER(float,		CloudSkyAOSampleCount)
	SHADER_PARAMETER(FVector4f,	CloudSkyAOSizeInvSize)
	SHADER_PARAMETER(FMatrix44f,	CloudSkyAOTranslatedWorldToLightClipMatrix)
	SHADER_PARAMETER(FMatrix44f,	CloudSkyAOTranslatedWorldToLightClipMatrixInv)
	SHADER_PARAMETER(FVector3f,	CloudSkyAOTraceDir)
END_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVolumetricCloudCommonGlobalShaderParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FVolumetricCloudCommonShaderParameters, VolumetricCloudCommonParams)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FLightCloudTransmittanceParameters, )
	SHADER_PARAMETER(FMatrix44f, CloudShadowmapTranslatedWorldToLightClipMatrix)
	SHADER_PARAMETER(float, CloudShadowmapFarDepthKm)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float3>, CloudShadowmapTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, CloudShadowmapSampler)
	SHADER_PARAMETER(float, CloudShadowmapStrength)
END_SHADER_PARAMETER_STRUCT()

bool SetupLightCloudTransmittanceParameters(FRDGBuilder& GraphBuilder, const FScene* Scene, const FViewInfo& View, const FLightSceneInfo* LightSceneInfo, FLightCloudTransmittanceParameters& OutParameters);

bool LightMayCastCloudShadow(const FScene* Scene, const FViewInfo& View, const FLightSceneInfo* LightSceneInfo);

/** Contains render data created render side for a FVolumetricCloudSceneProxy objects. */
class FVolumetricCloudRenderSceneInfo
{
public:

	/** Initialization constructor. */
	explicit FVolumetricCloudRenderSceneInfo(FVolumetricCloudSceneProxy& VolumetricCloudSceneProxy);
	~FVolumetricCloudRenderSceneInfo();

	FVolumetricCloudSceneProxy& GetVolumetricCloudSceneProxy() const { return VolumetricCloudSceneProxy; }

	FVolumetricCloudCommonShaderParameters& GetVolumetricCloudCommonShaderParameters() { return VolumetricCloudCommonShaderParameters; }
	const FVolumetricCloudCommonShaderParameters& GetVolumetricCloudCommonShaderParameters() const { return VolumetricCloudCommonShaderParameters; }

	TUniformBufferRef<FVolumetricCloudCommonGlobalShaderParameters>& GetVolumetricCloudCommonShaderParametersUB() { return VolumetricCloudCommonShaderParametersUB; }
	const TUniformBufferRef<FVolumetricCloudCommonGlobalShaderParameters>& GetVolumetricCloudCommonShaderParametersUB() const { return VolumetricCloudCommonShaderParametersUB; }

private:

	FVolumetricCloudSceneProxy& VolumetricCloudSceneProxy;

	FVolumetricCloudCommonShaderParameters VolumetricCloudCommonShaderParameters;
	TUniformBufferRef<FVolumetricCloudCommonGlobalShaderParameters> VolumetricCloudCommonShaderParametersUB;
};


bool ShouldRenderVolumetricCloud(const FScene* Scene, const FEngineShowFlags& EngineShowFlags);
bool ShouldRenderVolumetricCloudWithBlueNoise_GameThread(const FScene* Scene, const FSceneView& View);

bool ShouldViewVisualizeVolumetricCloudConservativeDensity(const FViewInfo& ViewInfo, const FEngineShowFlags& EngineShowFlags);
uint32 GetVolumetricCloudDebugViewMode(const FEngineShowFlags& ShowFlags);
bool ShouldVolumetricCloudTraceWithMinMaxDepth(const FViewInfo& ViewInfo);
bool ShouldVolumetricCloudTraceWithMinMaxDepth(const TArray<FViewInfo>& Views);
bool VolumetricCloudWantsSeparatedAtmosphereMieRayLeigh(const FScene* Scene);

bool ShouldVolumetricCloudsApplyFogDuringUpsampling(const FViewInfo& ViewInfo);
bool ShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent(const FViewInfo& ViewInfo);

// Structure with data necessary to specify a cloud render.
struct FCloudRenderContext
{
	///////////////////////////////////
	// Per scene parameters

	FVolumetricCloudRenderSceneInfo* CloudInfo;
	FMaterialRenderProxy* CloudVolumeMaterialProxy;

	FRDGTextureRef SceneDepthZ = nullptr;
	FRDGTextureRef SceneDepthMinAndMax = nullptr;

	///////////////////////////////////
	// Per view parameters

	FViewInfo* MainView;
	TUniformBufferRef<FViewUniformShaderParameters> ViewUniformBuffer;
	FRenderTargetBindingSlots RenderTargets;

	FRDGTextureRef SecondaryCloudTracingDataTexture = nullptr;
	FRDGTextureRef CloudAlphaHoldoutTexture = nullptr;

	bool bDisableCloudBlending;
	bool bShouldViewRenderVolumetricRenderTarget;
	bool bSkipAerialPerspective;
	bool bSkipHeightFog;
	bool bIsReflectionRendering;				// Reflection capture and real time sky capture
	bool bIsSkyRealTimeReflectionRendering;		// Real time sky capture only
	bool bSkipAtmosphericLightShadowmap;
	bool bSecondAtmosphereLightEnabled;

	bool bAsyncCompute;
	bool bCloudDebugViewModeEnabled;
	bool bAccumulateAlphaHoldOut;

	FUintVector4 TracingCoordToZbufferCoordScaleBias;
	FUintVector4 TracingCoordToFullResPixelCoordScaleBias;
	uint32 NoiseFrameIndexModPattern;

	FVolumeShadowingShaderParametersGlobal0 LightShadowShaderParams0;
	FRDGTextureRef VolumetricCloudShadowTexture[2];

	int VirtualShadowMapId0 = INDEX_NONE;

	FRDGTextureRef DefaultCloudColorCubeTexture = nullptr;
	FRDGTextureRef DefaultCloudColor02DTexture = nullptr;
	FRDGTextureRef DefaultCloudColor12DTexture = nullptr;
	FRDGTextureRef DefaultCloudDepthTexture = nullptr;
	FRDGTextureRef DefaultCloudAlphaHoldout = nullptr;
	FRDGTextureUAVRef DefaultCloudColorCubeTextureUAV = nullptr;
	FRDGTextureUAVRef DefaultCloudColor02DTextureUAV = nullptr;
	FRDGTextureUAVRef DefaultCloudColor12DTextureUAV = nullptr;
	FRDGTextureUAVRef DefaultCloudDepthTextureUAV = nullptr;
	FRDGTextureUAVRef DefaultCloudAlphaHoldoutUAV = nullptr;

	FRDGTextureUAVRef ComputeOverlapCloudColorCubeTextureUAVWithoutBarrier = nullptr;

	FCloudRenderContext();

	void CreateDefaultTexturesIfNeeded(FRDGBuilder& GraphBuilder);

private:
};

BEGIN_SHADER_PARAMETER_STRUCT(FVolumetricCloudShadowAOParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowMap0)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowMap1)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SkyAO)
END_SHADER_PARAMETER_STRUCT()

FVolumetricCloudShadowAOParameters GetCloudShadowAOParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVolumetricCloudRenderSceneInfo* CloudInfo);

struct FCloudShadowAOData
{
	bool bShouldSampleCloudShadow;
	bool bShouldSampleCloudSkyAO;
	FRDGTextureRef VolumetricCloudShadowMap[2];
	FRDGTextureRef VolumetricCloudSkyAO;
};

void GetCloudShadowAOData(const FVolumetricCloudRenderSceneInfo* CloudInfo, const FViewInfo& View, FRDGBuilder& GraphBuilder, FCloudShadowAOData& OutData);


