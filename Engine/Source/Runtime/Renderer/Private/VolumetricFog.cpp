// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VolumetricFog.cpp
=============================================================================*/

#include "VolumetricFog.h"
#include "CoreMinimal.h"
#include "BasePassRendering.h"
#include "CanvasTypes.h"
#include "Components/LightComponent.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DecalRenderingShared.h"
#include "DeferredShadingRenderer.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "DistanceFieldLightingShared.h"
#include "DynamicRHI.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "ExtensibleUniformBuffer.h"
#include "FogRendering.h"
#include "Froxel/FroxelGridUtils.h"
#include "GenerateConservativeDepthBuffer.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "GlobalDistanceField.h"
#include "GlobalDistanceFieldParameters.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "LightFunctionAtlas.h"
#include "LightRendering.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "LocalFogVolumeRendering.h"
#include "Logging/LogMacros.h"
#include "Lumen/LumenTranslucencyVolumeLighting.h"
#include "Math/Color.h"
#include "Math/Halton.h"
#include "Math/MathFwd.h"
#include "Math/Sphere.h"
#include "MeshMaterialShader.h"
#include "Misc/LargeWorldCoordinates.h"
#include "MultiGPU.h"
#include "Nanite/NaniteRayTracing.h"
#include "PipelineStateCache.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "PSOPrecache.h"
#include "RayTracing/RayTracingScene.h"
#include "RayTracingPayloadType.h"
#include "RayTracingShaderBindingLayout.h"
#include "RendererInterface.h"
#include "LogRenderer.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHIBufferInitializer.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIResourceUtils.h"
#include "RHIShaderParameters.h"
#include "RHIStaticStates.h"
#include "SceneCore.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneRendererInterface.h"
#include "SceneRendering.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneTypes.h"
#include "SceneView.h"
#include "ScreenRendering.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShadowRendering.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "SystemTextures.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VolumeLighting.h"
#include "VolumeRendering.h"
#include "VolumetricCloudRendering.h"
#include "VolumetricFogShared.h"
#include "SceneViewState.h"
#include "PSOPrecacheValidation.h"

using namespace LightFunctionAtlas;

int32 GVolumetricFog = 1;
FAutoConsoleVariableRef CVarVolumetricFog(
	TEXT("r.VolumetricFog"),
	GVolumetricFog,
	TEXT("Whether to allow the volumetric fog feature."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GVolumetricFogInjectShadowedLightsSeparately = 1;
FAutoConsoleVariableRef CVarVolumetricFogInjectShadowedLightsSeparately(
	TEXT("r.VolumetricFog.InjectShadowedLightsSeparately"),
	GVolumetricFogInjectShadowedLightsSeparately,
	TEXT("Whether to allow the volumetric fog feature."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GVolumetricFogDepthDistributionScale = 32.0f;
FAutoConsoleVariableRef CVarVolumetricFogDepthDistributionScale(
	TEXT("r.VolumetricFog.DepthDistributionScale"),
	GVolumetricFogDepthDistributionScale,
	TEXT("Scales the slice depth distribution."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GVolumetricFogGridPixelSize = 16;
FAutoConsoleVariableRef CVarVolumetricFogGridPixelSize(
	TEXT("r.VolumetricFog.GridPixelSize"),
	GVolumetricFogGridPixelSize,
	TEXT("XY Size of a cell in the voxel grid, in pixels."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GVolumetricFogGridSizeZ = 64;
FAutoConsoleVariableRef CVarVolumetricFogGridSizeZ(
	TEXT("r.VolumetricFog.GridSizeZ"),
	GVolumetricFogGridSizeZ,
	TEXT("How many Volumetric Fog cells to use in z."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GVolumetricFogTemporalReprojection = 1;
FAutoConsoleVariableRef CVarVolumetricFogTemporalReprojection(
	TEXT("r.VolumetricFog.TemporalReprojection"),
	GVolumetricFogTemporalReprojection,
	TEXT("Whether to use temporal reprojection on volumetric fog."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GVolumetricFogJitter = 1;
FAutoConsoleVariableRef CVarVolumetricFogJitter(
	TEXT("r.VolumetricFog.Jitter"),
	GVolumetricFogJitter,
	TEXT("Whether to apply jitter to each frame's volumetric fog computation, achieving temporal super sampling."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GVolumetricFogHistoryWeight = .9f;
FAutoConsoleVariableRef CVarVolumetricFogHistoryWeight(
	TEXT("r.VolumetricFog.HistoryWeight"),
	GVolumetricFogHistoryWeight,
	TEXT("How much the history value should be weighted each frame.  This is a tradeoff between visible jittering and responsiveness."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GVolumetricFogHistoryMissSupersampleCount = 4;
FAutoConsoleVariableRef CVarVolumetricFogHistoryMissSupersampleCount(
	TEXT("r.VolumetricFog.HistoryMissSupersampleCount"),
	GVolumetricFogHistoryMissSupersampleCount,
	TEXT("Number of lighting samples to compute for voxels whose history value is not available.\n")
	TEXT("This reduces noise when panning or on camera cuts, but introduces a variable cost to volumetric fog computation.  Valid range [1, 16]."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GInverseSquaredLightDistanceBiasScale = 1.0f;
FAutoConsoleVariableRef CVarInverseSquaredLightDistanceBiasScale(
	TEXT("r.VolumetricFog.InverseSquaredLightDistanceBiasScale"),
	GInverseSquaredLightDistanceBiasScale,
	TEXT("Scales the amount added to the inverse squared falloff denominator.  This effectively removes the spike from inverse squared falloff that causes extreme aliasing."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GVolumetricFogEmissive = 1;
FAutoConsoleVariableRef CVarVolumetricFogEmissive(
	TEXT("r.VolumetricFog.Emissive"),
	GVolumetricFogEmissive,
	TEXT("Whether to allow the volumetric fog emissive component."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GVolumetricFogRectLightTexture = 0;
FAutoConsoleVariableRef CVarVolumetricRectLightTexture(
	TEXT("r.VolumetricFog.RectLightTexture"),
	GVolumetricFogRectLightTexture,
	TEXT("Whether to allow the volumetric fog to use rect light source texture."),
	ECVF_RenderThreadSafe
);

int32 GVolumetricFogConservativeDepth = 1;
FAutoConsoleVariableRef CVarVolumetricFogConservativeDepth(
	TEXT("r.VolumetricFog.ConservativeDepth"),
	GVolumetricFogConservativeDepth,
	TEXT("[Experimental] Whether to allow the volumetric to use conservative depth to accelerate computations."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int GVolumetricFogInjectRaytracedLights = 0;
FAutoConsoleVariableRef CVarVolumetricInjectRaytracedLights(
	TEXT("r.VolumetricFog.InjectRaytracedLights"),
	GVolumetricFogInjectRaytracedLights,
	TEXT("Whether lights with ray traced shadows are injected into volumetric fog"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

float GLightScatteringSampleJitterMultiplier = 0;
FAutoConsoleVariableRef CVarLightScatteringSampleJitterMultiplier(
	TEXT("r.VolumetricFog.LightScatteringSampleJitterMultiplier"),
	GLightScatteringSampleJitterMultiplier,
	TEXT("Multiplier for random offset value used to jitter each world sample position when generating the 3D fog volume. Enable/disable with r.VolumetricFog.Jitter"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static TAutoConsoleVariable<float> CVarVolumetricFogLightSoftFading(
	TEXT("r.VolumetricFog.LightSoftFading"),
	0,
	TEXT("Enabled when >0, controls the soft fading of spot and rect light edges in order to make them appear smoother, resulting in less flickering. A value of 1 is a good starting point and it means that fading will be applied over the size of 1 froxel on the screen."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricFogGridCenterOffsetFromDepthBuffer(
	TEXT("r.VolumetricFog.GridCenterOffsetFromDepthBuffer"),
	0.5f,
	TEXT("Offset in grid units to move grid center sample out form the depth buffer along the Z direction. -1 means disabled. This reduces sample self intersection with geometry when tracing, and thus reduces flickering in those areas, as well as results in less leaking sometimes. Set to -1 to disable."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarVolumetricFogOffsetThresholdToAcceptDepthBufferOffset(
	TEXT("r.VolumetricFog.OffsetThresholdToAcceptDepthBufferOffset"),
	1.0f,
	TEXT("Offset in grid units to accept a sample to be moved forward in front of the depth buffer. This is to avoid moving all samples behind the depth buffer forward which would affect the lighting of translucent and volumetric at edges of mesh. Default to 1.0 to only allow moving the first layer of froxel intersecting depth."),
	ECVF_RenderThreadSafe
);

static const TCHAR* VolumetricFogGlobalPSOCollectorName = TEXT("VolumetricFogGlobalPSOCollector");

int32 GetVolumetricFogGridPixelSize()
{
	return FMath::Max(1, GVolumetricFogGridPixelSize);
}

static int32 GetVolumetricFogGridSizeZ()
{
	return FMath::Max(1, GVolumetricFogGridSizeZ);
}

static FIntPoint GetVolumetricFogTextureResourceRes(const FViewInfo& View)
{
	// Allocate texture using scene render targets size so we do not reallocate every frame when dynamic resolution is used in order to avoid resources allocation hitches.
	FIntPoint BufferSize = View.GetSceneTexturesConfig().Extent;
	// Make sure the buffer size has some minimum resolution to make sure everything is always valid.
	BufferSize.X = FMath::Max(1, BufferSize.X);
	BufferSize.Y = FMath::Max(1, BufferSize.Y);
	return BufferSize;
}

static bool PlatformSupportsVolumetricFogUbershader(FStaticShaderPlatform Platform)
{
	// GL does not have enough samplers for Ubershader permutations
	return !IsOpenGLPlatform(Platform);
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FVolumetricFogGlobalData, "VolumetricFog");

DECLARE_GPU_STAT(VolumetricFog);

FVolumetricFogGlobalData::FVolumetricFogGlobalData()
{}

FVector3f VolumetricFogTemporalRandom(uint32 FrameNumber)
{
	// Center of the voxel
	FVector3f RandomOffsetValue(.5f, .5f, .5f);

	if (GVolumetricFogJitter && GVolumetricFogTemporalReprojection)
	{
		RandomOffsetValue = FVector3f(Halton(FrameNumber & 1023, 2), Halton(FrameNumber & 1023, 3), Halton(FrameNumber & 1023, 5));
	}

	return RandomOffsetValue;
}

float GetVolumetricFogLightSoftFading()
{
	return FMath::Max(0.0f, CVarVolumetricFogLightSoftFading.GetValueOnAnyThread());
}

void SetupVolumetricFogIntegrationParameters(
	FVolumetricFogIntegrationParameters& Out,
	FViewInfo& View,
	const FVolumetricFogIntegrationParameterData& IntegrationData)
{
	Out.VolumetricFog = View.VolumetricFogResources.VolumetricFogGlobalData;

	FMatrix44f UnjitteredInvTranslatedViewProjectionMatrix = FMatrix44f(View.ViewMatrices.ComputeInvProjectionNoAAMatrix() * View.ViewMatrices.GetTranslatedViewMatrix().GetTransposed());
	Out.UnjitteredClipToTranslatedWorld = UnjitteredInvTranslatedViewProjectionMatrix;

	FMatrix TranslatedWorldToWorld = FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation());
	FMatrix44f UnjitteredTranslatedViewProjectionMatrix = FMatrix44f(TranslatedWorldToWorld * View.PrevViewInfo.ViewMatrices.GetWorldToView() * View.PrevViewInfo.ViewMatrices.ComputeProjectionNoAAMatrix());
	Out.UnjitteredPrevTranslatedWorldToClip = UnjitteredTranslatedViewProjectionMatrix;

	int32 OffsetCount = IntegrationData.FrameJitterOffsetValues.Num();
	for (int32 i = 0; i < OffsetCount; ++i)
	{
		Out.FrameJitterOffsets[i] = IntegrationData.FrameJitterOffsetValues.GetData()[i];
	}

	extern float GVolumetricFogHistoryWeight;
	Out.HistoryWeight = IntegrationData.bTemporalHistoryIsValid ? GVolumetricFogHistoryWeight : 0.0f;

	extern int32 GVolumetricFogHistoryMissSupersampleCount;
	Out.HistoryMissSuperSampleCount = FMath::Clamp(GVolumetricFogHistoryMissSupersampleCount, 1, 16);

	Out.GridCenterOffsetFromDepthBuffer = CVarVolumetricFogGridCenterOffsetFromDepthBuffer.GetValueOnRenderThread();
	Out.GridCenterOffsetThresholdToAcceptDepthBufferOffset = CVarVolumetricFogOffsetThresholdToAcceptDepthBufferOffset.GetValueOnRenderThread();
}

static const uint32 VolumetricFogGridInjectionGroupSize = 4;

namespace
{
class FPermutationUseEmissive : SHADER_PERMUTATION_BOOL("USE_EMISSIVE");
class FPermutationLocalFogVolume : SHADER_PERMUTATION_BOOL("USE_LOCAL_FOG_VOLUMES");
}

class FVolumetricFogMaterialSetupCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVolumetricFogMaterialSetupCS);
	SHADER_USE_PARAMETER_STRUCT(FVolumetricFogMaterialSetupCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FPermutationUseEmissive, FPermutationLocalFogVolume>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FLinearColor, GlobalAlbedo)
		SHADER_PARAMETER(FLinearColor, GlobalEmissive)
		SHADER_PARAMETER(float, GlobalExtinctionScale)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, Fog)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumetricFogIntegrationParameters, VolumetricFogParameters)

		SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWVBufferA)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWVBufferB)
	END_SHADER_PARAMETER_STRUCT()

public:

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), VolumetricFogGridInjectionGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVolumetricFogMaterialSetupCS, "/Engine/Private/VolumetricFog.usf", "MaterialSetupCS", SF_Compute);

/** Vertex shader used to write to a range of slices of a 3d volume texture. */
class FWriteToBoundingSphereVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FWriteToBoundingSphereVS);
	SHADER_USE_PARAMETER_STRUCT(FWriteToBoundingSphereVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumetricFogIntegrationParameters, VolumetricFogParameters)
		SHADER_PARAMETER(FMatrix44f, ViewToVolumeClip)
		SHADER_PARAMETER(FVector2f, ClipRatio)
		SHADER_PARAMETER(FVector4f, ViewSpaceBoundingSphere)
		SHADER_PARAMETER(int32, MinZ)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		if (RHISupportsGeometryShaders(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_VertexToGeometryShader);
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FWriteToBoundingSphereVS, "/Engine/Private/VolumetricFog.usf", "WriteToBoundingSphereVS", SF_Vertex);

BEGIN_SHADER_PARAMETER_STRUCT(FInjectShadowedLocalLightCommonParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FDeferredLightUniformStruct, DeferredLight)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLightFunctionAtlasGlobalParameters, LightFunctionAtlas)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WhiteDummyTexture)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVolumetricFogIntegrationParameters, VolumetricFogParameters)
	SHADER_PARAMETER(float, PhaseG)
	SHADER_PARAMETER(float, InverseSquaredLightDistanceBiasScale)
	SHADER_PARAMETER(uint32, LightFunctionAtlasLightIndex)
END_SHADER_PARAMETER_STRUCT()

static bool SetupInjectShadowedLocalLightCommonParameters(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	const FVolumetricFogIntegrationParameterData& IntegrationData,
	const FExponentialHeightFogSceneInfo& FogInfo,
	const FLightSceneInfo* LightSceneInfo,
	FInjectShadowedLocalLightCommonParameters& OutCommonParameters)
{
	// We also bind the default light function texture because when we are out of atlas tile, we fallback to use a white light function so we need the RHI to be created
	OutCommonParameters.WhiteDummyTexture = GSystemTextures.GetWhiteDummy(GraphBuilder);
	SetupVolumetricFogIntegrationParameters(OutCommonParameters.VolumetricFogParameters, View, IntegrationData);

	OutCommonParameters.ViewUniformBuffer = View.ViewUniformBuffer;
	OutCommonParameters.SceneTextures = GetSceneTextureShaderParameters(View);
	OutCommonParameters.PhaseG = FogInfo.VolumetricFogScatteringDistribution;
	OutCommonParameters.InverseSquaredLightDistanceBiasScale = GInverseSquaredLightDistanceBiasScale;

	FDeferredLightUniformStruct* DeferredLightStruct = GraphBuilder.AllocParameters<FDeferredLightUniformStruct>();
	*DeferredLightStruct = GetDeferredLightParameters(View, *LightSceneInfo);
	OutCommonParameters.DeferredLight = GraphBuilder.CreateUniformBuffer(DeferredLightStruct);

	return true;
}

static bool PlatformSupportsShadowedLocalLights(FStaticShaderPlatform Platform)
{
	if (IsMobilePlatform(Platform))
	{
		return (IsMobileMovableSpotlightShadowsEnabled(Platform) || FReadOnlyCVARCache::EnablePointLightShadows(Platform));
	}
	return true;
}

/** Shader that adds direct lighting contribution from the given light to the current volume lighting cascade. */
class FInjectShadowedLocalLightPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInjectShadowedLocalLightPS);
	SHADER_USE_PARAMETER_STRUCT(FInjectShadowedLocalLightPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FInjectShadowedLocalLightCommonParameters, Common)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParameters, VolumeShadowingShaderParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ConservativeDepthTexture)
		SHADER_PARAMETER(uint32, UseConservativeDepthTexture)
		SHADER_PARAMETER(int32, VirtualShadowMapId)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	class FDynamicallyShadowed		: SHADER_PERMUTATION_BOOL("DYNAMICALLY_SHADOWED");
	class FTemporalReprojection		: SHADER_PERMUTATION_BOOL("USE_TEMPORAL_REPROJECTION");
	class FSampleLightFunctionAtlas	: SHADER_PERMUTATION_BOOL("USE_LIGHT_FUNCTION_ATLAS");
	class FEnableShadows			: SHADER_PERMUTATION_BOOL("ENABLE_SHADOW_COMPUTATION");
	class FVirtualShadowMap			: SHADER_PERMUTATION_BOOL("VIRTUAL_SHADOW_MAP");
	class FRectLightTexture			: SHADER_PERMUTATION_BOOL("USE_RECT_LIGHT_TEXTURE");
	class FLightSoftFading			: SHADER_PERMUTATION_BOOL("USE_LIGHT_SOFT_FADING");

	using FPermutationDomain = TShaderPermutationDomain<
		FDynamicallyShadowed,
		FTemporalReprojection,
		FSampleLightFunctionAtlas,
		FEnableShadows,
		FVirtualShadowMap,
		FRectLightTexture,
		FLightSoftFading>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
	}
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FVirtualShadowMap>() && !DoesPlatformSupportVirtualShadowMaps(Parameters.Platform))
		{
			return false;
		}

		return PlatformSupportsShadowedLocalLights(Parameters.Platform);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FLightSoftFading>() && GetVolumetricFogLightSoftFading() <= 0.0f)
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		return FGlobalShader::ShouldPrecachePermutation(Parameters);
	}
};

IMPLEMENT_GLOBAL_SHADER(FInjectShadowedLocalLightPS, "/Engine/Private/VolumetricFog.usf", "InjectShadowedLocalLightPS", SF_Pixel);

#if RHI_RAYTRACING

class FInjectShadowedLocalLightRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInjectShadowedLocalLightRGS);
	SHADER_USE_ROOT_PARAMETER_STRUCT(FInjectShadowedLocalLightRGS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FInjectShadowedLocalLightCommonParameters, Common)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, OutVolumeTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER(int32, FirstSlice)
	END_SHADER_PARAMETER_STRUCT()

	class FTemporalReprojection		: SHADER_PERMUTATION_BOOL("USE_TEMPORAL_REPROJECTION");
	class FSampleLightFunctionAtlas : SHADER_PERMUTATION_BOOL("USE_LIGHT_FUNCTION_ATLAS");
	class FRectLightTexture			: SHADER_PERMUTATION_BOOL("USE_RECT_LIGHT_TEXTURE");
	class FLightSoftFading			: SHADER_PERMUTATION_BOOL("USE_LIGHT_SOFT_FADING");

	using FPermutationDomain = TShaderPermutationDomain<
		FTemporalReprojection,
		FSampleLightFunctionAtlas,
		FRectLightTexture,
		FLightSoftFading>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_RAYTRACED_SHADOWS"), TEXT("1"));

		// Only ray traced shadowed lights use this RGS
		OutEnvironment.SetDefine(TEXT("ENABLE_SHADOW_COMPUTATION"), TEXT("1"));
	}
};

IMPLEMENT_GLOBAL_SHADER(FInjectShadowedLocalLightRGS, "/Engine/Private/VolumetricFog.usf", "InjectShadowedLocalLightRGS", SF_RayGen);

class FRayTraceDirectionalLightVolumeShadowMapRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTraceDirectionalLightVolumeShadowMapRGS);
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRayTraceDirectionalLightVolumeShadowMapRGS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardDirectionalLightShadowMapParameters, ForwardDirLightShadowStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumetricFogIntegrationParameters, VolumetricFogParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, OutShadowVolumeTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER(float, LightScatteringSampleJitterMultiplier)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_RAYTRACED_SHADOWS"), TEXT("1"));
	}
};

IMPLEMENT_GLOBAL_SHADER(FRayTraceDirectionalLightVolumeShadowMapRGS, "/Engine/Private/VolumetricFog.usf", "InjectShadowedDirectionalLightRGS", SF_RayGen);

bool LightHasRayTracedShadows(const FLightSceneInfo* LightSceneInfo, const FSceneViewFamily& ViewFamily);

static void RenderRaytracedDirectionalShadowVolume(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	const FScene& Scene,
	const FVolumetricFogIntegrationParameterData& IntegrationData,
	FRDGTextureRef& OutRaytracedShadowsVolume)
{	
	const bool bUseRaytracedShadows = IsRayTracingEnabled(Scene.GetShaderPlatform())
		&& View.IsRayTracingAllowedForView()
		&& GRHISupportsRayTracing
		&& GRHISupportsRayTracingShaders
		&& GVolumetricFogInjectRaytracedLights;

	if (!bUseRaytracedShadows)
	{
		return;
	}

	// Following how RenderLightFunctionForVolumetricFog is selecting the main directional light, even though we could support all of them.
	const FLightSceneProxy* SelectedForwardDirectionalLightProxy = View.ForwardLightingResources.SelectedForwardDirectionalLightProxy;

	const FLightSceneInfo* DirectionalLightSceneInfo = nullptr;
	for (const FLightSceneInfo* LightSceneInfo : Scene.DirectionalLights)
	{
		if (LightSceneInfo->ShouldRenderLightViewIndependent()
			&& LightSceneInfo->ShouldRenderLight(View, true)
			&& LightHasRayTracedShadows(LightSceneInfo, *View.Family)
			&& LightSceneInfo->Proxy == SelectedForwardDirectionalLightProxy)
		{
			DirectionalLightSceneInfo = LightSceneInfo;
			break;
		}
	}

	if (DirectionalLightSceneInfo)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "VolumetricFog::RaytraceDirLightShadow");

		int32 VolumetricFogGridPixelSize;
		const FIntVector VolumetricFogResourceGridSize = GetVolumetricFogResourceGridSize(View, VolumetricFogGridPixelSize);
		FRDGTextureDesc RaytracedShadowsVolumeDesc(FRDGTextureDesc::Create3D(
			VolumetricFogResourceGridSize,
			PF_R16F,
			FClearValueBinding::Black,
			TexCreate_ShaderResource | TexCreate_UAV | TexCreate_ReduceMemoryWithTilingMode | TexCreate_3DTiling));

		OutRaytracedShadowsVolume = GraphBuilder.CreateTexture(RaytracedShadowsVolumeDesc, TEXT("VolumetricFog.RaytracedShadowVolume"));

		FRayTraceDirectionalLightVolumeShadowMapRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRayTraceDirectionalLightVolumeShadowMapRGS::FParameters>();
		PassParameters->OutShadowVolumeTexture = GraphBuilder.CreateUAV(OutRaytracedShadowsVolume);
		PassParameters->TLAS = View.GetRayTracingSceneLayerViewChecked(ERayTracingSceneLayer::Base);
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder, View);
		PassParameters->SceneTextures = GetSceneTextureShaderParameters(View);
		PassParameters->NaniteRayTracing = Nanite::GRayTracingManager.GetUniformBuffer();
		PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
		PassParameters->ForwardDirLightShadowStruct = View.ForwardLightingResources.ForwardDirLightShadowUniformBuffer;
		PassParameters->LightScatteringSampleJitterMultiplier = GVolumetricFogJitter ? GLightScatteringSampleJitterMultiplier : 0;
		SetupVolumetricFogIntegrationParameters(PassParameters->VolumetricFogParameters, View, IntegrationData);

		TShaderRef<FRayTraceDirectionalLightVolumeShadowMapRGS> RayGenerationShader = View.ShaderMap->GetShader<FRayTraceDirectionalLightVolumeShadowMapRGS>();
		ClearUnusedGraphResources(RayGenerationShader, PassParameters);
		
		const uint32 DispatchSize = VolumetricFogResourceGridSize.X * VolumetricFogResourceGridSize.Y * VolumetricFogResourceGridSize.Z;
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RayTracedShadowedDirectionalLight"),
			PassParameters,
			ERDGPassFlags::Compute,
			[&View, RayGenerationShader, PassParameters, DispatchSize](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
				SetShaderParameters(GlobalResources, RayGenerationShader, *PassParameters);	
				
				FRHIUniformBuffer* SceneUniformBuffer = PassParameters->Scene->GetRHI();
				FRHIUniformBuffer* NaniteRayTracingUniformBuffer = PassParameters->NaniteRayTracing->GetRHI();
				TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, RHICmdList);

				RHICmdList.RayTraceDispatch(View.MaterialRayTracingData.PipelineState, RayGenerationShader.GetRayTracingShader(), View.MaterialRayTracingData.ShaderBindingTable, GlobalResources, DispatchSize, 1);
			}
		);
	}
}

void FDeferredShadingSceneRenderer::PrepareRayTracingVolumetricFogShadows(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders)
{
	const bool bEnabled = View.bHasRayTracingShadows && View.IsRayTracingAllowedForView() && ::ShouldRenderVolumetricFog(&Scene, *View.Family) && GVolumetricFogInjectRaytracedLights;
	if (!bEnabled)
	{
		return;
	}

	for (int32 TemporalReprojection = 0; TemporalReprojection < 2; ++TemporalReprojection)
	{
		for (int32 UseLightFunction = 0; UseLightFunction < 2; ++UseLightFunction)
		{
			for (int32 UseRectLightTexture = 0; UseRectLightTexture < 2; ++UseRectLightTexture)
			{
				FInjectShadowedLocalLightRGS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FInjectShadowedLocalLightRGS::FTemporalReprojection>((bool)TemporalReprojection);
				PermutationVector.Set<FInjectShadowedLocalLightRGS::FSampleLightFunctionAtlas>((bool)UseLightFunction);
				PermutationVector.Set<FInjectShadowedLocalLightRGS::FRectLightTexture>((bool)UseRectLightTexture);

				TShaderMapRef<FInjectShadowedLocalLightRGS> RayGenerationShader(View.ShaderMap, PermutationVector);
				OutRayGenShaders.Add(RayGenerationShader.GetRayTracingShader());
			}
		}
	}

	{
		TShaderMapRef<FRayTraceDirectionalLightVolumeShadowMapRGS> RayGenerationShader(View.ShaderMap);
		OutRayGenShaders.Add(RayGenerationShader.GetRayTracingShader());
	}	
}

#endif // RHI_RAYTRACING

const FProjectedShadowInfo* GetShadowForInjectionIntoVolumetricFog(const FVisibleLightInfo& VisibleLightInfo)
{
	for (int32 ShadowIndex = 0; ShadowIndex < VisibleLightInfo.ShadowsToProject.Num(); ShadowIndex++)
	{
		FProjectedShadowInfo* ProjectedShadowInfo = VisibleLightInfo.ShadowsToProject[ShadowIndex];

		if (ProjectedShadowInfo->bAllocated
			&& ProjectedShadowInfo->bWholeSceneShadow
			&& !ProjectedShadowInfo->bRayTracedDistanceField)
		{
			return ProjectedShadowInfo;
		}
	}
	return nullptr;
}

bool LightHasRayTracedShadows(const FLightSceneInfo* LightSceneInfo, const FSceneViewFamily& ViewFamily)
{
	return GetLightOcclusionType(*LightSceneInfo->Proxy, ViewFamily) == ELightOcclusionType::Raytraced && GVolumetricFogInjectRaytracedLights;
}

bool LightNeedsSeparateInjectionIntoVolumetricFogForOpaqueShadow(const FViewInfo& View, const FLightSceneInfo* LightSceneInfo, const FVisibleLightInfo& VisibleLightInfo, const FScene& InScene)
{
#if RHI_RAYTRACING
	bool bTestRayTracedShadows = View.bHasRayTracingShadows && View.IsRayTracingAllowedForView();
#else
	bool bTestRayTracedShadows = false;
#endif
	const FLightSceneProxy* LightProxy = LightSceneInfo->Proxy;

	if (GVolumetricFogInjectShadowedLightsSeparately
		&& (LightProxy->GetLightType() == LightType_Point || LightProxy->GetLightType() == LightType_Spot || LightProxy->GetLightType() == LightType_Rect)
		&& !LightProxy->HasStaticLighting()
		&& LightProxy->CastsDynamicShadow()
		&& LightProxy->CastsVolumetricShadow())
	{
		const FStaticShadowDepthMap* StaticShadowDepthMap = LightProxy->GetStaticShadowDepthMap();
		const bool bStaticallyShadowed = LightSceneInfo->IsPrecomputedLightingValid() && StaticShadowDepthMap && StaticShadowDepthMap->Data && StaticShadowDepthMap->TextureRHI;
		const bool bHasVirtualShadowMap = VisibleLightInfo.GetVirtualShadowMapId( &View ) != INDEX_NONE;
		const bool bHasRayTracedShadows = (bTestRayTracedShadows) ? LightHasRayTracedShadows(LightSceneInfo, *View.Family) : false;

		return GetShadowForInjectionIntoVolumetricFog(VisibleLightInfo) != NULL || bStaticallyShadowed || bHasVirtualShadowMap || bHasRayTracedShadows;
	}

	return false;
}

void VolumetricFogOnRenderBegin(FScene& Scene, TArray<FViewInfo>& Views)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];

		// Similar test to what is in SetupVolumetricFog()
		if (View.ViewState && View.Family && !ShouldRenderVolumetricFog(&Scene, *View.Family))
		{
			View.ViewState->ResetVolumetricFogState();
		}
	}
}

FIntPoint CalculateVolumetricFogBoundsForLight(const FSphere& LightBounds, const FViewInfo& View, FIntVector VolumetricFogGridSize, FVector GridZParams)
{
	FIntPoint VolumeZBounds;

	FVector ViewSpaceLightBoundsOrigin = View.ViewMatrices.GetWorldToView().TransformPosition(LightBounds.Center);

	int32 FurthestSliceIndexUnclamped = ComputeZSliceFromDepth(ViewSpaceLightBoundsOrigin.Z + LightBounds.W, GridZParams);
	int32 ClosestSliceIndexUnclamped = ComputeZSliceFromDepth(ViewSpaceLightBoundsOrigin.Z - LightBounds.W, GridZParams);

	VolumeZBounds.X = FMath::Clamp(ClosestSliceIndexUnclamped, 0, VolumetricFogGridSize.Z - 1);
	VolumeZBounds.Y = FMath::Clamp(FurthestSliceIndexUnclamped, 0, VolumetricFogGridSize.Z - 1);

	return VolumeZBounds;
}

static bool OverrideDirectionalLightInScatteringUsingHeightFog(const FViewInfo& View, const FExponentialHeightFogSceneInfo& FogInfo)
{
	return FogInfo.bOverrideLightColorsWithFogInscatteringColors && View.bUseDirectionalInscattering && !View.FogInscatteringColorCubemap;
}

static bool OverrideSkyLightInScatteringUsingHeightFog(const FViewInfo& View, const FExponentialHeightFogSceneInfo& FogInfo)
{
	return FogInfo.bOverrideLightColorsWithFogInscatteringColors;
}

/**  */
class FCircleRasterizeVertexBuffer : public FVertexBuffer
{
public:

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FScreenVertex>(TEXT("FCircleRasterizeVertexBuffer"), NumVertices)
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		TRHIBufferInitializer<FScreenVertex> DestVertex = RHICmdList.CreateBufferInitializer(CreateDesc);

		const int32 NumSegments = NumVertices - 1;
		const float RadiansPerRingSegment = PI / (float)NumSegments;

		// Boost the effective radius so that the edges of the circle approximation lie on the circle, instead of the vertices
		const float RadiusScale = 1.0f / FMath::Cos(RadiansPerRingSegment);

		for (int32 VertexIndex = 0; VertexIndex < NumVertices; VertexIndex++)
		{
			float Angle = VertexIndex / (float)(NumVertices - 1) * 2 * PI;
			// WriteToBoundingSphereVS only uses UV
			DestVertex[VertexIndex].Position = FVector2f::ZeroVector;
			DestVertex[VertexIndex].UV = FVector2f(RadiusScale * FMath::Cos(Angle) * .5f + .5f, RadiusScale * FMath::Sin(Angle) * .5f + .5f);
		}

		VertexBufferRHI = DestVertex.Finalize();
	}

	static int32 NumVertices;
};

int32 FCircleRasterizeVertexBuffer::NumVertices = 8;

TGlobalResource<FCircleRasterizeVertexBuffer> GCircleRasterizeVertexBuffer;

/**  */
class FCircleRasterizeIndexBuffer : public FIndexBuffer
{
public:

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const int32 NumTriangles = FCircleRasterizeVertexBuffer::NumVertices - 2;

		TArray<uint16> Indices;
		Indices.Empty(NumTriangles * 3);

		for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; TriangleIndex++)
		{
			int32 LeadingVertexIndex = TriangleIndex + 2;
			Indices.Add(0);
			Indices.Add(LeadingVertexIndex - 1);
			Indices.Add(LeadingVertexIndex);
		}

		// Create index buffer. Fill buffer with initial data upon creation
		IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FCircleRasterizeIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Indices));
	}
};

TGlobalResource<FCircleRasterizeIndexBuffer> GCircleRasterizeIndexBuffer;

void SetupInjectShadowedLocalLightPSO(
	FRHIVertexShader* VertexShader,
	FRHIGeometryShader* GeometryShader,
	FRHIPixelShader* PixelShader,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit)
{
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	// Accumulate the contribution of multiple lights
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_One>::GetRHI();

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GScreenVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader;
	GraphicsPSOInit.BoundShaderState.SetGeometryShader(GeometryShader);
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader;
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
}

void FSceneRenderer::RenderLocalLightsForVolumetricFog(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View, int32 ViewIndex,
	bool bUseTemporalReprojection,
	const FVolumetricFogIntegrationParameterData& IntegrationData,
	const FExponentialHeightFogSceneInfo& FogInfo,
	FIntVector VolumetricFogViewGridSize,
	FVector GridZParams,
	const FRDGTextureDesc& VolumeDesc,
	FRDGTextureRef ConservativeDepthTexture,
	TConstArrayView<const FLightSceneInfo*> LightsToInject,
	TConstArrayView<const FLightSceneInfo*> RayTracedLightsToInject,
	FRDGTexture*& OutLocalShadowedLightScattering)
{
	RDG_EVENT_SCOPE(GraphBuilder, "VolumetricFog::LocalLights");

	// Setup the light function atlas
	const bool bUseLightFunctionAtlas = LightFunctionAtlas::IsEnabled(View, ELightFunctionAtlasSystem::VolumetricFog);
	TRDGUniformBufferRef<FLightFunctionAtlasGlobalParameters> LightFunctionAtlasGlobalParameters = LightFunctionAtlas::BindGlobalParameters(GraphBuilder, View);

	// Now voxelise all the light we have just gathered.
	bool bClearExecuted = false;
	if (LightsToInject.Num() > 0)
	{
		for (int32 LightIndex = 0; LightIndex < LightsToInject.Num(); LightIndex++)
		{
			const FLightSceneInfo* LightSceneInfo = LightsToInject[LightIndex];
			const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];

			const FSphere LightBounds = LightSceneInfo->Proxy->GetBoundingSphere();
			const FIntPoint VolumeZBounds = CalculateVolumetricFogBoundsForLight(LightBounds, View, VolumetricFogViewGridSize, GridZParams);
			if (VolumeZBounds.X < VolumeZBounds.Y)
			{
				bool bIsShadowed = LightNeedsSeparateInjectionIntoVolumetricFogForOpaqueShadow(View, LightSceneInfo, VisibleLightInfo, *Scene);
				bool bUsesRectLightTexture = GVolumetricFogRectLightTexture && LightSceneInfo->Proxy->HasSourceTexture();

				int32 VirtualShadowMapId = VisibleLightInfo.GetVirtualShadowMapId(&View);
				const bool bUseVSM = bIsShadowed && VirtualShadowMapArray.IsAllocated() && VirtualShadowMapId != INDEX_NONE;

				FInjectShadowedLocalLightPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInjectShadowedLocalLightPS::FParameters>();

				// Light function parameters
				bool bValid = SetupInjectShadowedLocalLightCommonParameters(
					GraphBuilder,
					View,
					IntegrationData,
					FogInfo,
					LightSceneInfo,
					PassParameters->Common
				);
				PassParameters->Common.LightFunctionAtlas = LightFunctionAtlasGlobalParameters;

				if (!bValid)
				{
					continue;
				}

				const bool bHasTextureBeenCreated = bClearExecuted == true;
				OutLocalShadowedLightScattering = bHasTextureBeenCreated ? OutLocalShadowedLightScattering : GraphBuilder.CreateTexture(VolumeDesc, TEXT("VolumetricFog.LocalShadowedLightScattering"));				

				PassParameters->RenderTargets[0] = FRenderTargetBinding(OutLocalShadowedLightScattering, bClearExecuted ? ERenderTargetLoadAction::ELoad : ERenderTargetLoadAction::EClear);
				bClearExecuted = true;

				PassParameters->VirtualShadowMapSamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
				PassParameters->ConservativeDepthTexture = ConservativeDepthTexture;
				PassParameters->UseConservativeDepthTexture = GVolumetricFogConservativeDepth > 0 ? 1 : 0;
				PassParameters->VirtualShadowMapId = VirtualShadowMapId;

				const FProjectedShadowInfo* ProjectedShadowInfo = GetShadowForInjectionIntoVolumetricFog(VisibleLightInfo);
				const bool bDynamicallyShadowed = ProjectedShadowInfo != NULL;
				GetVolumeShadowingShaderParameters(GraphBuilder, View, LightSceneInfo, ProjectedShadowInfo, PassParameters->VolumeShadowingShaderParameters);

				FInjectShadowedLocalLightPS::FPermutationDomain PermutationVector;
				PermutationVector.Set< FInjectShadowedLocalLightPS::FDynamicallyShadowed >(bDynamicallyShadowed);
				PermutationVector.Set< FInjectShadowedLocalLightPS::FTemporalReprojection >(bUseTemporalReprojection);
				PermutationVector.Set< FInjectShadowedLocalLightPS::FSampleLightFunctionAtlas >(bUseLightFunctionAtlas);
				PermutationVector.Set< FInjectShadowedLocalLightPS::FEnableShadows >(bIsShadowed);
				PermutationVector.Set< FInjectShadowedLocalLightPS::FVirtualShadowMap >(bUseVSM);
				PermutationVector.Set< FInjectShadowedLocalLightPS::FRectLightTexture >(bUsesRectLightTexture);
				PermutationVector.Set< FInjectShadowedLocalLightPS::FLightSoftFading >(GetVolumetricFogLightSoftFading() > 0.0f);

				auto VertexShader = View.ShaderMap->GetShader< FWriteToBoundingSphereVS >();
				TOptionalShaderMapRef<FWriteToSliceGS> GeometryShader(View.ShaderMap);
				auto PixelShader = View.ShaderMap->GetShader< FInjectShadowedLocalLightPS >(PermutationVector);

				ClearUnusedGraphResources(PixelShader, PassParameters);

				// We execute one pass per light: this is because RDG resources needs to be gathrered before and reference in the PassParameters.
				// Not many lights cast shadow so that is acceptable (LightRendering is doing the same things).
				// If light shadow maps woud be in a common resources (atlas, texture array, bindless) we could have a single pass for all the lights.
				// NOTE: light functions are already in an atlas so they are not a problem.
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("ShadowedLights"),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, &View, this, VertexShader, GeometryShader, PixelShader, VolumeZBounds, LightBounds, VolumetricFogViewGridSize](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
						FGraphicsPipelineStateInitializer GraphicsPSOInit;
						RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

						SetupInjectShadowedLocalLightPSO(VertexShader.GetVertexShader(), GeometryShader.GetGeometryShader(), PixelShader.GetPixelShader(), GraphicsPSOInit);
												
					#if PSO_PRECACHING_VALIDATE
						if (PSOCollectorStats::IsFullPrecachingValidationEnabled())
						{
							static const int32 GlobalPSOCollectorIndex = FGlobalPSOCollectorManager::GetIndex(VolumetricFogGlobalPSOCollectorName);
							PSOCollectorStats::CheckGlobalGraphicsPipelineStateInCache(GraphicsPSOInit, GlobalPSOCollectorIndex);
						}
					#endif // PSO_PRECACHING_VALIDATE

						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

						SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

						FWriteToBoundingSphereVS::FParameters VSPassParameters;
						VSPassParameters.MinZ = VolumeZBounds.X;
						VSPassParameters.ViewSpaceBoundingSphere = FVector4f(FVector4f(View.ViewMatrices.GetWorldToView().TransformPosition(LightBounds.Center)), LightBounds.W); // LWC_TODO: precision loss
						VSPassParameters.ViewToVolumeClip = FMatrix44f(View.ViewMatrices.ComputeProjectionNoAAMatrix());	// LWC_TODO: Precision loss?

						VSPassParameters.ClipRatio = GetVolumetricFogFroxelToScreenSVPosRatio(View);

						VSPassParameters.VolumetricFogParameters = PassParameters->Common.VolumetricFogParameters;
						SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSPassParameters);

						if (GeometryShader.IsValid())
						{
							SetShaderParametersLegacyGS(RHICmdList, GeometryShader, VolumeZBounds.X);
						}

						// Set the sub region of the texture according to the current dynamic resolution scale.
						RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, VolumetricFogViewGridSize.X, VolumetricFogViewGridSize.Y, 1.0f);

						RHICmdList.SetStreamSource(0, GCircleRasterizeVertexBuffer.VertexBufferRHI, 0);
						const int32 NumInstances = VolumeZBounds.Y - VolumeZBounds.X;
						const int32 NumTriangles = FCircleRasterizeVertexBuffer::NumVertices - 2;
						RHICmdList.DrawIndexedPrimitive(GCircleRasterizeIndexBuffer.IndexBufferRHI, 0, 0, FCircleRasterizeVertexBuffer::NumVertices, 0, NumTriangles, NumInstances);
				});
			}
		}
	}

#if RHI_RAYTRACING
	if (RayTracedLightsToInject.Num() > 0)
	{
		if (!bClearExecuted)
		{
			OutLocalShadowedLightScattering = GraphBuilder.CreateTexture(VolumeDesc, TEXT("VolumetricFog.LocalShadowedLightScattering"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(OutLocalShadowedLightScattering), 0.0f);
			bClearExecuted = true;
		}

		for (int32 LightIndex = 0; LightIndex < RayTracedLightsToInject.Num(); LightIndex++)
		{
			const FLightSceneInfo* LightSceneInfo = RayTracedLightsToInject[LightIndex];

			const FSphere LightBounds = LightSceneInfo->Proxy->GetBoundingSphere();
			const FIntPoint VolumeZBounds = CalculateVolumetricFogBoundsForLight(LightBounds, View, VolumetricFogViewGridSize, GridZParams);
			if (VolumeZBounds.X < VolumeZBounds.Y)
			{
				bool bUsesRectLightTexture = GVolumetricFogRectLightTexture && LightSceneInfo->Proxy->HasSourceTexture();

				FInjectShadowedLocalLightRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInjectShadowedLocalLightRGS::FParameters>();
				PassParameters->OutVolumeTexture = GraphBuilder.CreateUAV(OutLocalShadowedLightScattering);
				PassParameters->TLAS = View.GetRayTracingSceneLayerViewChecked(ERayTracingSceneLayer::Base);
				PassParameters->FirstSlice = VolumeZBounds.X;
				PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder);
				PassParameters->NaniteRayTracing = Nanite::GRayTracingManager.GetUniformBuffer();

				bool bValid = SetupInjectShadowedLocalLightCommonParameters(
					GraphBuilder,
					View,
					IntegrationData,
					FogInfo,
					LightSceneInfo,
					PassParameters->Common
				);
				PassParameters->Common.LightFunctionAtlas = LightFunctionAtlasGlobalParameters;

				if (!bValid)
				{
					continue;
				}

				FInjectShadowedLocalLightRGS::FPermutationDomain PermutationVector;
				PermutationVector.Set< FInjectShadowedLocalLightRGS::FTemporalReprojection >(bUseTemporalReprojection);
				PermutationVector.Set< FInjectShadowedLocalLightRGS::FSampleLightFunctionAtlas >(bUseLightFunctionAtlas);
				PermutationVector.Set< FInjectShadowedLocalLightRGS::FRectLightTexture >(bUsesRectLightTexture);
				PermutationVector.Set< FInjectShadowedLocalLightRGS::FLightSoftFading >(GetVolumetricFogLightSoftFading() > 0.0f);

				TShaderMapRef<FInjectShadowedLocalLightRGS> RayGenerationShader(GetGlobalShaderMap(FeatureLevel), PermutationVector);

				ClearUnusedGraphResources(RayGenerationShader, PassParameters);

				// TODO: better bounds
				const int32 NumSlices = VolumeZBounds.Y - VolumeZBounds.X;
				const uint32 DispatchSize = VolumeDesc.Extent.X * VolumeDesc.Extent.Y * NumSlices;
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("RayTracedShadowedLights"),
					PassParameters,
					ERDGPassFlags::Compute,
					[this, &View, RayGenerationShader, PassParameters, DispatchSize](FRDGAsyncTask, FRHICommandList& RHICmdList)
					{
						FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
						SetShaderParameters(GlobalResources, RayGenerationShader, *PassParameters);

						FRHIUniformBuffer* SceneUniformBuffer = PassParameters->Scene->GetRHI();
						FRHIUniformBuffer* NaniteRayTracingUniformBuffer = PassParameters->NaniteRayTracing->GetRHI();
						TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, RHICmdList);

						RHICmdList.RayTraceDispatch(View.MaterialRayTracingData.PipelineState, RayGenerationShader.GetRayTracingShader(), View.MaterialRayTracingData.ShaderBindingTable, GlobalResources, DispatchSize, 1);
					}
				);
			}
		}
	}
#endif
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLumenTranslucencyLightingUniforms, "LumenGIVolumeStruct");

class FVolumetricFogLightScatteringCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVolumetricFogLightScatteringCS);
	SHADER_USE_PARAMETER_STRUCT(FVolumetricFogLightScatteringCS, FGlobalShader);

	class FTemporalReprojection			: SHADER_PERMUTATION_BOOL("USE_TEMPORAL_REPROJECTION");
	class FDistanceFieldSkyOcclusion	: SHADER_PERMUTATION_BOOL("DISTANCE_FIELD_SKY_OCCLUSION");
	class FLumenGI						: SHADER_PERMUTATION_BOOL("LUMEN_GI");
	class FVirtualShadowMap				: SHADER_PERMUTATION_BOOL("VIRTUAL_SHADOW_MAP");
	class FRaytracedShadowsVolume		: SHADER_PERMUTATION_BOOL("USE_RAYTRACED_SHADOWS_VOLUME");
	class FSampleLightFunctionAtlas		: SHADER_PERMUTATION_BOOL("USE_LIGHT_FUNCTION_ATLAS");
	class FMegaLights					: SHADER_PERMUTATION_BOOL("USE_MEGA_LIGHTS");
	class FLightSoftFading				: SHADER_PERMUTATION_BOOL("USE_LIGHT_SOFT_FADING");
	class FSuperSampling				: SHADER_PERMUTATION_BOOL("USE_SUPER_SAMPLING");
	class FUbershader					: SHADER_PERMUTATION_BOOL("UBERSHADER");

	using FPermutationDomain = TShaderPermutationDomain<
		FTemporalReprojection,
		FDistanceFieldSkyOcclusion,
		FLumenGI,
		FVirtualShadowMap,
		FRaytracedShadowsVolume,
		FSampleLightFunctionAtlas,
		FMegaLights,
		FLightSoftFading,
		FSuperSampling,
		FUbershader>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardDirectionalLightShadowMapParameters, ForwardDirLightShadowStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, Fog)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLightFunctionAtlasGlobalParameters, LightFunctionAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumetricFogIntegrationParameters, VolumetricFogParameters)

		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, MegaLightsVolume)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VBufferA)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VBufferB)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, LocalShadowedLightScattering)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DirectionalLightLightFunctionTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DirectionalLightLightFunctionSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CloudShadowmapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, CloudShadowmapSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ConservativeDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PrevConservativeDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, LightScatteringHistory)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, RaytracedShadowsVolume)
		SHADER_PARAMETER_SAMPLER(SamplerState, LightScatteringHistorySampler)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLumenTranslucencyLightingUniforms, LumenGIVolumeStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGlobalDistanceFieldParameters2, GlobalDistanceFieldParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWLightScattering)
		SHADER_PARAMETER(uint32, SampleSkyLightDiffuseEnvMap)
		SHADER_PARAMETER(FMatrix44f, DirectionalLightFunctionTranslatedWorldToShadow)
		SHADER_PARAMETER(FMatrix44f, CloudShadowmapTranslatedWorldToLightClipMatrix)
		SHADER_PARAMETER(FVector3f, MobileDirectionalLightColor)
		SHADER_PARAMETER(FVector3f, MobileDirectionalLightDirection)
		SHADER_PARAMETER(FVector2f, PrevConservativeDepthTextureSize)
		SHADER_PARAMETER(FVector2f, UseHeightFogColors)
		SHADER_PARAMETER(FVector2f, LightScatteringHistoryPreExposureAndInv)
		SHADER_PARAMETER(float, StaticLightingScatteringIntensity)
		SHADER_PARAMETER(float, SkyLightUseStaticShadowing)
		SHADER_PARAMETER(float, PhaseG)
		SHADER_PARAMETER(float, InverseSquaredLightDistanceBiasScale)
		SHADER_PARAMETER(float, LightScatteringSampleJitterMultiplier)
		SHADER_PARAMETER(float, CloudShadowmapFarDepthKm)
		SHADER_PARAMETER(float, CloudShadowmapStrength)
		SHADER_PARAMETER(float, UseDirectionalLightShadowing)
		SHADER_PARAMETER(uint32, UseConservativeDepthTexture)
		SHADER_PARAMETER(uint32, UseEmissive)
		SHADER_PARAMETER(uint32, MobileHasDirectionalLight)
		SHADER_PARAMETER(uint32, DirectionalApplyLightFunctionFromAtlas)
		SHADER_PARAMETER(uint32, DirectionalLightFunctionAtlasLightIndex)
		SHADER_PARAMETER(uint32, bUseTemporalReprojection)
		SHADER_PARAMETER(uint32, bDistanceFieldSkyOcclusion)
		SHADER_PARAMETER(uint32, bLumenGI)
		SHADER_PARAMETER(uint32, bVirtualShadowMap)
		SHADER_PARAMETER(uint32, bUseRaytracedShadowsVolume)
		SHADER_PARAMETER(uint32, bUseLightFunctionAtlas)
		SHADER_PARAMETER(uint32, bUseMegaLights)
		SHADER_PARAMETER(uint32, bUseLightSoftFading)
		SHADER_PARAMETER(uint32, bUseSuperSampling)
	END_SHADER_PARAMETER_STRUCT()

	static FIntVector GetGroupSize()
	{
		return FIntVector(4, 4, 4);
	}

	static int32 GetSuperSampleCount(int32 InSampleCount)
	{
		if (InSampleCount <= 1)
		{
			return 1;
		}
		else if (InSampleCount <= 4)
		{
			return 4;
		}
		else if (InSampleCount <= 8)
		{
			return 8;
		}
		
		return 16;
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector, EShaderPlatform ShaderPlatform)
	{

		if (!FDataDrivenShaderPlatformInfo::GetSupportsLumenGI(ShaderPlatform))
		{
			PermutationVector.Set<FLumenGI>(false);
		}

		if (!ShouldCompileRayTracingShadersForProject(ShaderPlatform))
		{
			PermutationVector.Set<FRaytracedShadowsVolume>(false);
		}

		if (!MegaLights::ShouldCompileShaders(ShaderPlatform))
		{
			PermutationVector.Set<FMegaLights>(false);
		}

		if (PermutationVector.Get<FLumenGI>() || !DoesPlatformSupportDistanceFieldAO(ShaderPlatform))
		{
			PermutationVector.Set<FDistanceFieldSkyOcclusion>(false);
		}

		if (!DoesPlatformSupportVirtualShadowMaps(ShaderPlatform))
		{
			PermutationVector.Set<FVirtualShadowMap>(false);
		}

		if (PermutationVector.Get< FUbershader>())
		{
			PermutationVector.Set<FTemporalReprojection>(true);
			PermutationVector.Set<FDistanceFieldSkyOcclusion>(DoesPlatformSupportDistanceFieldAO(ShaderPlatform));
			PermutationVector.Set<FLumenGI>(FDataDrivenShaderPlatformInfo::GetSupportsLumenGI(ShaderPlatform));
			PermutationVector.Set<FVirtualShadowMap>(DoesPlatformSupportVirtualShadowMaps(ShaderPlatform));
			PermutationVector.Set<FRaytracedShadowsVolume>(RHISupportsRayTracingShaders(ShaderPlatform));
			PermutationVector.Set<FSampleLightFunctionAtlas>(true);
			PermutationVector.Set<FMegaLights>(MegaLights::ShouldCompileShaders(ShaderPlatform));
			PermutationVector.Set<FLightSoftFading>(true);
			PermutationVector.Set<FSuperSampling>(true);
		}

		return PermutationVector;
	}

	template<typename PermutationType>
	static bool GetIfSupported(FPermutationDomain PermutationVector, bool bSupported)
	{
		return bSupported ? PermutationVector.Get<PermutationType>() : PermutationVector.Get<PermutationType>() == 0;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FUbershader>())
		{
			return 
				GetIfSupported<FLumenGI>(PermutationVector, FDataDrivenShaderPlatformInfo::GetSupportsLumenGI(Parameters.Platform)) &&
				GetIfSupported<FMegaLights>(PermutationVector, MegaLights::ShouldCompileShaders(Parameters.Platform)) &&
				GetIfSupported<FVirtualShadowMap>(PermutationVector, DoesPlatformSupportVirtualShadowMaps(Parameters.Platform)) &&
				GetIfSupported<FDistanceFieldSkyOcclusion>(PermutationVector, DoesPlatformSupportDistanceFieldAO(Parameters.Platform)) &&
				GetIfSupported<FRaytracedShadowsVolume>(PermutationVector, RHISupportsRayTracingShaders(Parameters.Platform)) &&
				PermutationVector.Get<FTemporalReprojection>() &&
				PermutationVector.Get<FSampleLightFunctionAtlas>() &&
				PermutationVector.Get<FLightSoftFading>() &&
				PermutationVector.Get<FSuperSampling>()	&& 
				PlatformSupportsVolumetricFogUbershader(Parameters.Platform);
		}
		
		if (RemapPermutation(PermutationVector, Parameters.Platform) != PermutationVector)
		{
			return false;
		}

		return true;
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FShaderPermutationParameters& Parameters)
	{
		const bool bSupportsLumenGI = FDataDrivenShaderPlatformInfo::GetSupportsLumenGI(Parameters.Platform);

		FPermutationDomain PermutationVector = RemapPermutation(FPermutationDomain(Parameters.PermutationId), Parameters.Platform);

		if (PermutationVector.Get<FLumenGI>() && !bSupportsLumenGI)
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		// We cannot skip the precaching of FDistanceFieldSkyOcclusion because the runtime logic to enable it is complex and not only based on project support & cvar values.
		// It involves checking lumen support and the presence of GetLumenTranslucencyGIVolume().Texture0 which might not be present if no translucent are present.

		if (PermutationVector.Get<FTemporalReprojection>() && (GVolumetricFogTemporalReprojection == 0))
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		const int32 SuperSampleCount = FVolumetricFogLightScatteringCS::GetSuperSampleCount(GVolumetricFogHistoryMissSupersampleCount);
		if (PermutationVector.Get<FSuperSampling>() != bool(SuperSampleCount > 1))
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		if (!PermutationVector.Get<FUbershader>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}
		return EShaderPermutationPrecacheRequest::Precached;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GetGroupSize().X);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GetGroupSize().Y);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), GetGroupSize().Z);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVolumetricFogLightScatteringCS, "/Engine/Private/VolumetricFog.usf", "LightScatteringCS", SF_Compute);

uint32 VolumetricFogIntegrationGroupSize = 8;

class FVolumetricFogFinalIntegrationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVolumetricFogFinalIntegrationCS);
	SHADER_USE_PARAMETER_STRUCT(FVolumetricFogFinalIntegrationCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, LightScattering)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWIntegratedLightScattering)
		SHADER_PARAMETER(float, VolumetricFogNearFadeInDistanceInv)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumetricFogIntegrationParameters, VolumetricFogParameters)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), VolumetricFogIntegrationGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVolumetricFogFinalIntegrationCS, "/Engine/Private/VolumetricFog.usf", "FinalIntegrationCS", SF_Compute);

bool ShouldRenderVolumetricFog(const FScene* Scene, const FSceneViewFamily& ViewFamily)
{
	return ShouldRenderFog(ViewFamily)
		&& Scene
		&& GVolumetricFog
		&& ViewFamily.EngineShowFlags.VolumetricFog
		&& Scene->ExponentialFogs.Num() > 0
		&& Scene->ExponentialFogs[0].bEnableVolumetricFog
		&& Scene->ExponentialFogs[0].VolumetricFogDistance > 0;
}

FVector GetVolumetricFogGridZParams(float VolumetricFogStartDistance, float NearPlane, float FarPlane, int32 GridSizeZ)
{
	// Don't spend lots of resolution right in front of the near plane
	NearPlane = FMath::Max(NearPlane, double(VolumetricFogStartDistance));

	return CalculateGridZParams(NearPlane, FarPlane, GVolumetricFogDepthDistributionScale, GridSizeZ);
}

static FIntVector GetVolumetricFogGridSize(const FIntPoint& TargetResolution, int32& OutVolumetricFogGridPixelSize)
{
	OutVolumetricFogGridPixelSize = GetVolumetricFogGridPixelSize();
	return GetFroxelGridSize(TargetResolution, OutVolumetricFogGridPixelSize, GetVolumetricFogGridSizeZ());
}

FIntVector GetVolumetricFogResourceGridSize(const FViewInfo& View, int32& OutVolumetricFogGridPixelSize)
{
	return GetVolumetricFogGridSize(GetVolumetricFogTextureResourceRes(View), OutVolumetricFogGridPixelSize);
}

FIntVector GetVolumetricFogViewGridSize(const FViewInfo& View, int32& OutVolumetricFogGridPixelSize)
{
	return GetVolumetricFogGridSize(View.ViewRect.Size(), OutVolumetricFogGridPixelSize);
}

FVector2f GetVolumetricFogFroxelToScreenSVPosRatio(const FViewInfo& View)
{
	const FIntPoint ViewRectSize = View.ViewRect.Size();

	// Calculate how much the Fog froxel volume "overhangs" the actual view frustum to the right and bottom.
	// This needs to be applied on SVPos because froxel pixel size (see r.VolumetricFog.GridPixelSize) does not align perfectly with view rect.
	int32 VolumetricFogGridPixelSize;
	const FIntVector VolumetricFogGridSize = GetVolumetricFogViewGridSize(View, VolumetricFogGridPixelSize);
	const FVector2f FogPhysicalSize = FVector2f(VolumetricFogGridSize.X, VolumetricFogGridSize.Y) * VolumetricFogGridPixelSize;
	const FVector2f ClipRatio = FogPhysicalSize / FVector2f(ViewRectSize);
	return ClipRatio;
}

FRDGTextureDesc GetVolumetricFogRDGTextureDesc(const FIntVector& VolumetricFogResourceGridSize)
{
	return FRDGTextureDesc::Create3D(
		VolumetricFogResourceGridSize,
		PF_FloatRGBA,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV | TexCreate_ReduceMemoryWithTilingMode | TexCreate_3DTiling);
}

void SetupVolumetricFogGlobalData(const FViewInfo& View, FVolumetricFogGlobalData& Parameters)
{
	const FScene* Scene = (FScene*)View.Family->Scene;
	const FExponentialHeightFogSceneInfo& FogInfo = Scene->ExponentialFogs[0];

	int32 VolumetricFogGridPixelSize;
	const FIntVector VolumetricFogViewGridSize = GetVolumetricFogViewGridSize(View, VolumetricFogGridPixelSize);
	const FIntVector VolumetricFogResourceGridSize = GetVolumetricFogResourceGridSize(View, VolumetricFogGridPixelSize);  

	Parameters.ViewGridSizeInt = VolumetricFogViewGridSize;
	Parameters.ViewGridSize = FVector3f(VolumetricFogViewGridSize);
	Parameters.ResourceGridSizeInt = VolumetricFogResourceGridSize;
	Parameters.ResourceGridSize = FVector3f(VolumetricFogResourceGridSize);

	FVector ZParams = GetVolumetricFogGridZParams(FogInfo.VolumetricFogStartDistance, View.NearClippingDistance, FogInfo.VolumetricFogDistance, VolumetricFogResourceGridSize.Z);
	Parameters.GridZParams = (FVector3f)ZParams;

	Parameters.SVPosToVolumeUV = FVector2f::UnitVector / (FVector2f(VolumetricFogResourceGridSize.X, VolumetricFogResourceGridSize.Y) * VolumetricFogGridPixelSize);
	Parameters.FogGridToPixelXY = FIntPoint(VolumetricFogGridPixelSize, VolumetricFogGridPixelSize);
	Parameters.MaxDistance = FogInfo.VolumetricFogDistance;

	Parameters.HeightFogInscatteringColor = View.ExponentialFogColor;

	Parameters.HeightFogDirectionalLightInscatteringColor = FVector3f::ZeroVector;
	if (OverrideDirectionalLightInScatteringUsingHeightFog(View, FogInfo))
	{
		Parameters.HeightFogDirectionalLightInscatteringColor = FVector3f(View.DirectionalInscatteringColor);
	}

	Parameters.LightSoftFading = GetVolumetricFogLightSoftFading();
}

void FViewInfo::SetupVolumetricFogUniformBufferParameters(FViewUniformShaderParameters& ViewUniformShaderParameters) const
{
	const FScene* Scene = (const FScene*)Family->Scene;

	if (ShouldRenderVolumetricFog(Scene, *Family))
	{
		const FExponentialHeightFogSceneInfo& FogInfo = Scene->ExponentialFogs[0];

		int32 VolumetricFogResourceGridPixelSize;
		int32 VolumetricFogViewGridPixelSize;
		const FIntVector VolumetricFogResourceGridSize = GetVolumetricFogResourceGridSize(*this, VolumetricFogResourceGridPixelSize);
		const FIntVector VolumetricFogViewGridSize = GetVolumetricFogViewGridSize(*this, VolumetricFogViewGridPixelSize);

		ViewUniformShaderParameters.VolumetricFogGridSize = FVector3f(VolumetricFogResourceGridSize.X, VolumetricFogResourceGridSize.Y, VolumetricFogResourceGridSize.Z);
		ViewUniformShaderParameters.VolumetricFogInvGridSize = FVector3f(1.0f / VolumetricFogResourceGridSize.X, 1.0f / VolumetricFogResourceGridSize.Y, 1.0f / VolumetricFogResourceGridSize.Z);

		const FVector ZParams = GetVolumetricFogGridZParams(FogInfo.VolumetricFogStartDistance, NearClippingDistance, FogInfo.VolumetricFogDistance, VolumetricFogResourceGridSize.Z);
		ViewUniformShaderParameters.VolumetricFogGridZParams = (FVector3f)ZParams;

		ViewUniformShaderParameters.VolumetricFogSVPosToVolumeUV = FVector2f::UnitVector / (FVector2f(VolumetricFogResourceGridSize.X, VolumetricFogResourceGridSize.Y) * VolumetricFogResourceGridPixelSize);
		ViewUniformShaderParameters.VolumetricFogMaxDistance = FogInfo.VolumetricFogDistance;
	}
	else
	{
		ViewUniformShaderParameters.VolumetricFogGridSize = FVector3f::ZeroVector;
		ViewUniformShaderParameters.VolumetricFogInvGridSize = FVector3f::ZeroVector;
		ViewUniformShaderParameters.VolumetricFogGridZParams = FVector3f::ZeroVector;
		ViewUniformShaderParameters.VolumetricFogSVPosToVolumeUV = FVector2f::ZeroVector;
		ViewUniformShaderParameters.VolumetricFogViewGridUVToPrevViewRectUV = FVector2f::ZeroVector;
		ViewUniformShaderParameters.VolumetricFogMaxDistance = 0;
	}
}

bool FSceneRenderer::ShouldRenderVolumetricFog() const
{
	return ::ShouldRenderVolumetricFog(Scene, ViewFamily);
}

void FSceneRenderer::SetupVolumetricFog()
{
	if (ShouldRenderVolumetricFog())
	{
		const FExponentialHeightFogSceneInfo& FogInfo = Scene->ExponentialFogs[0];

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			FViewInfo& View = Views[ViewIndex];

			FVolumetricFogGlobalData GlobalData;
			SetupVolumetricFogGlobalData(View, GlobalData);
			View.VolumetricFogResources.VolumetricFogGlobalData = TUniformBufferRef<FVolumetricFogGlobalData>::CreateUniformBufferImmediate(GlobalData, UniformBuffer_SingleFrame);
		}
	}
}

void FSceneRenderer::ComputeVolumetricFog(FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures)
{
	if (!ShouldRenderVolumetricFog())
	{
		return;
	}
	
	const FExponentialHeightFogSceneInfo& FogInfo = Scene->ExponentialFogs[0];

	TRACE_CPUPROFILER_EVENT_SCOPE(FSceneRenderer::ComputeVolumetricFog);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_VolumetricFog);
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, VolumetricFog);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, VolumetricFog, "ComputeVolumetricFog");

	// Gather lights that need to be rendered with shadow from opaque or light functions.

	struct FLightsToInject
	{
		TArray<const FLightSceneInfo*, SceneRenderingAllocator> Lights;
		TArray<const FLightSceneInfo*, SceneRenderingAllocator> RayTracedLights;
		FLightSceneInfo* DirectionalLightFunction = nullptr;
		bool bUseDirectionalLightShadowing = false;
	};

	TArray<FLightsToInject, TInlineAllocator<2>> LightsToInjectPerView;
	LightsToInjectPerView.SetNum(Views.Num());

	for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
	{

		// Nothing to do for black lights.
		if (LightSceneInfoCompact.Color.IsAlmostBlack())
		{
			continue;
		}

		FLightSceneInfo* LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			FViewInfo& View = Views[ViewIndex];
			FLightsToInject& LightsToInject = LightsToInjectPerView[ViewIndex];

			if (!LightSceneInfo->ShouldRenderLight(View))
			{
				continue;
			}

			const FLightSceneProxy* LightSceneProxy = LightSceneInfo->Proxy;

			if (LightSceneProxy->GetVolumetricScatteringIntensity() > SMALL_NUMBER)
			{
				bool bIsShadowed = LightNeedsSeparateInjectionIntoVolumetricFogForOpaqueShadow(View, LightSceneInfo, VisibleLightInfos[LightSceneInfo->Id], *Scene);
				bool bUsesRectLightTexture = GVolumetricFogRectLightTexture && LightSceneProxy->HasSourceTexture();

				if (bIsShadowed || bUsesRectLightTexture)
				{
					const FSphere LightBounds = LightSceneProxy->GetBoundingSphere();

					if ((View.ViewMatrices.GetViewOrigin() - LightBounds.Center).SizeSquared() < (FogInfo.VolumetricFogDistance + LightBounds.W) * (FogInfo.VolumetricFogDistance + LightBounds.W))
					{
#if RHI_RAYTRACING
						const bool bRayTracedLight = (View.bHasRayTracingShadows && View.IsRayTracingAllowedForView()) ? LightHasRayTracedShadows(LightSceneInfo, ViewFamily) : false;
						if (bRayTracedLight)
						{
							LightsToInject.RayTracedLights.Add(LightSceneInfo);
						}
						else
#endif
						{
							LightsToInject.Lights.Add(LightSceneInfo);
						}
					}
				}
			}

			// The only directional light we can accept in the volumetric fog because we use the forward lighting data in the Scattering compute shader.
			const FLightSceneProxy* SelectedForwardDirectionalLightProxy = View.ForwardLightingResources.SelectedForwardDirectionalLightProxy;

			if (LightSceneProxy == SelectedForwardDirectionalLightProxy && LightSceneProxy->GetLightType() == LightType_Directional)
			{
				LightsToInject.bUseDirectionalLightShadowing = LightSceneProxy->CastsVolumetricShadow();

				if (CheckForLightFunction(LightSceneInfo) && ViewFamily.EngineShowFlags.LightFunctions)
				{
					LightsToInject.DirectionalLightFunction = LightSceneInfo;
				}
			}
		}
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];
		const FLightsToInject& LightsToInject = LightsToInjectPerView[ViewIndex];
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

		// Allocate texture using scene render targets size so we do not reallocate every frame when dynamic resolution is used in order to avoid resources allocation hitches.
		const FIntPoint BufferSize = View.GetSceneTexturesConfig().Extent;

		int32 VolumetricFogGridPixelSize;
		const FIntVector VolumetricFogResourceGridSize = GetVolumetricFogResourceGridSize(View, VolumetricFogGridPixelSize);
		const FIntVector VolumetricFogViewGridSize = GetVolumetricFogViewGridSize(View, VolumetricFogGridPixelSize); 
		const FVector GridZParams = GetVolumetricFogGridZParams(FogInfo.VolumetricFogStartDistance, View.NearClippingDistance, FogInfo.VolumetricFogDistance, VolumetricFogViewGridSize.Z);

		FVolumetricFogIntegrationParameterData IntegrationData;
		IntegrationData.FrameJitterOffsetValues.Empty(16);
		IntegrationData.FrameJitterOffsetValues.AddZeroed(16);
		IntegrationData.FrameJitterOffsetValues[0] = VolumetricFogTemporalRandom(View.Family->FrameNumber);

		for (int32 FrameOffsetIndex = 1; FrameOffsetIndex < GVolumetricFogHistoryMissSupersampleCount; FrameOffsetIndex++)
		{
			IntegrationData.FrameJitterOffsetValues[FrameOffsetIndex] = VolumetricFogTemporalRandom(View.Family->FrameNumber - FrameOffsetIndex);
		}

		const bool bUseLightFunctionAtlasEnabledAndSupported = LightFunctionAtlas::IsEnabled(*Scene, ELightFunctionAtlasSystem::VolumetricFog);

		const bool bUseTemporalReprojection =
			GVolumetricFogTemporalReprojection
			&& View.ViewState;

		IntegrationData.bTemporalHistoryIsValid =
			bUseTemporalReprojection
			&& !View.bCameraCut
			&& !View.bPrevTransformsReset
			&& ViewFamily.bRealtimeUpdate
			&& View.ViewState->LightScatteringHistory;

		FMatrix44f DirectionalLightFunctionTranslatedWorldToShadow = FMatrix44f::Identity;


		RDG_EVENT_SCOPE(GraphBuilder, "VolumetricFog");

		FRDGTextureRef ConservativeDepthTexture;
		// To use a depth target format, and depth tests, we will have to render depth from a PS depth output. Keeping it simple for now with all the tests happening in shader.
		if (GVolumetricFogConservativeDepth > 0)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "VolumetricFog::ConservativeDepth");
			FIntPoint ConservativeDepthTextureSize = FIntPoint(VolumetricFogViewGridSize.X, VolumetricFogViewGridSize.Y);
			ConservativeDepthTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(ConservativeDepthTextureSize, PF_R16F,
				FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV), TEXT("VolumetricFog.ConservativeDepthTexture"));
			AddGenerateConservativeDepthBufferPass(View, GraphBuilder, ConservativeDepthTexture, GetVolumetricFogGridPixelSize());
		}
		else
		{
			ConservativeDepthTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		}

		FRDGTexture* LightFunctionTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.WhiteDummy);
		FRDGTexture* BlackDummyTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		FRDGTexture* VolumetricBlackDummyTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.VolumetricBlackDummy);
		const bool bUseEmissive = GVolumetricFogEmissive > 0;

		// The potential light function for the main directional light is kept separate to be applied during the main VolumetricFogLightScattering pass (as an optimisation).
		FRDGTexture* DirectionalLightFunctionTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.WhiteDummy);

		// Recover the information about the light use as the forward directional light for cloud shadowing
		int AtmosphericDirectionalLightIndex = -1;
		FLightSceneProxy* AtmosphereLightProxy = nullptr;
		if(View.ForwardLightingResources.SelectedForwardDirectionalLightProxy)
		{
			FLightSceneProxy* AtmosphereLight0Proxy = Scene->AtmosphereLights[0] ? Scene->AtmosphereLights[0]->Proxy : nullptr;
			FLightSceneProxy* AtmosphereLight1Proxy = Scene->AtmosphereLights[1] ? Scene->AtmosphereLights[1]->Proxy : nullptr;
			FVolumetricCloudRenderSceneInfo* CloudInfo = Scene->GetVolumetricCloudSceneInfo();
			const bool VolumetricCloudShadowMap0Valid = View.VolumetricCloudShadowExtractedRenderTarget[0] != nullptr;
			const bool VolumetricCloudShadowMap1Valid = View.VolumetricCloudShadowExtractedRenderTarget[1] != nullptr;
			const bool bLight0CloudPerPixelTransmittance = CloudInfo && VolumetricCloudShadowMap0Valid && View.ForwardLightingResources.SelectedForwardDirectionalLightProxy == AtmosphereLight0Proxy && AtmosphereLight0Proxy && AtmosphereLight0Proxy->GetCloudShadowOnSurfaceStrength() > 0.0f;
			const bool bLight1CloudPerPixelTransmittance = CloudInfo && VolumetricCloudShadowMap1Valid && View.ForwardLightingResources.SelectedForwardDirectionalLightProxy == AtmosphereLight1Proxy && AtmosphereLight1Proxy && AtmosphereLight1Proxy->GetCloudShadowOnSurfaceStrength() > 0.0f;
			if (bLight0CloudPerPixelTransmittance)
			{
				AtmosphereLightProxy = AtmosphereLight0Proxy;
				AtmosphericDirectionalLightIndex = 0;
			}
			else if (bLight1CloudPerPixelTransmittance)
			{
				AtmosphereLightProxy = AtmosphereLight1Proxy;
				AtmosphericDirectionalLightIndex = 1;
			}
		}
		
		if (LightsToInject.DirectionalLightFunction)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "VolumetricFog::DirLightFunction");
			RenderLightFunctionForVolumetricFog(
				GraphBuilder,
				View,
				SceneTextures,
				VolumetricFogViewGridSize,
				FogInfo.VolumetricFogDistance,
					LightsToInject.DirectionalLightFunction,
				DirectionalLightFunctionTranslatedWorldToShadow,
					DirectionalLightFunctionTexture);
		}
			
		View.VolumetricFogResources.IntegratedLightScatteringTexture = nullptr;
		TRDGUniformBufferRef<FFogUniformParameters> FogUniformBuffer = CreateFogUniformBuffer(GraphBuilder, View);

		FRDGTextureDesc VolumeDesc = GetVolumetricFogRDGTextureDesc(VolumetricFogResourceGridSize);

		FRDGTextureDesc VolumeDescNoRT = VolumeDesc;
		VolumeDescNoRT.Flags &= ~TexCreate_RenderTargetable;

		FRDGTextureDesc VolumeDescFastVRAM = VolumeDesc;
		VolumeDescFastVRAM.Flags |= GFastVRamConfig.VolumetricFog;

		IntegrationData.VBufferA = GraphBuilder.CreateTexture(VolumeDescFastVRAM, TEXT("VolumetricFog.VBufferA"));
		IntegrationData.VBufferA_UAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(IntegrationData.VBufferA));
		IntegrationData.VBufferB = nullptr;
		IntegrationData.VBufferB_UAV = nullptr;
		if (bUseEmissive)
		{
			IntegrationData.VBufferB = GraphBuilder.CreateTexture(VolumeDescFastVRAM, TEXT("VolumetricFog.VBufferB"));
			IntegrationData.VBufferB_UAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(IntegrationData.VBufferB));
		}

		FRDGTexture* LocalShadowedLightScattering = GraphBuilder.RegisterExternalTexture(GSystemTextures.VolumetricBlackDummy);
		RenderLocalLightsForVolumetricFog(GraphBuilder, View, ViewIndex, bUseTemporalReprojection, IntegrationData, FogInfo,
			VolumetricFogViewGridSize, GridZParams, VolumeDescFastVRAM, ConservativeDepthTexture, LightsToInject.Lights, LightsToInject.RayTracedLights, LocalShadowedLightScattering);
	
		FRDGTextureRef RaytracedShadowsVolume = nullptr;
#if RHI_RAYTRACING		
		RenderRaytracedDirectionalShadowVolume(GraphBuilder, View, *Scene, IntegrationData, RaytracedShadowsVolume);
#endif
		const bool bUseRaytracedShadowsVolume = RaytracedShadowsVolume != nullptr;
		if (!bUseRaytracedShadowsVolume)
		{
			RaytracedShadowsVolume = VolumetricBlackDummyTexture;
		}

		{
			RDG_EVENT_SCOPE(GraphBuilder, "VolumetricFog::InitialiseVolume");

			FVolumetricFogMaterialSetupCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVolumetricFogMaterialSetupCS::FParameters>();
			PassParameters->GlobalAlbedo = FogInfo.VolumetricFogAlbedo;
			PassParameters->GlobalEmissive = FogInfo.VolumetricFogEmissive;
			PassParameters->GlobalExtinctionScale = FogInfo.VolumetricFogExtinctionScale;

			PassParameters->RWVBufferA = IntegrationData.VBufferA_UAV;
			PassParameters->RWVBufferB = IntegrationData.VBufferB_UAV; // FVolumetricFogMaterialSetupCS uses a permutation to not reference that UAV when bUseEmissive is false.

			PassParameters->LFV = View.LocalFogVolumeViewData.UniformParametersStruct;

			PassParameters->Fog = FogUniformBuffer; 
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->SceneTextures = GetSceneTextureShaderParameters(View);
			SetupVolumetricFogIntegrationParameters(PassParameters->VolumetricFogParameters, View, IntegrationData);

			FVolumetricFogMaterialSetupCS::FPermutationDomain PermutationVector;
			PermutationVector.Set< FPermutationUseEmissive >(bUseEmissive);
			PermutationVector.Set< FPermutationLocalFogVolume >(ShouldRenderLocalFogVolumeInVolumetricFog(Scene, ViewFamily, ShouldRenderLocalFogVolume(Scene, ViewFamily)));
			auto ComputeShader = View.ShaderMap->GetShader< FVolumetricFogMaterialSetupCS >(PermutationVector);
			ClearUnusedGraphResources(ComputeShader, PassParameters);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("InitializeVolumeAttributes"),
				PassParameters,
				ERDGPassFlags::Compute,
				[PassParameters, &View, VolumetricFogViewGridSize, IntegrationData, ComputeShader](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				const FIntVector NumGroups = FIntVector::DivideAndRoundUp(VolumetricFogViewGridSize, VolumetricFogGridInjectionGroupSize);

				SetComputePipelineState(RHICmdList, ComputeShader.GetComputeShader());

				SetShaderParameters(RHICmdList, ComputeShader, ComputeShader.GetComputeShader(), *PassParameters);
				DispatchComputeShader(RHICmdList, ComputeShader.GetShader(), NumGroups.X, NumGroups.Y, NumGroups.Z);
				UnsetShaderUAVs(RHICmdList, ComputeShader, ComputeShader.GetComputeShader());
			});

			VoxelizeFogVolumePrimitives(
				GraphBuilder,
				View,
				IntegrationData,
				VolumetricFogViewGridSize,
				GridZParams,
				FogInfo.VolumetricFogDistance,
				bUseEmissive);
		}

		IntegrationData.LightScattering = GraphBuilder.CreateTexture(VolumeDescNoRT, TEXT("VolumetricFog.LightScattering"));
		IntegrationData.LightScatteringUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(IntegrationData.LightScattering));

		{
			RDG_EVENT_SCOPE(GraphBuilder, "VolumetricFog::LightScattering");

			FVolumetricFogLightScatteringCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVolumetricFogLightScatteringCS::FParameters>();

			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->SceneTextures = GetSceneTextureShaderParameters(View);
			PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
			PassParameters->ForwardDirLightShadowStruct = View.ForwardLightingResources.ForwardDirLightShadowUniformBuffer;
			PassParameters->Fog = FogUniformBuffer;
			SetupVolumetricFogIntegrationParameters(PassParameters->VolumetricFogParameters, View, IntegrationData);

			PassParameters->VBufferA = IntegrationData.VBufferA;
			PassParameters->VBufferB = IntegrationData.VBufferB ? IntegrationData.VBufferB : VolumetricBlackDummyTexture;
			PassParameters->LocalShadowedLightScattering = LocalShadowedLightScattering;
			PassParameters->ConservativeDepthTexture = ConservativeDepthTexture;
			PassParameters->UseConservativeDepthTexture = GVolumetricFogConservativeDepth > 0 ? 1 : 0;
			PassParameters->UseEmissive = bUseEmissive ? 1 : 0;
			if (GVolumetricFogConservativeDepth > 0 && bUseTemporalReprojection && View.ViewState->PrevLightScatteringConservativeDepthTexture.IsValid())
			{
				PassParameters->PrevConservativeDepthTexture = GraphBuilder.RegisterExternalTexture(View.ViewState->PrevLightScatteringConservativeDepthTexture);
				FIntVector TextureSize = View.ViewState->PrevLightScatteringConservativeDepthTexture->GetDesc().GetSize();
				PassParameters->PrevConservativeDepthTextureSize = FVector2f(TextureSize.X, TextureSize.Y);
			}
			else
			{
				PassParameters->PrevConservativeDepthTexture = BlackDummyTexture;
				PassParameters->PrevConservativeDepthTextureSize = FVector2f::UnitVector;
			}

			PassParameters->DirectionalLightFunctionTranslatedWorldToShadow = DirectionalLightFunctionTranslatedWorldToShadow;
			PassParameters->DirectionalLightLightFunctionTexture = DirectionalLightFunctionTexture;
			PassParameters->DirectionalLightLightFunctionSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

			auto* LumenUniforms = GraphBuilder.AllocParameters<FLumenTranslucencyLightingUniforms>();
			LumenUniforms->Parameters = GetLumenTranslucencyLightingParameters(GraphBuilder, View.GetLumenTranslucencyGIVolume());
			PassParameters->LumenGIVolumeStruct = GraphBuilder.CreateUniformBuffer(LumenUniforms);
			PassParameters->MegaLightsVolume = View.GetMegaLightsVolume().Texture;
			PassParameters->RWLightScattering = IntegrationData.LightScatteringUAV;
			PassParameters->VirtualShadowMapSamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
			if (!PassParameters->MegaLightsVolume)
			{
				PassParameters->MegaLightsVolume = VolumetricBlackDummyTexture;
			}

			FDistanceFieldAOParameters AOParameterData(Scene->DefaultMaxDistanceFieldOcclusionDistance);
			if (Scene->SkyLight
				// Skylights with static lighting had their diffuse contribution baked into lightmaps
				&& !Scene->SkyLight->bHasStaticLighting
				&& View.Family->EngineShowFlags.SkyLighting)
			{
				AOParameterData = FDistanceFieldAOParameters(Scene->SkyLight->OcclusionMaxDistance, Scene->SkyLight->Contrast);
			}
			PassParameters->AOParameters = DistanceField::SetupAOShaderParameters(AOParameterData);
			PassParameters->GlobalDistanceFieldParameters = SetupGlobalDistanceFieldParameters(View.GlobalDistanceFieldInfo.ParameterData);

			FVolumetricCloudRenderSceneInfo* CloudInfo = Scene->GetVolumetricCloudSceneInfo();
			FRDGTexture* LightScatteringHistoryRDGTexture = VolumetricBlackDummyTexture;
			float LightScatteringHistoryPreExposure = 1.0f;
			if (bUseTemporalReprojection && View.ViewState->LightScatteringHistory.IsValid())
			{
				LightScatteringHistoryRDGTexture = GraphBuilder.RegisterExternalTexture(View.ViewState->LightScatteringHistory);
				LightScatteringHistoryPreExposure = View.ViewState->LightScatteringHistoryPreExposure;
			}

			PassParameters->LightScatteringHistory = LightScatteringHistoryRDGTexture;
			PassParameters->LightScatteringHistorySampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->LightScatteringHistoryPreExposureAndInv = FVector2f(LightScatteringHistoryPreExposure, LightScatteringHistoryPreExposure > 0.0f ? 1.0f / LightScatteringHistoryPreExposure : 1.0f);

			FSkyLightSceneProxy* SkyLight = Scene->SkyLight;
			if (SkyLight
				// Skylights with static lighting had their diffuse contribution baked into lightmaps
				&& !SkyLight->bHasStaticLighting
				&& View.Family->EngineShowFlags.SkyLighting)
			{
				PassParameters->SkyLightUseStaticShadowing = SkyLight->bWantsStaticShadowing && SkyLight->bCastShadows ? 1.0f : 0.0f;
				PassParameters->SampleSkyLightDiffuseEnvMap = 1;
			}
			else
			{
				PassParameters->SkyLightUseStaticShadowing = 0.0f;
				PassParameters->SampleSkyLightDiffuseEnvMap = 0;
			}

			// Mobile handles directional differently as of today to handle light masking (does not use and fill up the FForwardLightData). 
			// Volumetric fog does not work with light mask so we simply pick up the first one available. In the long run we might want something more common.
			PassParameters->MobileDirectionalLightColor		= FVector3f::Zero();
			PassParameters->MobileDirectionalLightDirection = FVector3f::Zero();
			PassParameters->MobileHasDirectionalLight		= 0;
			for (uint32 ChannelIdx = 0; ChannelIdx < UE_ARRAY_COUNT(Scene->MobileDirectionalLights); ChannelIdx++)
			{
				FLightSceneInfo* Light = Scene->GetMobileDirectionalLightForView(ChannelIdx, View);
				if (Light != nullptr)
				{
					PassParameters->MobileDirectionalLightColor		= FVector3f(Light->Proxy->GetSunIlluminanceAccountingForSkyAtmospherePerPixelTransmittance() * Light->Proxy->GetVolumetricScatteringIntensity());
					PassParameters->MobileDirectionalLightDirection = FVector3f(-Light->Proxy->GetDirection());
					PassParameters->MobileHasDirectionalLight		= 1;
					break;
				}
			}

			float StaticLightingScatteringIntensityValue = 0;
			if (View.Family->EngineShowFlags.GlobalIllumination && View.Family->EngineShowFlags.VolumetricLightmap)
			{
				StaticLightingScatteringIntensityValue = FogInfo.VolumetricFogStaticLightingScatteringIntensity;
			}
			PassParameters->StaticLightingScatteringIntensity = StaticLightingScatteringIntensityValue;

			PassParameters->PhaseG = FogInfo.VolumetricFogScatteringDistribution;
			PassParameters->InverseSquaredLightDistanceBiasScale = GInverseSquaredLightDistanceBiasScale;
			PassParameters->UseDirectionalLightShadowing = LightsToInject.bUseDirectionalLightShadowing ? 1.0f : 0.0f;
			PassParameters->LightScatteringSampleJitterMultiplier = GVolumetricFogJitter ? GLightScatteringSampleJitterMultiplier : 0;
			PassParameters->UseHeightFogColors = FVector2f(
				OverrideDirectionalLightInScatteringUsingHeightFog(View, FogInfo) ? 1.0f : 0.0f,
				OverrideSkyLightInScatteringUsingHeightFog(View, FogInfo) ? 1.0f : 0.0f);

			FMatrix44f CloudWorldToLightClipShadowMatrix = FMatrix44f::Identity;
			float CloudShadowmap_FarDepthKm = 0.0f;
			float CloudShadowmap_Strength = 0.0f;
			FRDGTexture* CloudShadowmap_RDGTexture = BlackDummyTexture;
			if (CloudInfo && AtmosphericDirectionalLightIndex >= 0 && AtmosphereLightProxy)
			{
				CloudShadowmap_RDGTexture = GraphBuilder.RegisterExternalTexture(View.VolumetricCloudShadowExtractedRenderTarget[AtmosphericDirectionalLightIndex]);
				CloudWorldToLightClipShadowMatrix = CloudInfo->GetVolumetricCloudCommonShaderParameters().CloudShadowmapTranslatedWorldToLightClipMatrix[AtmosphericDirectionalLightIndex];
				CloudShadowmap_FarDepthKm = CloudInfo->GetVolumetricCloudCommonShaderParameters().CloudShadowmapFarDepthKm[AtmosphericDirectionalLightIndex].X;
				CloudShadowmap_Strength = AtmosphereLightProxy->GetCloudShadowOnSurfaceStrength();
			}
			PassParameters->CloudShadowmapTexture = CloudShadowmap_RDGTexture;
			PassParameters->CloudShadowmapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->CloudShadowmapFarDepthKm = CloudShadowmap_FarDepthKm;
			PassParameters->CloudShadowmapStrength = CloudShadowmap_Strength;
			PassParameters->CloudShadowmapTranslatedWorldToLightClipMatrix = CloudWorldToLightClipShadowMatrix;
			PassParameters->RaytracedShadowsVolume = GraphBuilder.CreateSRV(RaytracedShadowsVolume);

			PassParameters->LightFunctionAtlas = LightFunctionAtlas::BindGlobalParameters(GraphBuilder, View);
			if (LightsToInject.DirectionalLightFunction && bUseLightFunctionAtlasEnabledAndSupported)
			{
				PassParameters->DirectionalApplyLightFunctionFromAtlas = LightsToInject.DirectionalLightFunction->Proxy->HasValidLightFunctionAtlasSlot() ? 1 :0;
				PassParameters->DirectionalLightFunctionAtlasLightIndex = PassParameters->DirectionalApplyLightFunctionFromAtlas == 1 ? LightsToInject.DirectionalLightFunction->Proxy->GetLightFunctionAtlasLightIndex() : 0;
			}
			else
			{
				PassParameters->DirectionalApplyLightFunctionFromAtlas = 0;
				PassParameters->DirectionalLightFunctionAtlasLightIndex = 0;
			}

			const bool bUseLumenGI = View.GetLumenTranslucencyGIVolume().Texture0 != nullptr && FDataDrivenShaderPlatformInfo::GetSupportsLumenGI(View.GetShaderPlatform());
			const bool bUseMegaLights = View.GetMegaLightsVolume().Texture != nullptr && MegaLights::IsEnabled(ViewFamily);
			const bool bUseGlobalDistanceField = UseGlobalDistanceField() && Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0;

			const bool bUseDistanceFieldSkyOcclusion =
				ViewFamily.EngineShowFlags.AmbientOcclusion
				&& !bUseLumenGI
				&& Scene->SkyLight
				&& Scene->SkyLight->bCastShadows
				&& Scene->SkyLight->bCastVolumetricShadow
				&& ShouldRenderDistanceFieldAO(Views, ViewFamily.EngineShowFlags, AnyViewHasGIMethodSupportingDFAO())
				&& SupportsDistanceFieldAO(View.GetFeatureLevel(), View.GetShaderPlatform())
				&& bUseGlobalDistanceField
				&& Views.Num() == 1
				&& View.IsPerspectiveProjection();

			const int32 SuperSampleCount = FVolumetricFogLightScatteringCS::GetSuperSampleCount(GVolumetricFogHistoryMissSupersampleCount);
			check(PassParameters->VolumetricFogParameters.HistoryMissSuperSampleCount == SuperSampleCount);

			// Ubershader "dynamic-if" parameters
			PassParameters->bUseTemporalReprojection = bUseTemporalReprojection;
			PassParameters->bDistanceFieldSkyOcclusion = bUseDistanceFieldSkyOcclusion;
			PassParameters->bLumenGI = bUseLumenGI;
			PassParameters->bVirtualShadowMap = VirtualShadowMapArray.IsAllocated();
			PassParameters->bUseRaytracedShadowsVolume = bUseRaytracedShadowsVolume;
			PassParameters->bUseLightFunctionAtlas = bUseLightFunctionAtlasEnabledAndSupported;
			PassParameters->bUseMegaLights = bUseMegaLights;
			PassParameters->bUseLightSoftFading = GetVolumetricFogLightSoftFading() > 0.0f;
			PassParameters->bUseSuperSampling = SuperSampleCount > 1;

			FVolumetricFogLightScatteringCS::FPermutationDomain PermutationVector;
			PermutationVector.Set< FVolumetricFogLightScatteringCS::FTemporalReprojection >(PassParameters->bUseTemporalReprojection > 0);
			PermutationVector.Set< FVolumetricFogLightScatteringCS::FDistanceFieldSkyOcclusion >(PassParameters->bDistanceFieldSkyOcclusion > 0);
			PermutationVector.Set< FVolumetricFogLightScatteringCS::FSuperSampling>(PassParameters->bUseSuperSampling > 0);
			PermutationVector.Set< FVolumetricFogLightScatteringCS::FLumenGI >(PassParameters->bLumenGI > 0);
			PermutationVector.Set< FVolumetricFogLightScatteringCS::FVirtualShadowMap >(PassParameters->bVirtualShadowMap > 0);
			PermutationVector.Set< FVolumetricFogLightScatteringCS::FRaytracedShadowsVolume >(PassParameters->bUseRaytracedShadowsVolume > 0);
			PermutationVector.Set< FVolumetricFogLightScatteringCS::FSampleLightFunctionAtlas >(PassParameters->bUseLightFunctionAtlas > 0);
			PermutationVector.Set< FVolumetricFogLightScatteringCS::FMegaLights>(PassParameters->bUseMegaLights > 0);
			PermutationVector.Set< FVolumetricFogLightScatteringCS::FLightSoftFading >(PassParameters->bUseLightSoftFading > 0);

			PermutationVector = FVolumetricFogLightScatteringCS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
			auto ComputeShader = View.ShaderMap->GetShader< FVolumetricFogLightScatteringCS >(PermutationVector);

			// Ubershader default parameters and feature dynamic branching
			if (!PrecachePSO(FComputePipelineStateInitializer(ComputeShader.GetComputeShader())) &&
				PlatformSupportsVolumetricFogUbershader(View.GetShaderPlatform()))
			{
				PermutationVector.Set< FVolumetricFogLightScatteringCS::FUbershader >(true);
				PermutationVector = FVolumetricFogLightScatteringCS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
				ComputeShader = View.ShaderMap->GetShader< FVolumetricFogLightScatteringCS >( PermutationVector );
			}
			ClearUnusedGraphResources(ComputeShader, PassParameters);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("LightScattering %dx%dx%d SS:%d %s %s %s",
					VolumetricFogViewGridSize.X,
					VolumetricFogViewGridSize.Y,
					VolumetricFogViewGridSize.Z,
					SuperSampleCount,
					bUseDistanceFieldSkyOcclusion ? TEXT("DFAO") : TEXT(""),
					PassParameters->DirectionalLightLightFunctionTexture ? TEXT("LF") : TEXT(""),
					bUseLumenGI ? TEXT("Lumen") : TEXT("")),
				PassParameters,
				ERDGPassFlags::Compute,
				[PassParameters, ComputeShader, &View, this, VolumetricFogViewGridSize](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				const FIntVector NumGroups = FComputeShaderUtils::GetGroupCount(VolumetricFogViewGridSize, FVolumetricFogLightScatteringCS::GetGroupSize());

				SetComputePipelineState(RHICmdList, ComputeShader.GetComputeShader());

				SetShaderParameters(RHICmdList, ComputeShader, ComputeShader.GetComputeShader(), *PassParameters);
				DispatchComputeShader(RHICmdList, ComputeShader.GetShader(), NumGroups.X, NumGroups.Y, NumGroups.Z);
				UnsetShaderUAVs(RHICmdList, ComputeShader, ComputeShader.GetComputeShader());
			});
		}

		FRDGTexture* IntegratedLightScattering = GraphBuilder.CreateTexture(VolumeDescNoRT, TEXT("VolumetricFog.IntegratedLightScattering"));
		FRDGTextureUAV* IntegratedLightScatteringUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(IntegratedLightScattering));

		{
			RDG_EVENT_SCOPE(GraphBuilder, "VolumetricFog::FinalIntegration");

			FVolumetricFogFinalIntegrationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVolumetricFogFinalIntegrationCS::FParameters>();
			PassParameters->LightScattering = IntegrationData.LightScattering;
			PassParameters->RWIntegratedLightScattering = IntegratedLightScatteringUAV;
			PassParameters->VolumetricFogNearFadeInDistanceInv = View.VolumetricFogNearFadeInDistanceInv;
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			SetupVolumetricFogIntegrationParameters(PassParameters->VolumetricFogParameters, View, IntegrationData);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("FinalIntegration"),
				PassParameters,
				ERDGPassFlags::Compute,
				[PassParameters, &View, VolumetricFogViewGridSize, IntegrationData, this](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				const FIntVector NumGroups = FIntVector::DivideAndRoundUp(VolumetricFogViewGridSize, VolumetricFogIntegrationGroupSize);

				auto ComputeShader = View.ShaderMap->GetShader< FVolumetricFogFinalIntegrationCS >();
				SetComputePipelineState(RHICmdList, ComputeShader.GetComputeShader());

				SetShaderParameters(RHICmdList, ComputeShader, ComputeShader.GetComputeShader(), *PassParameters);
				DispatchComputeShader(RHICmdList, ComputeShader.GetShader(), NumGroups.X, NumGroups.Y, 1);
				UnsetShaderUAVs(RHICmdList, ComputeShader, ComputeShader.GetComputeShader());
			});
		}

		View.VolumetricFogResources.IntegratedLightScatteringTexture = IntegratedLightScattering;

		if (bUseTemporalReprojection)
		{
			const FVector2f ViewRectSize = FVector2f(View.ViewRect.Size());

			GraphBuilder.QueueTextureExtraction(IntegrationData.LightScattering, &View.ViewState->LightScatteringHistory);
			View.ViewState->LightScatteringHistoryPreExposure = View.CachedViewUniformShaderParameters->PreExposure;
			View.ViewState->PrevLightScatteringViewGridUVToViewRectVolumeUV = ViewRectSize / (FVector2f(VolumetricFogViewGridSize.X, VolumetricFogViewGridSize.Y) * VolumetricFogGridPixelSize);

			View.ViewState->VolumetricFogPrevViewGridRectUVToResourceUV = FVector2f(VolumetricFogViewGridSize.X, VolumetricFogViewGridSize.Y) / FVector2f(VolumetricFogResourceGridSize.X, VolumetricFogResourceGridSize.Y);
			View.ViewState->VolumetricFogPrevUVMax = GetFroxelGridUVMaxForSampling(ViewRectSize, VolumetricFogResourceGridSize, VolumetricFogGridPixelSize);
			View.ViewState->VolumetricFogPrevUVMaxForTemporalBlend = GetFroxelGridPrevUVMaxForTemporalBlend(ViewRectSize, VolumetricFogResourceGridSize, VolumetricFogGridPixelSize);
			View.ViewState->VolumetricFogPrevResourceGridSize = VolumetricFogResourceGridSize;
		}
		else if (View.ViewState)
		{
			View.ViewState->LightScatteringHistory = nullptr;
			View.ViewState->LightScatteringHistoryPreExposure = 1.0f;
			View.ViewState->PrevLightScatteringViewGridUVToViewRectVolumeUV = FVector2f::One();
			View.ViewState->VolumetricFogPrevViewGridRectUVToResourceUV = FVector2f::One();
			View.ViewState->VolumetricFogPrevUVMax = FVector2f::One();
			View.ViewState->VolumetricFogPrevUVMaxForTemporalBlend = FVector2f::One();
			View.ViewState->VolumetricFogPrevResourceGridSize = VolumetricFogResourceGridSize;
		}

		if (bUseTemporalReprojection && GVolumetricFogConservativeDepth > 0)
		{
			GraphBuilder.QueueTextureExtraction(ConservativeDepthTexture, &View.ViewState->PrevLightScatteringConservativeDepthTexture);
		}
		else if (View.ViewState)
		{
			View.ViewState->PrevLightScatteringConservativeDepthTexture = NULL;
		}
	}
}

void VolumetricFogGlobalPSOCollector(const FSceneTexturesConfig& SceneTexturesConfig, int32 GlobalPSOCollectorIndex, TArray<FPSOPrecacheData>& PSOInitializers)
{
	EShaderPlatform ShaderPlatform = SceneTexturesConfig.ShaderPlatform;
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(ShaderPlatform);

	auto AddPSOInitializer = [&](FRHIPixelShader* PixelShaderRHI)
	{		
		auto VertexShader = GlobalShaderMap->GetShader< FWriteToBoundingSphereVS >();
		TOptionalShaderMapRef<FWriteToSliceGS> GeometryShader(GlobalShaderMap);
		
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		SetupInjectShadowedLocalLightPSO(VertexShader.GetVertexShader(), GeometryShader.GetGeometryShader(), PixelShaderRHI, GraphicsPSOInit);

		FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
		RenderTargetsInfo.NumSamples = 1;
		FRDGTextureDesc VolumetricFogDesc = GetVolumetricFogRDGTextureDesc(FIntVector());
		AddRenderTargetInfo(VolumetricFogDesc.Format, VolumetricFogDesc.Flags, RenderTargetsInfo);

		GraphicsPSOInit.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(GraphicsPSOInit);		
		ApplyTargetsInfo(GraphicsPSOInit, RenderTargetsInfo);

		FPSOPrecacheData PSOPrecacheData;
		PSOPrecacheData.bRequired = true;
		PSOPrecacheData.Type = FPSOPrecacheData::EType::Graphics;
		PSOPrecacheData.GraphicsPSOInitializer = GraphicsPSOInit;
		#if PSO_PRECACHING_VALIDATE
		PSOPrecacheData.PSOCollectorIndex = GlobalPSOCollectorIndex;
		PSOPrecacheData.VertexFactoryType = nullptr;
		#endif // PSO_PRECACHING_VALIDATE

		PSOInitializers.Add(MoveTemp(PSOPrecacheData));
	};
	   
	// Precache PSOs are never required
	const bool bRequired = false;

	EShaderPermutationFlags PermutationFlags = EShaderPermutationFlags::None;

	FShaderType* ShaderType = FShaderType::GetShaderTypeByName(FInjectShadowedLocalLightPS::GetStaticType().GetName());
	FGlobalShaderType* GlobalShaderType = ShaderType->GetGlobalShaderType();
	for (int32 PermutationId = 0; PermutationId < GlobalShaderType->GetPermutationCount(); PermutationId++)
	{
		if (GlobalShaderType->ShouldCompilePermutation(ShaderPlatform, PermutationId, PermutationFlags) &&
			GlobalShaderType->ShouldPrecachePermutation(ShaderPlatform, PermutationId, PermutationFlags) == EShaderPermutationPrecacheRequest::Precached)
		{
			TShaderRef<FShader> GlobalShader = GlobalShaderMap->GetShader(GlobalShaderType, PermutationId);
			
			FRHIPixelShader* RHIPixelShader = static_cast<FRHIPixelShader*>(GlobalShader.GetRHIShaderBase(SF_Pixel, bRequired));
			if (RHIPixelShader)
			{
				AddPSOInitializer(RHIPixelShader);
			}
		}
	}
}

FRegisterGlobalPSOCollectorFunction RegisterVolumetricFogGlobalPSOCollector(&VolumetricFogGlobalPSOCollector, VolumetricFogGlobalPSOCollectorName);
