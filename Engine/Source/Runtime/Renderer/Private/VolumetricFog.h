// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricFog.h
=============================================================================*/

#pragma once

#include "RHIDefinitions.h"
#include "SceneView.h"
#include "SceneRendering.h"

extern int32 GetVolumetricFogGridPixelSize();

// Grid size for resource allocation to be independent of dynamic resolution
extern FIntVector GetVolumetricFogResourceGridSize(const FViewInfo& View, int32& OutVolumetricFogGridPixelSize);
// Grid size for the view rectangle within the allocated resource
extern FIntVector GetVolumetricFogViewGridSize(const FViewInfo& View, int32& OutVolumetricFogGridPixelSize);

extern FVector2f GetVolumetricFogFroxelToScreenSVPosRatio(const FViewInfo& View);

extern FRDGTextureDesc GetVolumetricFogRDGTextureDesc(const FIntVector& VolumetricFogResourceGridSize);

extern bool ShouldRenderVolumetricFog(const FScene* Scene, const FSceneViewFamily& ViewFamily);
extern const FProjectedShadowInfo* GetShadowForInjectionIntoVolumetricFog(const FVisibleLightInfo& VisibleLightInfo);

extern bool LightNeedsSeparateInjectionIntoVolumetricFogForOpaqueShadow(const FViewInfo& View, const FLightSceneInfo* LightSceneInfo, const FVisibleLightInfo& VisibleLightInfo, const FScene& InScene);

extern void VolumetricFogOnRenderBegin(FScene& Scene, TArray<FViewInfo>& Views);
