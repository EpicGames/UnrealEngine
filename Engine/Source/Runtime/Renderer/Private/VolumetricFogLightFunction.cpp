// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricFogLightFunction.cpp
=============================================================================*/

#include "VolumetricFog.h"
#include "ScenePrivate.h"
#include "SceneUtils.h"
#include "LightFunctionRendering.h"
#include "LightRendering.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessing.h"
#include "PipelineStateCache.h"
#include "ShadowRendering.h"
#include "Materials/MaterialRenderProxy.h"
#include "LogRenderer.h"

float GVolumetricFogLightFunctionDirectionalLightSupersampleScale = 2.0f;
FAutoConsoleVariableRef CVarVolumetricFogLightFunctionSupersampleScale(
	TEXT("r.VolumetricFog.LightFunction.DirectionalLightSupersampleScale"),
	GVolumetricFogLightFunctionDirectionalLightSupersampleScale,
	TEXT("Scales the slice depth distribution."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

extern int GVolumetricFogLightFunction;
static bool inline LocalLightLighFunctionsEnabled()
{
	return GVolumetricFogLightFunction > 0;
}

class FVolumetricFogLightFunctionPS : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FVolumetricFogLightFunctionPS, Material);

	class FLightType : SHADER_PERMUTATION_INT("LIGHT_TYPE", 4);
	using FPermutationDomain = TShaderPermutationDomain< FLightType >;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FMatrix44f, LightFunctionTranslatedWorldToLight)
		SHADER_PARAMETER(FMatrix44f, ShadowToTranslatedWorld)
		SHADER_PARAMETER(FVector4f, LightFunctionParameters)
		SHADER_PARAMETER(FVector3f, LightFunctionParameters2)
		SHADER_PARAMETER(FVector3f, LightTranslatedWorldPosition)
		SHADER_PARAMETER(FVector2f, LightFunctionTexelSize)
		SHADER_PARAMETER(FVector3f, CameraRelativeLightPosition)
	END_SHADER_PARAMETER_STRUCT()

	FVolumetricFogLightFunctionPS() {}
	FVolumetricFogLightFunctionPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(
			this,
			Initializer.PermutationId,
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(),
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false);
	}

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.MaterialParameters.MaterialDomain == MD_LightFunction;
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FMaterialRenderProxy* MaterialProxy)
	{
		const FMaterial& Material = MaterialProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialProxy);
		FMaterialShader::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
		FMaterialShader::SetParameters(BatchedParameters, MaterialProxy, Material, View);
	}

	FParameters GetParameters(
		const FSceneView& View, 
		const FLightSceneInfo* LightSceneInfo, 
		const FVector2D& LightFunctionTexelSizeValue,
		const FMatrix44f& ShadowToTranslatedWorldValue)
	{
		FParameters PS;

		PS.LightFunctionParameters = FLightFunctionSharedParameters::GetLightFunctionSharedParameters(LightSceneInfo, 1.0f);
		PS.LightFunctionParameters2 = FVector3f(
			LightSceneInfo->Proxy->GetLightFunctionFadeDistance(),
			LightSceneInfo->Proxy->GetLightFunctionDisabledBrightness(),
			0.0f);

		{
			const FVector Scale = LightSceneInfo->Proxy->GetLightFunctionScale();
			// Switch x and z so that z of the user specified scale affects the distance along the light direction
			const FVector InverseScale = FVector(1.f / Scale.Z, 1.f / Scale.Y, 1.f / Scale.X);
			const FMatrix WorldToLight = LightSceneInfo->Proxy->GetWorldToLight() * FScaleMatrix(FVector(InverseScale));
			const FMatrix TranslatedWorldToWorld = FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation());

			PS.LightFunctionTranslatedWorldToLight = FMatrix44f(TranslatedWorldToWorld * WorldToLight);
		}

		PS.LightFunctionTexelSize = FVector2f(LightFunctionTexelSizeValue);
		PS.ShadowToTranslatedWorld = ShadowToTranslatedWorldValue;
		PS.LightTranslatedWorldPosition = FVector4f(LightSceneInfo->Proxy->GetPosition() + View.ViewMatrices.GetPreViewTranslation());
		PS.CameraRelativeLightPosition = GetCamRelativeLightPosition(View.ViewMatrices, *LightSceneInfo);

		return PS;
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FVolumetricFogLightFunctionPS, TEXT("/Engine/Private/VolumetricFogLightFunction.usf"), TEXT("Main"), SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FVolumetricFogLightFunctionParameters, )
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FSceneRenderer::RenderLightFunctionForVolumetricFog(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FIntVector VolumetricFogGridSize,
	float VolumetricFogMaxDistance,
	FLightSceneInfo* DirectionalLightSceneInfo,
	FMatrix44f& OutDirectionalLightFunctionTranslatedWorldToShadow,
	FRDGTexture*& OutDirectionalLightFunctionTexture)
{
	RDG_EVENT_SCOPE(GraphBuilder, "DirectionalLightFunction");

	// Estimate the resolution and the projection matrix.
	FProjectedShadowInfo ProjectedShadowInfo;
	FIntPoint LightFunctionResolution;
	{
		const FVector ViewForward = View.ViewMatrices.GetOverriddenTranslatedViewMatrix().GetColumn(2);
		const FVector ViewUp = View.ViewMatrices.GetOverriddenTranslatedViewMatrix().GetColumn(1);
		const FVector ViewRight = View.ViewMatrices.GetOverriddenTranslatedViewMatrix().GetColumn(0);

		const FVector LightDirection = DirectionalLightSceneInfo->Proxy->GetDirection().GetSafeNormal();
		FVector AxisWeights;
		AxisWeights.X = FMath::Abs(LightDirection | ViewRight) * VolumetricFogGridSize.X;
		AxisWeights.Y = FMath::Abs(LightDirection | ViewUp) * VolumetricFogGridSize.Y;
		AxisWeights.Z = FMath::Abs(LightDirection | ViewForward) * VolumetricFogGridSize.Z;

		const float VolumeResolutionEstimate = FMath::Max(AxisWeights.X, FMath::Max(AxisWeights.Y, AxisWeights.Z)) * GVolumetricFogLightFunctionDirectionalLightSupersampleScale;
		LightFunctionResolution = FIntPoint(FMath::TruncToInt(VolumeResolutionEstimate), FMath::TruncToInt(VolumeResolutionEstimate));

		// Snap the resolution to allow render target pool hits most of the time
		const int32 ResolutionSnapFactor = 32;
		LightFunctionResolution.X = FMath::DivideAndRoundUp(LightFunctionResolution.X, ResolutionSnapFactor) * ResolutionSnapFactor;
		LightFunctionResolution.Y = FMath::DivideAndRoundUp(LightFunctionResolution.Y, ResolutionSnapFactor) * ResolutionSnapFactor;

		// Guard against invalid resolutions
		const uint32 LiFuncResXU32 = (uint32)LightFunctionResolution.X;
		const uint32 LiFuncResYU32 = (uint32)LightFunctionResolution.Y;
		const uint32 MaxTextureResU32 = GMaxTextureDimensions;
		if (LiFuncResXU32 > MaxTextureResU32 || LiFuncResYU32 > MaxTextureResU32)
		{
#if !(UE_BUILD_SHIPPING)
			UE_LOGF(LogRenderer, Error,
				"Invalid LightFunctionResolution %dx%d, View={ %ls}, LightDirection={ %ls }",
				LightFunctionResolution.X,
				LightFunctionResolution.Y,
				*View.ViewMatrices.GetOverriddenTranslatedViewMatrix().ToString(),
				*LightDirection.ToString());
#endif
			const uint32 ClampedRes = FMath::Max(FMath::Min3(LiFuncResXU32, LiFuncResYU32, MaxTextureResU32), (uint32)ResolutionSnapFactor);
			LightFunctionResolution.X = ClampedRes;
			LightFunctionResolution.Y = ClampedRes;
		}

		FWholeSceneProjectedShadowInitializer ShadowInitializer;

		check(VolumetricFogMaxDistance > 0);
		FSphere Bounds = DirectionalLightSceneInfo->Proxy->GetShadowSplitBoundsDepthRange(View, View.ViewMatrices.GetViewOrigin(), 0, VolumetricFogMaxDistance, NULL);
		check(Bounds.W > 0);

		const float ShadowExtent = Bounds.W / FMath::Sqrt(3.0f);
		const FBoxSphereBounds SubjectBounds(Bounds.Center, FVector(ShadowExtent, ShadowExtent, ShadowExtent), Bounds.W);
		ShadowInitializer.PreShadowTranslation = -Bounds.Center;
		ShadowInitializer.WorldToLight = FInverseRotationMatrix(LightDirection.Rotation());
		ShadowInitializer.Scales = FVector2D(1.0f / Bounds.W, 1.0f / Bounds.W);
		ShadowInitializer.SubjectBounds = FBoxSphereBounds(FVector::ZeroVector, SubjectBounds.BoxExtent, SubjectBounds.SphereRadius);
		ShadowInitializer.WAxis = FVector4(0, 0, 0, 1);
		ShadowInitializer.MinLightW = -HALF_WORLD_MAX;
		// Reduce casting distance on a directional light
		// This is necessary to improve floating point precision in several places, especially when deriving frustum verts from InvReceiverMatrix
		ShadowInitializer.MaxDistanceToCastInLightW = HALF_WORLD_MAX / 32.0f;
		ShadowInitializer.bRayTracedDistanceField = false;
		ShadowInitializer.CascadeSettings.bFarShadowCascade = false;

		ProjectedShadowInfo.SetupWholeSceneProjection(
			DirectionalLightSceneInfo,
			&View,
			ShadowInitializer,
			LightFunctionResolution.X,
			LightFunctionResolution.Y,
			LightFunctionResolution.X,
			LightFunctionResolution.Y,
			0
		);

		OutDirectionalLightFunctionTranslatedWorldToShadow = ProjectedShadowInfo.TranslatedWorldToClipInnerMatrix;
	}

	// Now render the texture
	{
		FRDGTextureDesc LightFunctionTextureDesc = FRDGTextureDesc::Create2D(LightFunctionResolution, PF_G8, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_RenderTargetable);
		LightFunctionTextureDesc.Flags |= GFastVRamConfig.VolumetricFog;

		FRDGTexture* LightFunctionTexture = GraphBuilder.CreateTexture(LightFunctionTextureDesc, TEXT("VolumetricFog.LightFunction"));
		OutDirectionalLightFunctionTexture = LightFunctionTexture;

		const FMaterialRenderProxy* MaterialProxyForRendering = DirectionalLightSceneInfo->Proxy->GetLightFunctionMaterial();
		const FMaterial& Material = MaterialProxyForRendering->GetMaterialWithFallback(Scene->GetFeatureLevel(), MaterialProxyForRendering);

		FVolumetricFogLightFunctionParameters* PassParameters = GraphBuilder.AllocParameters<FVolumetricFogLightFunctionParameters>();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(LightFunctionTexture, ERenderTargetLoadAction::ENoAction);
		PassParameters->SceneTextures = SceneTextures.UniformBuffer;

		FMatrix44f LightFunctionTranslatedWorldToShadowMatrix = OutDirectionalLightFunctionTranslatedWorldToShadow;
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("LightFunction %ux%u Material=%s", LightFunctionResolution.X, LightFunctionResolution.Y, *(Material.GetFriendlyName())),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, &View, MaterialProxyForRendering, &Material, LightFunctionResolution, DirectionalLightSceneInfo, LightFunctionTranslatedWorldToShadowMatrix](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(0.f, 0.f, 0.f, LightFunctionResolution.X, LightFunctionResolution.Y, 1.f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;

				const FMaterialShaderMap* MaterialShaderMap = Material.GetRenderingThreadShaderMap();
				TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap);

				check(DirectionalLightSceneInfo->Proxy->GetLightType() == LightType_Directional)
				FVolumetricFogLightFunctionPS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FVolumetricFogLightFunctionPS::FLightType>(3);
				TShaderRef<FVolumetricFogLightFunctionPS> PixelShader = MaterialShaderMap->GetShader<FVolumetricFogLightFunctionPS>(PermutationVector);

				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				FVolumetricFogLightFunctionPS::FParameters PS = PixelShader->GetParameters(View, DirectionalLightSceneInfo, FVector2D(1.0f / LightFunctionResolution.X, 1.0f / LightFunctionResolution.Y), LightFunctionTranslatedWorldToShadowMatrix.Inverse());

				ClearUnusedGraphResources(PixelShader, &PS);
				SetShaderParametersMixedPS(RHICmdList, PixelShader, PS, View, MaterialProxyForRendering);

				DrawRectangle(
					RHICmdList,
					0, 0,
					LightFunctionResolution.X, LightFunctionResolution.Y,
					0, 0,
					LightFunctionResolution.X, LightFunctionResolution.Y,
					LightFunctionResolution,
					LightFunctionResolution,
					VertexShader);
			}
		);
	}
}
