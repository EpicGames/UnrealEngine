// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"
#include "SceneView.h"
#include "SceneRendering.h"

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVoxelizeVolumePassUniformParameters, )
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER(FMatrix44f, ViewToVolumeClip)
	SHADER_PARAMETER(FVector2f, ClipRatio)
	SHADER_PARAMETER(FVector4f, FrameJitterOffset0)
	SHADER_PARAMETER_STRUCT(FVolumetricFogGlobalData, VolumetricFog)
	SHADER_PARAMETER(FVector3f, RenderVolumetricCloudParametersCloudLayerCenterKm)
	SHADER_PARAMETER(float, RenderVolumetricCloudParametersPlanetRadiusKm)
	SHADER_PARAMETER(float, RenderVolumetricCloudParametersBottomRadiusKm)
	SHADER_PARAMETER(float, RenderVolumetricCloudParametersTopRadiusKm)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

extern FVector3f VolumetricFogTemporalRandom(uint32 FrameNumber);

float GetVolumetricFogLightSoftFading();

struct FVolumetricFogIntegrationParameterData
{
	FVolumetricFogIntegrationParameterData() :
		bTemporalHistoryIsValid(false)
	{}

	bool bTemporalHistoryIsValid;
	TArray<FVector4f, TInlineAllocator<16>> FrameJitterOffsetValues;
	FRDGTexture* VBufferA;
	FRDGTexture* VBufferB;
	FRDGTextureUAV* VBufferA_UAV;
	FRDGTextureUAV* VBufferB_UAV;

	FRDGTexture* LightScattering;
	FRDGTextureUAV* LightScatteringUAV;
};

BEGIN_SHADER_PARAMETER_STRUCT(FVolumetricFogIntegrationParameters, )
	SHADER_PARAMETER_STRUCT_REF(FVolumetricFogGlobalData, VolumetricFog)
	SHADER_PARAMETER(FMatrix44f, UnjitteredClipToTranslatedWorld)
	SHADER_PARAMETER(FMatrix44f, UnjitteredPrevTranslatedWorldToClip)
	SHADER_PARAMETER_ARRAY(FVector4f, FrameJitterOffsets, [16])
	SHADER_PARAMETER(float, HistoryWeight)
	SHADER_PARAMETER(uint32, HistoryMissSuperSampleCount)
	SHADER_PARAMETER(float, GridCenterOffsetFromDepthBuffer)
	SHADER_PARAMETER(float, GridCenterOffsetThresholdToAcceptDepthBufferOffset)
END_SHADER_PARAMETER_STRUCT()

void SetupVolumetricFogIntegrationParameters(
	FVolumetricFogIntegrationParameters& Out,
	FViewInfo& View,
	const FVolumetricFogIntegrationParameterData& IntegrationData);

inline int32 ComputeZSliceFromDepth(float SceneDepth, FVector GridZParams)
{
	return FMath::TruncToInt(FMath::Log2(SceneDepth * GridZParams.X + GridZParams.Y) * GridZParams.Z);
}

extern FVector GetVolumetricFogGridZParams(float NearPlane, float FarPlane, int32 GridSizeZ);