// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricFogVoxelization.cpp
=============================================================================*/

#include "VolumetricFog.h"
#include "ScenePrivate.h"
#include "SceneUtils.h"
#include "VolumetricFogShared.h"
#include "LocalVertexFactory.h"
#include "DynamicMeshBuilder.h"
#include "SpriteIndexBuffer.h"
#include "StaticMeshResources.h"
#include "MeshPassProcessor.inl"
#include "VolumetricCloudRendering.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "SceneUniformBuffer.h"

int32 GVolumetricFogVoxelizationSlicesPerGSPass = 8;
FAutoConsoleVariableRef CVarVolumetricFogVoxelizationSlicesPerPass(
	TEXT("r.VolumetricFog.VoxelizationSlicesPerGSPass"),
	GVolumetricFogVoxelizationSlicesPerGSPass,
	TEXT("How many depth slices to render in a single voxelization pass (max geometry shader expansion).  Must recompile voxelization shaders to propagate changes."),
	ECVF_ReadOnly
	);

int32 GVolumetricFogVoxelizationShowOnlyPassIndex = -1;
FAutoConsoleVariableRef CVarVolumetricFogVoxelizationShowOnlyPassIndex(
	TEXT("r.VolumetricFog.VoxelizationShowOnlyPassIndex"),
	GVolumetricFogVoxelizationShowOnlyPassIndex,
	TEXT("When >= 0, indicates a single voxelization pass to render for debugging."),
	ECVF_RenderThreadSafe
	);

static bool DoesPlatformSupportVolumetricFogVoxelization(const FStaticShaderPlatform Platform)
{
	return RHISupportsGeometryShaders(Platform) || RHISupportsVertexShaderLayer(Platform);
}

static FORCEINLINE int32 GetVoxelizationSlicesPerPass(EShaderPlatform Platform)
{
	return RHISupportsGeometryShaders(Platform) ? GVolumetricFogVoxelizationSlicesPerGSPass : 1;
}

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FVoxelizeVolumePassUniformParameters, "VoxelizeVolumePass", SceneTextures);

TRDGUniformBufferRef<FVoxelizeVolumePassUniformParameters> CreateVoxelizeVolumePassUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, 
	const FVolumetricFogIntegrationParameterData& IntegrationData,
	FVector2D Jitter,
	const FVolumetricCloudRenderSceneInfo* CloudInfo)
{
	auto* Parameters = GraphBuilder.AllocParameters<FVoxelizeVolumePassUniformParameters>();
	SetupSceneTextureUniformParameters(GraphBuilder, &View.GetSceneTextures(), View.FeatureLevel, ESceneTextureSetupMode::None, Parameters->SceneTextures);

	Parameters->ClipRatio = GetVolumetricFogFroxelToScreenSVPosRatio(View);

	Parameters->ViewToVolumeClip = FMatrix44f(View.ViewMatrices.ComputeProjectionNoAAMatrix());		// LWC_TODO: Precision loss?
	Parameters->ViewToVolumeClip.M[2][0] += Jitter.X;
	Parameters->ViewToVolumeClip.M[2][1] += Jitter.Y;

	Parameters->FrameJitterOffset0 = IntegrationData.FrameJitterOffsetValues[0];

	SetupVolumetricFogGlobalData(View, Parameters->VolumetricFog);

	if (CloudInfo)
	{
		const FVolumetricCloudCommonShaderParameters& CloudGlobalShaderParams = CloudInfo->GetVolumetricCloudCommonShaderParameters();
		Parameters->RenderVolumetricCloudParametersCloudLayerCenterKm = CloudGlobalShaderParams.CloudLayerCenterKm;
		Parameters->RenderVolumetricCloudParametersPlanetRadiusKm = CloudGlobalShaderParams.PlanetRadiusKm;
		Parameters->RenderVolumetricCloudParametersBottomRadiusKm = CloudGlobalShaderParams.BottomRadiusKm;
		Parameters->RenderVolumetricCloudParametersTopRadiusKm = CloudGlobalShaderParams.TopRadiusKm;
	}
	else
	{
		Parameters->RenderVolumetricCloudParametersCloudLayerCenterKm = FVector3f::ZeroVector;
		Parameters->RenderVolumetricCloudParametersPlanetRadiusKm = 0.001f;
		Parameters->RenderVolumetricCloudParametersBottomRadiusKm = 0.5f;
		Parameters->RenderVolumetricCloudParametersTopRadiusKm = 1.0f;
	}

	return GraphBuilder.CreateUniformBuffer(Parameters);
}

class FQuadMeshVertexBuffer : public FRenderResource
{
public:
	FStaticMeshVertexBuffers Buffers;

	FQuadMeshVertexBuffer()
	{
		TArray<FDynamicMeshVertex> Vertices;

		// Vertex position constructed in the shader
		Vertices.Add(FDynamicMeshVertex(FVector3f(0.0f, 0.0f, 0.0f)));
		Vertices.Add(FDynamicMeshVertex(FVector3f(0.0f, 0.0f, 0.0f)));
		Vertices.Add(FDynamicMeshVertex(FVector3f(0.0f, 0.0f, 0.0f)));
		Vertices.Add(FDynamicMeshVertex(FVector3f(0.0f, 0.0f, 0.0f)));

		Buffers.PositionVertexBuffer.Init(Vertices.Num());
		Buffers.StaticMeshVertexBuffer.Init(Vertices.Num(), 1);

		for (int32 i = 0; i < Vertices.Num(); i++)
		{
			const FDynamicMeshVertex& Vertex = Vertices[i];

			Buffers.PositionVertexBuffer.VertexPosition(i) = Vertex.Position;
			Buffers.StaticMeshVertexBuffer.SetVertexTangents(i, Vertex.TangentX.ToFVector3f(), Vertex.GetTangentY(), Vertex.TangentZ.ToFVector3f());
			Buffers.StaticMeshVertexBuffer.SetVertexUV(i, 0, Vertex.TextureCoordinate[0]);
		}
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		Buffers.PositionVertexBuffer.InitResource(RHICmdList);
		Buffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
	}

	virtual void ReleaseRHI() override
	{
		Buffers.PositionVertexBuffer.ReleaseResource();
		Buffers.StaticMeshVertexBuffer.ReleaseResource();
	}
};

TGlobalResource<FQuadMeshVertexBuffer> GQuadMeshVertexBuffer;

TGlobalResource<FSpriteIndexBuffer<1>> GQuadMeshIndexBuffer;

class FQuadMeshVertexFactory final : public FLocalVertexFactory
{
public:
	FQuadMeshVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FLocalVertexFactory(InFeatureLevel, "FQuadMeshVertexFactory")
	{}

	~FQuadMeshVertexFactory()
	{
		ReleaseResource();
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FQuadMeshVertexBuffer* VertexBuffer = &GQuadMeshVertexBuffer;
		FLocalVertexFactory::FDataType NewData;
		VertexBuffer->Buffers.PositionVertexBuffer.BindPositionVertexBuffer(this, NewData);
		VertexBuffer->Buffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(this, NewData);
		VertexBuffer->Buffers.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(this, NewData);
		VertexBuffer->Buffers.StaticMeshVertexBuffer.BindLightMapVertexBuffer(this, NewData, 0);
		FColorVertexBuffer::BindDefaultColorVertexBuffer(this, NewData, FColorVertexBuffer::NullBindStride::ZeroForDefaultBufferBind);
		// Don't call SetData(), because that ends up calling UpdateRHI(), and if the resource has already been initialized
		// (e.g. when switching the feature level in the editor), that calls InitRHI(), resulting in an infinite loop.
		Data = NewData;
		FLocalVertexFactory::InitRHI(RHICmdList);
	}

	bool HasIncompatibleFeatureLevel(ERHIFeatureLevel::Type InFeatureLevel)
	{
		return InFeatureLevel != GetFeatureLevel();
	}
};

FQuadMeshVertexFactory* GQuadMeshVertexFactory = NULL;

class FVoxelizeVolumeShaderElementData : public FMeshMaterialShaderElementData
{
public:
	FVoxelizeVolumeShaderElementData(int32 InVoxelizationPassIndex)
		: VoxelizationPassIndex(InVoxelizationPassIndex)
	{}

	int32 VoxelizationPassIndex;
};

class FVoxelizeVolumeVS : public FMeshMaterialShader
{
public:
	DECLARE_INLINE_TYPE_LAYOUT(FVoxelizeVolumeVS, NonVirtual);

	FVoxelizeVolumeVS() = default;
	FVoxelizeVolumeVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		VoxelizationPassIndex.Bind(Initializer.ParameterMap, TEXT("VoxelizationPassIndex"));
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportVolumetricFogVoxelization(Parameters.Platform)
			&& Parameters.MaterialParameters.MaterialDomain == MD_Volume;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		if (RHISupportsGeometryShaders(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add( CFLAG_VertexToGeometryShader );
		}
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FVoxelizeVolumeShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
		if (!RHISupportsGeometryShaders(Scene->GetShaderPlatform()))
		{
			ShaderBindings.Add(VoxelizationPassIndex, ShaderElementData.VoxelizationPassIndex);
		}
	}

protected:
	LAYOUT_FIELD(FShaderParameter, VoxelizationPassIndex);
};

enum EVoxelizeShapeMode
{
	VMode_Primitive_Sphere,
	VMode_Object_Box
};

template<EVoxelizeShapeMode Mode>
class TVoxelizeVolumeVS : public FVoxelizeVolumeVS
{
public:
	DECLARE_SHADER_TYPE(TVoxelizeVolumeVS,MeshMaterial);
	typedef FVoxelizeVolumeVS Super;

	TVoxelizeVolumeVS() = default;
	TVoxelizeVolumeVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: Super(Initializer)
	{}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		
		if (Mode == VMode_Primitive_Sphere)
		{
			OutEnvironment.SetDefine(TEXT("VOXELIZE_SHAPE_MODE"), TEXT("PRIMITIVE_SPHERE_MODE"));
		}
		else if (Mode == VMode_Object_Box)
		{
			OutEnvironment.SetDefine(TEXT("VOXELIZE_SHAPE_MODE"), TEXT("OBJECT_BOX_MODE"));
		}
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TVoxelizeVolumeVS<VMode_Primitive_Sphere>,TEXT("/Engine/Private/VolumetricFogVoxelization.usf"),TEXT("VoxelizeVS"),SF_Vertex); 
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TVoxelizeVolumeVS<VMode_Object_Box>,TEXT("/Engine/Private/VolumetricFogVoxelization.usf"),TEXT("VoxelizeVS"),SF_Vertex); 

class FVoxelizeVolumeGS : public FMeshMaterialShader
{
public:
	DECLARE_INLINE_TYPE_LAYOUT(FVoxelizeVolumeGS, NonVirtual);

	FVoxelizeVolumeGS() = default;
	FVoxelizeVolumeGS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		VoxelizationPassIndex.Bind(Initializer.ParameterMap, TEXT("VoxelizationPassIndex"));
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return
			RHISupportsGeometryShaders(Parameters.Platform)
			&& DoesPlatformSupportVolumetricFogVoxelization(Parameters.Platform)
			&& Parameters.MaterialParameters.MaterialDomain == MD_Volume;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MAX_SLICES_PER_VOXELIZATION_PASS"), GetVoxelizationSlicesPerPass(Parameters.Platform));
		OutEnvironment.SetCompileArgument(TEXT("PIPELINE_CONTAINS_GEOMETRYSHADER"), true);
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FVoxelizeVolumeShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		ShaderBindings.Add(VoxelizationPassIndex, ShaderElementData.VoxelizationPassIndex);
	}

protected:
	LAYOUT_FIELD(FShaderParameter, VoxelizationPassIndex);
};

template<EVoxelizeShapeMode Mode>
class TVoxelizeVolumeGS : public FVoxelizeVolumeGS
{
public:
	DECLARE_SHADER_TYPE(TVoxelizeVolumeGS,MeshMaterial);
	typedef FVoxelizeVolumeGS Super;

	TVoxelizeVolumeGS() = default;
	TVoxelizeVolumeGS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: Super(Initializer)
	{}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		
		if (Mode == VMode_Primitive_Sphere)
		{
			OutEnvironment.SetDefine(TEXT("VOXELIZE_SHAPE_MODE"), TEXT("PRIMITIVE_SPHERE_MODE"));
		}
		else if (Mode == VMode_Object_Box)
		{
			OutEnvironment.SetDefine(TEXT("VOXELIZE_SHAPE_MODE"), TEXT("OBJECT_BOX_MODE"));
		}
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TVoxelizeVolumeGS<VMode_Primitive_Sphere>,TEXT("/Engine/Private/VolumetricFogVoxelization.usf"),TEXT("VoxelizeGS"),SF_Geometry); 
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TVoxelizeVolumeGS<VMode_Object_Box>,TEXT("/Engine/Private/VolumetricFogVoxelization.usf"),TEXT("VoxelizeGS"),SF_Geometry); 

class FVoxelizeVolumePS : public FMeshMaterialShader
{
public:
	DECLARE_INLINE_TYPE_LAYOUT(FVoxelizeVolumePS, NonVirtual);

	FVoxelizeVolumePS() = default;
	FVoxelizeVolumePS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportVolumetricFogVoxelization(Parameters.Platform)
			&& Parameters.MaterialParameters.MaterialDomain == MD_Volume;
	}
};

template<EVoxelizeShapeMode Mode>
class TVoxelizeVolumePS : public FVoxelizeVolumePS
{
	DECLARE_SHADER_TYPE(TVoxelizeVolumePS,MeshMaterial);
	typedef FVoxelizeVolumePS Super;

protected:

	TVoxelizeVolumePS() = default;
	TVoxelizeVolumePS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer):
		Super(Initializer)
	{}

public:

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		
		if (Mode == VMode_Primitive_Sphere)
		{
			OutEnvironment.SetDefine(TEXT("VOXELIZE_SHAPE_MODE"), TEXT("PRIMITIVE_SPHERE_MODE"));
		}
		else if (Mode == VMode_Object_Box)
		{
			OutEnvironment.SetDefine(TEXT("VOXELIZE_SHAPE_MODE"), TEXT("OBJECT_BOX_MODE"));
		}
		OutEnvironment.SetDefine(TEXT("CLOUD_LAYER_PIXEL_SHADER"), TEXT("1")); // This is to enable cloud data on the MaterialParameter structure.
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TVoxelizeVolumePS<VMode_Primitive_Sphere>,TEXT("/Engine/Private/VolumetricFogVoxelization.usf"),TEXT("VoxelizePS"),SF_Pixel); 
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TVoxelizeVolumePS<VMode_Object_Box>,TEXT("/Engine/Private/VolumetricFogVoxelization.usf"),TEXT("VoxelizePS"),SF_Pixel);

class FVoxelizeVolumeMeshProcessor : public FMeshPassProcessor
{
public:
	FVoxelizeVolumeMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FViewInfo* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);

	void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, int32 NumVoxelizationPasses, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final
	{
		checkf(false, TEXT("Default AddMeshBatch can't be used as rendering requires extra parameters per pass."));
	}

	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		int32 NumVoxelizationPasses);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		int32 NumVoxelizationPasses,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	FMeshPassProcessorRenderState PassDrawRenderState;
};

static const TCHAR* VoxelizeVolumePassName = TEXT("VoxelizeVolume");

FVoxelizeVolumeMeshProcessor::FVoxelizeVolumeMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FViewInfo* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(VoxelizeVolumePassName, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
{
	PassDrawRenderState.SetBlendState(TStaticBlendState<
		CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
		CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI());
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
}

void FVoxelizeVolumeMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, int32 NumVoxelizationPasses, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy)
{
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, *MaterialRenderProxy, *Material, NumVoxelizationPasses))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

static bool GetVoxelizeVolumePassShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	bool bUsePrimitiveSphere,
	TShaderRef<FVoxelizeVolumeVS>& VertexShader,
	TShaderRef<FVoxelizeVolumeGS>& GeometryShader,
	TShaderRef<FVoxelizeVolumePS>& PixelShader)
{
	FMaterialShaderTypes ShaderTypes;
	if (bUsePrimitiveSphere)
	{
		ShaderTypes.AddShaderType<TVoxelizeVolumeVS<VMode_Primitive_Sphere>>();
		if (RHISupportsGeometryShaders(GShaderPlatformForFeatureLevel[FeatureLevel]))
		{
			ShaderTypes.AddShaderType<TVoxelizeVolumeGS<VMode_Primitive_Sphere>>();
		}
		ShaderTypes.AddShaderType<TVoxelizeVolumePS<VMode_Primitive_Sphere>>();
	}
	else
	{
		ShaderTypes.AddShaderType<TVoxelizeVolumeVS<VMode_Object_Box>>();
		if (RHISupportsGeometryShaders(GShaderPlatformForFeatureLevel[FeatureLevel]))
		{
			ShaderTypes.AddShaderType<TVoxelizeVolumeGS<VMode_Object_Box>>();
		}
		ShaderTypes.AddShaderType<TVoxelizeVolumePS<VMode_Object_Box>>();
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(VertexShader);
	if (RHISupportsGeometryShaders(GShaderPlatformForFeatureLevel[FeatureLevel]))
	{
		Shaders.TryGetGeometryShader(GeometryShader);
	}
	Shaders.TryGetPixelShader(PixelShader);

	return true;
}

bool FVoxelizeVolumeMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	int32 NumVoxelizationPasses)
{
	// Determine the mesh's material and blend mode.
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = CM_None;

	return Process(MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material, NumVoxelizationPasses, MeshFillMode, MeshCullMode);
}

bool FVoxelizeVolumeMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	int32 NumVoxelizationPasses,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;
	const bool bUsePrimitiveSphere = VertexFactory != GQuadMeshVertexFactory;

	TMeshProcessorShaders<
		FVoxelizeVolumeVS,
		FVoxelizeVolumePS,
		FVoxelizeVolumeGS> PassShaders;
	if (!GetVoxelizeVolumePassShaders(
		MaterialResource,
		VertexFactory->GetType(),
		FeatureLevel,
		bUsePrimitiveSphere,
		PassShaders.VertexShader,
		PassShaders.GeometryShader,
		PassShaders.PixelShader))
	{
		return false;
	}
	
	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);

	for (int32 VoxelizationPassIndex = 0; VoxelizationPassIndex < NumVoxelizationPasses; VoxelizationPassIndex++)
	{
		if (GVolumetricFogVoxelizationShowOnlyPassIndex < 0 || GVolumetricFogVoxelizationShowOnlyPassIndex == VoxelizationPassIndex)
		{
			FVoxelizeVolumeShaderElementData ShaderElementData(VoxelizationPassIndex);
			ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, -1, true);

			BuildMeshDrawCommands(
				MeshBatch,
				BatchElementMask,
				PrimitiveSceneProxy,
				MaterialRenderProxy,
				MaterialResource,
				PassDrawRenderState,
				PassShaders,
				MeshFillMode,
				MeshCullMode,
				SortKey,
				EMeshPassFeatures::Default,
				ShaderElementData);
		}
	}

	return true;
}

void FVoxelizeVolumeMeshProcessor::CollectPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FPSOPrecacheParams& PreCacheParams,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	if (Material.GetMaterialDomain() != MD_Volume)
	{
		return;
	}

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	ERasterizerCullMode MeshCullMode = CM_None;

	FRDGTextureDesc VolumeDesc = GetVolumetricFogRDGTextureDesc(FIntVector(0));

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;
	AddRenderTargetInfo(VolumeDesc.Format, VolumeDesc.Flags, RenderTargetsInfo);

	static const auto CVarVolumetrixFogEmissive = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VolumetricFog.Emissive"));
	const bool bUseEmissive = (!CVarVolumetrixFogEmissive) || (CVarVolumetrixFogEmissive->GetInt() > 0);
	if (bUseEmissive)
	{
		AddRenderTargetInfo(VolumeDesc.Format, VolumeDesc.Flags, RenderTargetsInfo);
	}

	const auto AddPSOInitializer = [&](bool bUsePrimitiveSphere)
	{
		TMeshProcessorShaders<
			FVoxelizeVolumeVS,
			FVoxelizeVolumePS,
			FVoxelizeVolumeGS> PassShaders;
		if (!GetVoxelizeVolumePassShaders(
			Material,
			VertexFactoryData.VertexFactoryType,
			FeatureLevel,
			bUsePrimitiveSphere,
			PassShaders.VertexShader,
			PassShaders.GeometryShader,
			PassShaders.PixelShader))
		{
			return;
		}

		AddGraphicsPipelineStateInitializer(
			VertexFactoryData,
			Material,
			PassDrawRenderState,
			RenderTargetsInfo,
			PassShaders,
			MeshFillMode,
			MeshCullMode,
			(EPrimitiveType)PreCacheParams.PrimitiveType,
			EMeshPassFeatures::Default,
			true /*bRequired*/,
			PSOInitializers);
	};

	AddPSOInitializer(true /*bUsePrimitiveSphere*/);
	AddPSOInitializer(false /*bUsePrimitiveSphere*/);	
}

IPSOCollector* CreatePSOCollectorVoxelizeVolume(ERHIFeatureLevel::Type FeatureLevel)
{
	return new FVoxelizeVolumeMeshProcessor(nullptr, FeatureLevel, nullptr, nullptr);
}
FRegisterPSOCollectorCreateFunction RegisterPSOCollectorVoxelizeVolume(&CreatePSOCollectorVoxelizeVolume, EShadingPath::Deferred, VoxelizeVolumePassName);

void VoxelizeVolumePrimitive(FVoxelizeVolumeMeshProcessor& PassMeshProcessor,
	const FViewInfo& View,
	FIntVector VolumetricFogViewGridSize,
	FVector GridZParams,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMeshBatch& OriginalMesh)
{
	const FMaterialRenderProxy* MaterialProxy = OriginalMesh.MaterialRenderProxy;
	const FMaterial& Material = OriginalMesh.MaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialProxy);

	if (Material.GetMaterialDomain() == MD_Volume)
	{
		FMeshBatch LocalQuadMesh;

		// The voxelization shaders require camera facing quads as input
		// Vertex factories like particle sprites can work as-is, everything else needs to override with a camera facing quad
		const bool bOverrideWithQuadMesh = !OriginalMesh.VertexFactory->RendersPrimitivesAsCameraFacingSprites();

		if (bOverrideWithQuadMesh)
		{
			LocalQuadMesh.VertexFactory = GQuadMeshVertexFactory;
			LocalQuadMesh.MaterialRenderProxy = MaterialProxy;
			LocalQuadMesh.Elements[0].IndexBuffer = &GQuadMeshIndexBuffer;
			LocalQuadMesh.Elements[0].PrimitiveUniformBuffer = OriginalMesh.Elements[0].PrimitiveUniformBuffer;
			LocalQuadMesh.Elements[0].FirstIndex = 0;
			LocalQuadMesh.Elements[0].NumPrimitives = 2;
			LocalQuadMesh.Elements[0].MinVertexIndex = 0;
			LocalQuadMesh.Elements[0].MaxVertexIndex = 3;
		}

		const FMeshBatch& Mesh = bOverrideWithQuadMesh ? LocalQuadMesh : OriginalMesh;

		FBoxSphereBounds Bounds = PrimitiveSceneProxy->GetBounds();
		//@todo - compute NumSlices based on the largest particle size.  Bounds is overly conservative in most cases.
		float BoundsCenterDepth = View.ViewMatrices.GetWorldToView().TransformPosition(Bounds.Origin).Z;
		int32 NearSlice = ComputeZSliceFromDepth(BoundsCenterDepth - Bounds.SphereRadius, GridZParams);
		int32 FarSlice = ComputeZSliceFromDepth(BoundsCenterDepth + Bounds.SphereRadius, GridZParams);

		NearSlice = FMath::Clamp(NearSlice, 0, VolumetricFogViewGridSize.Z - 1);
		FarSlice = FMath::Clamp(FarSlice, 0, VolumetricFogViewGridSize.Z - 1);

		const int32 NumSlices = FarSlice - NearSlice + 1;
		const int32 NumVoxelizationPasses = FMath::DivideAndRoundUp(NumSlices, GetVoxelizationSlicesPerPass(View.GetShaderPlatform()));

		const uint64 DefaultBatchElementMask = ~0ull;
		PassMeshProcessor.AddMeshBatch(Mesh, DefaultBatchElementMask, NumVoxelizationPasses, PrimitiveSceneProxy);
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FVoxelizeVolumePassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVoxelizeVolumePassUniformParameters, Pass)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FInstanceCullingGlobalUniforms, InstanceCulling)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FSceneRenderer::VoxelizeFogVolumePrimitives(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FVolumetricFogIntegrationParameterData& IntegrationData,
	FIntVector VolumetricFogViewGridSize,
	FVector GridZParams,
	float VolumetricFogDistance,
	bool bVoxelizeEmissive)
{
	if (View.VolumetricMeshBatches.Num() > 0 && DoesPlatformSupportVolumetricFogVoxelization(View.GetShaderPlatform()))
	{
		RDG_EVENT_SCOPE(GraphBuilder, "VolumetricFog::VoxelizePrimitives");

		const FVector2D Jitter(
			IntegrationData.FrameJitterOffsetValues[0].X / VolumetricFogViewGridSize.X,
			IntegrationData.FrameJitterOffsetValues[0].Y / VolumetricFogViewGridSize.Y);

		auto* PassParameters = GraphBuilder.AllocParameters<FVoxelizeVolumePassParameters>();
		PassParameters->Pass = CreateVoxelizeVolumePassUniformBuffer(GraphBuilder, View, IntegrationData, Jitter, Scene->GetVolumetricCloudSceneInfo());
		PassParameters->InstanceCulling = FInstanceCullingContext::CreateDummyInstanceCullingUniformBuffer(GraphBuilder);
		PassParameters->RenderTargets[0] = FRenderTargetBinding(IntegrationData.VBufferA, ERenderTargetLoadAction::ELoad);
		if (bVoxelizeEmissive)
		{
			PassParameters->RenderTargets[1] = FRenderTargetBinding(IntegrationData.VBufferB, ERenderTargetLoadAction::ELoad);
		}

		{
			FViewUniformShaderParameters ViewVoxelizeParameters = *View.CachedViewUniformShaderParameters;

			// Update the parts of VoxelizeParameters which are dependent on the buffer size and view rect
			View.SetupViewRectUniformBufferParameters(
				ViewVoxelizeParameters,
				FIntPoint(VolumetricFogViewGridSize.X, VolumetricFogViewGridSize.Y),
				FIntRect(0, 0, VolumetricFogViewGridSize.X, VolumetricFogViewGridSize.Y),
				View.ViewMatrices,
				View.PrevViewInfo.ViewMatrices
			);

			PassParameters->View.View = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(ViewVoxelizeParameters, UniformBuffer_SingleFrame);;
			PassParameters->View.InstancedView = View.GetInstancedViewUniformBuffer();
		}
		PassParameters->Scene = GetSceneUniforms().GetBuffer(GraphBuilder);

		if (!GQuadMeshVertexFactory || GQuadMeshVertexFactory->HasIncompatibleFeatureLevel(View.GetFeatureLevel()))
		{
			if (GQuadMeshVertexFactory)
			{
				GQuadMeshVertexFactory->ReleaseResource();
				delete GQuadMeshVertexFactory;
			}
			GQuadMeshVertexFactory = new FQuadMeshVertexFactory(View.GetFeatureLevel());
			GQuadMeshVertexBuffer.UpdateRHI(GraphBuilder.RHICmdList);
			GQuadMeshVertexFactory->InitResource(GraphBuilder.RHICmdList);
		}

		// Set the sub region of the texture according to the current dynamic resolution scale.
		const FIntRect ViewRect(0, 0, VolumetricFogViewGridSize.X, VolumetricFogViewGridSize.Y);

		AddDrawDynamicMeshPass(GraphBuilder, RDG_EVENT_NAME("VoxelizeVolumePrimitives"), PassParameters, View, ViewRect,
			[&View, VolumetricFogDistance, VolumetricFogViewGridSize, GridZParams](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FVoxelizeVolumeMeshProcessor PassMeshProcessor(
					View.Family->Scene->GetRenderScene(),
					View.GetFeatureLevel(),
					&View,
					DynamicMeshPassContext);

				const bool bShouldRenderHeterogeneousVolumes = ShouldRenderHeterogeneousVolumesForView(View);

				for (int32 MeshBatchIndex = 0; MeshBatchIndex < View.VolumetricMeshBatches.Num(); ++MeshBatchIndex)
				{
					// Skip volumes flagged as rendered with HeterogenousVolumes
					const FMeshBatch* Mesh = View.VolumetricMeshBatches[MeshBatchIndex].Mesh;
					const FPrimitiveSceneProxy* PrimitiveSceneProxy = View.VolumetricMeshBatches[MeshBatchIndex].Proxy;
					if (ShouldRenderMeshBatchWithHeterogeneousVolumes(Mesh, PrimitiveSceneProxy, View.GetFeatureLevel()))
					{
						continue;
					}

					const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
					const FBoxSphereBounds Bounds = PrimitiveSceneProxy->GetBounds();

					if ((View.ViewMatrices.GetViewOrigin() - Bounds.Origin).SizeSquared() < (VolumetricFogDistance + Bounds.SphereRadius) * (VolumetricFogDistance + Bounds.SphereRadius))
					{
						VoxelizeVolumePrimitive(PassMeshProcessor, View, VolumetricFogViewGridSize, GridZParams, PrimitiveSceneProxy, *Mesh);
					}
				}
			},

			// Force off instanced stereo.
			// With instanced stereo on, primitives were being drawn to the left eye twice,  thickening the fog more in the
			// left eye.  It seemed better to force off instanced stereo anyway because of cache coherency in the 3d grids,
			// which are per-eye (far away in cache). The engine is already instancing across slices, which should be nearby
			// in cache).
			//
			// It may be a tradeoff where small primitives do better with instancing (their texture lookups stay cached)
			// and large ones covering lots of the voxel grid do worse (grid writes use up too much cache?), and could be
			// decided based on bounds?  GPUs presumably may have separate caches for read-only data in a way that changes
			// this tradeoff as well.
			true /*bForceInstanceStereoOff*/);
	}
}
