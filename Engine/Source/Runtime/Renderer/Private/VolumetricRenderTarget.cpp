// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricRenderTarget.cpp
=============================================================================*/

#include "VolumetricRenderTarget.h"
#include "DeferredShadingRenderer.h"
#include "RenderGraphUtils.h"
#include "PixelShaderUtils.h"
#include "SceneTextureParameters.h"
#include "SingleLayerWaterRendering.h"
#include "VolumetricCloudRendering.h"
#include "RendererUtils.h"
#include "PostProcess/PostProcessing.h" // IsPostProcessingWithAlphaChannelSupported
#include "EnvironmentComponentsFlags.h"
#include "LogRenderer.h"
#include "SceneViewState.h"
#include "RenderTargetPool.h"
#include "FogSeparateComposition.h"

static TAutoConsoleVariable<int32> CVarVolumetricRenderTarget(
	TEXT("r.VolumetricRenderTarget"), 1,
	TEXT(""),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricRenderTargetUvNoiseSampleAcceptanceWeight(
	TEXT("r.VolumetricRenderTarget.UvNoiseSampleAcceptanceWeight"), 20.0f,
	TEXT("Used when r.VolumetricRenderTarget.UpsamplingMode is in a mode using jitter - this value control the acceptance of noisy cloud samples according to their similarities. A higher value means large differences will be less accepted for blending."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricRenderTargetMode(
	TEXT("r.VolumetricRenderTarget.Mode"), 0,
	TEXT("[0] trace quarter resolution + reconstruct at half resolution + upsample [1] trace half res + upsample [2] trace at quarter resolution + reconstruct full resolution (cannot intersect with opaque meshes and forces UpsamplingMode=2 [3] Cinematic mode with tracing done at full reoslution in render target so that clouds can also be applied on translucent.)"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricRenderTargetUpsamplingMode(
	TEXT("r.VolumetricRenderTarget.UpsamplingMode"), 4,
	TEXT("Used in compositing volumetric RT over the scene. [0] bilinear [1] bilinear + jitter [2] nearest + depth test [3] bilinear + jitter + keep closest [4] bilaterial upsampling"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricRenderTargetScale(
	TEXT("r.VolumetricRenderTarget.Scale"), 1.0f,
	TEXT("Scales volumetric render target size (1.0 = 100%). Supported by VRT mode 2 only."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricRenderTargetPreferAsyncCompute(
	TEXT("r.VolumetricRenderTarget.PreferAsyncCompute"), 0,
	TEXT("Whether to prefer using async compute to generate volumetric cloud render targets. When this is set to true, it is recommend to also use r.VolumetricCloud.ApplyFogLate=1 for correct volumetric fog lighting on clouds."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricRenderTargetViewRect(
	TEXT("r.VolumetricRenderTarget.ViewRect"), 1,
	TEXT("Enable ViewRect support: does not reallocate new render targets when dynamic resolution changes"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricRenderTargetReprojectionBoxConstraint(
	TEXT("r.VolumetricRenderTarget.ReprojectionBoxConstraint"), 0,
	TEXT("Whether reprojected data should be constrained to the new incoming cloud data neighborhod value."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricRenderTargetMinimumDistanceKmToEnableReprojection(
	TEXT("r.VolumetricRenderTarget.MinimumDistanceKmToEnableReprojection"), 0.0f,
	TEXT("This is the distance in kilometer at which the `cloud surface` must be before we enable reprojection of the previous frame data. One could start with a value of 4km. This helps hide reprojection issues due to imperfect approximation of cloud depth as a single front surface, especially visible when flying through the cloud layer. It is not perfect but will help in lots of cases. The problem when using this method: clouds will look noisier when closer to that distance."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarVolumetricRenderTargetMinimumDistanceKmToDisableDisoclusion(
	TEXT("r.VolumetricRenderTarget.MinimumDistanceKmToDisableDisoclusion"), 5.0f,
	TEXT("This is the distance in kilometer at which we stop applying disocclusion, if all the traced and reprojected cloud depth are larger. Otherwise we might be hitting an edge. In this case, cloud information will be like a layer blended on top without upsampling."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarVolumetricRenderTargetSimulateNullResolution(
	TEXT("r.VolumetricRenderTarget.SimulateNullResolution"), 0,
	TEXT("Simulate a view port resolution of 0x0 for debugging."),
	ECVF_RenderThreadSafe);

static float GetUvNoiseSampleAcceptanceWeight()
{
	return FMath::Max(0.0f, CVarVolumetricRenderTargetUvNoiseSampleAcceptanceWeight.GetValueOnRenderThread());
}

static float GetMinimumDistanceKmToDisableDisoclusion()
{
	return FMath::Max(0.0f, CVarVolumetricRenderTargetMinimumDistanceKmToDisableDisoclusion.GetValueOnRenderThread());;
}

static bool ShouldPipelineCompileVolumetricRenderTargetShaders(EShaderPlatform ShaderPlatform)
{
	return GetMaxSupportedFeatureLevel(ShaderPlatform) >= ERHIFeatureLevel::SM5;
}

bool ShouldViewRenderVolumetricCloudRenderTarget(const FViewInfo& ViewInfo)
{
	return CVarVolumetricRenderTarget.GetValueOnRenderThread() && ShouldPipelineCompileVolumetricRenderTargetShaders(ViewInfo.GetShaderPlatform())
		&& (ViewInfo.ViewState != nullptr) && !ViewInfo.bIsReflectionCapture && ViewInfo.IsPerspectiveProjection(); //Do not use for ortho as the resolution resolves do not blend well when depth is uniform anyway.
}

bool IsVolumetricRenderTargetEnabled()
{
	return CVarVolumetricRenderTarget.GetValueOnRenderThread() > 0;
}

bool IsVolumetricRenderTargetAsyncCompute()
{
	// TODO remove that when we remove the pixel shading path in 5.0
	static auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VolumetricCloud.DisableCompute"));
	const bool bCloudComputePathDisabled = CVar && CVar->GetInt() > 1;

	return GSupportsEfficientAsyncCompute && CVarVolumetricRenderTargetPreferAsyncCompute.GetValueOnRenderThread() > 0 && !bCloudComputePathDisabled;
}

static bool ShouldViewComposeVolumetricRenderTarget(const FViewInfo& ViewInfo, bool bComposeCameraIntersectingWater = false)
{
	// IsUnderwater() is imprecise, but is currently what's used to decide whether to render clouds into SceneColor before or after water. In order to handle cases where
	// IsUnderwater() == false but the camera is still fully or partially underwater, we need to composite clouds into SceneColorWithoutWater in a special pass. Since this is more of an edge case,
	// we rely on FSceneView::WaterIntersection to only do run this pass when absolutely necessary.
	const bool bComposeCameraIntersectingWaterRelevant = (ViewInfo.WaterIntersection == EViewWaterIntersection::PossiblyIntersectingWater) && !ViewInfo.IsUnderwater();
	return ShouldViewRenderVolumetricCloudRenderTarget(ViewInfo) && (!bComposeCameraIntersectingWater || bComposeCameraIntersectingWaterRelevant);
}

static uint32 GetMainDownsampleFactor(int32 Mode)
{
	switch (Mode)
	{
	case 0:
		return 2; // Reconstruct at half resolution of view
	case 1:
	case 2:
		return 1; // Reconstruct at full resolution of view
	case 3:
		return 1; // SKip reconstruct, tracing at full resolution.
	}
	check(false); // unhandled mode
	return 2;
}

static uint32 GetTraceDownsampleFactor(int32 Mode)
{
	switch (Mode)
	{
	case 0:
		return 2; // Trace at half resolution of the reconstructed buffer (with it being at the half the resolution of the main view)
	case 1:
		return 2; // Trace at half resolution of the reconstructed buffer (with it being at the same resolution as main view)
	case 2:
		return 4; // Trace at quarter resolution of the reconstructed buffer (with it being at the same resolution as main view)
	case 3:
		return 1; // Trace at full resolution
	}
	check(false); // unhandled mode
	return 2;
}

static void GetTextureSafeUvCoordBound(const FIntPoint& TextureViewRect, FRDGTextureRef Texture, FUintVector4& TextureValidCoordRect, FVector4f& TextureValidUvRect, FVector2f& UVScale)
{
	FIntVector TexSize = Texture->Desc.GetSize();
    UVScale.X = (float)TextureViewRect.X / (float)TexSize.X;
    UVScale.Y = (float)TextureViewRect.Y / (float)TexSize.Y;

	TextureValidCoordRect.X = 0;
	TextureValidCoordRect.Y = 0;
	TextureValidCoordRect.Z = TextureViewRect.X - 1;
	TextureValidCoordRect.W = TextureViewRect.Y - 1;
	TextureValidUvRect.X = 0.51f / float(TexSize.X);
	TextureValidUvRect.Y = 0.51f / float(TexSize.Y);
	TextureValidUvRect.Z = (float(TextureViewRect.X) - 0.51f) / float(TexSize.X);
	TextureValidUvRect.W = (float(TextureViewRect.Y) - 0.51f) / float(TexSize.Y);
};

static bool AnyViewRequiresProcessing(TArrayView<FViewInfo> Views, bool bComposeCameraIntersectingWater = false)
{
	bool bAnyViewRequiresProcessing = false;
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& ViewInfo = Views[ViewIndex];
		bAnyViewRequiresProcessing |= ShouldViewComposeVolumetricRenderTarget(ViewInfo, bComposeCameraIntersectingWater);
	}
	return bAnyViewRequiresProcessing;
}

DECLARE_GPU_STAT(VolCloudReconstruction);
DECLARE_GPU_STAT(VolCloudComposeOverScene);
DECLARE_GPU_STAT(VolCloudComposeUnderSLW);
DECLARE_GPU_STAT(VolCloudComposeForVis);

/*=============================================================================
	UVolumetricCloudComponent implementation.
=============================================================================*/

FVolumetricRenderTargetViewStateData::FVolumetricRenderTargetViewStateData()
	: VolumetricReconstructRTDownsampleFactor(0)
	, VolumetricTracingRTDownsampleFactor(0)
	, CurrentRT(1)
	, bFirstTimeUsed(true)
	, bHistoryValid(false)
	, bHoldoutValid(false)
	, bValid(false)
	, PreViewExposure(1.0f)
	, StartTracingDistance(0.0f)
	, FrameId(0)
	, NoiseFrameIndex(0)
	, NoiseFrameIndexModPattern(0)
	, CurrentPixelOffset(FIntPoint::ZeroValue)
	, FullResolution(FIntPoint::ZeroValue)
	, VolumetricReconstructRTResolution(FIntPoint::ZeroValue)
	, VolumetricTracingRTResolution(FIntPoint::ZeroValue)
	, VolumetricTracingViewRect(FIntPoint::ZeroValue)
	, Mode(0)
	, UpsamplingMode(0)
{
	for (uint32 i = 0; i < kRenderTargetCount; ++i)
	{
		VolumetricReconstructViewRect[i] = FIntPoint::ZeroValue;
	}
}

FVolumetricRenderTargetViewStateData::~FVolumetricRenderTargetViewStateData()
{
}

void FVolumetricRenderTargetViewStateData::Initialise(
	FIntPoint& TextureResolutionIn,
	FIntPoint& ViewRectResolutionIn,
	int32 InMode,
	int32 InUpsamplingMode,
	bool bCameraCut)
{
	FIntPoint TextureResolutionInCopy(TextureResolutionIn);
	if (CVarVolumetricRenderTargetViewRect.GetValueOnAnyThread() == 0)
	{
		TextureResolutionInCopy = ViewRectResolutionIn;
	}

	// Update internal settings
	Mode = FMath::Clamp(InMode, 0, 3);
	UpsamplingMode = Mode == 2 || Mode == 3 ? 2 : FMath::Clamp(InUpsamplingMode, 0, 4); // if we are using mode 2 then we cannot intersect with depth and upsampling should be 2 (simple on/off intersection)

	bHoldoutValid = false;

	bValid = TextureResolutionInCopy.X > 0 && TextureResolutionInCopy.Y > 0;
	if (!bValid)
	{
		UE_LOGF(LogRenderer, Warning, "Warning: A viewport of resolution 0x0 was specified - VolumetricCloud not rendered.");
		return;
	}

	if (bFirstTimeUsed || bCameraCut)
	{
		bFirstTimeUsed = false;
		bHistoryValid = false;
		PreViewExposure = 1.0f;
		StartTracingDistance = 0.0f;
		FrameId = 0;
		NoiseFrameIndex = 0;
		NoiseFrameIndexModPattern = 0;
		CurrentPixelOffset = FIntPoint::ZeroValue;
	}

	{
		CurrentRT = 1 - CurrentRT;
		const uint32 PreviousRT = 1 - CurrentRT;

		// We always reallocate on a resolution change to adapt to dynamic resolution scaling.
		// TODO allocate once at max resolution and change source and destination coord/uvs/rect.
		if (FullResolution != TextureResolutionInCopy || GetMainDownsampleFactor(Mode) != VolumetricReconstructRTDownsampleFactor || GetTraceDownsampleFactor(Mode) != VolumetricTracingRTDownsampleFactor)
		{
			VolumetricReconstructRTDownsampleFactor = GetMainDownsampleFactor(Mode);
			VolumetricTracingRTDownsampleFactor = GetTraceDownsampleFactor(Mode);

			FullResolution = TextureResolutionInCopy;
			VolumetricReconstructRTResolution = FIntPoint::DivideAndRoundUp(FullResolution, VolumetricReconstructRTDownsampleFactor);							// Half resolution
			VolumetricTracingRTResolution = FIntPoint::DivideAndRoundUp(VolumetricReconstructRTResolution, VolumetricTracingRTDownsampleFactor);	// Half resolution of the volumetric buffer

			// Need a new size so release the low resolution trace buffer
			VolumetricTracingRT.SafeRelease();
			VolumetricSecondaryTracingRT.SafeRelease();
			VolumetricTracingRTDepth.SafeRelease();
		}

		VolumetricReconstructViewRect[CurrentRT] = FIntPoint::DivideAndRoundUp(ViewRectResolutionIn, VolumetricReconstructRTDownsampleFactor);							// Half resolution
		VolumetricTracingViewRect = FIntPoint::DivideAndRoundUp(VolumetricReconstructViewRect[CurrentRT], VolumetricTracingRTDownsampleFactor);	// Half resolution of the volumetric buffer

		FIntVector CurrentTargetResVec = VolumetricReconstructRT[CurrentRT].IsValid() ? VolumetricReconstructRT[CurrentRT]->GetDesc().GetSize() : FIntVector::ZeroValue;
		FIntPoint CurrentTargetRes = FIntPoint::DivideAndRoundUp(FullResolution, VolumetricReconstructRTDownsampleFactor);
		if (VolumetricReconstructRT[CurrentRT].IsValid() && FIntPoint(CurrentTargetResVec.X, CurrentTargetResVec.Y) != CurrentTargetRes)
		{
			// Resolution does not match so release target we are going to render in
			VolumetricReconstructRT[CurrentRT].SafeRelease();
			VolumetricReconstructSecondaryRT[CurrentRT].SafeRelease();
			VolumetricReconstructRTDepth[CurrentRT].SafeRelease();
			VolumetricReconstructRTForCloudOnTranslucent.SafeRelease();
		}

		// Regular every frame update
		{
			// Do not mark history as valid if the half resolution buffer is not valid. That means nothing has been rendered last frame.
			// That can happen when cloud is used to render into that buffer
			bHistoryValid = !bCameraCut && VolumetricReconstructRT[PreviousRT].IsValid();

			NoiseFrameIndex += FrameId == 0 ? 1 : 0;
			NoiseFrameIndexModPattern = NoiseFrameIndex % (VolumetricTracingRTDownsampleFactor * VolumetricTracingRTDownsampleFactor);

			FrameId++;
			FrameId = FrameId % (VolumetricTracingRTDownsampleFactor * VolumetricTracingRTDownsampleFactor);

			if (VolumetricTracingRTDownsampleFactor == 2)
			{
				static int32 OrderDithering2x2[4] = { 0, 2, 3, 1 };
				int32 LocalFrameId = OrderDithering2x2[FrameId];
				CurrentPixelOffset = FIntPoint(LocalFrameId % VolumetricTracingRTDownsampleFactor, LocalFrameId / VolumetricTracingRTDownsampleFactor);
			}
			else if (VolumetricTracingRTDownsampleFactor == 4)
			{
				static int32 OrderDithering4x4[16] = { 0, 8, 2, 10, 12, 4, 14, 6, 3, 11, 1, 9, 15, 7, 13, 5 };
				int32 LocalFrameId = OrderDithering4x4[FrameId];
				CurrentPixelOffset = FIntPoint(LocalFrameId % VolumetricTracingRTDownsampleFactor, LocalFrameId / VolumetricTracingRTDownsampleFactor);
			}
			else
			{
				// Default linear parse
				CurrentPixelOffset = FIntPoint(FrameId % VolumetricTracingRTDownsampleFactor, FrameId / VolumetricTracingRTDownsampleFactor);
			}
		}

		if (Mode == 1 || Mode == 3)
		{
			// No need to jitter in this case. Mode on is tracing half res and then upsample without reconstruction.
			CurrentPixelOffset = FIntPoint::ZeroValue;
		}
	}
}

void FVolumetricRenderTargetViewStateData::Reset()
{
	bFirstTimeUsed = false;
	bHistoryValid = false;
	bHoldoutValid = false;
	bValid = false;
	PreViewExposure = 1.0f;
	StartTracingDistance = 0.0f;
	FrameId = 0;
	NoiseFrameIndex = 0;
	NoiseFrameIndexModPattern = 0;
	CurrentPixelOffset = FIntPoint::ZeroValue;
	CurrentRT = 0;
	Mode = 0;
	UpsamplingMode = 0;

	// Release GPU resources
	VolumetricTracingRT.SafeRelease();
	VolumetricSecondaryTracingRT.SafeRelease();
	VolumetricTracingRTDepth.SafeRelease();
	for(uint32 i = 0 ; i < kRenderTargetCount; ++i)
	{
		VolumetricReconstructRT[i].SafeRelease();
		VolumetricReconstructSecondaryRT[i].SafeRelease();
		VolumetricReconstructRTDepth[i].SafeRelease();
	}
	VolumetricReconstructRTForCloudOnTranslucent.SafeRelease();
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateVolumetricTracingRT(FRDGBuilder& GraphBuilder)
{
	check(FullResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	if (!VolumetricTracingRT.IsValid())
	{
		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			VolumetricTracingRTResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f)),
			TexCreate_None, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV, false);
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, VolumetricTracingRT, TEXT("VolumetricRenderTarget.Tracing"));
	}

	return GraphBuilder.RegisterExternalTexture(VolumetricTracingRT);
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateVolumetricSecondaryTracingRT(FRDGBuilder& GraphBuilder)
{
	check(FullResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	if (!VolumetricSecondaryTracingRT.IsValid())
	{
		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			VolumetricTracingRTResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f)),
			TexCreate_None, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV, false);
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, VolumetricSecondaryTracingRT, TEXT("VolumetricRenderTarget.SecondaryTracing"));
	}

	return GraphBuilder.RegisterExternalTexture(VolumetricSecondaryTracingRT);
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateVolumetricTracingRTDepth(FRDGBuilder& GraphBuilder)
{
	check(FullResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	if (!VolumetricTracingRTDepth.IsValid())
	{
		EPixelFormat DepthDataFormat = Mode == 0 ? PF_FloatRGBA : PF_G16R16F; // Mode 0 supports MinAndMax depth tracing when the compute path is used so always allocate a 4-components texture in this case.

		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			VolumetricTracingRTResolution, DepthDataFormat, FClearValueBinding(FLinearColor(63000.0f, 63000.0f, 63000.0f, 63000.0f)),
			TexCreate_None, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV, false);
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, VolumetricTracingRTDepth, TEXT("VolumetricRenderTarget.TracingDepth"));
	}

	return GraphBuilder.RegisterExternalTexture(VolumetricTracingRTDepth);
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateVolumetricTracingRTHoldout(FRDGBuilder& GraphBuilder)
{
	check(FullResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	if (!VolumetricTracingRTHoldout.IsValid())
	{
		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			VolumetricTracingRTResolution, PF_R16F, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f)),
			TexCreate_None, TexCreate_ShaderResource | TexCreate_UAV, false);
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, VolumetricTracingRTHoldout, TEXT("VolumetricRenderTarget.TracingHoldOut"));
	}
	bHoldoutValid = true;
	return GraphBuilder.RegisterExternalTexture(VolumetricTracingRTHoldout);
}

FVector2f FVolumetricRenderTargetViewStateData::GetVolumetricTracingUVScale() const
{
	return FVector2f(float(VolumetricTracingViewRect.X) / float(VolumetricTracingRTResolution.X), 
					 float(VolumetricTracingViewRect.Y) / float(VolumetricTracingRTResolution.Y));
}

FVector2f FVolumetricRenderTargetViewStateData::GetVolumetricTracingUVMax() const
{
	const FVector2f TracingViewRect = FVector2f(GetCurrentVolumetricTracingViewRect());
	const FVector2f UVScale = GetVolumetricTracingUVScale();

	// To make sure the maximum UV will not result in outof bound filtered data, we only need to reduce it by hald a texel
	const FVector2f UVMax = UVScale * FVector2f(
		(float(TracingViewRect.X) - 0.51f) / TracingViewRect.X,
		(float(TracingViewRect.Y) - 0.51f) / TracingViewRect.Y);

	return UVMax;
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateVolumetricReconstructRTForCloudOnTranslucent(FRDGBuilder& GraphBuilder)
{
	check(VolumetricReconstructRTResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	if (!VolumetricReconstructRTForCloudOnTranslucent.IsValid())
	{
		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			VolumetricReconstructRTResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f)),
			TexCreate_None, TexCreate_ShaderResource | TexCreate_RenderTargetable, false);
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, VolumetricReconstructRTForCloudOnTranslucent, TEXT("VolumetricRenderTarget.Reconstruct.WithFog"));
	}

	return GraphBuilder.RegisterExternalTexture(VolumetricReconstructRTForCloudOnTranslucent);
}

TRefCountPtr<IPooledRenderTarget> FVolumetricRenderTargetViewStateData::GetVolumetricReconstructRTForCloudOnTranslucent()
{
	return VolumetricReconstructRTForCloudOnTranslucent;
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetDstVolumetricReconstructRT(FRDGBuilder& GraphBuilder)
{
	return VolumetricReconstructRT[CurrentRT].IsValid() ? GraphBuilder.RegisterExternalTexture(VolumetricReconstructRT[CurrentRT]) : nullptr;
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetDstVolumetricReconstructSecondaryRT(FRDGBuilder& GraphBuilder)
{
	return VolumetricReconstructSecondaryRT[CurrentRT].IsValid() ? GraphBuilder.RegisterExternalTexture(VolumetricReconstructSecondaryRT[CurrentRT]) : nullptr;
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateDstVolumetricReconstructRT(FRDGBuilder& GraphBuilder)
{
	check(VolumetricReconstructRTResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	if (!VolumetricReconstructRT[CurrentRT].IsValid())
	{
		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			VolumetricReconstructRTResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f)),
			TexCreate_None, TexCreate_ShaderResource | TexCreate_RenderTargetable, false);
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, VolumetricReconstructRT[CurrentRT], TEXT("VolumetricRenderTarget.Reconstruct"));
	}

	return GraphBuilder.RegisterExternalTexture(VolumetricReconstructRT[CurrentRT]);
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateDstVolumetricReconstructSecondaryRT(FRDGBuilder& GraphBuilder)
{
	check(VolumetricReconstructRTResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	if (!VolumetricReconstructSecondaryRT[CurrentRT].IsValid())
	{
		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			VolumetricReconstructRTResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f)),
			TexCreate_None, TexCreate_ShaderResource | TexCreate_RenderTargetable, false);
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, VolumetricReconstructSecondaryRT[CurrentRT], TEXT("VolumetricRenderTarget.Reconstruct"));
	}

	return GraphBuilder.RegisterExternalTexture(VolumetricReconstructSecondaryRT[CurrentRT]);
}


FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateDstVolumetricReconstructRTDepth(FRDGBuilder& GraphBuilder)
{
	check(VolumetricReconstructRTResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	if (!VolumetricReconstructRTDepth[CurrentRT].IsValid())
	{
		FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::Create2DDesc(
			VolumetricReconstructRTResolution, PF_FloatRGBA, FClearValueBinding(FLinearColor(63000.0f, 63000.0f, 63000.0f, 63000.0f)),
			TexCreate_None, TexCreate_ShaderResource | TexCreate_RenderTargetable, false);
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, Desc, VolumetricReconstructRTDepth[CurrentRT], TEXT("VolumetricRenderTarget.ReconstructDepth"));
	}

	return GraphBuilder.RegisterExternalTexture(VolumetricReconstructRTDepth[CurrentRT]);
}

TRefCountPtr<IPooledRenderTarget> FVolumetricRenderTargetViewStateData::GetDstVolumetricReconstructRT()
{
	return VolumetricReconstructRT[CurrentRT];
}
TRefCountPtr<IPooledRenderTarget> FVolumetricRenderTargetViewStateData::GetDstVolumetricReconstructSecondaryRT()
{
	return VolumetricReconstructSecondaryRT[CurrentRT];
}
TRefCountPtr<IPooledRenderTarget> FVolumetricRenderTargetViewStateData::GetDstVolumetricReconstructRTDepth()
{
	return VolumetricReconstructRTDepth[CurrentRT];
}

const FIntPoint& FVolumetricRenderTargetViewStateData::GetDstVolumetricReconstructViewRect() const
{
	return VolumetricReconstructViewRect[CurrentRT];
}

FVector2f FVolumetricRenderTargetViewStateData::GetDstVolumetricReconstructUVScale() const
{
	return FVector2f(float(VolumetricReconstructViewRect[CurrentRT].X) / float(VolumetricReconstructRTResolution.X), 
					 float(VolumetricReconstructViewRect[CurrentRT].Y) / float(VolumetricReconstructRTResolution.Y));
}

FVector2f FVolumetricRenderTargetViewStateData::GetDstVolumetricReconstructUVMax() const
{
	const FVector2f ReconstructViewRect = FVector2f(GetDstVolumetricReconstructViewRect());
	const FVector2f UVScale = GetDstVolumetricReconstructUVScale();

	// To make sure the maximum UV will not result in outof bound filtered data, we only need to reduce it by hald a texel
	const FVector2f UVMax = UVScale * FVector2f(
		(float(ReconstructViewRect.X) - 0.51f) / ReconstructViewRect.X,
		(float(ReconstructViewRect.Y) - 0.51f) / ReconstructViewRect.Y);

	return UVMax;
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateSrcVolumetricReconstructRT(FRDGBuilder& GraphBuilder)
{
	check(VolumetricReconstructRTResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	check(VolumetricReconstructRT[1u - CurrentRT].IsValid());
	return GraphBuilder.RegisterExternalTexture(VolumetricReconstructRT[1u - CurrentRT]);
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateSrcVolumetricReconstructSecondaryRT(FRDGBuilder& GraphBuilder)
{
	check(VolumetricReconstructRTResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	check(VolumetricReconstructSecondaryRT[1u - CurrentRT].IsValid());
	return GraphBuilder.RegisterExternalTexture(VolumetricReconstructSecondaryRT[1u - CurrentRT]);
}

FRDGTextureRef FVolumetricRenderTargetViewStateData::GetOrCreateSrcVolumetricReconstructRTDepth(FRDGBuilder& GraphBuilder)
{
	check(VolumetricReconstructRTResolution != FIntPoint::ZeroValue); // check that initialization has been done at least once

	check(VolumetricReconstructRT[1u - CurrentRT].IsValid());
	return GraphBuilder.RegisterExternalTexture(VolumetricReconstructRTDepth[1u - CurrentRT]);
}

const FIntPoint& FVolumetricRenderTargetViewStateData::GetSrcVolumetricReconstructViewRect() const
{
	return VolumetricReconstructViewRect[1u - CurrentRT];
}

static float GetVolumetricBufferResolutionScale(uint32 VRTMode)
{
	if (VRTMode == 2) // Only valid for mode 2
	{
		return FMath::Clamp(CVarVolumetricRenderTargetScale.GetValueOnAnyThread(), 0.1f, 1.0f);
	}
	return 1.0f;
}

FUintVector4 FVolumetricRenderTargetViewStateData::GetTracingCoordToZbufferCoordScaleBias() const
{
	uint32 InvRenderTargetScale = (uint32)FMath::RoundToInt(1.0f / GetVolumetricBufferResolutionScale(Mode));
	
	if (Mode == 2 || Mode == 3)
	{
		// In this case, the source depth buffer is the full resolution scene one
		const uint32 CombinedDownsampleFactor = InvRenderTargetScale * VolumetricReconstructRTDownsampleFactor * VolumetricTracingRTDownsampleFactor;
		return FUintVector4(CombinedDownsampleFactor, CombinedDownsampleFactor,																// Scale is the combined downsample factor
			CurrentPixelOffset.X * VolumetricReconstructRTDownsampleFactor, CurrentPixelOffset.Y * VolumetricReconstructRTDownsampleFactor);// Each sample will then sample from full res according to reconstructed RT offset times its downsample factor
	}

	// Otherwise, a half resolution depth buffer is used
	const uint32 SourceDepthBufferRTDownsampleFactor = 2;
	const uint32 CombinedDownsampleFactor = InvRenderTargetScale * VolumetricReconstructRTDownsampleFactor * VolumetricTracingRTDownsampleFactor / SourceDepthBufferRTDownsampleFactor;
	return FUintVector4( CombinedDownsampleFactor, CombinedDownsampleFactor,										// Scale is the combined downsample factor
		CurrentPixelOffset.X * VolumetricReconstructRTDownsampleFactor / VolumetricReconstructRTDownsampleFactor,	// Each sample will then sample from full res according to reconstructed RT offset times its downsample factor
		CurrentPixelOffset.Y * VolumetricReconstructRTDownsampleFactor / VolumetricReconstructRTDownsampleFactor);
}

FUintVector4 FVolumetricRenderTargetViewStateData::GetTracingCoordToFullResPixelCoordScaleBias() const
{
	uint32 InvRenderTargetScale = (uint32)FMath::RoundToInt(1.0f / GetVolumetricBufferResolutionScale(Mode));

	// In this case, the source depth buffer full resolution depth buffer is the full resolution scene one
	const uint32 CombinedDownsampleFactor = InvRenderTargetScale * VolumetricReconstructRTDownsampleFactor * VolumetricTracingRTDownsampleFactor;
	return FUintVector4(CombinedDownsampleFactor, CombinedDownsampleFactor,																// Scale is the combined downsample factor
		CurrentPixelOffset.X * VolumetricReconstructRTDownsampleFactor, CurrentPixelOffset.Y * VolumetricReconstructRTDownsampleFactor);// Each sample will then sample from full res according to reconstructed RT offset times its downsample factor
}


/*=============================================================================
	FSceneRenderer implementation.
=============================================================================*/

FIntPoint ComputeVolumetricBufferResolution(const FIntPoint& InViewRect, int32 VRTMode)
{
	FIntPoint OutViewRect = InViewRect;
	const float VolumetricRenderTargetScale = GetVolumetricBufferResolutionScale(VRTMode);
	OutViewRect.X = FMath::RoundToInt((float)OutViewRect.X * VolumetricRenderTargetScale);
	OutViewRect.Y = FMath::RoundToInt((float)OutViewRect.Y * VolumetricRenderTargetScale);
	return OutViewRect;
}

void InitVolumetricRenderTargetForViews(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views, const FMinimalSceneTextures& SceneTextures)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& ViewInfo = Views[ViewIndex];
		if (!ShouldViewRenderVolumetricCloudRenderTarget(ViewInfo))
		{
			continue;
		}
		FVolumetricRenderTargetViewStateData& VolumetricCloudRT = ViewInfo.ViewState->VolumetricCloudRenderTarget;

		// Determine if we are initializing or we should reset the persistent state
		const bool bCameraCut = ViewInfo.bCameraCut || ViewInfo.bForceCameraVisibilityReset || ViewInfo.bPrevTransformsReset;

		int32 VRTMode = FMath::Clamp(CVarVolumetricRenderTargetMode.GetValueOnRenderThread(), 0, 3);

		FIntPoint SceneTexturesExtent = ComputeVolumetricBufferResolution(SceneTextures.Config.Extent, VRTMode);
		FIntPoint ViewRect = ComputeVolumetricBufferResolution(ViewInfo.ViewRect.Size(), VRTMode);

#if !UE_BUILD_SHIPPING
		if (CVarVolumetricRenderTargetSimulateNullResolution.GetValueOnRenderThread() > 0.0f)
		{
			SceneTexturesExtent = ViewRect = FIntPoint::ZeroValue;
		}
#endif

		VolumetricCloudRT.Initialise(	// TODO this is going to reallocate a buffer each time dynamic resolution scaling is applied 
			SceneTexturesExtent,
			ViewRect,
			VRTMode,
			CVarVolumetricRenderTargetUpsamplingMode.GetValueOnAnyThread(),
			bCameraCut);

		if (!VolumetricCloudRT.IsValid())
		{
			continue;
		}

		FViewUniformShaderParameters ViewVolumetricCloudRTParameters = *ViewInfo.CachedViewUniformShaderParameters;
		{
			const FIntPoint& VolumetricTracingResolution = VolumetricCloudRT.GetCurrentVolumetricTracingRTResolution();
			const FIntPoint& VolumetricReconstructViewRect = VolumetricCloudRT.GetDstVolumetricReconstructViewRect();
			const FIntPoint& VolumetricTracingViewRect = VolumetricCloudRT.GetCurrentVolumetricTracingViewRect();
			const FIntPoint& CurrentPixelOffset = VolumetricCloudRT.GetCurrentTracingPixelOffset();
			const uint32 VolumetricReconstructRTDownSample = VolumetricCloudRT.GetVolumetricReconstructRTDownsampleFactor();
			const uint32 VolumetricTracingRTDownSample = VolumetricCloudRT.GetVolumetricTracingRTDownsampleFactor();

			// We jitter and reconstruct the volumetric view before TAA so we do not want any of its jitter.
			// We do use TAA remove bilinear artifact at up sampling time.
			FViewMatrices ViewMatrices = ViewInfo.ViewMatrices;
			ViewMatrices.HackRemoveTemporalAAProjectionJitter();

			// Offset to the correct half resolution pixel
			FVector2D CenterCoord = FVector2D(VolumetricReconstructRTDownSample / 2.0f);
			FVector2D TargetCoord = FVector2D(CurrentPixelOffset) + FVector2D(0.5f, 0.5f);
			FVector2D OffsetCoord = (TargetCoord - CenterCoord) * (FVector2D(-2.0f, 2.0f) / FVector2D(VolumetricReconstructViewRect));
			ViewMatrices.HackAddTemporalAAProjectionJitter(OffsetCoord);

			ViewInfo.SetupViewRectUniformBufferParameters(
				ViewVolumetricCloudRTParameters,
				VolumetricTracingResolution,
				FIntRect(0, 0, VolumetricTracingViewRect.X, VolumetricTracingViewRect.Y),
				ViewMatrices,
				ViewInfo.PrevViewInfo.ViewMatrices // This could also be changed if needed
			);
		}
		ViewInfo.VolumetricRenderTargetViewUniformBuffer = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(ViewVolumetricCloudRTParameters, UniformBuffer_SingleFrame);
	}
}

void ResetVolumetricRenderTargetForViews(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& ViewInfo = Views[ViewIndex];
		if (ViewInfo.ViewState)
		{
			FVolumetricRenderTargetViewStateData& VolumetricCloudRT = ViewInfo.ViewState->VolumetricCloudRenderTarget;
			VolumetricCloudRT.Reset();
		}
	}
}

//////////////////////////////////////////////////////////////////////////

class FReconstructVolumetricRenderTargetPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FReconstructVolumetricRenderTargetPS);
	SHADER_USE_PARAMETER_STRUCT(FReconstructVolumetricRenderTargetPS, FGlobalShader);

	class FHistoryAvailable : SHADER_PERMUTATION_BOOL("PERMUTATION_HISTORY_AVAILABLE");
	class FReprojectionBoxConstraint : SHADER_PERMUTATION_BOOL("PERMUTATION_REPROJECTION_BOX_CONSTRAINT");
	class FCloudMinAndMaxDepth : SHADER_PERMUTATION_BOOL("PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH");
	class FApplyFog : SHADER_PERMUTATION_BOOL("PERMUTATION_APPLY_FOG");
	class FApplyLocalFogVolume : SHADER_PERMUTATION_BOOL("PERMUTATION_APPLY_LOCAL_FOG_VOLUME");
	using FPermutationDomain = TShaderPermutationDomain<FHistoryAvailable, FReprojectionBoxConstraint, FCloudMinAndMaxDepth, FApplyFog, FApplyLocalFogVolume>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, FogStruct)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, TracingVolumetricTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SecondaryTracingVolumetricTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, TracingVolumetricDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PreviousFrameVolumetricTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PreviousFrameVolumetricSecondaryTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PreviousFrameVolumetricDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HalfResDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearTextureSampler)
		RENDER_TARGET_BINDING_SLOTS()
		SHADER_PARAMETER(FVector4f, DstVolumetricTextureSizeAndInvSize)
		SHADER_PARAMETER(FVector4f, PreviousVolumetricTextureSizeAndInvSize)
		SHADER_PARAMETER(FIntPoint, CurrentTracingPixelOffset)
		SHADER_PARAMETER(FIntPoint, ViewViewRectMin)
		SHADER_PARAMETER(int32, DownSampleFactor)
		SHADER_PARAMETER(int32, VolumetricRenderTargetMode)
		SHADER_PARAMETER(FVector2f, TracingVolumetricTextureUVScale)
		SHADER_PARAMETER(FUintVector4, TracingVolumetricTextureValidCoordRect)
		SHADER_PARAMETER(FVector4f, TracingVolumetricTextureValidUvRect)
		SHADER_PARAMETER(FUintVector4, PreviousFrameVolumetricTextureValidCoordRect)
		SHADER_PARAMETER(FVector4f, PreviousFrameVolumetricTextureValidUvRect)
		SHADER_PARAMETER(float, MinimumDistanceKmToEnableReprojection)
		SHADER_PARAMETER(float, MinimumDistanceKmToDisableDisoclusion)
		SHADER_PARAMETER(float, HistoryPreExposureCorrection)
		SHADER_PARAMETER(FVector2f, PreviousFrameVolumetricTextureUVScale)
		SHADER_PARAMETER(FVector2f, SourceViewRectMin)
		SHADER_PARAMETER(FVector2f, SourceViewRectSize)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldPipelineCompileVolumetricRenderTargetShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RECONSTRUCT_VOLUMETRICRT"), TEXT("1"));
	}
};

IMPLEMENT_GLOBAL_SHADER(FReconstructVolumetricRenderTargetPS, "/Engine/Private/VolumetricRenderTarget.usf", "ReconstructVolumetricRenderTargetPS", SF_Pixel);

//////////////////////////////////////////////////////////////////////////

void ReconstructVolumetricRenderTarget(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> Views,
	FRDGTextureRef SceneDepthTexture,
	FRDGTextureRef HalfResolutionDepthCheckerboardMinMaxTexture,
	bool bWaitFinishFence)
{
	if (!AnyViewRequiresProcessing(Views))
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, VolCloudReconstruction, "VolCloudReconstruction");
	SCOPED_NAMED_EVENT(VolCloudReconstruction, FColor::Emerald);

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& ViewInfo = Views[ViewIndex];
		if (!ShouldViewComposeVolumetricRenderTarget(ViewInfo))
		{
			continue;
		}

		FVolumetricRenderTargetViewStateData& VolumetricCloudRT = ViewInfo.ViewState->VolumetricCloudRenderTarget;

		if (!VolumetricCloudRT.IsValid())
		{
			continue;
		}

		if (VolumetricCloudRT.GetMode() == 1 || VolumetricCloudRT.GetMode() == 3)
		{
			// In this case, we trace at half resolution using checker boarded min max depth.
			// We will then directly up sample on screen from half resolution to full resolution. 
			// No reconstruction needed.
			continue;
		}
		const bool bShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent = ShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent(ViewInfo);

		FRDGTextureRef DstVolumetric = VolumetricCloudRT.GetOrCreateDstVolumetricReconstructRT(GraphBuilder);
		FRDGTextureRef DstVolumetricSecondary = VolumetricCloudRT.GetOrCreateDstVolumetricReconstructSecondaryRT(GraphBuilder);
		FRDGTextureRef DstVolumetricDepth = VolumetricCloudRT.GetOrCreateDstVolumetricReconstructRTDepth(GraphBuilder);
		FRDGTextureRef SrcTracingVolumetric = VolumetricCloudRT.GetOrCreateVolumetricTracingRT(GraphBuilder);
		FRDGTextureRef SrcSecondaryTracingVolumetric = VolumetricCloudRT.GetOrCreateVolumetricSecondaryTracingRT(GraphBuilder);
		FRDGTextureRef SrcTracingVolumetricDepth = VolumetricCloudRT.GetOrCreateVolumetricTracingRTDepth(GraphBuilder);
		FRDGTextureRef PreviousFrameVolumetricTexture = VolumetricCloudRT.GetHistoryValid() ? VolumetricCloudRT.GetOrCreateSrcVolumetricReconstructRT(GraphBuilder) : SystemTextures.Black;
		FRDGTextureRef PreviousFrameVolumetricSecondaryTexture = VolumetricCloudRT.GetHistoryValid() ? VolumetricCloudRT.GetOrCreateSrcVolumetricReconstructSecondaryRT(GraphBuilder) : SystemTextures.Black;
		FRDGTextureRef PreviousFrameVolumetricDepthTexture = VolumetricCloudRT.GetHistoryValid() ? VolumetricCloudRT.GetOrCreateSrcVolumetricReconstructRTDepth(GraphBuilder) : SystemTextures.Black;

		const uint32 TracingVolumetricCloudRTDownSample = VolumetricCloudRT.GetVolumetricTracingRTDownsampleFactor();

		const bool bMinMaxDepth = ShouldVolumetricCloudTraceWithMinMaxDepth(ViewInfo);

		FReconstructVolumetricRenderTargetPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FReconstructVolumetricRenderTargetPS::FHistoryAvailable>(VolumetricCloudRT.GetHistoryValid());
		PermutationVector.Set<FReconstructVolumetricRenderTargetPS::FReprojectionBoxConstraint>(CVarVolumetricRenderTargetReprojectionBoxConstraint.GetValueOnAnyThread() > 0);
		PermutationVector.Set<FReconstructVolumetricRenderTargetPS::FCloudMinAndMaxDepth>(bMinMaxDepth);
		PermutationVector.Set<FReconstructVolumetricRenderTargetPS::FApplyFog>(bShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent && ShouldRenderFog(*ViewInfo.Family));
		PermutationVector.Set<FReconstructVolumetricRenderTargetPS::FApplyLocalFogVolume>(bShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent && ViewInfo.LocalFogVolumeViewData.GPUInstanceCount > 0);
		TShaderMapRef<FReconstructVolumetricRenderTargetPS> PixelShader(ViewInfo.ShaderMap, PermutationVector);

		FReconstructVolumetricRenderTargetPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FReconstructVolumetricRenderTargetPS::FParameters>();
		PassParameters->ViewUniformBuffer = ViewInfo.VolumetricRenderTargetViewUniformBuffer; // Using a special uniform buffer because the view has some special resolution and no split screen offset.
		PassParameters->RenderTargets[0] = FRenderTargetBinding(DstVolumetric, ERenderTargetLoadAction::ENoAction);
		if (bMinMaxDepth)
		{
			PassParameters->RenderTargets[1] = FRenderTargetBinding(DstVolumetricSecondary, ERenderTargetLoadAction::ENoAction);
			PassParameters->RenderTargets[2] = FRenderTargetBinding(DstVolumetricDepth, ERenderTargetLoadAction::ENoAction);
			if (bShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent)
			{
				PassParameters->RenderTargets[3] = FRenderTargetBinding(VolumetricCloudRT.GetOrCreateVolumetricReconstructRTForCloudOnTranslucent(GraphBuilder), ERenderTargetLoadAction::ENoAction);
			}
		}
		else
		{
			PassParameters->RenderTargets[1] = FRenderTargetBinding(DstVolumetricDepth, ERenderTargetLoadAction::ENoAction);
			if (bShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent)
			{
				PassParameters->RenderTargets[2] = FRenderTargetBinding(VolumetricCloudRT.GetOrCreateVolumetricReconstructRTForCloudOnTranslucent(GraphBuilder), ERenderTargetLoadAction::ENoAction);
			}
		}
		PassParameters->TracingVolumetricTexture = SrcTracingVolumetric;
		PassParameters->SecondaryTracingVolumetricTexture = SrcSecondaryTracingVolumetric;
		PassParameters->TracingVolumetricDepthTexture = SrcTracingVolumetricDepth;
		PassParameters->PreviousFrameVolumetricTexture = PreviousFrameVolumetricTexture;
		PassParameters->PreviousFrameVolumetricSecondaryTexture = PreviousFrameVolumetricSecondaryTexture;
		PassParameters->PreviousFrameVolumetricDepthTexture = PreviousFrameVolumetricDepthTexture;
		PassParameters->LinearTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->CurrentTracingPixelOffset = VolumetricCloudRT.GetCurrentTracingPixelOffset();
		PassParameters->ViewViewRectMin = ViewInfo.ViewRect.Min / GetMainDownsampleFactor(VolumetricCloudRT.GetMode());// because we use the special VolumetricRenderTargetViewUniformBuffer, we have to specify View.RectMin separately.
		PassParameters->DownSampleFactor = TracingVolumetricCloudRTDownSample;
		PassParameters->VolumetricRenderTargetMode = VolumetricCloudRT.GetMode();
		PassParameters->HalfResDepthTexture = (VolumetricCloudRT.GetMode() == 0 || VolumetricCloudRT.GetMode() == 3) ? HalfResolutionDepthCheckerboardMinMaxTexture : SceneDepthTexture;
		PassParameters->MinimumDistanceKmToEnableReprojection = FMath::Max(0.0f, CVarVolumetricRenderTargetMinimumDistanceKmToEnableReprojection.GetValueOnRenderThread());
		PassParameters->MinimumDistanceKmToDisableDisoclusion = GetMinimumDistanceKmToDisableDisoclusion();
		PassParameters->HistoryPreExposureCorrection = ViewInfo.PreExposure / VolumetricCloudRT.GetPrevViewExposure();

		PassParameters->SourceViewRectMin = ViewInfo.ViewRect.Min;
		PassParameters->SourceViewRectSize = ViewInfo.ViewRect.Size();

		if (bShouldVolumetricCloudsApplyFogDuringUpsamplingForCloudOnTranslucent)
		{
			PassParameters->FogStruct = CreateFogUniformBuffer(GraphBuilder, ViewInfo);
			PassParameters->LFV = ViewInfo.LocalFogVolumeViewData.UniformParametersStruct;
		}

		const bool bVisualizeConservativeDensity = ShouldViewVisualizeVolumetricCloudConservativeDensity(ViewInfo, ViewInfo.Family->EngineShowFlags);
		PassParameters->HalfResDepthTexture = bVisualizeConservativeDensity ?
			SystemTextures.Black :
			((VolumetricCloudRT.GetMode() == 0 || VolumetricCloudRT.GetMode() == 3) ?
				HalfResolutionDepthCheckerboardMinMaxTexture :
				SceneDepthTexture);

		const FIntPoint CurrentVolumetricTracingViewRect = VolumetricCloudRT.GetCurrentVolumetricTracingViewRect();
		const FIntPoint SrcVolumetricReconstructViewRect = VolumetricCloudRT.GetSrcVolumetricReconstructViewRect();
		
		GetTextureSafeUvCoordBound(CurrentVolumetricTracingViewRect, SrcTracingVolumetric, PassParameters->TracingVolumetricTextureValidCoordRect, PassParameters->TracingVolumetricTextureValidUvRect, PassParameters->TracingVolumetricTextureUVScale);
		GetTextureSafeUvCoordBound(SrcVolumetricReconstructViewRect, PreviousFrameVolumetricTexture, PassParameters->PreviousFrameVolumetricTextureValidCoordRect, PassParameters->PreviousFrameVolumetricTextureValidUvRect, PassParameters->PreviousFrameVolumetricTextureUVScale);

		FIntPoint DstVolumetricSize = VolumetricCloudRT.GetDstVolumetricReconstructViewRect();
		FVector2D DstVolumetricTextureSize = FVector2D(float(DstVolumetricSize.X), float(DstVolumetricSize.Y));
		FVector2D PreviousVolumetricTextureSize = FVector2D(float(PreviousFrameVolumetricTexture->Desc.GetSize().X), float(PreviousFrameVolumetricTexture->Desc.GetSize().Y));
		PassParameters->DstVolumetricTextureSizeAndInvSize = FVector4f(DstVolumetricTextureSize.X, DstVolumetricTextureSize.Y, 1.0f / DstVolumetricTextureSize.X, 1.0f / DstVolumetricTextureSize.Y);
		PassParameters->PreviousVolumetricTextureSizeAndInvSize = FVector4f(PreviousVolumetricTextureSize.X, PreviousVolumetricTextureSize.Y, 1.0f / PreviousVolumetricTextureSize.X, 1.0f / PreviousVolumetricTextureSize.Y);

		FPixelShaderUtils::AddFullscreenPass<FReconstructVolumetricRenderTargetPS>(
			GraphBuilder, ViewInfo.ShaderMap, RDG_EVENT_NAME("VolumetricReconstruct"), PixelShader, PassParameters, 
			FIntRect(0, 0, DstVolumetricSize.X, DstVolumetricSize.Y));

		VolumetricCloudRT.PostRenderUpdate(ViewInfo.PreExposure);
	}

}

//////////////////////////////////////////////////////////////////////////

class FComposeVolumetricRTOverScenePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComposeVolumetricRTOverScenePS);
	SHADER_USE_PARAMETER_STRUCT(FComposeVolumetricRTOverScenePS, FGlobalShader);

	class FUpsamplingMode : SHADER_PERMUTATION_RANGE_INT("PERMUTATION_UPSAMPLINGMODE", 0, 5);
	class FRenderUnderWaterBuffer : SHADER_PERMUTATION_BOOL("PERMUTATION_RENDER_UNDERWATER_BUFFER");	// Render into the water scene color buffer (used when rendering from water system)
	class FRenderCameraComposeWithWater : SHADER_PERMUTATION_BOOL("PERMUTATION_COMPOSE_WITH_WATER");	// When water us used and the camera is under water, use that permutation (to handle camera intersection with water and double cloud composition)
	class FMSAASampleCount : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_MSAA_SAMPLE_COUNT", 1, 2, 4, 8);
	class FApplyFog : SHADER_PERMUTATION_BOOL("PERMUTATION_APPLY_FOG");
	class FApplyLocalFogVolume : SHADER_PERMUTATION_BOOL("PERMUTATION_APPLY_LOCAL_FOG_VOLUME");
	class FMinMaxDepthAvailable : SHADER_PERMUTATION_BOOL("PERMUTATION_MINMAXDEPTH_AVAILABLE");
	using FPermutationDomain = TShaderPermutationDomain<FUpsamplingMode, FRenderUnderWaterBuffer, FRenderCameraComposeWithWater, FMSAASampleCount, FApplyFog, FApplyLocalFogVolume, FMinMaxDepthAvailable>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, FogStruct)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VolumetricTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VolumetricSecondaryTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VolumetricDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterLinearDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS<float>, MSAADepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, WaterLinearDepthSampler)
		RENDER_TARGET_BINDING_SLOTS()
		SHADER_PARAMETER(uint32, ForwardShadingEnable)
		SHADER_PARAMETER(uint32, OutputAlphaHoldout)
		SHADER_PARAMETER(float, VolumeTracingStartDistanceFromCamera)
		SHADER_PARAMETER(float, UvOffsetSampleAcceptanceWeight)
		SHADER_PARAMETER(float, MinimumDistanceKmToDisableDisoclusion)
		SHADER_PARAMETER(FVector4f, VolumetricTextureSizeAndInvSize)
		SHADER_PARAMETER(FVector2f, FullResolutionToVolumetricBufferResolutionScale)
		SHADER_PARAMETER(FVector2f, FullResolutionToWaterBufferScale)
		SHADER_PARAMETER(FVector4f, SceneWithoutSingleLayerWaterViewRect)
		SHADER_PARAMETER(FUintVector4, VolumetricTextureValidCoordRect)
		SHADER_PARAMETER(FVector4f, VolumetricTextureValidUvRect)
		SHADER_PARAMETER(FVector2f, ViewSeparateFogDepthOriginOffset)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if ((!IsForwardShadingEnabled(Parameters.Platform) || !RHISupportsMSAA(Parameters.Platform)) && PermutationVector.Get<FMSAASampleCount>() > 1)
		{
			// We only compile the MSAA support when Forward shading is enabled because MSAA can only be used in this case.
			return false;
		}

		return ShouldPipelineCompileVolumetricRenderTargetShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_COMPOSE_VOLUMETRICRT"), TEXT("1"));
	}
};

IMPLEMENT_GLOBAL_SHADER(FComposeVolumetricRTOverScenePS, "/Engine/Private/VolumetricRenderTarget.usf", "ComposeVolumetricRTOverScenePS", SF_Pixel);

//////////////////////////////////////////////////////////////////////////

static FVector2f GetCompositionFullResolutionToVolumetricBufferResolutionScale(uint32 VRTMode)
{
	if (VRTMode == 1)
	{
		return FVector2f(0.5f, 2.0f);
	}
	return FVector2f(1.0f / float(GetMainDownsampleFactor(VRTMode)), float(GetMainDownsampleFactor(VRTMode)));
}

static void GetCompositionCloudTextures(uint32 VRTMode, FVolumetricRenderTargetViewStateData& VolumetricCloudRT, FRDGBuilder& GraphBuilder, FRDGTextureRef& VolumetricTexture, FRDGTextureRef& VolumetricSecondaryTexture, FRDGTextureRef& VolumetricDepthTexture,
										FIntPoint& VolumetricViewRect)
{
	if (VRTMode == 1 || VRTMode == 3)
	{
		// In this case, we trace at half resolution using checker boarded min max depth.
		// We will then directly up sample on screen from half resolution to full resolution. 
		// No reconstruction needed.
		VolumetricTexture = VolumetricCloudRT.GetOrCreateVolumetricTracingRT(GraphBuilder);
		VolumetricSecondaryTexture = GSystemTextures.GetBlackAlphaOneDummy(GraphBuilder);
		VolumetricDepthTexture = VolumetricCloudRT.GetOrCreateVolumetricTracingRTDepth(GraphBuilder);
		VolumetricViewRect = VolumetricCloudRT.GetCurrentVolumetricTracingViewRect();
	}
	else
	{
		VolumetricTexture = VolumetricCloudRT.GetOrCreateDstVolumetricReconstructRT(GraphBuilder);
		VolumetricSecondaryTexture = VRTMode == 0 ? VolumetricCloudRT.GetOrCreateDstVolumetricReconstructSecondaryRT(GraphBuilder) : GSystemTextures.GetBlackAlphaOneDummy(GraphBuilder);
		VolumetricDepthTexture = VolumetricCloudRT.GetOrCreateDstVolumetricReconstructRTDepth(GraphBuilder);
		VolumetricViewRect = VolumetricCloudRT.GetDstVolumetricReconstructViewRect();
	}
}

static int32 GetCompositionUpsamplingMode(uint32 VRTMode, int32 UpsamplingMode)
{
	return UpsamplingMode == 3 && (VRTMode == 1 || VRTMode == 2 || VRTMode == 3) ? 2 : UpsamplingMode;
}

void ComposeVolumetricRenderTargetOverScene(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	TArrayView<FViewInfo> Views,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	bool bComposeWithWater,
	const FSceneWithoutWaterTextures& WaterPassData,
	const FMinimalSceneTextures& SceneTextures,
	TArray<FFogSeparateCompositionViewResources>* FogSeparateCompositionViewResources)
{
	if (!Scene || !AnyViewRequiresProcessing(Views))
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, VolCloudComposeOverScene, "VolCloudComposeOverScene");
	SCOPED_NAMED_EVENT(VolCloudComposeOverScene, FColor::Emerald);

	FRHIBlendState* PreMultipliedColorTransmittanceBlend;
	const bool bSupportsAlpha = IsPostProcessingWithAlphaChannelSupported();
	if (bSupportsAlpha)
	{
		// When alpha channel is enabled, we always write transmittance to impact other alpha holdout values from sky or fog for instance.
		// We will run a second pass later accumulating the cloud contribution to hold out
		PreMultipliedColorTransmittanceBlend = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
	}
	else
	{
		PreMultipliedColorTransmittanceBlend = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& ViewInfo = Views[ViewIndex];
		if (!ShouldViewRenderVolumetricCloudRenderTarget(ViewInfo))
		{
			continue;
		}
		if (ViewInfo.CachedViewUniformShaderParameters->RenderingReflectionCaptureMask == 0 && !IsVolumetricCloudRenderedInMain(ViewInfo.CachedViewUniformShaderParameters->EnvironmentComponentsFlags))
		{
			continue;
		}

		FVolumetricRenderTargetViewStateData& VolumetricCloudRT = ViewInfo.ViewState->VolumetricCloudRenderTarget;

		if (!VolumetricCloudRT.IsValid())
		{
			continue;
		}

		// When reconstructed and back buffer resolution matches, force using a pixel perfect upsampling.
		const uint32 VRTMode = VolumetricCloudRT.GetMode();
		int32 UpsamplingMode = GetCompositionUpsamplingMode(VRTMode, VolumetricCloudRT.GetUpsamplingMode());
		const bool bMinMaxDepth = ShouldVolumetricCloudTraceWithMinMaxDepth(ViewInfo);

		FRDGTextureRef VolumetricTexture = nullptr;
		FRDGTextureRef VolumetricSecondaryTexture = nullptr;
		FRDGTextureRef VolumetricDepthTexture = nullptr;
		FIntPoint VolumetricViewRect(FIntPoint::ZeroValue);
		GetCompositionCloudTextures(VRTMode, VolumetricCloudRT, GraphBuilder, VolumetricTexture, VolumetricSecondaryTexture, VolumetricDepthTexture, VolumetricViewRect);

		// We only support MSAA up to 8 sample and in forward
		check(SceneDepthTexture->Desc.NumSamples <= 8);
		// We only support MSAA in forward, not in deferred.
		const bool bForwardShading = IsForwardShadingEnabled(ViewInfo.GetShaderPlatform());
		check(bForwardShading || (!bForwardShading && SceneDepthTexture->Desc.NumSamples==1));

		const bool bShouldVolumetricCloudsApplyFogDuringUpsampling = ShouldVolumetricCloudsApplyFogDuringUpsampling(ViewInfo);

		// Check if we should apply fog separate composition on clouds
		check(!FogSeparateCompositionViewResources || FogSeparateCompositionViewResources->Num() > ViewIndex);
		const bool bApplyFogSeparateComposition = FogSeparateCompositionViewResources
			&& (*FogSeparateCompositionViewResources)[ViewIndex].FogSeparateCompositionTexture0 != nullptr
			&& (*FogSeparateCompositionViewResources)[ViewIndex].FogSeparateCompositionTexture1 != nullptr;
		FRDGTextureRef SceneSeparateColorTextureOverride = bApplyFogSeparateComposition ? (*FogSeparateCompositionViewResources)[ViewIndex].FogSeparateCompositionTexture0 : nullptr;

		FComposeVolumetricRTOverScenePS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FUpsamplingMode>(UpsamplingMode);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FRenderUnderWaterBuffer>(0);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FRenderCameraComposeWithWater>((bComposeWithWater && ViewInfo.IsUnderwater()) ? 1 : 0);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FMSAASampleCount>(SceneDepthTexture->Desc.NumSamples);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FApplyFog>(bShouldVolumetricCloudsApplyFogDuringUpsampling && (ViewInfo.Family->Scene->HasAnyExponentialHeightFog() || ViewInfo.LocalFogVolumeViewData.GPUInstanceCount > 0) && ShouldRenderFog(*ViewInfo.Family));
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FApplyLocalFogVolume>(bShouldVolumetricCloudsApplyFogDuringUpsampling && ViewInfo.LocalFogVolumeViewData.GPUInstanceCount > 0);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FMinMaxDepthAvailable>(bMinMaxDepth);
		TShaderMapRef<FComposeVolumetricRTOverScenePS> PixelShader(ViewInfo.ShaderMap, PermutationVector);

		FComposeVolumetricRTOverScenePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComposeVolumetricRTOverScenePS::FParameters>();
		PassParameters->ViewUniformBuffer = ViewInfo.ViewUniformBuffer;
		if (SceneSeparateColorTextureOverride)
		{
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneSeparateColorTextureOverride, ERenderTargetLoadAction::ELoad);
			PassParameters->ViewSeparateFogDepthOriginOffset = FVector2f(ViewInfo.ViewRect.Min);
		}
		else
		{
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);
			PassParameters->ViewSeparateFogDepthOriginOffset = FVector2f(0.0f, 0.0f);
		}
		PassParameters->MSAADepthTexture = SceneDepthTexture;
		PassParameters->VolumetricTexture = VolumetricTexture;
		PassParameters->VolumetricSecondaryTexture = bMinMaxDepth ? VolumetricSecondaryTexture : VolumetricTexture;
		PassParameters->VolumetricDepthTexture = VolumetricDepthTexture;
		PassParameters->LinearTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->UvOffsetSampleAcceptanceWeight = GetUvNoiseSampleAcceptanceWeight();
		PassParameters->MinimumDistanceKmToDisableDisoclusion = GetMinimumDistanceKmToDisableDisoclusion();
		PassParameters->FullResolutionToVolumetricBufferResolutionScale = GetCompositionFullResolutionToVolumetricBufferResolutionScale(VRTMode) * GetVolumetricBufferResolutionScale(VRTMode);
		PassParameters->SceneTextures = SceneTextures.UniformBuffer;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(ViewInfo);
		PassParameters->OutputAlphaHoldout = 0;
		PassParameters->ForwardShadingEnable = bForwardShading ? 1 : 0;
		PassParameters->VolumeTracingStartDistanceFromCamera = VolumetricCloudRT.GetStartTracingDistance();
		FVector2f DummyUVScale;
		GetTextureSafeUvCoordBound(VolumetricViewRect, PassParameters->VolumetricTexture, PassParameters->VolumetricTextureValidCoordRect, PassParameters->VolumetricTextureValidUvRect, DummyUVScale);

		PassParameters->WaterLinearDepthTexture = WaterPassData.DepthTexture;
		PassParameters->WaterLinearDepthSampler = TStaticSamplerState<SF_Point>::GetRHI();
		if (bComposeWithWater)
		{
			const FSceneWithoutWaterTextures::FView& WaterPassViewData = WaterPassData.Views[ViewIndex];
			PassParameters->FullResolutionToWaterBufferScale = FVector2f(1.0f / WaterPassData.RefractionDownsampleFactor, WaterPassData.RefractionDownsampleFactor);
			PassParameters->SceneWithoutSingleLayerWaterViewRect = FVector4f(WaterPassViewData.ViewRect.Min.X, WaterPassViewData.ViewRect.Min.Y,
				WaterPassViewData.ViewRect.Max.X, WaterPassViewData.ViewRect.Max.Y);
		}

		FVector2D VolumetricTextureSize = FVector2D(float(VolumetricTexture->Desc.GetSize().X), float(VolumetricTexture->Desc.GetSize().Y));
		PassParameters->VolumetricTextureSizeAndInvSize = FVector4f(VolumetricTextureSize.X, VolumetricTextureSize.Y, 1.0f / VolumetricTextureSize.X, 1.0f / VolumetricTextureSize.Y);

		PassParameters->FogStruct = CreateFogUniformBuffer(GraphBuilder, ViewInfo);
		PassParameters->LFV = ViewInfo.LocalFogVolumeViewData.UniformParametersStruct;

		if (SceneSeparateColorTextureOverride)
		{
			FRHIBlendState* AlphaBlendMode = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI(); // multiply transmitance of clouds into alpha.

			FIntRect SceneSeparateColorRect = ViewInfo.ViewRect;
			SceneSeparateColorRect -= ViewInfo.ViewRect.Min; // Make it relative to 0,0 

			FPixelShaderUtils::AddFullscreenPass<FComposeVolumetricRTOverScenePS>(
				GraphBuilder, ViewInfo.ShaderMap, RDG_EVENT_NAME("VolumetricComposeCloudOverScene"), PixelShader, PassParameters, SceneSeparateColorRect,
				AlphaBlendMode);
		}
		else
		{
			FPixelShaderUtils::AddFullscreenPass<FComposeVolumetricRTOverScenePS>(
				GraphBuilder, ViewInfo.ShaderMap, RDG_EVENT_NAME("VolumetricComposeCloudOverScene"), PixelShader, PassParameters, ViewInfo.ViewRect,
				PreMultipliedColorTransmittanceBlend);
		}

		if (bSupportsAlpha && VolumetricCloudRT.GetHoldoutValid() && !SceneSeparateColorTextureOverride)
		{
			check(VRTMode == 3);	// This is the only supported way today.

			// Also compose the alpha value the same way
			PermutationVector.Set<FComposeVolumetricRTOverScenePS::FApplyFog>(0);
			PermutationVector.Set<FComposeVolumetricRTOverScenePS::FApplyLocalFogVolume>(0);
			TShaderMapRef<FComposeVolumetricRTOverScenePS> HoldoutPixelShader(ViewInfo.ShaderMap, PermutationVector);

			FComposeVolumetricRTOverScenePS::FParameters* HoldoutPassParameters = GraphBuilder.AllocParameters<FComposeVolumetricRTOverScenePS::FParameters>();
			*HoldoutPassParameters = *PassParameters;
			HoldoutPassParameters->VolumetricTexture = VolumetricCloudRT.GetOrCreateVolumetricTracingRTHoldout(GraphBuilder);
			HoldoutPassParameters->OutputAlphaHoldout = 1;

			FRHIBlendState* AddAlphaBlendMode = TStaticBlendState<CW_ALPHA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();

			FPixelShaderUtils::AddFullscreenPass<FComposeVolumetricRTOverScenePS>(
				GraphBuilder, ViewInfo.ShaderMap, RDG_EVENT_NAME("VolumetricComposeHoldoutOverScene"), HoldoutPixelShader, HoldoutPassParameters, ViewInfo.ViewRect,
				AddAlphaBlendMode);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

void ComposeVolumetricRenderTargetOverSceneUnderWater(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> Views,
	const FSceneWithoutWaterTextures& WaterPassData,
	const FMinimalSceneTextures& SceneTextures)
{
	if (!AnyViewRequiresProcessing(Views, true /*bComposeCameraIntersectingWater*/))
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, VolCloudComposeUnderSLW, "VolCloudComposeUnderSLW");
	SCOPED_NAMED_EVENT(VolCloudComposeUnderSLW, FColor::Emerald);

	FRHIBlendState* PreMultipliedColorTransmittanceBlend = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& ViewInfo = Views[ViewIndex];
		if (!ShouldViewComposeVolumetricRenderTarget(ViewInfo, true /*bComposeCameraIntersectingWater*/) || !ViewInfo.ShouldRenderView())
		{
			continue;
		}

		const FSceneWithoutWaterTextures::FView& WaterPassViewData = WaterPassData.Views[ViewIndex];
		FVolumetricRenderTargetViewStateData& VolumetricCloudRT = ViewInfo.ViewState->VolumetricCloudRenderTarget;

		if (!VolumetricCloudRT.IsValid())
		{
			continue;
		}

		// When reconstructed and back buffer resolution matches, force using a pixel perfect upsampling.
		const uint32 VRTMode = VolumetricCloudRT.GetMode();
		int32 UpsamplingMode = GetCompositionUpsamplingMode(VRTMode, VolumetricCloudRT.GetUpsamplingMode());
		const bool bMinMaxDepth = ShouldVolumetricCloudTraceWithMinMaxDepth(ViewInfo);

		const bool bShouldVolumetricCloudsApplyFogDuringUpsampling = ShouldVolumetricCloudsApplyFogDuringUpsampling(ViewInfo);

		FRDGTextureRef VolumetricTexture = nullptr;
		FRDGTextureRef VolumetricSecondaryTexture = nullptr;
		FRDGTextureRef VolumetricDepthTexture = nullptr;
		FIntPoint VolumetricViewRect(FIntPoint::ZeroValue);
		GetCompositionCloudTextures(VRTMode, VolumetricCloudRT, GraphBuilder, VolumetricTexture, VolumetricSecondaryTexture, VolumetricDepthTexture, VolumetricViewRect);

		FComposeVolumetricRTOverScenePS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FUpsamplingMode>(UpsamplingMode);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FRenderUnderWaterBuffer>(1);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FRenderCameraComposeWithWater>(0);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FMSAASampleCount>(1);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FApplyFog>(bShouldVolumetricCloudsApplyFogDuringUpsampling && (ViewInfo.Family->Scene->HasAnyExponentialHeightFog() || ViewInfo.LocalFogVolumeViewData.GPUInstanceCount > 0) && ShouldRenderFog(*ViewInfo.Family));
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FApplyLocalFogVolume>(bShouldVolumetricCloudsApplyFogDuringUpsampling && ViewInfo.LocalFogVolumeViewData.GPUInstanceCount > 0);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FMinMaxDepthAvailable>(bMinMaxDepth);
		TShaderMapRef<FComposeVolumetricRTOverScenePS> PixelShader(ViewInfo.ShaderMap, PermutationVector);

		FComposeVolumetricRTOverScenePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComposeVolumetricRTOverScenePS::FParameters>();
		PassParameters->ViewUniformBuffer = ViewInfo.ViewUniformBuffer;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(WaterPassData.ColorTexture, ERenderTargetLoadAction::ELoad);
		PassParameters->VolumetricTexture = VolumetricTexture;
		PassParameters->VolumetricSecondaryTexture = bMinMaxDepth ? VolumetricSecondaryTexture : VolumetricTexture;
		PassParameters->VolumetricDepthTexture = VolumetricDepthTexture;
		PassParameters->WaterLinearDepthTexture = WaterPassData.DepthTexture;
		PassParameters->LinearTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->WaterLinearDepthSampler = TStaticSamplerState<SF_Point>::GetRHI();
		PassParameters->UvOffsetSampleAcceptanceWeight = GetUvNoiseSampleAcceptanceWeight();
		PassParameters->MinimumDistanceKmToDisableDisoclusion = GetMinimumDistanceKmToDisableDisoclusion();
		PassParameters->FullResolutionToVolumetricBufferResolutionScale = GetCompositionFullResolutionToVolumetricBufferResolutionScale(VRTMode) * GetVolumetricBufferResolutionScale(VRTMode);
		PassParameters->FullResolutionToWaterBufferScale = FVector2f(1.0f / WaterPassData.RefractionDownsampleFactor, WaterPassData.RefractionDownsampleFactor);
		PassParameters->SceneWithoutSingleLayerWaterViewRect = FVector4f(WaterPassViewData.ViewRect.Min.X, WaterPassViewData.ViewRect.Min.Y,
																		WaterPassViewData.ViewRect.Max.X, WaterPassViewData.ViewRect.Max.Y);
		PassParameters->SceneTextures = SceneTextures.UniformBuffer;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(ViewInfo);
		PassParameters->OutputAlphaHoldout = 0;
		PassParameters->ForwardShadingEnable = 0;
		PassParameters->VolumeTracingStartDistanceFromCamera = VolumetricCloudRT.GetStartTracingDistance();

		if (bShouldVolumetricCloudsApplyFogDuringUpsampling)
		{
			PassParameters->FogStruct = CreateFogUniformBuffer(GraphBuilder, ViewInfo);
			PassParameters->LFV = ViewInfo.LocalFogVolumeViewData.UniformParametersStruct;
		}

		FVector2f DummyUVScale;
		GetTextureSafeUvCoordBound(VolumetricViewRect, PassParameters->VolumetricTexture, PassParameters->VolumetricTextureValidCoordRect, PassParameters->VolumetricTextureValidUvRect, DummyUVScale);

		FVector2D VolumetricTextureSize = FVector2D(float(VolumetricTexture->Desc.GetSize().X), float(VolumetricTexture->Desc.GetSize().Y));
		PassParameters->VolumetricTextureSizeAndInvSize = FVector4f(VolumetricTextureSize.X, VolumetricTextureSize.Y, 1.0f / VolumetricTextureSize.X, 1.0f / VolumetricTextureSize.Y);

		FPixelShaderUtils::AddFullscreenPass<FComposeVolumetricRTOverScenePS>(
			GraphBuilder, ViewInfo.ShaderMap, RDG_EVENT_NAME("SLW::VolumetricComposeOverScene"), PixelShader, PassParameters, WaterPassViewData.ViewRect,
			PreMultipliedColorTransmittanceBlend);
	}
}

//////////////////////////////////////////////////////////////////////////

void ComposeVolumetricRenderTargetOverSceneForVisualization(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> Views,
	FRDGTextureRef SceneColorTexture,
	const FMinimalSceneTextures& SceneTextures)
{
	if (!AnyViewRequiresProcessing(Views))
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, VolCloudComposeForVis, "VolCloudComposeForVis");
	SCOPED_NAMED_EVENT(VolCloudComposeForVis, FColor::Emerald);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& ViewInfo = Views[ViewIndex];
		if (!ShouldViewRenderVolumetricCloudRenderTarget(ViewInfo))
		{
			continue;
		}
		FVolumetricRenderTargetViewStateData& VolumetricCloudRT = ViewInfo.ViewState->VolumetricCloudRenderTarget;

		if (!VolumetricCloudRT.IsValid())
		{
			continue;
		}

		// When reconstructed and back buffer resolution matches, force using a pixel perfect upsampling.
		const uint32 VRTMode = VolumetricCloudRT.GetMode();
		int32 UpsamplingMode = GetCompositionUpsamplingMode(VRTMode, VolumetricCloudRT.GetUpsamplingMode());
		const bool bMinMaxDepth = ShouldVolumetricCloudTraceWithMinMaxDepth(ViewInfo);

		FRDGTextureRef VolumetricTexture = nullptr;
		FRDGTextureRef VolumetricSecondaryTexture = nullptr;
		FRDGTextureRef VolumetricDepthTexture = nullptr;
		FIntPoint VolumetricViewRect(FIntPoint::ZeroValue);
		GetCompositionCloudTextures(VRTMode, VolumetricCloudRT, GraphBuilder, VolumetricTexture, VolumetricSecondaryTexture, VolumetricDepthTexture, VolumetricViewRect);

		FComposeVolumetricRTOverScenePS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FUpsamplingMode>(0);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FRenderUnderWaterBuffer>(0);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FRenderCameraComposeWithWater>(0);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FMSAASampleCount>(1);
		PermutationVector.Set<FComposeVolumetricRTOverScenePS::FMinMaxDepthAvailable>(bMinMaxDepth);
		TShaderMapRef<FComposeVolumetricRTOverScenePS> PixelShader(ViewInfo.ShaderMap, PermutationVector);

		FComposeVolumetricRTOverScenePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComposeVolumetricRTOverScenePS::FParameters>();
		PassParameters->ViewUniformBuffer = ViewInfo.ViewUniformBuffer;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);
		PassParameters->VolumetricTexture = VolumetricTexture;
		PassParameters->VolumetricSecondaryTexture = bMinMaxDepth ? VolumetricSecondaryTexture : VolumetricTexture;
		PassParameters->VolumetricDepthTexture = VolumetricDepthTexture;
		PassParameters->LinearTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->UvOffsetSampleAcceptanceWeight = GetUvNoiseSampleAcceptanceWeight();
		PassParameters->MinimumDistanceKmToDisableDisoclusion = GetMinimumDistanceKmToDisableDisoclusion();
		PassParameters->FullResolutionToVolumetricBufferResolutionScale = GetCompositionFullResolutionToVolumetricBufferResolutionScale(VRTMode) * GetVolumetricBufferResolutionScale(VRTMode);
		PassParameters->SceneTextures = SceneTextures.UniformBuffer;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(ViewInfo);
		PassParameters->OutputAlphaHoldout = 0;
		PassParameters->ForwardShadingEnable = 0;
		PassParameters->VolumeTracingStartDistanceFromCamera = VolumetricCloudRT.GetStartTracingDistance();;
		FVector2f DummyUVScale;
		GetTextureSafeUvCoordBound(VolumetricViewRect, PassParameters->VolumetricTexture, PassParameters->VolumetricTextureValidCoordRect, PassParameters->VolumetricTextureValidUvRect, DummyUVScale);

		PassParameters->WaterLinearDepthTexture = GSystemTextures.GetBlackDummy(GraphBuilder);

		FVector2D VolumetricTextureSize = FVector2D(float(VolumetricTexture->Desc.GetSize().X), float(VolumetricTexture->Desc.GetSize().Y));
		PassParameters->VolumetricTextureSizeAndInvSize = FVector4f(VolumetricTextureSize.X, VolumetricTextureSize.Y, 1.0f / VolumetricTextureSize.X, 1.0f / VolumetricTextureSize.Y);

		FPixelShaderUtils::AddFullscreenPass<FComposeVolumetricRTOverScenePS>(
			GraphBuilder, ViewInfo.ShaderMap, RDG_EVENT_NAME("VolumetricComposeOverSceneForVisualization"), PixelShader, PassParameters, ViewInfo.ViewRect);
	}
}

//////////////////////////////////////////////////////////////////////////



FTemporalRenderTargetState::FTemporalRenderTargetState()
	: CurrentRT(1)
	, FrameId(0)
	, bFirstTimeUsed(true)
	, bHistoryValid(false)
	, Resolution(FIntPoint::ZeroValue)
	, Format(PF_MAX)
{ 
}

FTemporalRenderTargetState::~FTemporalRenderTargetState()
{
}

void FTemporalRenderTargetState::Initialise(const FIntPoint& ResolutionIn, EPixelFormat FormatIn)
{
	// Update internal settings

	if (bFirstTimeUsed)
	{
		bFirstTimeUsed = false;
		bHistoryValid = false;
		FrameId = 0;
	}

	CurrentRT = 1 - CurrentRT;
	const uint32 PreviousRT = 1 - CurrentRT;

	FIntVector ResolutionVector = FIntVector(ResolutionIn.X, ResolutionIn.Y, 0);
	for (int32 i = 0; i < kRenderTargetCount; ++i)
	{
		if (RenderTargets[i].IsValid() && (RenderTargets[i]->GetDesc().GetSize() != ResolutionVector || Format != FormatIn))
		{
			// Resolution does not match so release target we are going to render in, keep the previous one at a different resolution.
			RenderTargets[i].SafeRelease();
		}
	}
	Resolution = ResolutionIn;
	Format = FormatIn;

	// Regular every frame update
	bHistoryValid = RenderTargets[PreviousRT].IsValid();
}

FRDGTextureRef FTemporalRenderTargetState::GetOrCreateCurrentRT(FRDGBuilder& GraphBuilder)
{
	check(Resolution.X > 0 && Resolution.Y > 0);

	if (RenderTargets[CurrentRT].IsValid())
	{
		return GraphBuilder.RegisterExternalTexture(RenderTargets[CurrentRT]);
	}

	FRDGTextureRef RDGTexture = GraphBuilder.CreateTexture(
		FRDGTextureDesc::Create2D(Resolution, Format, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f)), 
			TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable), TEXT("VolumetricRenderTarget.GeneralTemporalTexture"));
	return RDGTexture;
}
void FTemporalRenderTargetState::ExtractCurrentRT(FRDGBuilder& GraphBuilder, FRDGTextureRef RDGTexture)
{
	check(Resolution.X > 0 && Resolution.Y > 0);

	RenderTargets[CurrentRT] = GraphBuilder.ConvertToExternalTexture(RDGTexture);
}

FRDGTextureRef FTemporalRenderTargetState::GetOrCreatePreviousRT(FRDGBuilder& GraphBuilder)
{
	check(Resolution.X > 0 && Resolution.Y > 0);
	const uint32 PreviousRT = 1u - CurrentRT;
	check(RenderTargets[PreviousRT].IsValid());

	return GraphBuilder.RegisterExternalTexture(RenderTargets[PreviousRT]);
}

void FTemporalRenderTargetState::Reset()
{
	bFirstTimeUsed = false;
	bHistoryValid = false;
	FrameId = 0;
	for (int32 i = 0; i < kRenderTargetCount; ++i)
	{
		RenderTargets[i].SafeRelease();
	}
	Resolution = FIntPoint::ZeroValue;
	Format = PF_MAX;
}

