// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricRenderTarget.h
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "RendererInterface.h"
#include "RenderResource.h"
#include "VolumetricRenderTargetViewStateData.h"

class FScene;
class FViewInfo;
struct FSceneWithoutWaterTextures;
struct FMinimalSceneTextures;
class FFogSeparateCompositionViewResources;

void InitVolumetricRenderTargetForViews(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views, const FMinimalSceneTextures& SceneTextures);
void ResetVolumetricRenderTargetForViews(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views);

void ReconstructVolumetricRenderTarget(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> Views,
	FRDGTextureRef SceneDepthTexture,
	FRDGTextureRef HalfResolutionDepthCheckerboardMinMaxTexture,
	bool bWaitFinishFence);

void ComposeVolumetricRenderTargetOverScene(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	TArrayView<FViewInfo> Views,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthResolveTexture,
	bool bShouldRenderSingleLayerWater,
	const FSceneWithoutWaterTextures& WaterPassData,
	const FMinimalSceneTextures& SceneTextures,
	TArray<FFogSeparateCompositionViewResources>* FogSeparateCompositionViewResources = nullptr);

void ComposeVolumetricRenderTargetOverSceneUnderWater(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> Views,
	const FSceneWithoutWaterTextures& WaterPassData,
	const FMinimalSceneTextures& SceneTextures);

void ComposeVolumetricRenderTargetOverSceneForVisualization(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> Views,
	FRDGTextureRef SceneColorTexture,
	const FMinimalSceneTextures& SceneTextures);

bool ShouldViewRenderVolumetricCloudRenderTarget(const FViewInfo& ViewInfo);
bool IsVolumetricRenderTargetEnabled();
bool IsVolumetricRenderTargetAsyncCompute();