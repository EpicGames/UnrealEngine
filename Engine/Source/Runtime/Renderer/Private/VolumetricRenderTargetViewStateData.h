// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricRenderTargetViewStatedata.h
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"

class FRDGBuilder;

class FVolumetricRenderTargetViewStateData
{

public:

	FVolumetricRenderTargetViewStateData();
	~FVolumetricRenderTargetViewStateData();

	void Initialise(
		FIntPoint& TextureResolutionIn,
		FIntPoint& ViewRectResolutionIn,
		int32 Mode,
		int32 UpsamplingMode,
		bool bCameraCut);

	void Reset();

	void PostRenderUpdate(float ViewExposure)
	{
		PreViewExposure = ViewExposure;
	}

	float GetPrevViewExposure() const 
	{
		return PreViewExposure;
	}

	void SetStartTracingDistance(float InStartTracingDistance)
	{
		StartTracingDistance = InStartTracingDistance;
	}
	float GetStartTracingDistance() const
	{
		return StartTracingDistance;
	}

	FRDGTextureRef GetOrCreateVolumetricTracingRT(FRDGBuilder& GraphBuilder);
	FRDGTextureRef GetOrCreateVolumetricSecondaryTracingRT(FRDGBuilder& GraphBuilder);
	FRDGTextureRef GetOrCreateVolumetricTracingRTDepth(FRDGBuilder& GraphBuilder);
	FRDGTextureRef GetOrCreateVolumetricTracingRTHoldout(FRDGBuilder& GraphBuilder);
	FVector2f GetVolumetricTracingUVScale() const;
	FVector2f GetVolumetricTracingUVMax() const;

	FRDGTextureRef GetOrCreateVolumetricReconstructRTForCloudOnTranslucent(FRDGBuilder& GraphBuilder);
	TRefCountPtr<IPooledRenderTarget> GetVolumetricReconstructRTForCloudOnTranslucent();

	FRDGTextureRef GetDstVolumetricReconstructRT(FRDGBuilder& GraphBuilder);
	FRDGTextureRef GetDstVolumetricReconstructSecondaryRT(FRDGBuilder& GraphBuilder);
	FRDGTextureRef GetOrCreateDstVolumetricReconstructRT(FRDGBuilder& GraphBuilder);
	FRDGTextureRef GetOrCreateDstVolumetricReconstructSecondaryRT(FRDGBuilder& GraphBuilder);
	FRDGTextureRef GetOrCreateDstVolumetricReconstructRTDepth(FRDGBuilder& GraphBuilder);
	const FIntPoint& GetDstVolumetricReconstructViewRect() const;
	FVector2f GetDstVolumetricReconstructUVScale() const;
	FVector2f GetDstVolumetricReconstructUVMax() const;

	TRefCountPtr<IPooledRenderTarget> GetDstVolumetricReconstructRT();
	TRefCountPtr<IPooledRenderTarget> GetDstVolumetricReconstructSecondaryRT();
	TRefCountPtr<IPooledRenderTarget> GetDstVolumetricReconstructRTDepth();

	FRDGTextureRef GetOrCreateSrcVolumetricReconstructRT(FRDGBuilder& GraphBuilder);
	FRDGTextureRef GetOrCreateSrcVolumetricReconstructSecondaryRT(FRDGBuilder& GraphBuilder);
	FRDGTextureRef GetOrCreateSrcVolumetricReconstructRTDepth(FRDGBuilder& GraphBuilder);
	const FIntPoint& GetSrcVolumetricReconstructViewRect() const;

	bool IsValid() const { return bValid; }

	bool GetHistoryValid() const { return bHistoryValid; }
	bool GetHoldoutValid() const { return bHoldoutValid; }
	const FIntPoint& GetCurrentVolumetricReconstructRTResolution() const { return VolumetricReconstructRTResolution; }
	const FIntPoint& GetCurrentVolumetricTracingRTResolution() const { return VolumetricTracingRTResolution; }
	const FIntPoint& GetCurrentVolumetricTracingViewRect() const { return VolumetricTracingViewRect; }
	const FIntPoint& GetCurrentTracingPixelOffset() const { return CurrentPixelOffset; }
	const uint32 GetNoiseFrameIndexModPattern() const { return NoiseFrameIndexModPattern; }

	const uint32 GetVolumetricReconstructRTDownsampleFactor() const { return VolumetricReconstructRTDownsampleFactor; }
	const uint32 GetVolumetricTracingRTDownsampleFactor() const { return VolumetricTracingRTDownsampleFactor; }

	FUintVector4 GetTracingCoordToZbufferCoordScaleBias() const;
	FUintVector4 GetTracingCoordToFullResPixelCoordScaleBias() const;

	int32 GetMode()				const { return Mode; }
	int32 GetUpsamplingMode()	const { return UpsamplingMode; }

	uint64 GetGPUSizeBytes(bool bLogSizes) const;

private:

	uint32 VolumetricReconstructRTDownsampleFactor;
	uint32 VolumetricTracingRTDownsampleFactor;

	uint32 CurrentRT;
	bool bFirstTimeUsed;
	bool bHistoryValid;
	bool bHoldoutValid;
	bool bValid;
	float PreViewExposure;
	float StartTracingDistance;	// The distance at which the tracing starts, and thus the composition can be clipped for pixel closer to that distance.

	int32 FrameId;
	uint32 NoiseFrameIndex;	// This is only incremented once all Volumetric render target samples have been iterated
	uint32 NoiseFrameIndexModPattern;
	FIntPoint CurrentPixelOffset;

	FIntPoint FullResolution;
	FIntPoint VolumetricReconstructRTResolution;
	FIntPoint VolumetricTracingRTResolution;
	FIntPoint VolumetricTracingViewRect;

	static constexpr uint32 kRenderTargetCount = 2;
	TRefCountPtr<IPooledRenderTarget> VolumetricReconstructRT[kRenderTargetCount];
	TRefCountPtr<IPooledRenderTarget> VolumetricReconstructSecondaryRT[kRenderTargetCount];
	TRefCountPtr<IPooledRenderTarget> VolumetricReconstructRTDepth[kRenderTargetCount];
	FIntPoint VolumetricReconstructViewRect[kRenderTargetCount];

	TRefCountPtr<IPooledRenderTarget> VolumetricReconstructRTForCloudOnTranslucent;

	TRefCountPtr<IPooledRenderTarget> VolumetricTracingRT;
	TRefCountPtr<IPooledRenderTarget> VolumetricSecondaryTracingRT;
	TRefCountPtr<IPooledRenderTarget> VolumetricTracingRTDepth;
	TRefCountPtr<IPooledRenderTarget> VolumetricTracingRTHoldout;

	int32 Mode;
	int32 UpsamplingMode;
};


class FTemporalRenderTargetState
{

public:

	FTemporalRenderTargetState();
	~FTemporalRenderTargetState();

	void Initialise(const FIntPoint& ResolutionIn, EPixelFormat FormatIn);

	FRDGTextureRef GetOrCreateCurrentRT(FRDGBuilder& GraphBuilder);
	void ExtractCurrentRT(FRDGBuilder& GraphBuilder, FRDGTextureRef RDGRT);

	FRDGTextureRef GetOrCreatePreviousRT(FRDGBuilder& GraphBuilder);

	bool GetHistoryValid() const { return bHistoryValid; }

	bool CurrentIsValid() const { return RenderTargets[CurrentRT].IsValid(); }
	TRefCountPtr<IPooledRenderTarget> CurrentRenderTarget() const { return RenderTargets[CurrentRT]; }

	uint32 GetCurrentIndex() { return CurrentRT; }
	uint32 GetPreviousIndex() { return 1 - CurrentRT; }

	void Reset();

	uint64 GetGPUSizeBytes(bool bLogSizes) const;

private:

	uint32 CurrentRT;
	int32 FrameId;

	bool bFirstTimeUsed;
	bool bHistoryValid;

	FIntPoint Resolution;
	EPixelFormat Format;

	static constexpr uint32 kRenderTargetCount = 2;
	TRefCountPtr<IPooledRenderTarget> RenderTargets[kRenderTargetCount];
};



