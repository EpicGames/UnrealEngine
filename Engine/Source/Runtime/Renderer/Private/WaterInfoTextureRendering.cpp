// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaterInfoTextureRendering.h"
#include "UnrealClient.h"
#include "RenderGraph.h"
#include "DeferredShadingRenderer.h"
#include "SceneUtils.h"
#include "ScenePrivate.h"
#include "MeshPassProcessor.inl"
#include "PixelShaderUtils.h"
#include "BasePassRendering.h"
#include "MobileBasePassRendering.h"
#include "RenderCaptureInterface.h"
#include "PSOPrecacheSettings.h"

DECLARE_GPU_DRAWCALL_STAT(WaterInfoTexture);

static TAutoConsoleVariable<float> CVarWaterInfoUndergroundDilationDepthOffset(
	TEXT("r.Water.WaterInfo.UndergroundDilationDepthOffset"),
	64.f,
	TEXT("The minimum distance below the ground when we allow dilation to write on top of water"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarWaterInfoDilationOverwriteMinimumDistance(
	TEXT("r.Water.WaterInfo.DilationOverwriteMinimumDistance"),
	128.f,
	TEXT("The minimum distance below the ground when we allow dilation to write on top of water"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRenderCaptureNextWaterInfoDraws(
	TEXT("r.Water.WaterInfo.RenderCaptureNextWaterInfoDraws"),
	0,
	TEXT("Enable capturing of the water info texture for the next N draws"),
	ECVF_RenderThreadSafe);

BEGIN_SHADER_PARAMETER_STRUCT(FWaterInfoTexturePassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FWaterInfoTextureDepthPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FWaterInfoTextureMergePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterInfoTextureMergePS);
	SHADER_USE_PARAMETER_STRUCT(FWaterInfoTextureMergePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, GroundDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, GroundDepthTextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, WaterBodyTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, WaterBodyTextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, WaterBodyDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, WaterBodyDepthTextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, DilatedWaterBodyDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DilatedWaterBodyDepthTextureSampler)
		SHADER_PARAMETER(FVector2f, WaterHeightExtents)
		SHADER_PARAMETER(float, GroundZMin)
		SHADER_PARAMETER(float, CaptureZ)
		SHADER_PARAMETER(float, UndergroundDilationDepthOffset)
		SHADER_PARAMETER(float, DilationOverwriteMinimumDistance)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	class FEnable128BitRT : SHADER_PERMUTATION_BOOL("ENABLE_128_BIT");
	using FPermutationDomain = TShaderPermutationDomain<FEnable128BitRT>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		OutEnvironment.SetRenderTargetOutputFormat(0, PermutationVector.Get<FEnable128BitRT>() ? PF_A32B32G32R32F : PF_FloatRGBA);
	}
};

IMPLEMENT_GLOBAL_SHADER(FWaterInfoTextureMergePS, "/Engine/Private/WaterInfoTextureMerge.usf", "Main", SF_Pixel);

class FWaterInfoTextureBlurPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FWaterInfoTextureBlurPS);
	SHADER_USE_PARAMETER_STRUCT(FWaterInfoTextureBlurPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterInfoTexture)
		SHADER_PARAMETER(float, WaterZMin)
		SHADER_PARAMETER(float, WaterZMax)
		SHADER_PARAMETER(float, GroundZMin)
		SHADER_PARAMETER(float, CaptureZ)
		SHADER_PARAMETER(int, BlurRadius)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	class FEnable128BitRT : SHADER_PERMUTATION_BOOL("ENABLE_128_BIT");
	using FPermutationDomain = TShaderPermutationDomain<FEnable128BitRT>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		OutEnvironment.SetRenderTargetOutputFormat(0, PermutationVector.Get<FEnable128BitRT>() ? PF_A32B32G32R32F : PF_FloatRGBA);
	}
};

IMPLEMENT_GLOBAL_SHADER(FWaterInfoTextureBlurPS, "/Engine/Private/WaterInfoTextureBlur.usf", "Main", SF_Pixel);

/**
* MeshPassProcessor for the "color" pass required for generating the water info texture. The associated pass draws water body meshes with an unlit material
* in order to write water surface depth, river velocity and possibly other data too.
*/
class FWaterInfoTexturePassMeshProcessor : public FSceneRenderingAllocatorObject<FWaterInfoTexturePassMeshProcessor>, public FMeshPassProcessor
{
public:
	FWaterInfoTexturePassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext, bool bIsMobile);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	FMeshPassProcessorRenderState PassDrawRenderState;
	TArray<const TCHAR*, TInlineAllocator<1>> MaterialAllowList;
	bool bIsMobile;
};

FWaterInfoTexturePassMeshProcessor::FWaterInfoTexturePassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext, bool bIsMobile)
	: FMeshPassProcessor(EMeshPass::WaterInfoTexturePass, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext),
	bIsMobile(bIsMobile)
{
	PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	PassDrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilNop);
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());

	// HACK: This whole path for rendering the water info texture is a temporary solution, so in order to avoid supporting generic material setups, we use an allow list to restrict
	// the set of supported materials to known working (and required) materials.
	MaterialAllowList.Add(TEXT("DrawWaterInfo"));
}

void FWaterInfoTexturePassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (!MeshBatch.bUseForMaterial)
	{
		return;
	}

	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material && MaterialAllowList.Contains(Material->GetAssetName()))
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

void FWaterInfoTexturePassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	// Try to reduce the number of materials considered for this mesh pass:
	// Materials for drawing the water info texture are supposed to be unlit (they write velocity and possibly other data into Emissive).
	// They also need to be applied to meshes (MD_Surface) and we can safely exclude sky materials which also use the unlit shading model.
	if (Material.GetShadingModels().IsUnlit() && Material.GetMaterialDomain() == MD_Surface && !Material.IsSky() && MaterialAllowList.Contains(Material.GetAssetName()))
	{
		// Determine the mesh's material and blend mode.
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

		// Setup the render target info
		FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
		RenderTargetsInfo.NumSamples = 1;
		AddRenderTargetInfo(PF_FloatRGBA, TexCreate_RenderTargetable | TexCreate_ShaderResource, RenderTargetsInfo);
		SetupDepthStencilInfo(PF_DepthStencil, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop, RenderTargetsInfo);

		if (!bIsMobile)
		{
			// Get the shaders if possible for given vertex factory
			TMeshProcessorShaders<
				TBasePassVertexShaderPolicyParamType<FUniformLightMapPolicy>,
				TBasePassPixelShaderPolicyParamType<FUniformLightMapPolicy>> PassShaders;
			if (!GetBasePassShaders<FUniformLightMapPolicy>(
				Material,
				VertexFactoryData.VertexFactoryType,
				FUniformLightMapPolicy(LMP_NO_LIGHTMAP),
				FeatureLevel,
				false, // 128bit
				false, // bIsDebug
				GBL_Default, // Currently only Nanite supports non-default layout
				nullptr, // Mesh Shader
				&PassShaders.VertexShader,
				&PassShaders.PixelShader
			))
			{
				return;
			}

			AddGraphicsPipelineStateInitializer(
				VertexFactoryData,
				Material,
				PassDrawRenderState,
				RenderTargetsInfo,
				PassShaders,
				MeshFillMode,
				MeshCullMode,
				(EPrimitiveType)PreCacheParams.PrimitiveType,
				EMeshPassFeatures::Default,
				true /*bRequired*/,
				PSOInitializers);
		}
		else
		{
			TMeshProcessorShaders<
				TMobileBasePassVSPolicyParamType<FUniformLightMapPolicy>,
				TMobileBasePassPSPolicyParamType<FUniformLightMapPolicy>> PassShaders;

			FMaterialShaderTypes ShaderTypes;
			ShaderTypes.AddShaderType<TMobileBasePassVS<TUniformLightMapPolicy<LMP_NO_LIGHTMAP>>>();
			ShaderTypes.AddShaderType<TMobileBasePassPS<TUniformLightMapPolicy<LMP_NO_LIGHTMAP>, LOCAL_LIGHTS_DISABLED>>();
			FMaterialShaders Shaders;
			if (!Material.TryGetShaders(ShaderTypes, VertexFactoryData.VertexFactoryType, Shaders))
			{
				return;
			}

			Shaders.TryGetVertexShader(PassShaders.VertexShader);
			Shaders.TryGetPixelShader(PassShaders.PixelShader);
	
			AddGraphicsPipelineStateInitializer(
				VertexFactoryData,
				Material,
				PassDrawRenderState,
				RenderTargetsInfo,
				PassShaders,
				MeshFillMode,
				MeshCullMode,
				(EPrimitiveType)PreCacheParams.PrimitiveType,
				EMeshPassFeatures::Default,
				true /*bRequired*/,
				PSOInitializers);
		}
		
	}
}

bool FWaterInfoTexturePassMeshProcessor::TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material)
{
	// Try to reduce the number of materials considered for this mesh pass:
	// Materials for drawing the water info texture are supposed to be unlit (they write velocity and possibly other data into Emissive).
	// They also need to be applied to meshes (MD_Surface) and we can safely exclude sky materials which also use the unlit shading model.
	if (Material.GetShadingModels().IsUnlit() && Material.GetMaterialDomain() == MD_Surface && !Material.IsSky())
	{
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

		const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

		if (!bIsMobile)
		{
			TMeshProcessorShaders<
				TBasePassVertexShaderPolicyParamType<FUniformLightMapPolicy>,
				TBasePassPixelShaderPolicyParamType<FUniformLightMapPolicy>> PassShaders;

			if (!GetBasePassShaders<FUniformLightMapPolicy>(
				Material,
				VertexFactory->GetType(),
				FUniformLightMapPolicy(LMP_NO_LIGHTMAP),
				FeatureLevel,
				false, // 128bit
				false, // bIsDebug
				GBL_Default, // Currently only Nanite uses non-default layout
				nullptr, // Mesh Shader
				&PassShaders.VertexShader,
				&PassShaders.PixelShader
			))
			{
				return false;
			}

			FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);

			TBasePassShaderElementData<FUniformLightMapPolicy> ShaderElementData(MeshBatch.LCI);
			ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, true);

			BuildMeshDrawCommands(
				MeshBatch,
				BatchElementMask,
				PrimitiveSceneProxy,
				MaterialRenderProxy,
				Material,
				PassDrawRenderState,
				PassShaders,
				MeshFillMode,
				MeshCullMode,
				SortKey,
				EMeshPassFeatures::Default,
				ShaderElementData);
		}
		else
		{
			TMeshProcessorShaders<
				TMobileBasePassVSPolicyParamType<FUniformLightMapPolicy>,
				TMobileBasePassPSPolicyParamType<FUniformLightMapPolicy>> PassShaders;

			FMaterialShaderTypes ShaderTypes;
			ShaderTypes.AddShaderType<TMobileBasePassVS<TUniformLightMapPolicy<LMP_NO_LIGHTMAP>>>();
			ShaderTypes.AddShaderType<TMobileBasePassPS<TUniformLightMapPolicy<LMP_NO_LIGHTMAP>, LOCAL_LIGHTS_DISABLED>>();
			FMaterialShaders Shaders;
			if (!Material.TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
			{
				return false;
			}

			Shaders.TryGetVertexShader(PassShaders.VertexShader);
			Shaders.TryGetPixelShader(PassShaders.PixelShader);

			FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);

			TMobileBasePassShaderElementData<FUniformLightMapPolicy> ShaderElementData(MeshBatch.LCI, false);
			ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

			BuildMeshDrawCommands(
				MeshBatch,
				BatchElementMask,
				PrimitiveSceneProxy,
				MaterialRenderProxy,
				Material,
				PassDrawRenderState,
				PassShaders,
				MeshFillMode,
				MeshCullMode,
				SortKey,
				EMeshPassFeatures::Default,
				ShaderElementData);
		}
		

		return true;
	}

	return false;
}

FMeshPassProcessor* CreateWaterInfoTexturePassMeshProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FWaterInfoTexturePassMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext, false);
}

FMeshPassProcessor* CreateWaterInfoTexturePassMeshProcessorMobile(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FWaterInfoTexturePassMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext, true);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(WaterInfoTexturePass, CreateWaterInfoTexturePassMeshProcessor, EShadingPath::Deferred, EMeshPass::WaterInfoTexturePass, EMeshPassFlags::CachedMeshCommands);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileWaterInfoTexturePass, CreateWaterInfoTexturePassMeshProcessorMobile, EShadingPath::Mobile, EMeshPass::WaterInfoTexturePass, EMeshPassFlags::CachedMeshCommands);


/**
* MeshPassProcessor for the two depth-only passes involved in generating the water info texture. The two passes can use the same processor. The first (terrain) pass renders terrain
* with a fixed grid vertex factory and possibly other static meshes too. The second pass renders dilated water body meshes. All these meshes should only be rendered into this view/texture, 
  which is why MeshBatches need to have the bUseForWaterInfoTextureDepth bool set.
*/
class FWaterInfoTextureDepthPassMeshProcessor : public FSceneRenderingAllocatorObject<FWaterInfoTextureDepthPassMeshProcessor>, public FMeshPassProcessor
{
public:
	FWaterInfoTextureDepthPassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	template<bool bPositionOnly>
	bool Process(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	template<bool bPositionOnly>
	void CollectPSOInitializersInternal(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		const FPSOPrecacheParams& PreCacheParams,
		TArray<FPSOPrecacheData>& PSOInitializers);

	FMeshPassProcessorRenderState PassDrawRenderState;
};

FWaterInfoTextureDepthPassMeshProcessor::FWaterInfoTextureDepthPassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::WaterInfoTextureDepthPass, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
{
	PassDrawRenderState.SetBlendState(TStaticBlendState<CW_NONE>::GetRHI());
	PassDrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilNop);
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
}

void FWaterInfoTextureDepthPassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

bool FWaterInfoTextureDepthPassMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	if (MeshBatch.bUseForWaterInfoTextureDepth && Material.GetMaterialDomain() == MD_Surface && !IsTranslucentBlendMode(Material))
	{
		// Determine the mesh's material and blend mode.
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
		const bool bVFTypeSupportsNullPixelShader = MeshBatch.VertexFactory->SupportsNullPixelShader();
		const bool bEvaluateWPO = Material.MaterialModifiesMeshPosition_RenderThread() && (!ShouldOptimizedWPOAffectNonNaniteShaderSelection() || PrimitiveSceneProxy->EvaluateWorldPositionOffset());

		if (IsOpaqueBlendMode(Material)
			&& MeshBatch.VertexFactory->SupportsPositionOnlyStream()
			&& !bEvaluateWPO
			&& Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader))
		{
			const FMaterialRenderProxy& DefaultProxy = *UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			const FMaterial& DefaultMaterial = *DefaultProxy.GetMaterialNoFallback(FeatureLevel);
			return Process<true>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, DefaultProxy, DefaultMaterial, MeshFillMode, MeshCullMode);
		}
		else
		{
			const bool bMaterialMasked = !Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader);
			const FMaterialRenderProxy* EffectiveMaterialRenderProxy = &MaterialRenderProxy;
			const FMaterial* EffectiveMaterial = &Material;

			if (!bMaterialMasked && !bEvaluateWPO)
			{
				// Override with the default material for opaque materials that are not two sided
				EffectiveMaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
				EffectiveMaterial = EffectiveMaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
				check(EffectiveMaterial);
			}

			return Process<false>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *EffectiveMaterialRenderProxy, *EffectiveMaterial, MeshFillMode, MeshCullMode);
		}
	}

	return true;
}

template<bool bPositionOnly>
bool FWaterInfoTextureDepthPassMeshProcessor::Process(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	TMeshProcessorShaders<TDepthOnlyVS<bPositionOnly>, FDepthOnlyPS> DepthPassShaders;
	FShaderPipelineRef ShaderPipeline;

	if (!GetDepthPassShaders<bPositionOnly>(
		MaterialResource,
		MeshBatch.VertexFactory->GetType(),
		FeatureLevel,
		MaterialResource.MaterialUsesPixelDepthOffset_GameThread(),
		DepthPassShaders.VertexShader,
		DepthPassShaders.PixelShader,
		ShaderPipeline))
	{
		return false;
	}

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, true);

	const bool bIsMasked = IsMaskedBlendMode(MaterialResource);
	FMeshDrawCommandSortKey SortKey = CalculateDepthPassMeshStaticSortKey(bIsMasked, DepthPassShaders.VertexShader.GetShader(), DepthPassShaders.PixelShader.GetShader());

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		DepthPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		bPositionOnly ? EMeshPassFeatures::PositionOnly : EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

void FWaterInfoTextureDepthPassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	// Early out if disabled
	if (!UPSOPrecacheSettingsManager::GetSettings().bWaterInfoDepth)
	{
		return;
	}

	// We need to support all materials that could possibly be rendered in a depth-only pass. Unfortunately there doesn't seem to be a way to filter for bUseForWaterInfoTextureDepth at this point.
	if (Material.GetMaterialDomain() == MD_Surface && !IsTranslucentBlendMode(Material))
	{
		// Determine the mesh's material and blend mode.
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
		const bool bSupportPositionOnlyStream = VertexFactoryData.VertexFactoryType->SupportsPositionOnly();
		const bool bVFTypeSupportsNullPixelShader = VertexFactoryData.VertexFactoryType->SupportsNullPixelShader();

		if (IsOpaqueBlendMode(Material)
			&& bSupportPositionOnlyStream
			&& !Material.MaterialModifiesMeshPosition_GameThread()
			&& Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader))
		{
			EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
			const FMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface)->GetMaterialResource(SceneTexturesConfig.ShaderPlatform, ActiveQualityLevel);
			check(DefaultMaterial);

			CollectPSOInitializersInternal<true>(SceneTexturesConfig, VertexFactoryData, *DefaultMaterial, MeshFillMode, MeshCullMode, PreCacheParams, PSOInitializers);
		}
		else
		{
			const bool bMaterialMasked = !Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader);
			const FMaterial* EffectiveMaterial = &Material;

			if (!bMaterialMasked && !Material.MaterialModifiesMeshPosition_GameThread())
			{
				// Override with the default material for opaque materials that are not two sided
				EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
				EffectiveMaterial = UMaterial::GetDefaultMaterial(MD_Surface)->GetMaterialResource(SceneTexturesConfig.ShaderPlatform, ActiveQualityLevel);
				check(EffectiveMaterial);
			}

			CollectPSOInitializersInternal<false>(SceneTexturesConfig, VertexFactoryData, *EffectiveMaterial, MeshFillMode, MeshCullMode, PreCacheParams, PSOInitializers);
		}
	}
}

template<bool bPositionOnly>
void FWaterInfoTextureDepthPassMeshProcessor::CollectPSOInitializersInternal(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	const FPSOPrecacheParams& PreCacheParams,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	TMeshProcessorShaders<TDepthOnlyVS<bPositionOnly>, FDepthOnlyPS> DepthPassShaders;
	FShaderPipelineRef ShaderPipeline;

	if (!GetDepthPassShaders<bPositionOnly>(
		MaterialResource,
		VertexFactoryData.VertexFactoryType,
		FeatureLevel,
		MaterialResource.MaterialUsesPixelDepthOffset_GameThread(),
		DepthPassShaders.VertexShader,
		DepthPassShaders.PixelShader,
		ShaderPipeline))
	{
		return;
	}

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;
	SetupDepthStencilInfo(PF_DepthStencil, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource, ERenderTargetLoadAction::EClear,
		ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop, RenderTargetsInfo);

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		MaterialResource,
		PassDrawRenderState,
		RenderTargetsInfo,
		DepthPassShaders,
		MeshFillMode,
		MeshCullMode,
		(EPrimitiveType)PreCacheParams.PrimitiveType,
		bPositionOnly ? EMeshPassFeatures::PositionOnly : EMeshPassFeatures::Default,
		true /*bRequired*/,
		PSOInitializers);
}

FMeshPassProcessor* CreateWaterInfoTextureDepthPassMeshProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FWaterInfoTextureDepthPassMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(WaterInfoTextureDepthPass, CreateWaterInfoTextureDepthPassMeshProcessor, EShadingPath::Deferred, EMeshPass::WaterInfoTextureDepthPass, EMeshPassFlags::CachedMeshCommands);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileWaterInfoTextureDepthPass, CreateWaterInfoTextureDepthPassMeshProcessor, EShadingPath::Mobile, EMeshPass::WaterInfoTextureDepthPass, EMeshPassFlags::CachedMeshCommands);



enum class EWaterInfoTextureMeshPass : int32
{
	// Depth-only pass for actors which are considered the terrain for water rendering
	TerrainDepth,

	// Depth-only pass for water body dilated sections which get composited back into the water info texture at a lower priority than regular water body data
	DilatedWaterBodyDepth,
	
	// Depth and color pass for non-dilated water body meshes, rendering out water surface depth, river velocity/flow and possibly other data in the future.
	WaterBody,
	
	Num
};

struct FWaterInfoTextureDraws
{
	EMeshPass::Type MeshPass = EMeshPass::WaterInfoTextureDepthPass;
	const TSet<FPrimitiveComponentId>* ComponentIDsToDraw = nullptr;
	uint32 PrimitiveInstanceIndex = 0;
	FMeshCommandOneFrameArray VisibleMeshCommands;
	TArray<uint32, SceneRenderingAllocator> InstanceRuns;
	FInstanceCullingContext* InstanceCullingContext = nullptr;
	FInstanceCullingResult InstanceCullingResult;

	FWaterInfoTextureDraws() = default;
	FWaterInfoTextureDraws(EMeshPass::Type InMeshPass, const TSet<FPrimitiveComponentId>* InComponentIDsToDraw, uint32 InPrimitiveInstanceIndex)
		: MeshPass(InMeshPass), ComponentIDsToDraw(InComponentIDsToDraw), PrimitiveInstanceIndex(InPrimitiveInstanceIndex)
	{}
};

static void AddWaterInfoTextureDraws(const FScene* Scene, const TArrayView<FWaterInfoTextureDraws>& PassDrawLists)
{
	const int32 NumDrawLists = PassDrawLists.Num();
	TArray<uint32, TInlineAllocator<3>> MaxVisibleMeshDrawCommands;
	MaxVisibleMeshDrawCommands.SetNum(NumDrawLists);

	// Compute upper bounds for mesh draw commands
	for (const FPrimitiveSceneInfo* PrimitiveSceneInfo : Scene->Primitives)
	{
		if (PrimitiveSceneInfo && !PrimitiveSceneInfo->Proxy->IsNaniteMesh())
		{
			for (int32 DrawListIdx = 0; DrawListIdx < NumDrawLists; ++DrawListIdx)
			{
				if (PassDrawLists[DrawListIdx].ComponentIDsToDraw->Contains(PrimitiveSceneInfo->PrimitiveComponentId))
				{
					MaxVisibleMeshDrawCommands[DrawListIdx] += PrimitiveSceneInfo->StaticMeshRelevances.Num();
				}
			}
			
		}
	}
	for (int32 DrawListIdx = 0; DrawListIdx < NumDrawLists; ++DrawListIdx)
	{
		PassDrawLists[DrawListIdx].InstanceRuns.Reserve(2 * MaxVisibleMeshDrawCommands[DrawListIdx]);
	}

	// Collect mesh draw commands
	for (const FPrimitiveSceneInfo* PrimitiveSceneInfo : Scene->Primitives)
	{
		if (PrimitiveSceneInfo && !PrimitiveSceneInfo->Proxy->IsNaniteMesh())
		{
			for (int32 DrawListIdx = 0; DrawListIdx < NumDrawLists; ++DrawListIdx)
			{
				const EWaterInfoTextureMeshPass PassType = static_cast<EWaterInfoTextureMeshPass>(DrawListIdx);
				FWaterInfoTextureDraws& Draws = PassDrawLists[DrawListIdx];
				if (Draws.ComponentIDsToDraw->Contains(PrimitiveSceneInfo->PrimitiveComponentId))
				{
					FMeshDrawCommandPrimitiveIdInfo IdInfo(PrimitiveSceneInfo->GetMDCIdInfo());

					for (int32 MeshIndex = 0; MeshIndex < PrimitiveSceneInfo->StaticMeshRelevances.Num(); MeshIndex++)
					{
						const FStaticMeshBatchRelevance& StaticMeshRelevance = PrimitiveSceneInfo->StaticMeshRelevances[MeshIndex];
						const FStaticMeshBatch& StaticMesh = PrimitiveSceneInfo->StaticMeshes[MeshIndex];

						const bool bUseInDepthPass = (PassType == EWaterInfoTextureMeshPass::TerrainDepth || PassType == EWaterInfoTextureMeshPass::DilatedWaterBodyDepth) && StaticMeshRelevance.bUseForWaterInfoTextureDepth;
						const bool bUseInColorPass = (PassType == EWaterInfoTextureMeshPass::WaterBody) && StaticMeshRelevance.bUseForMaterial;

						if ((bUseInDepthPass || bUseInColorPass) && StaticMeshRelevance.GetLODIndex() == 0)
						{
							const int32 StaticMeshCommandInfoIndex = StaticMeshRelevance.GetStaticMeshCommandInfoIndex(Draws.MeshPass);
							if (StaticMeshCommandInfoIndex >= 0)
							{
								const FCachedMeshDrawCommandInfo& CachedMeshDrawCommand = PrimitiveSceneInfo->StaticMeshCommandInfos[StaticMeshCommandInfoIndex];
								const FCachedPassMeshDrawList& SceneDrawList = Scene->CachedDrawLists[Draws.MeshPass];

								const FMeshDrawCommand* MeshDrawCommand = nullptr;
								if (CachedMeshDrawCommand.StateBucketId >= 0)
								{
									MeshDrawCommand = &Scene->CachedMeshDrawCommandStateBuckets[Draws.MeshPass].GetByElementId(CachedMeshDrawCommand.StateBucketId).Key;
								}
								else
								{
									MeshDrawCommand = &SceneDrawList.MeshDrawCommands[CachedMeshDrawCommand.CommandIndex];
								}

								const uint32* InstanceRunArray = nullptr;
								uint32 NumInstanceRuns = 0;

								if (MeshDrawCommand->NumInstances > 1 && Draws.PrimitiveInstanceIndex >= 0)
								{
									// Render only a single specified instance, by specifying an inclusive [x;x] range

									ensure(Draws.InstanceRuns.Num() + 2 <= Draws.InstanceRuns.Max());
									InstanceRunArray = Draws.InstanceRuns.GetData() + Draws.InstanceRuns.Num();
									NumInstanceRuns = 1;

									Draws.InstanceRuns.Add(Draws.PrimitiveInstanceIndex);
									Draws.InstanceRuns.Add(Draws.PrimitiveInstanceIndex);
								}

								FVisibleMeshDrawCommand NewVisibleMeshDrawCommand;

								NewVisibleMeshDrawCommand.Setup(
									MeshDrawCommand,
									IdInfo,
									CachedMeshDrawCommand.StateBucketId,
									CachedMeshDrawCommand.MeshFillMode,
									CachedMeshDrawCommand.MeshCullMode,
									CachedMeshDrawCommand.Flags,
									CachedMeshDrawCommand.SortKey,
									CachedMeshDrawCommand.CullingPayload,
									EMeshDrawCommandCullingPayloadFlags::NoScreenSizeCull,
									InstanceRunArray,
									NumInstanceRuns);

								Draws.VisibleMeshCommands.Add(NewVisibleMeshDrawCommand);
							}
						}
					}
				}
			}
		}
	}
}

void RenderWaterInfoTexture(
	FRDGBuilder& GraphBuilder,
	FSceneRenderer& SceneRenderer,
	const FScene* Scene
)
{
	bool bAnyWaterInfoTextureUpdates = false;
	for (const FViewInfo& View : SceneRenderer.Views)
	{
		bAnyWaterInfoTextureUpdates = bAnyWaterInfoTextureUpdates || !View.WaterInfoTextureRenderingParams.IsEmpty();
	}

	if (!bAnyWaterInfoTextureUpdates)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(WaterInfo::RenderWaterInfoTexture);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, WaterInfoTexture, "WaterInfoTexture");

	int32 RenderCaptureNextWaterInfoDraws = CVarRenderCaptureNextWaterInfoDraws.GetValueOnRenderThread();
	RenderCaptureInterface::FScopedCapture RenderCapture((RenderCaptureNextWaterInfoDraws != 0), GraphBuilder, TEXT("RenderWaterInfo"));
	if (RenderCaptureNextWaterInfoDraws != 0)
	{
		RenderCaptureNextWaterInfoDraws = FMath::Max(0, RenderCaptureNextWaterInfoDraws - 1);
		CVarRenderCaptureNextWaterInfoDraws->SetWithCurrentPriority(RenderCaptureNextWaterInfoDraws);
	}

	// Keep track of all the result textures so we can batch call UseExternalAccessMode() on them.
	TArray<FRDGViewableResource*, TInlineAllocator<4>> OutputTextures;

	// Check all views for WaterInfoTexture updates that we might need to execute.
	for (const FViewInfo& View : SceneRenderer.Views)
	{
		for (const FViewInfo::FWaterInfoTextureRenderingParams& RenderingParams : View.WaterInfoTextureRenderingParams)
		{
			const FIntPoint TextureSize = RenderingParams.RenderTarget->GetSizeXY();
			const FIntRect Viewport(0, 0, TextureSize.X, TextureSize.Y);

			FViewInfo& WaterView = *View.CreateSnapshot();
			{
				WaterView.ViewState = nullptr;
				WaterView.DynamicPrimitiveCollector = FGPUScenePrimitiveCollector(&SceneRenderer.GetGPUSceneDynamicContext());
				WaterView.StereoPass = EStereoscopicPass::eSSP_FULL;
				WaterView.DrawDynamicFlags = EDrawDynamicFlags::ForceLowestLOD;
				WaterView.MaterialTextureMipBias = 0;
				WaterView.PreExposure = 1.0f;
				WaterView.ViewRect = Viewport;

				WaterView.CachedViewUniformShaderParameters = MakeUnique<FViewUniformShaderParameters>();
				FBox VolumeBounds[TVC_MAX];
				WaterView.SetupUniformBufferParameters(VolumeBounds, TVC_MAX, *WaterView.CachedViewUniformShaderParameters);

				WaterView.UpdateProjectionMatrix(RenderingParams.ProjectionMatrix);

				FViewMatrices::FMinimalInitializer Initializer;
				Initializer.ViewRotationMatrix = RenderingParams.ViewRotationMatrix;
				Initializer.ViewOrigin = RenderingParams.ViewLocation;
				Initializer.ProjectionMatrix = RenderingParams.ProjectionMatrix;
				Initializer.ConstrainedViewRect = Viewport;
				WaterView.ViewMatrices = FViewMatrices(Initializer);

				WaterView.SetupCommonViewUniformBufferParameters(
					*WaterView.CachedViewUniformShaderParameters, 
					TextureSize, 
					1, 
					Initializer.ConstrainedViewRect, 
					WaterView.ViewMatrices, 
					WaterView.ViewMatrices);

				WaterView.CreateViewUniformBuffers(*WaterView.CachedViewUniformShaderParameters);
			}

			const FTextureRHIRef OutputRenderTarget = RenderingParams.RenderTarget->GetRenderTargetTexture();
			FRDGTexture* OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputRenderTarget, TEXT("WaterInfo.OutputTexture")));
			GraphBuilder.UseInternalAccessMode(OutputTexture); // Make sure the texture is actually managed by RDG as a call to RegisterExternalTexture does not guarantee this if the texture was already registered and then had UseExternalAccessMode() called on it.

			FRDGTextureDesc TerrainDepthBufferDesc = FRDGTextureDesc::Create2D(TextureSize, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
			FRDGTexture* TerrainDepthBuffer = GraphBuilder.CreateTexture(TerrainDepthBufferDesc, TEXT("WaterInfo.TerrainDepthTexture"));

			FRDGTextureDesc WaterInfoTextureDesc = FRDGTextureDesc::Create2D(TextureSize, PF_FloatRGBA, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
			FRDGTexture* WaterInfoColorTexture = GraphBuilder.CreateTexture(WaterInfoTextureDesc, TEXT("WaterInfo.ColorTexture"));

			FRDGTextureDesc WaterInfoDepthBufferDesc = FRDGTextureDesc::Create2D(TextureSize, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
			FRDGTexture* WaterInfoDepthBuffer = GraphBuilder.CreateTexture(WaterInfoDepthBufferDesc, TEXT("WaterInfo.DepthTexture"));

			FRDGTextureDesc DilatedDepthBufferDesc = FRDGTextureDesc::Create2D(TextureSize, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
			FRDGTexture* DilatedDepthBuffer = GraphBuilder.CreateTexture(DilatedDepthBufferDesc, TEXT("WaterInfo.DilatedDepthTexture"));

			FRDGTextureDesc MergedTextureDesc = FRDGTextureDesc::Create2D(TextureSize, OutputTexture->Desc.Format, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
			FRDGTexture* MergedTexture = GraphBuilder.CreateTexture(MergedTextureDesc, TEXT("WaterInfo.MergedTexture"));

			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(WaterView.GetFeatureLevel());

			TStaticArray<FWaterInfoTextureDraws, static_cast<int32>(EWaterInfoTextureMeshPass::Num)> PassDraws;
			PassDraws[static_cast<int32>(EWaterInfoTextureMeshPass::TerrainDepth)] =			FWaterInfoTextureDraws(EMeshPass::WaterInfoTextureDepthPass, &RenderingParams.TerrainComponentIds, 0);
			PassDraws[static_cast<int32>(EWaterInfoTextureMeshPass::DilatedWaterBodyDepth)] =	FWaterInfoTextureDraws(EMeshPass::WaterInfoTextureDepthPass, &RenderingParams.DilatedWaterBodyComponentIds, 0);
			PassDraws[static_cast<int32>(EWaterInfoTextureMeshPass::WaterBody)] =				FWaterInfoTextureDraws(EMeshPass::WaterInfoTexturePass, &RenderingParams.WaterBodyComponentIds, 0);
			
			AddWaterInfoTextureDraws(Scene, PassDraws);

			// Build rendering commands or prepare primitive ID buffer for the mesh draw commands
			for (int32 PassIdx = 0; PassIdx < 3; ++PassIdx)
			{
				if (Scene->GPUScene.IsEnabled())
				{
					int32 MaxInstances = 0;
					int32 VisibleMeshDrawCommandsNum = 0;
					int32 NewPassVisibleMeshDrawCommandsNum = 0;

					PassDraws[PassIdx].InstanceCullingContext = GraphBuilder.AllocObject<FInstanceCullingContext>(TEXT("WaterInfoTexture"), WaterView.GetShaderPlatform(), nullptr, TArrayView<const int32>(&WaterView.SceneRendererPrimaryViewId, 1), nullptr);

					PassDraws[PassIdx].InstanceCullingContext->SetupDrawCommands(PassDraws[PassIdx].VisibleMeshCommands, false, Scene, MaxInstances, VisibleMeshDrawCommandsNum, NewPassVisibleMeshDrawCommandsNum);
					// Not supposed to do any compaction here.
					ensure(VisibleMeshDrawCommandsNum == PassDraws[PassIdx].VisibleMeshCommands.Num());

					PassDraws[PassIdx].InstanceCullingContext->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, WaterView.DynamicPrimitiveCollector.GetInstanceSceneDataOffset(), WaterView.DynamicPrimitiveCollector.NumInstances(), PassDraws[PassIdx].InstanceCullingResult);
				}

				PassDraws[PassIdx].InstanceCullingResult.Parameters.Scene = SceneRenderer.GetSceneUniforms().GetBuffer(GraphBuilder);
			}

			// Execute the three mesh passes involved in generating the water info texture: terrain depth-only, dilated water body depth-only and finally regular water body depth + river velocity
			for (uint32 PassIndex = 0; PassIndex < static_cast<int32>(EWaterInfoTextureMeshPass::Num); ++PassIndex)
			{
				const EWaterInfoTextureMeshPass PassType = static_cast<EWaterInfoTextureMeshPass>(PassIndex);

				// Terrain depth and dilated water body depth share the same setup, they just render a different set of meshes into different depth buffers
				if (PassType == EWaterInfoTextureMeshPass::TerrainDepth || PassType == EWaterInfoTextureMeshPass::DilatedWaterBodyDepth)
				{
					const bool bIsTerrainDepthPass = PassType == EWaterInfoTextureMeshPass::TerrainDepth;
					FWaterInfoTextureDepthPassParameters* PassParameters = GraphBuilder.AllocParameters<FWaterInfoTextureDepthPassParameters>();
					PassParameters->View = WaterView.GetShaderParameters();
					PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(bIsTerrainDepthPass ? TerrainDepthBuffer : DilatedDepthBuffer, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);
					PassDraws[PassIndex].InstanceCullingResult.GetDrawParameters(PassParameters->InstanceCullingDrawParams);

					GraphBuilder.AddPass(
						bIsTerrainDepthPass ? RDG_EVENT_NAME("WaterInfoTexture(Terrain)") : RDG_EVENT_NAME("WaterInfoTexture(DilatedWaterBodies)"),
						PassParameters,
						ERDGPassFlags::Raster,
						[MeshDrawCmds = MoveTemp(PassDraws[PassIndex].VisibleMeshCommands), Scene = Scene, Viewport,
						PassParameters, InstanceCullingContext = PassDraws[PassIndex].InstanceCullingContext](FRHICommandList& RHICmdList)
						{
							QUICK_SCOPE_CYCLE_COUNTER(MeshPass);

							RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
							FGraphicsMinimalPipelineStateSet PipelineStateSet;
							if (Scene->GPUScene.IsEnabled())
							{
								InstanceCullingContext->SubmitDrawCommands(MeshDrawCmds, PipelineStateSet, GetMeshDrawCommandOverrideArgs(PassParameters->InstanceCullingDrawParams), 0, MeshDrawCmds.Num(), 1, RHICmdList);
							}
							else
							{
								FMeshDrawCommandSceneArgs SceneArgs;
								SubmitMeshDrawCommandsRange(MeshDrawCmds, PipelineStateSet, SceneArgs, FInstanceCullingContext::GetInstanceIdBufferStride(Scene->GetShaderPlatform()), false, 0, MeshDrawCmds.Num(), 1, RHICmdList);
							}
						});
				}
				else if (PassType == EWaterInfoTextureMeshPass::WaterBody)
				{
					FWaterInfoTexturePassParameters* PassParameters = GraphBuilder.AllocParameters<FWaterInfoTexturePassParameters>();
					PassParameters->View = WaterView.GetShaderParameters();
					PassParameters->RenderTargets = GetRenderTargetBindings(ERenderTargetLoadAction::EClear, MakeArrayView(&WaterInfoColorTexture, 1));
					PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(WaterInfoDepthBuffer, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);
					PassDraws[PassIndex].InstanceCullingResult.GetDrawParameters(PassParameters->InstanceCullingDrawParams);

					GraphBuilder.AddPass(
						RDG_EVENT_NAME("WaterInfoTexture(WaterBodies)"),
						PassParameters,
						ERDGPassFlags::Raster,
						[MeshDrawCmds = MoveTemp(PassDraws[PassIndex].VisibleMeshCommands), Scene = Scene, Viewport,
						PassParameters, InstanceCullingContext = PassDraws[PassIndex].InstanceCullingContext](FRHICommandList& RHICmdList)
						{
							QUICK_SCOPE_CYCLE_COUNTER(MeshPass);

							RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
							FGraphicsMinimalPipelineStateSet PipelineStateSet;
							if (Scene->GPUScene.IsEnabled())
							{
								InstanceCullingContext->SubmitDrawCommands(MeshDrawCmds, PipelineStateSet, GetMeshDrawCommandOverrideArgs(PassParameters->InstanceCullingDrawParams), 0, MeshDrawCmds.Num(), 1, RHICmdList);
							}
							else
							{
								FMeshDrawCommandSceneArgs SceneArgs;
								SubmitMeshDrawCommandsRange(MeshDrawCmds, PipelineStateSet, SceneArgs, FInstanceCullingContext::GetInstanceIdBufferStride(Scene->GetShaderPlatform()), false, 0, MeshDrawCmds.Num(), 1, RHICmdList);
							}
						});
				}
				
			}

			// Merge terrain depth, dilated water body depth, water body depth and velocity into a single texture
			{
				FWaterInfoTextureMergePS::FPermutationDomain PixelPermutationVector;
				// The output format depends on the user configurable format of the passed in render target. Since we store depth data in the texture,
				// it is sometimes desirable to have full 32bit float precision.
				PixelPermutationVector.Set<FWaterInfoTextureMergePS::FEnable128BitRT>(OutputTexture->Desc.Format == PF_A32B32G32R32F);
				TShaderMapRef<FWaterInfoTextureMergePS> PixelShader(ShaderMap, PixelPermutationVector);

				FWaterInfoTextureMergePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterInfoTextureMergePS::FParameters>();
				PassParameters->View = WaterView.GetShaderParameters();
				PassParameters->RenderTargets[0] = FRenderTargetBinding(MergedTexture, ERenderTargetLoadAction::ENoAction);
				PassParameters->SceneTextures = GetSceneTextureShaderParameters(WaterView);
				PassParameters->GroundDepthTexture = TerrainDepthBuffer;
				PassParameters->GroundDepthTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->WaterBodyTexture = WaterInfoColorTexture;
				PassParameters->WaterBodyTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->WaterBodyDepthTexture = WaterInfoDepthBuffer;
				PassParameters->WaterBodyDepthTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->DilatedWaterBodyDepthTexture = DilatedDepthBuffer;
				PassParameters->DilatedWaterBodyDepthTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->CaptureZ = RenderingParams.CaptureZ;
				PassParameters->WaterHeightExtents = RenderingParams.WaterHeightExtents;
				PassParameters->GroundZMin = RenderingParams.GroundZMin;
				PassParameters->DilationOverwriteMinimumDistance = CVarWaterInfoDilationOverwriteMinimumDistance.GetValueOnRenderThread();
				PassParameters->UndergroundDilationDepthOffset = CVarWaterInfoUndergroundDilationDepthOffset.GetValueOnRenderThread();

				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder,
					ShaderMap,
					RDG_EVENT_NAME("WaterInfoTextureMerge"),
					PixelShader,
					PassParameters,
					Viewport);
			}

			// Blur the velocity component in the water info texture to get a smooth gradient between water body transitions
			{
				FWaterInfoTextureBlurPS::FPermutationDomain PixelPermutationVector;
				// The output format depends on the user configurable format of the passed in render target. Since we store depth data in the texture,
				// it is sometimes desirable to have full 32bit float precision.
				PixelPermutationVector.Set<FWaterInfoTextureBlurPS::FEnable128BitRT>(OutputTexture->Desc.Format == PF_A32B32G32R32F);
				TShaderMapRef<FWaterInfoTextureBlurPS> PixelShader(ShaderMap, PixelPermutationVector);

				FWaterInfoTextureBlurPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FWaterInfoTextureBlurPS::FParameters>();
				PassParameters->View = WaterView.GetShaderParameters();
				PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ENoAction, 0, RenderingParams.RenderTargetArrayLayer);
				PassParameters->SceneTextures = GetSceneTextureShaderParameters(WaterView);
				PassParameters->WaterInfoTexture = MergedTexture;
				PassParameters->WaterZMin = RenderingParams.WaterHeightExtents.X;
				PassParameters->WaterZMax = RenderingParams.WaterHeightExtents.Y;
				PassParameters->GroundZMin = RenderingParams.GroundZMin;
				PassParameters->CaptureZ = RenderingParams.CaptureZ;
				PassParameters->BlurRadius = RenderingParams.VelocityBlurRadius;

				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder,
					ShaderMap,
					RDG_EVENT_NAME("WaterInfoTextureBlur"),
					PixelShader,
					PassParameters,
					Viewport);
			}

			// Add the output texture to an array to batch call UseExternalAccessMode() on all of them.
			OutputTextures.AddUnique(OutputTexture);
		}
	}

	// Make sure the textures are in the SRV state by the time they are used in water draws (referenced outside of RDG).
	GraphBuilder.UseExternalAccessMode(OutputTextures, ERHIAccess::SRVMask);

	// Reset the WIT updates on all the views to ensure we don't accidentally keep executing the same updates.
	for (FViewInfo& View : SceneRenderer.Views)
	{
		View.WaterInfoTextureRenderingParams.Reset();
	}
}
