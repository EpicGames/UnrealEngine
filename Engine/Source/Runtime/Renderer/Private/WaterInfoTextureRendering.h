// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FRDGBuilder;
class FSceneRenderer;
class FScene;


void RenderWaterInfoTexture(FRDGBuilder& GraphBuilder, FSceneRenderer& SceneRenderer, const FScene* Scene);