// Copyright Epic Games, Inc. All Rights Reserved.

#include "WideCustomResolveShaders.h"
#include "StaticBoundShaderState.h"
#include "ShaderParameterUtils.h"
#include "PipelineStateCache.h"
#include "RenderUtils.h"

IMPLEMENT_SHADER_TYPE(, FWideCustomResolveVS, TEXT("/Engine/Private/WideCustomResolveShaders.usf"), TEXT("WideCustomResolveVS"), SF_Vertex);

#define IMPLEMENT_RESOLVE_SHADER(Width, MSAA, UseFMask) \
	typedef FWideCustomResolvePS<Width,MSAA,UseFMask> FWideCustomResolve##Width##_##MSAA##x_##UseFMask##PS; \
	IMPLEMENT_SHADER_TYPE(template<>, FWideCustomResolve##Width##_##MSAA##x_##UseFMask##PS, TEXT("/Engine/Private/WideCustomResolveShaders.usf"), TEXT("WideCustomResolvePS"), SF_Pixel)

IMPLEMENT_RESOLVE_SHADER(0, 1, false);
IMPLEMENT_RESOLVE_SHADER(2, 0, false);
IMPLEMENT_RESOLVE_SHADER(2, 1, false);
IMPLEMENT_RESOLVE_SHADER(2, 2, false);
IMPLEMENT_RESOLVE_SHADER(2, 3, false);
IMPLEMENT_RESOLVE_SHADER(4, 0, false);
IMPLEMENT_RESOLVE_SHADER(4, 1, false);
IMPLEMENT_RESOLVE_SHADER(4, 2, false);
IMPLEMENT_RESOLVE_SHADER(4, 3, false);
IMPLEMENT_RESOLVE_SHADER(8, 0, false);
IMPLEMENT_RESOLVE_SHADER(8, 1, false);
IMPLEMENT_RESOLVE_SHADER(8, 2, false);
IMPLEMENT_RESOLVE_SHADER(8, 3, false);

IMPLEMENT_RESOLVE_SHADER(0, 1, true);
IMPLEMENT_RESOLVE_SHADER(2, 0, true);
IMPLEMENT_RESOLVE_SHADER(2, 1, true);
IMPLEMENT_RESOLVE_SHADER(2, 2, true);
IMPLEMENT_RESOLVE_SHADER(2, 3, true);
IMPLEMENT_RESOLVE_SHADER(4, 0, true);
IMPLEMENT_RESOLVE_SHADER(4, 1, true);
IMPLEMENT_RESOLVE_SHADER(4, 2, true);
IMPLEMENT_RESOLVE_SHADER(4, 3, true);
IMPLEMENT_RESOLVE_SHADER(8, 0, true);
IMPLEMENT_RESOLVE_SHADER(8, 1, true);
IMPLEMENT_RESOLVE_SHADER(8, 2, true);
IMPLEMENT_RESOLVE_SHADER(8, 3, true);

template <unsigned Width, unsigned MSAA, bool UseFMask>
static void ResolveColorWideInternal2(
	FRHICommandList& RHICmdList,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	const ERHIFeatureLevel::Type CurrentFeatureLevel,
	const FTextureRHIRef& SrcTexture,
	FRHIShaderResourceView* FmaskSRV,
	const FIntPoint& SrcOrigin,
	FRHIBuffer* DummyVB)
{
	auto ShaderMap = GetGlobalShaderMap(CurrentFeatureLevel);

	TShaderMapRef<FWideCustomResolveVS> VertexShader(ShaderMap);
	TShaderMapRef<FWideCustomResolvePS<MSAA, Width, UseFMask>> PixelShader(ShaderMap);

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyPS(RHICmdList, PixelShader, SrcTexture, FmaskSRV, SrcOrigin);

	RHICmdList.SetStreamSource(0, DummyVB, 0);
	RHICmdList.DrawPrimitive(0, 1, 1);
}

template <unsigned MSAA, bool UseFMask>
static void ResolveColorWideInternal(
	FRHICommandList& RHICmdList, 
	FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	const ERHIFeatureLevel::Type CurrentFeatureLevel,
	const FTextureRHIRef& SrcTexture,
	FRHIShaderResourceView* FmaskSRV,
	const FIntPoint& SrcOrigin, 
	int32 WideFilterWidth,
	FRHIBuffer* DummyVB)
{
	switch (WideFilterWidth)
	{
	case 0: ResolveColorWideInternal2<0, MSAA, UseFMask>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, DummyVB); break;
	case 1: ResolveColorWideInternal2<1, MSAA, UseFMask>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, DummyVB); break;
	case 2: ResolveColorWideInternal2<2, MSAA, UseFMask>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, DummyVB); break;
	case 3: ResolveColorWideInternal2<3, MSAA, UseFMask>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, DummyVB); break;
	}
}

void ResolveFilterWide(
	FRHICommandList& RHICmdList, 
	FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	const ERHIFeatureLevel::Type CurrentFeatureLevel,
	const FTextureRHIRef& SrcTexture,
	FRHIShaderResourceView* FmaskSRV,
	const FIntPoint& SrcOrigin, 
	int32 NumSamples,
	int32 WideFilterWidth,
	FRHIBuffer* DummyVB)
{	
	if (FmaskSRV)
	{
		if (NumSamples <= 1)
		{
			ResolveColorWideInternal2<1, 0, true>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, DummyVB);
		}
		else if (NumSamples == 2)
		{
			ResolveColorWideInternal<2, true>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, WideFilterWidth, DummyVB);
		}
		else if (NumSamples == 4)
		{
			ResolveColorWideInternal<4, true>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, WideFilterWidth, DummyVB);
		}
		else if (NumSamples == 8)
		{
			ResolveColorWideInternal<8, true>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, WideFilterWidth, DummyVB);
		}
		else
		{
			// Need to implement more.
			check(0);
		}
	}
	else
	{
		if (NumSamples <= 1)
		{
			ResolveColorWideInternal2<1, 0, false>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, DummyVB);
		}
		else if (NumSamples == 2)
		{
			ResolveColorWideInternal<2, false>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, WideFilterWidth, DummyVB);
		}
		else if (NumSamples == 4)
		{
			ResolveColorWideInternal<4, false>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, WideFilterWidth, DummyVB);
		}
		else if (NumSamples == 8)
		{
			ResolveColorWideInternal<8, false>(RHICmdList, GraphicsPSOInit, CurrentFeatureLevel, SrcTexture, FmaskSRV, SrcOrigin, WideFilterWidth, DummyVB);
		}
		else
		{
			// Need to implement more.
			check(0);
		}
	}
}
