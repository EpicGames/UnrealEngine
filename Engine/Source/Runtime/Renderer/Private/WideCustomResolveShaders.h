// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "DataDrivenShaderPlatformInfo.h"

class FWideCustomResolveVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FWideCustomResolveVS,Global);
public:
	FWideCustomResolveVS() {}
	FWideCustomResolveVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader( Initializer )
	{
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

template <unsigned MSAASampleCount, unsigned Width, bool UseFMask>
class FWideCustomResolvePS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FWideCustomResolvePS,Global);
public:
	FWideCustomResolvePS() {}
	FWideCustomResolvePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader( Initializer )
	{
		static_assert(Width >= 0 && Width <= 3, "Invalid width");
		static_assert(MSAASampleCount == 0 || MSAASampleCount == 2 || MSAASampleCount == 4 || MSAASampleCount == 8, "Invalid sample count");

		Tex.Bind(Initializer.ParameterMap, TEXT("Tex"), SPF_Mandatory);
		if (MSAASampleCount > 0)
		{
			FMaskTex.Bind(Initializer.ParameterMap, TEXT("FMaskTex"), SPF_Optional);
		}
		ResolveOrigin.Bind(Initializer.ParameterMap, TEXT("ResolveOrigin"));
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHITexture* Texture2DMS, FRHIShaderResourceView* FmaskSRV, FIntPoint Origin)
	{
		SetTextureParameter(BatchedParameters, Tex, Texture2DMS);
		if (MSAASampleCount > 0)
		{
			SetSRVParameter(BatchedParameters, FMaskTex, FmaskSRV);
		}
		SetShaderValue(BatchedParameters, ResolveOrigin, FVector2f(Origin));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("WIDE_RESOLVE_WIDTH"), Width);
		OutEnvironment.SetDefine(TEXT("MSAA_SAMPLE_COUNT"), MSAASampleCount);
		OutEnvironment.SetDefine(TEXT("USE_FMASK"), UseFMask);
	}

protected:
	LAYOUT_FIELD(FShaderResourceParameter, Tex);
	LAYOUT_FIELD(FShaderResourceParameter, FMaskTex);
	LAYOUT_FIELD(FShaderParameter, ResolveOrigin);
};

extern void ResolveFilterWide(
	FRHICommandList& RHICmdList,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	const ERHIFeatureLevel::Type CurrentFeatureLevel,
	const FTextureRHIRef& SrcTexture,
	FRHIShaderResourceView* FmaskSRV,
	const FIntPoint& SrcOrigin,
	int32 NumSamples,
	int32 WideFilterWidth,
	FRHIBuffer* DummyVB);
