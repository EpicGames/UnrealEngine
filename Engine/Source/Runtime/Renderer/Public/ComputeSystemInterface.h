// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "RHIDefinitions.h"

class FSceneInterface;
class IComputeTaskWorker;

/** 
 * Interface for compute systems. 
 * IComputeSystem objects are registered with the renderer.
 * Whenever a scene is created it instantiates the IComputeTaskWorker objects required by the registered IComputeSystem.
 * IComputeTaskWorker objects have their work executed by the renderer at specific points in the frame.
 * The compute tasks are scheduled by the IComputeSystem.
 */
class IComputeSystem
{
public:
	virtual ~IComputeSystem() = default;

	/** Create compute workers and add to OutWorkers. The created compute workers will only be used by InScene. */
	virtual void CreateWorkers(FSceneInterface const* InScene, TArray<IComputeTaskWorker*>& OutWorkers) = 0;

	/** Destroy any compute workers created by this system that are found the InOutWorkers array. Also remove them from the array. */
	virtual void DestroyWorkers(FSceneInterface const* InScene, TArray<IComputeTaskWorker*>& InOutWorkers) = 0;
};

namespace ComputeSystemInterface
{
	/** Register a system that can provide a compute work scheduler for every scene. */
	RENDERER_API void RegisterSystem(IComputeSystem* InSystem);

	/** Unregister a system that provides compute workers. */
	RENDERER_API void UnregisterSystem(IComputeSystem* InSystem);

	/** Create compute workers for all registered systems. */
	void CreateWorkers(FSceneInterface const* InScene, TArray<IComputeTaskWorker*>& OutSchedulers);

	/** Destroy compute workers from all registered systems. */
	void DestroyWorkers(FSceneInterface const* InScene, TArray<IComputeTaskWorker*>& InOutSchedulers);
}
