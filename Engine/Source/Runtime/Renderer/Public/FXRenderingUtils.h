// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "RenderGraphFwd.h"
#include "Shader.h"
#include "RHIDefinitions.h"
#include "SceneRenderTargetParameters.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "UObject/ObjectMacros.h"
#include "RendererUtils.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

class FMaterial;
class FPrimitiveSceneProxy;
class FRDGBuilder;
class FRHIShaderResourceView;
class FSceneInterface;
class FSceneView;
class FSceneViewFamily;
class FShaderParametersMetadata;
class FSceneUniformBuffer;
class FViewInfo;
class FGlobalDistanceFieldParameterData;
class FSceneTextureUniformParameters;
class FSubstratePublicGlobalUniformParameters;
class FMobileSceneTextureUniformParameters;
DECLARE_UNIFORM_BUFFER_STRUCT(FSceneUniformParameters, RENDERER_API)

#if RHI_RAYTRACING
class FRHIRayTracingScene;
class FRayTracingShaderBindingData;
class FRayTracingMeshCommandStorage;
#endif

namespace UE::FXRenderingUtils
{
	RENDERER_API TConstStridedView<FSceneView> ConvertViewArray(TConstArrayView<FViewInfo> Views);
	RENDERER_API FIntRect GetRawViewRectUnsafe(const FSceneView& View);

	RENDERER_API bool CanMaterialRenderBeforeFXPostOpaque(const FSceneViewFamily& ViewFamily, const FPrimitiveSceneProxy& SceneProxy, const FMaterial& Material);

	RENDERER_API const FGlobalDistanceFieldParameterData* GetGlobalDistanceFieldParameterData(TConstStridedView<FSceneView> Views);
	RENDERER_API FRDGTextureRef GetSceneVelocityTexture(const FSceneView& View);

	RENDERER_API TRDGUniformBufferRef<FSceneTextureUniformParameters> GetOrCreateSceneTextureUniformBuffer(
		FRDGBuilder& GraphBuilder,
		TConstStridedView<FSceneView> Views,
		ERHIFeatureLevel::Type FeatureLevel,
		ESceneTextureSetupMode SetupMode = ESceneTextureSetupMode::All);

	RENDERER_API TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> GetOrCreateMobileSceneTextureUniformBuffer(
		FRDGBuilder& GraphBuilder,
		TConstStridedView<FSceneView> Views,
		EMobileSceneTextureSetupMode SetupMode = EMobileSceneTextureSetupMode::All);

	RENDERER_API TRDGUniformBufferRef<FSubstratePublicGlobalUniformParameters> GetSubstratePublicGlobalUniformBuffer(
		FRDGBuilder& GraphBuilder,
		TConstStridedView<FSceneView> Views);

	namespace DistanceFields
	{
		RENDERER_API const FShaderParametersMetadata* GetObjectBufferParametersMetadata();
		RENDERER_API const FShaderParametersMetadata* GetAtlasParametersMetadata();

		RENDERER_API bool HasDataToBind(const FSceneView& View);

		RENDERER_API void SetupObjectBufferParameters(FRDGBuilder& GraphBuilder, uint8* DestinationData, const FSceneView* View);
		RENDERER_API void SetupAtlasParameters(FRDGBuilder& GraphBuilder, uint8* DestinationData, const FSceneView* View);
	}

	/**
	 * Creates a minimal Scene Uniform buffer for the given Scene, allocated using the FRDGBuilder (making it safe to keep reference).
	 */
	RENDERER_API FSceneUniformBuffer &CreateSceneUniformBuffer(FRDGBuilder& GraphBuilder, const FSceneInterface* Scene);
	/**
	 * Get the RDG uniform buffer from the FSceneUniformBuffer (not exposed in public API).
	 */
	RENDERER_API TRDGUniformBufferRef<FSceneUniformParameters> GetSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer &SceneUniformBuffer);

#if RHI_RAYTRACING
	namespace RayTracing
	{
		RENDERER_API bool HasRayTracingScene(const FSceneInterface& Scene);
		RENDERER_API FRHIRayTracingScene* GetRayTracingScene(const FSceneInterface& Scene, const FSceneView& View);
		RENDERER_API FRHIShaderResourceView* GetRayTracingSceneView(FRHICommandListBase& RHICmdList, const FSceneInterface& Scene, const FSceneView& View);
		RENDERER_API FRDGBufferSRVRef GetRayTracingSceneViewRDG(const FSceneInterface& Scene, const FSceneView& View);

		RENDERER_API FRHIShaderResourceView* GetInlineRayTracingBindingDataBuffer(const FSceneInterface& Scene);

		UE_DEPRECATED(5.7, "Please provide Scene reference instead.")
		inline bool HasRayTracingScene(const FSceneInterface* Scene)
		{
			return HasRayTracingScene(*Scene);
		}

		UE_DEPRECATED(5.7, "Please provide Scene and View reference.")
		inline FRHIRayTracingScene* GetRayTracingScene(const FSceneInterface* Scene)
		{
			return nullptr;
		}

		UE_DEPRECATED(5.7, "Please provide Scene and View reference.")
		inline FRHIShaderResourceView* GetRayTracingSceneView(FRHICommandListBase& RHICmdList, const FSceneInterface* Scene)
		{
			return nullptr;
		}

		RENDERER_API FShaderBindingTableRHIRef CreateShaderBindingTable(FRHICommandListBase& RHICmdList, const FSceneInterface* Scene,  uint32 LocalBindingDataSize);
		RENDERER_API TConstArrayView<FRayTracingShaderBindingData> GetVisibleRayTracingShaderBindings(const FSceneView& View);
		RENDERER_API const FRayTracingMeshCommandStorage* GetRayTracingMeshCommands(const FSceneInterface* Scene);
	}
#endif
}
