// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameters.h"
#include "RenderGraphFwd.h"
#include "SceneManagement.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "MeshBatch.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

class FSceneView;
class FRDGBuilder;
class FMeshElementCollector;

DECLARE_UNIFORM_BUFFER_STRUCT(FGPUSceneWriterUniformParameters, RENDERER_API)

/**
 * Note: this should not be in a public header, the above UB exists for this reason but doesn't work on DX11 RHI.
 */

#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND

// Parameter sub struct that is common to all parameter use-cases
BEGIN_SHADER_PARAMETER_STRUCT(FGPUSceneCommonParameters, RENDERER_API)
	SHADER_PARAMETER(uint32, GPUSceneInstanceDataTileSizeLog2)
	SHADER_PARAMETER(uint32, GPUSceneInstanceDataTileSizeMask)
	SHADER_PARAMETER(uint32, GPUSceneInstanceDataTileStride)
	SHADER_PARAMETER(uint32, GPUSceneFrameNumber)
	SHADER_PARAMETER(int32, GPUSceneMaxAllocatedInstanceId)
	SHADER_PARAMETER(int32, GPUSceneMaxPersistentPrimitiveIndex)
	SHADER_PARAMETER(int32, GPUSceneNumLightmapDataItems)
END_SHADER_PARAMETER_STRUCT()

#endif

/**
 * Deprecated for 5.6!
 * Use SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FGPUSceneWriterUniformParameters, GPUSceneWriterUB) instead.
 */
BEGIN_SHADER_PARAMETER_STRUCT(FGPUSceneWriterParameters, RENDERER_API)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, GPUSceneInstanceSceneDataRW)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, GPUSceneInstancePayloadDataRW)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, GPUScenePrimitiveSceneDataRW)
	SHADER_PARAMETER(uint32, GPUSceneInstanceSceneDataSOAStride)
	SHADER_PARAMETER(uint32, GPUSceneNumAllocatedInstances)
	SHADER_PARAMETER(uint32, GPUSceneNumAllocatedPrimitives)

	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FGPUSceneWriterUniformParameters, GPUSceneWriterUB)
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
	SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneCommonParameters, CommonParameters)
#else
	// This is retained purely for backwards API compatibility
	SHADER_PARAMETER(uint32, GPUSceneFrameNumber)
#endif

END_SHADER_PARAMETER_STRUCT()

/** The parameters passed to the GPUScene writer delegate */
struct FGPUSceneWriteDelegateParams
{
	/** The ID of the primitive that writes must be limited to. */
	uint32 PersistentPrimitiveId = (uint32)INDEX_NONE;
	/** The ID of the first instance scene data of the primitive */
	uint32 InstanceSceneDataOffset = (uint32)INDEX_NONE;
	/** Number of custom data floats in the instance payload data. */
	uint32 NumCustomDataFloats = (uint32)INDEX_NONE;
	/** Packed instance scene data flags suitable for writing to instance scene data. */
	uint32 PackedInstanceSceneDataFlags = 0u;
	/** The GPU Scene write pass that is currently executing. (NOTE: A value of None specifies that it is occurring on upload) */
	EGPUSceneGPUWritePass GPUWritePass = EGPUSceneGPUWritePass::None;
	/** The view for which this primitive belongs (for dynamic primitives) */
	FSceneView* View = nullptr;
	/** The shader parameters the delegate can use to perform writes on GPU Scene data */
	UE_DEPRECATED(5.6, "Only the opaque UB (see GPUSceneWriterUB below) should be referenced")
	FGPUSceneWriterParameters GPUWriteParams;
	/** 
	 * Include a reference to the UB in the compute shader parameter struct, for example:
	 *   SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FGPUSceneWriterUniformParameters, GPUSceneWriterUB)
	 */
	TRDGUniformBufferRef<FGPUSceneWriterUniformParameters> GPUSceneWriterUB;
};

DECLARE_DELEGATE_TwoParams(FGPUSceneWriteDelegateImpl, FRDGBuilder&, const FGPUSceneWriteDelegateParams&);

/** 
 * Special implementation of FGPUSceneWriteDelegate to keep API similar, but allow abstracting the internals for the mesh batch path.
 * Used by GPUScene to stage writing to the GPUScene primitive and instance data buffers via the GPU.
 */
class FGPUSceneWriteDelegate : public FOneFrameResource
{
public:
	FGPUSceneWriteDelegate() = default;
	FGPUSceneWriteDelegate(FGPUSceneWriteDelegate&&) = default;
	FGPUSceneWriteDelegate &operator=(FGPUSceneWriteDelegate&&) = default;
	RENDERER_API FGPUSceneWriteDelegate(FGPUSceneWriteDelegateImpl&& GPUSceneWriteDelegateImpl);

	/**
	 * This version returns a FGPUSceneWriteDelegateRef and this must be used to associate a FGPUSceneWriteDelegate with a mesh batch.
	 */
	template <typename LambdaType>
	static FGPUSceneWriteDelegateRef CreateLambda(FMeshElementCollector* MeshElementCollector, LambdaType&& Lambda)
	{
		return CreateInternal(MeshElementCollector, FGPUSceneWriteDelegateImpl::CreateLambda(MoveTemp(Lambda)));
	}

	/**
	 */
	template <typename LambdaType>
	static FGPUSceneWriteDelegate CreateLambda(LambdaType&& Lambda)
	{
		return FGPUSceneWriteDelegate(FGPUSceneWriteDelegateImpl::CreateLambda(MoveTemp(Lambda)));
	}

	/**
	 */
	RENDERER_API void Execute(FRDGBuilder& GraphBuilder, const FGPUSceneWriteDelegateParams& Params) const;

	/**
	 */
	bool IsBound() const { return Delegate.IsBound(); }

private:
	RENDERER_API static FGPUSceneWriteDelegateRef CreateInternal(FMeshElementCollector* MeshElementCollector, FGPUSceneWriteDelegateImpl&& DelegateImpl);

	FGPUSceneWriteDelegateImpl Delegate;
};
