// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace GlobalDistanceField
{
	/** Must match global distance field shaders. */
	inline const int32 MaxClipmaps = 6;
}
