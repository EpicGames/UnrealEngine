// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GlobalDistanceFieldConstants.h"
#include "GlobalRenderResources.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameters.h"
#include "ShaderParameterUtils.h"
#include "RHIStaticStates.h"
#include "RenderGraphResources.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "RenderUtils.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

class FShaderParameterMap;

class FGlobalDistanceFieldParameterData
{
public:
	FGlobalDistanceFieldParameterData()
	{
		FPlatformMemory::Memzero(this, sizeof(FGlobalDistanceFieldParameterData));
	}

	FVector4f TranslatedCenterAndExtent[GlobalDistanceField::MaxClipmaps];
	FVector4f TranslatedWorldToUVAddAndMul[GlobalDistanceField::MaxClipmaps];
	FVector4f MipTranslatedWorldToUVScale[GlobalDistanceField::MaxClipmaps];
	FVector4f MipTranslatedWorldToUVBias[GlobalDistanceField::MaxClipmaps];
	float MipFactor;
	float MipTransition;
	FRHITexture* PageAtlasTexture;
	FRHITexture* CoverageAtlasTexture;
	FRDGPooledBuffer* PageObjectGridBuffer;
	FRHITexture* PageTableTexture;
	FRHITexture* MipTexture;
	int32 ClipmapSizeInPages;
	FVector InvPageAtlasSize;
	FVector InvCoverageAtlasSize;
	int32 MaxPageNum;
	float GlobalDFResolution;
	float MaxDFAOConeDistance;
	int32 NumGlobalSDFClipmaps;
};

BEGIN_SHADER_PARAMETER_STRUCT(FGlobalDistanceFieldParameters2, )
	SHADER_PARAMETER_TEXTURE(Texture3D, GlobalDistanceFieldPageAtlasTexture)
	SHADER_PARAMETER_TEXTURE(Texture3D, GlobalDistanceFieldCoverageAtlasTexture)
	SHADER_PARAMETER_TEXTURE(Texture3D<uint>, GlobalDistanceFieldPageTableTexture)
	SHADER_PARAMETER_TEXTURE(Texture3D, GlobalDistanceFieldMipTexture)
	SHADER_PARAMETER_ARRAY(FVector4f, GlobalVolumeTranslatedCenterAndExtent, [GlobalDistanceField::MaxClipmaps])
	SHADER_PARAMETER_ARRAY(FVector4f, GlobalVolumeTranslatedWorldToUVAddAndMul, [GlobalDistanceField::MaxClipmaps])
	SHADER_PARAMETER_ARRAY(FVector4f, GlobalDistanceFieldMipTranslatedWorldToUVScale, [GlobalDistanceField::MaxClipmaps])
	SHADER_PARAMETER_ARRAY(FVector4f, GlobalDistanceFieldMipTranslatedWorldToUVBias, [GlobalDistanceField::MaxClipmaps])
	SHADER_PARAMETER(float, GlobalDistanceFieldMipFactor)
	SHADER_PARAMETER(float, GlobalDistanceFieldMipTransition)
	SHADER_PARAMETER(int32, GlobalDistanceFieldClipmapSizeInPages)
	SHADER_PARAMETER(FVector3f, GlobalDistanceFieldInvPageAtlasSize)
	SHADER_PARAMETER(FVector3f, GlobalDistanceFieldInvCoverageAtlasSize)
	SHADER_PARAMETER(float, GlobalVolumeDimension)
	SHADER_PARAMETER(float, GlobalVolumeTexelSize)
	SHADER_PARAMETER(float, MaxGlobalDFAOConeDistance)
	SHADER_PARAMETER(uint32, NumGlobalSDFClipmaps)
	SHADER_PARAMETER(float, CoveredExpandSurfaceScale)
	SHADER_PARAMETER(float, NotCoveredExpandSurfaceScale)
	SHADER_PARAMETER(float, NotCoveredMinStepScale)
	SHADER_PARAMETER(float, DitheredTransparencyStepThreshold)
	SHADER_PARAMETER(float, DitheredTransparencyTraceThreshold)
	SHADER_PARAMETER_SAMPLER(SamplerState, GlobalDistanceFieldCoverageAtlasTextureSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, GlobalDistanceFieldPageAtlasTextureSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, GlobalDistanceFieldMipTextureSampler)
END_SHADER_PARAMETER_STRUCT()

FGlobalDistanceFieldParameters2 SetupGlobalDistanceFieldParameters(const FGlobalDistanceFieldParameterData& ParameterData);

inline FGlobalDistanceFieldParameters2 SetupGlobalDistanceFieldParameters_Minimal(const FGlobalDistanceFieldParameterData& ParameterData)
{
	FGlobalDistanceFieldParameters2 ShaderParameters{};

	ShaderParameters.GlobalDistanceFieldPageAtlasTexture = ParameterData.PageAtlasTexture ? ParameterData.PageAtlasTexture : GBlackVolumeTexture->TextureRHI.GetReference();
	ShaderParameters.GlobalDistanceFieldPageTableTexture = ParameterData.PageTableTexture ? ParameterData.PageTableTexture : GBlackUintVolumeTexture->TextureRHI.GetReference();
	ShaderParameters.GlobalDistanceFieldMipTexture = ParameterData.MipTexture ? ParameterData.MipTexture : GBlackVolumeTexture->TextureRHI.GetReference();

	for (int32 Index = 0; Index < GlobalDistanceField::MaxClipmaps; Index++)
	{
		ShaderParameters.GlobalVolumeTranslatedCenterAndExtent[Index] = ParameterData.TranslatedCenterAndExtent[Index];
		ShaderParameters.GlobalVolumeTranslatedWorldToUVAddAndMul[Index] = ParameterData.TranslatedWorldToUVAddAndMul[Index];
		ShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVScale[Index] = ParameterData.MipTranslatedWorldToUVScale[Index];
		ShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVBias[Index] = ParameterData.MipTranslatedWorldToUVBias[Index];
	}

	ShaderParameters.GlobalDistanceFieldMipFactor = ParameterData.MipFactor;
	ShaderParameters.GlobalDistanceFieldMipTransition = ParameterData.MipTransition;
	ShaderParameters.GlobalDistanceFieldClipmapSizeInPages = ParameterData.ClipmapSizeInPages;
	ShaderParameters.GlobalDistanceFieldInvPageAtlasSize = (FVector3f)ParameterData.InvPageAtlasSize;
	ShaderParameters.GlobalDistanceFieldInvCoverageAtlasSize = (FVector3f)ParameterData.InvCoverageAtlasSize;
	ShaderParameters.GlobalVolumeDimension = ParameterData.GlobalDFResolution;
	ShaderParameters.GlobalVolumeTexelSize = 1.0f / ParameterData.GlobalDFResolution;
	ShaderParameters.MaxGlobalDFAOConeDistance = ParameterData.MaxDFAOConeDistance;
	ShaderParameters.NumGlobalSDFClipmaps = ParameterData.NumGlobalSDFClipmaps;

	return ShaderParameters;
}

class UE_DEPRECATED(5.2, "FGlobalDistanceFieldParameters2 should be used from now on.") FGlobalDistanceFieldParameters
{
	DECLARE_INLINE_TYPE_LAYOUT(FGlobalDistanceFieldParameters, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		GlobalDistanceFieldPageAtlasTexture.Bind(ParameterMap, TEXT("GlobalDistanceFieldPageAtlasTexture"));
		GlobalDistanceFieldPageTableTexture.Bind(ParameterMap, TEXT("GlobalDistanceFieldPageTableTexture"));
		GlobalDistanceFieldMipTexture.Bind(ParameterMap, TEXT("GlobalDistanceFieldMipTexture"));
		GlobalDistanceFieldMipFactor.Bind(ParameterMap, TEXT("GlobalDistanceFieldMipFactor"));
		GlobalDistanceFieldMipTransition.Bind(ParameterMap, TEXT("GlobalDistanceFieldMipTransition"));
		GlobalVolumeTranslatedCenterAndExtent.Bind(ParameterMap, TEXT("GlobalVolumeTranslatedCenterAndExtent"));
		GlobalVolumeTranslatedWorldToUVAddAndMul.Bind(ParameterMap, TEXT("GlobalVolumeTranslatedWorldToUVAddAndMul"));
		GlobalDistanceFieldMipTranslatedWorldToUVScale.Bind(ParameterMap, TEXT("GlobalDistanceFieldMipTranslatedWorldToUVScale"));
		GlobalDistanceFieldMipTranslatedWorldToUVBias.Bind(ParameterMap, TEXT("GlobalDistanceFieldMipTranslatedWorldToUVBias"));
		GlobalDistanceFieldClipmapSizeInPages.Bind(ParameterMap, TEXT("GlobalDistanceFieldClipmapSizeInPages"));
		GlobalDistanceFieldInvPageAtlasSize.Bind(ParameterMap, TEXT("GlobalDistanceFieldInvPageAtlasSize"));
		GlobalVolumeDimension.Bind(ParameterMap,TEXT("GlobalVolumeDimension"));
		GlobalVolumeTexelSize.Bind(ParameterMap,TEXT("GlobalVolumeTexelSize"));
		MaxGlobalDFAOConeDistance.Bind(ParameterMap,TEXT("MaxGlobalDFAOConeDistance"));
		NumGlobalSDFClipmaps.Bind(ParameterMap,TEXT("NumGlobalSDFClipmaps"));
	}

	bool IsBound() const
	{
		return GlobalVolumeTranslatedCenterAndExtent.IsBound() || GlobalVolumeTranslatedWorldToUVAddAndMul.IsBound();
	}

	friend FArchive& operator<<(FArchive& Ar,FGlobalDistanceFieldParameters& Parameters)
	{
		Ar << Parameters.GlobalDistanceFieldPageAtlasTexture;
		Ar << Parameters.GlobalDistanceFieldPageTableTexture;
		Ar << Parameters.GlobalDistanceFieldMipTexture;
		Ar << Parameters.GlobalVolumeTranslatedCenterAndExtent;
		Ar << Parameters.GlobalVolumeTranslatedWorldToUVAddAndMul;
		Ar << Parameters.GlobalDistanceFieldMipTranslatedWorldToUVScale;
		Ar << Parameters.GlobalDistanceFieldMipTranslatedWorldToUVBias;
		Ar << Parameters.GlobalDistanceFieldMipFactor;
		Ar << Parameters.GlobalDistanceFieldMipTransition;
		Ar << Parameters.GlobalDistanceFieldClipmapSizeInPages;
		Ar << Parameters.GlobalDistanceFieldInvPageAtlasSize;
		Ar << Parameters.GlobalVolumeDimension;
		Ar << Parameters.GlobalVolumeTexelSize;
		Ar << Parameters.MaxGlobalDFAOConeDistance;
		Ar << Parameters.NumGlobalSDFClipmaps;
		return Ar;
	}

	template<typename ShaderRHIParamRef>
	inline void Set(FRHICommandList& RHICmdList, const ShaderRHIParamRef ShaderRHI, const FGlobalDistanceFieldParameterData& ParameterData) const
	{
		if (IsBound())
		{
			FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

			SetTextureParameter(BatchedParameters, GlobalDistanceFieldPageAtlasTexture, ParameterData.PageAtlasTexture ? ParameterData.PageAtlasTexture : GBlackVolumeTexture->TextureRHI.GetReference());
			SetTextureParameter(BatchedParameters, GlobalDistanceFieldPageTableTexture, ParameterData.PageTableTexture ? ParameterData.PageTableTexture : GBlackUintVolumeTexture->TextureRHI.GetReference());
			SetTextureParameter(BatchedParameters, GlobalDistanceFieldMipTexture, ParameterData.MipTexture ? ParameterData.MipTexture : GBlackVolumeTexture->TextureRHI.GetReference());

			SetShaderValueArray(BatchedParameters, GlobalVolumeTranslatedCenterAndExtent, ParameterData.TranslatedCenterAndExtent, GlobalDistanceField::MaxClipmaps);
			SetShaderValueArray(BatchedParameters, GlobalVolumeTranslatedWorldToUVAddAndMul, ParameterData.TranslatedWorldToUVAddAndMul, GlobalDistanceField::MaxClipmaps);
			SetShaderValueArray(BatchedParameters, GlobalDistanceFieldMipTranslatedWorldToUVScale, ParameterData.MipTranslatedWorldToUVScale, GlobalDistanceField::MaxClipmaps);
			SetShaderValueArray(BatchedParameters, GlobalDistanceFieldMipTranslatedWorldToUVBias, ParameterData.MipTranslatedWorldToUVBias, GlobalDistanceField::MaxClipmaps);
			SetShaderValue(BatchedParameters, GlobalDistanceFieldMipFactor, ParameterData.MipFactor);
			SetShaderValue(BatchedParameters, GlobalDistanceFieldMipTransition, ParameterData.MipTransition);
			SetShaderValue(BatchedParameters, GlobalDistanceFieldClipmapSizeInPages, ParameterData.ClipmapSizeInPages);
			SetShaderValue(BatchedParameters, GlobalDistanceFieldInvPageAtlasSize, (FVector3f)ParameterData.InvPageAtlasSize);
			SetShaderValue(BatchedParameters, GlobalVolumeDimension, ParameterData.GlobalDFResolution);
			SetShaderValue(BatchedParameters, GlobalVolumeTexelSize, 1.0f / ParameterData.GlobalDFResolution);
			SetShaderValue(BatchedParameters, MaxGlobalDFAOConeDistance, ParameterData.MaxDFAOConeDistance);
			SetShaderValue(BatchedParameters, NumGlobalSDFClipmaps, ParameterData.NumGlobalSDFClipmaps);

			RHICmdList.SetBatchedShaderParameters(ShaderRHI, BatchedParameters);
		}
	}

private:

	LAYOUT_FIELD(FShaderResourceParameter, GlobalDistanceFieldPageAtlasTexture)
	LAYOUT_FIELD(FShaderResourceParameter, GlobalDistanceFieldPageTableTexture)
	LAYOUT_FIELD(FShaderResourceParameter, GlobalDistanceFieldMipTexture)
	LAYOUT_FIELD(FShaderParameter, GlobalVolumeTranslatedCenterAndExtent)
	LAYOUT_FIELD(FShaderParameter, GlobalVolumeTranslatedWorldToUVAddAndMul)
	LAYOUT_FIELD(FShaderParameter, GlobalDistanceFieldMipTranslatedWorldToUVScale)
	LAYOUT_FIELD(FShaderParameter, GlobalDistanceFieldMipTranslatedWorldToUVBias)
	LAYOUT_FIELD(FShaderParameter, GlobalDistanceFieldMipFactor)
	LAYOUT_FIELD(FShaderParameter, GlobalDistanceFieldMipTransition)
	LAYOUT_FIELD(FShaderParameter, GlobalDistanceFieldClipmapSizeInPages)	
	LAYOUT_FIELD(FShaderParameter, GlobalDistanceFieldInvPageAtlasSize)
	LAYOUT_FIELD(FShaderParameter, GlobalVolumeDimension)
	LAYOUT_FIELD(FShaderParameter, GlobalVolumeTexelSize)
	LAYOUT_FIELD(FShaderParameter, MaxGlobalDFAOConeDistance)
	LAYOUT_FIELD(FShaderParameter, NumGlobalSDFClipmaps)
};
