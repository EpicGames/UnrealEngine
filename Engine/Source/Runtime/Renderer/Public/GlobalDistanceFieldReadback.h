// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "HAL/IConsoleManager.h"
#include "Math/Box.h"
#include "Math/Float16Color.h"
#include "Math/IntVector.h"

class FGlobalDistanceFieldReadback
{
public:
	DECLARE_DELEGATE(FCompleteDelegate);

	FBox Bounds;
	FIntVector Size;
	TArray<FFloat16Color> ReadbackData;
	FCompleteDelegate ReadbackComplete;
	ENamedThreads::Type CallbackThread = ENamedThreads::UnusedAnchor;
};

/**
 * Retrieves the GPU data of a global distance field clipmap for access by the CPU
 *
 * @note: Currently only works with the highest res clipmap on the first updated view in the frame
 **/
extern void RequestGlobalDistanceFieldReadback(FGlobalDistanceFieldReadback* Readback);

void RENDERER_API RequestGlobalDistanceFieldReadback_GameThread(FGlobalDistanceFieldReadback* Readback);
