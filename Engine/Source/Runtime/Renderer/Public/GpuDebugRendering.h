// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This file is a deprecated include file. 
// Forward to the new include file
#include "ShaderPrint.h"
