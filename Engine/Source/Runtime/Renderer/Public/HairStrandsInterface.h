// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairStrandsInterface.h: public interface for hair strands rendering.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Templates/RefCounting.h"
#include "RHIFwd.h"
#include "RenderGraphFwd.h"
#include "Shader.h"
#include "RenderGraphDefinitions.h"
#include "HairStrandsDefinitions.h"
#include "CachedGeometry.h"
#include "Serialization/BulkData.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "RendererInterface.h"
#include "Engine/EngineTypes.h"
#include "RenderResource.h"
#include "RenderGraphResources.h"
#include "ShaderPrintParameters.h"
#include "GroomVisualizationData.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8


class UTexture2D;
class FSceneInterface;
class FGlobalShaderMap;

class FViewInfo;
struct FShaderPrintData;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Shader parameters

// Instance attribute parameters
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstanceAttributeParameters, RENDERER_API)
	SHADER_PARAMETER(uint32, CurveAttributeIndexToChunkDivAsShift)
	SHADER_PARAMETER(uint32, CurveAttributeChunkElementCount)
	SHADER_PARAMETER(uint32, CurveAttributeChunkStrideInBytes)
	SHADER_PARAMETER(uint32, PointAttributeIndexToChunkDivAsShift)
	SHADER_PARAMETER(uint32, PointAttributeChunkElementCount)
	SHADER_PARAMETER(uint32, PointAttributeChunkStrideInBytes)
	SHADER_PARAMETER_ARRAY(FUintVector4, CurveAttributeOffsets, [HAIR_CURVE_ATTRIBUTE_OFFSET_COUNT])
	SHADER_PARAMETER_ARRAY(FUintVector4, PointAttributeOffsets, [HAIR_POINT_ATTRIBUTE_OFFSET_COUNT])
END_SHADER_PARAMETER_STRUCT()

// Instance common parameters
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstanceCommonParameters, RENDERER_API)
	SHADER_PARAMETER(float,  Density)
	SHADER_PARAMETER(float,  Radius)
	SHADER_PARAMETER(float,  RootScale)
	SHADER_PARAMETER(float,  TipScale)
	SHADER_PARAMETER(float,  Length)
	SHADER_PARAMETER(float,  LengthScale)
	SHADER_PARAMETER(float,  RaytracingRadiusScale)
	SHADER_PARAMETER(uint32, RegisteredIndex)
	SHADER_PARAMETER(uint32, GroupIndex)
	SHADER_PARAMETER(uint32, GroupCount)
	SHADER_PARAMETER(uint32, PointCount)
	SHADER_PARAMETER(uint32, CurveCount)
	SHADER_PARAMETER(uint32, RaytracingProceduralSplits)
	SHADER_PARAMETER(uint32, Flags)
	SHADER_PARAMETER(uint32, bSimulation)
	SHADER_PARAMETER(uint32, bSingleGuide)
	SHADER_PARAMETER(uint32, bPointRemapping)
	SHADER_PARAMETER(uint32, bCurveRemapping)
	SHADER_PARAMETER(uint32, SourcePointCount)
	SHADER_PARAMETER(uint32, SourceCurveCount)
	SHADER_PARAMETER(FVector3f, PositionOffset)
	SHADER_PARAMETER(FVector3f, PrevPositionOffset)
	SHADER_PARAMETER(FMatrix44f, LocalToWorldPrimitiveTransform)
	SHADER_PARAMETER(FMatrix44f, LocalToTranslatedWorldPrimitiveTransform)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceAttributeParameters, Attributes)
END_SHADER_PARAMETER_STRUCT()

// Instance resources (RDG)
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstanceResourceParameters, RENDERER_API)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PositionBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, PositionOffsetBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CurveBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointToCurveBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CurveAttributeBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointAttributeBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CurveMappingBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointMappingBuffer)
END_SHADER_PARAMETER_STRUCT()

// Instance prev. resources (RDG)
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstancePrevResourceParameters, RENDERER_API)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PrevPositionBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, PrevPositionOffsetBuffer)
END_SHADER_PARAMETER_STRUCT()

// Instance culling resources (RDG)
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstanceCullingParameters, RENDERER_API)
	SHADER_PARAMETER(uint32, bCullingEnable)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, CullingIndirectBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, CullingIndexBuffer)
	RDG_BUFFER_ACCESS(CullingIndirectBufferArgs, ERHIAccess::IndirectArgs)
END_SHADER_PARAMETER_STRUCT()

// Instance interpolation resources (RDG)
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstanceInterpolationParameters, RENDERER_API)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CurveInterpolationBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PointInterpolationBuffer)
END_SHADER_PARAMETER_STRUCT()

// Instance resources (Raw)
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstanceResourceRawParameters, RENDERER_API)
	SHADER_PARAMETER_SRV(ByteAddressBuffer, PositionBuffer)
	SHADER_PARAMETER_SRV(StructuredBuffer<float4>, PositionOffsetBuffer)
	SHADER_PARAMETER_SRV(ByteAddressBuffer, CurveBuffer)
	SHADER_PARAMETER_SRV(ByteAddressBuffer, PointToCurveBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, TangentBuffer)
	SHADER_PARAMETER_SRV(ByteAddressBuffer, CurveAttributeBuffer)
	SHADER_PARAMETER_SRV(ByteAddressBuffer, PointAttributeBuffer)
END_SHADER_PARAMETER_STRUCT()

// Instance prev. resources (Raw)
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstancePrevResourceRawParameters, RENDERER_API)
	SHADER_PARAMETER_SRV(ByteAddressBuffer, PreviousPositionBuffer)
	SHADER_PARAMETER_SRV(StructuredBuffer<float4>, PreviousPositionOffsetBuffer)
END_SHADER_PARAMETER_STRUCT()

// Instance culling resources (Raw)
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstanceCullingRawParameters, RENDERER_API)
	SHADER_PARAMETER(uint32, bCullingEnable)
//	SHADER_PARAMETER_SRV(Buffer<uint>, CullingIndirectBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint>, CullingIndexBuffer)
END_SHADER_PARAMETER_STRUCT()

// Intermediate struct which can be referenced by FHairStrandsInstanceParameters for getting 
// the HairStrandsVF_ decoration in shader
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstanceIntermediateParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceCommonParameters, Common)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceResourceParameters, Resources)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceCullingParameters, Culling)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsInstanceParameters, )
	SHADER_PARAMETER_STRUCT(FHairStrandsInstanceIntermediateParameters, HairStrandsVF)
END_SHADER_PARAMETER_STRUCT()

////////////////////////////////////////////////////////////////////////////////////////////////////
// Utils buffers for importing/exporting hair resources

enum class ERDGImportedBufferFlags
{
	None = 0,
	CreateSRV = 0x1,
	CreateUAV = 0x2,
	CreateViews = CreateSRV | CreateUAV
};
ENUM_CLASS_FLAGS(ERDGImportedBufferFlags);

struct FRDGExternalBuffer
{
	TRefCountPtr<FRDGPooledBuffer> Buffer = nullptr;
	FShaderResourceViewRHIRef SRV = nullptr;
	FUnorderedAccessViewRHIRef UAV = nullptr;
	EPixelFormat Format = PF_Unknown;
	RENDERER_API void Release();
};

struct FRDGImportedBuffer
{
	FRDGBufferRef Buffer = nullptr;
	FRDGBufferSRVRef SRV = nullptr;
	FRDGBufferUAVRef UAV = nullptr;
};

RENDERER_API FRDGImportedBuffer Register(FRDGBuilder& GraphBuilder, const FRDGExternalBuffer& In, ERDGImportedBufferFlags Flags, ERDGUnorderedAccessViewFlags UAVFlags = ERDGUnorderedAccessViewFlags::None);
RENDERER_API FRDGBufferSRVRef   RegisterAsSRV(FRDGBuilder& GraphBuilder, const FRDGExternalBuffer& In);
RENDERER_API FRDGBufferUAVRef   RegisterAsUAV(FRDGBuilder& GraphBuilder, const FRDGExternalBuffer& In, ERDGUnorderedAccessViewFlags Flags = ERDGUnorderedAccessViewFlags::None);

RENDERER_API void AddTransitionPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	EShaderPlatform InPlatform,
	const TArray<FRDGBufferSRVRef>& Transitions);

////////////////////////////////////////////////////////////////////////////////////////////////////
// Misc/Helpers

struct FHairStrandClusterData;
struct IPooledRenderTarget;
struct FRWBuffer;
class  FRDGPooledBuffer;
class  FHairGroupPublicData;
class  FRDGShaderResourceView;
class  FResourceArrayInterface;
class  FSceneView;

enum EHairGeometryType
{
	Strands,
	Cards,
	Meshes,
	NoneGeometry
};

enum class EHairStrandsRayTracingPrimitiveType : uint8
{
	Procedural,
	Triangle,
	LinearSweptSphere,
	MAX
};

enum EHairBindingType
{
	NoneBinding,
	Rigid,
	Skinning
};

enum EHairInterpolationType
{
	NoneSkinning,				// Use no skinning data (i.e. with no binding)
	RigidSkinning,				// Use skinning data & apply a rigid triangle deformation
	OffsetSkinning,				// Use skinning data & apply a offset deformation
	SmoothSkinning,				// Use skinning data & apply a smooth (rotation) offset
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Public group data 

struct FHairStrandsInstance : public FRefCountedObject
{
	virtual ~FHairStrandsInstance() = default;
	int32 RegisteredIndex = -1;
	virtual const FBoxSphereBounds& GetBounds() const = 0;
	virtual const FBoxSphereBounds& GetLocalBounds() const = 0;
	virtual const FHairGroupPublicData* GetHairData() const { return nullptr; }
	virtual const EHairGeometryType GetHairGeometry() const { return EHairGeometryType::NoneGeometry; }
};
typedef TArray<TRefCountPtr<FHairStrandsInstance>> FHairStrandsInstances;

class FHairGroupPublicData
{
public:
	RENDERER_API FHairGroupPublicData(uint32 InGroupIndex, const FName& OwnerName);
	
	uint32 GetGroupIndex() const { return GroupIndex; }

	FRDGExternalBuffer& GetDrawIndirectRasterComputeBuffer() { return Culling->DrawIndirectRasterComputeBuffer; }
	const FRDGExternalBuffer& GetDrawIndirectRasterComputeBuffer() const { return Culling->DrawIndirectRasterComputeBuffer; }
	FRDGExternalBuffer& GetDrawIndirectBuffer() { return Culling->DrawIndirectBuffer; }

	const FRDGExternalBuffer& GetCulledCurveBuffer() const { return Culling->CulledCurveBuffer; }
	const FRDGExternalBuffer& GetCulledVertexIdBuffer() const { return Culling->CulledVertexIdBuffer; }

	FRDGExternalBuffer& GetCulledCurveBuffer() { return Culling->CulledCurveBuffer; }
	FRDGExternalBuffer& GetCulledVertexIdBuffer() { return Culling->CulledVertexIdBuffer; }

	bool GetCullingResultAvailable() const { return Culling->bCullingResultAvailable; }
	void SetCullingResultAvailable(bool b) { Culling->bCullingResultAvailable = b; }

	void SupportVoxelization(bool InVoxelize) { bSupportVoxelization = InVoxelize; }
	bool DoesSupportVoxelization() const { return bSupportVoxelization; }

	void SetLODGeometryTypes(const TArray<EHairGeometryType>& InTypes) { LODGeometryTypes = InTypes; }
	const TArray<EHairGeometryType>& GetLODGeometryTypes() const { return LODGeometryTypes; }

	void SetLODVisibilities(const TArray<bool>& InLODVisibility) { LODVisibilities = InLODVisibility; }
	const TArray<bool>& GetLODVisibilities() const { return LODVisibilities; }

	bool IsVisible(int32 InLODIndex) const
	{
		return LODVisibilities.IsValidIndex(InLODIndex) ? LODVisibilities[InLODIndex] : false;
	}

	EHairGeometryType GetGeometryType(int32 InLODIndex) const
	{
		return LODGeometryTypes.IsValidIndex(InLODIndex) ? LODGeometryTypes[InLODIndex] : EHairGeometryType::NoneGeometry;
	}

	EHairBindingType GetBindingType(int32 InLODIndex) const
	{ 
		return BindingTypes.IsValidIndex(InLODIndex) ? BindingTypes[InLODIndex] :EHairBindingType::NoneBinding;
	}	

	bool IsSimulationEnable(int32 InLODIndex) const
	{
		return LODSimulations.IsValidIndex(InLODIndex) ? LODSimulations[InLODIndex] : false;
	}

	bool IsGlobalInterpolationEnable(int32 InLODIndex) const 
	{
		return LODGlobalInterpolations.IsValidIndex(InLODIndex) ? LODGlobalInterpolations[InLODIndex] : false;
	}

	void SetLODScreenSizes(const TArray<float>& ScreenSizes) { LODScreenSizes = ScreenSizes; }
	const TArray<float>& GetLODScreenSizes() const { return LODScreenSizes;  }

	void SetLODBias(float InLODBias) { LODBias = InLODBias; }
	float GetLODBias() const { return LODBias; }

	void SetLODIndex(float InLODIndex) { LODIndex = InLODIndex; }
	float GetLODIndex() const { return LODIndex; }
	int32 GetIntLODIndex() const { return FMath::Max(0, FMath::FloorToInt(LODIndex)); }

	void SetMeshLODIndex(float InMeshLODIndex) { MeshLODIndex = InMeshLODIndex; }
	float GetMeshLODIndex() const { return MeshLODIndex; }

	void SetLODVisibility(bool bVisible) { bLODVisibility = bVisible; }
	bool GetLODVisibility() const { return bLODVisibility; }

	// Return the number of active point/curve for strand geometry
	RENDERER_API uint32 GetActiveStrandsPointCount(bool bPrevious=false) const;
	RENDERER_API uint32 GetActiveStrandsCurveCount(bool bPrevious=false) const;
	RENDERER_API float  GetActiveStrandsCoverageScale() const;
	RENDERER_API float  GetActiveStrandsRadiusScale() const;

	struct FVertexFactoryInput 
	{
		struct FStrands
		{
			FRDGImportedBuffer PositionBuffer;
			FRDGImportedBuffer PrevPositionBuffer;
			FRDGImportedBuffer TangentBuffer;
			FRDGImportedBuffer CurveAttributeBuffer;
			FRDGImportedBuffer PointAttributeBuffer;
			FRDGImportedBuffer PointToCurveBuffer;
			FRDGImportedBuffer PositionOffsetBuffer;
			FRDGImportedBuffer PrevPositionOffsetBuffer;
			FRDGImportedBuffer CurveBuffer;
			FRDGImportedBuffer CurveMappingBuffer;
			FRDGImportedBuffer PointMappingBuffer;

			FRDGExternalBuffer PositionBufferExternal;
			FRDGExternalBuffer PrevPositionBufferExternal;
			FRDGExternalBuffer TangentBufferExternal;
			FRDGExternalBuffer CurveAttributeBufferExternal;
			FRDGExternalBuffer PointAttributeBufferExternal;
			FRDGExternalBuffer PointToCurveBufferExternal;
			FRDGExternalBuffer PositionOffsetBufferExternal;
			FRDGExternalBuffer PrevPositionOffsetBufferExternal;
			FRDGExternalBuffer CurveBufferExternal;
			FRDGExternalBuffer CurveMappingBufferExternal;
			FRDGExternalBuffer PointMappingBufferExternal;

			FShaderResourceViewRHIRef PositionBufferRHISRV				= nullptr;
			FShaderResourceViewRHIRef PrevPositionBufferRHISRV			= nullptr;
			FShaderResourceViewRHIRef TangentBufferRHISRV				= nullptr;
			FShaderResourceViewRHIRef CurveAttributeBufferRHISRV		= nullptr;
			FShaderResourceViewRHIRef PointAttributeBufferRHISRV		= nullptr;
			FShaderResourceViewRHIRef PointToCurveBufferRHISRV			= nullptr;
			FShaderResourceViewRHIRef PositionOffsetBufferRHISRV		= nullptr;
			FShaderResourceViewRHIRef PrevPositionOffsetBufferRHISRV	= nullptr;
			FShaderResourceViewRHIRef CurveBufferRHISRV					= nullptr;
			FShaderResourceViewRHIRef CurveMappingBufferRHISRV			= nullptr;
			FShaderResourceViewRHIRef PointMappingBufferRHISRV			= nullptr;

			FHairStrandsInstanceCommonParameters Common; 
		} Strands;

		struct FCards
		{

		} Cards;

		struct FMeshes
		{
			
		} Meshes;

		bool bHasLODSwitch = false;
		bool bHasLODSwitchBindingType = false;
		EHairGeometryType GeometryType = EHairGeometryType::NoneGeometry;
		EHairBindingType BindingType = EHairBindingType::NoneBinding;
		FTransform LocalToWorldTransform;
	};

	struct FCulling
	{
		/* Indirect draw buffer to draw everything or the result of the culling per pass */
		FRDGExternalBuffer DrawIndirectBuffer;
		FRDGExternalBuffer DrawIndirectRasterComputeBuffer;

		/* Culling & LODing results for a hair group */ // Better to be transient?
		FRDGExternalBuffer CulledCurveBuffer;
		FRDGExternalBuffer CulledVertexIdBuffer;
		bool bCullingResultAvailable = false;
	};

	// Current hook for retriving instance data. 
	// This needs to be refactor to merge FHairGroupPublicData & HairStrandsInstance
	FHairStrandsInstance* Instance = nullptr;
	FCulling* Culling = nullptr;

	FVertexFactoryInput VFInput;
	uint32 ClusterDataIndex = ~0; // #hair_todo: move this into instance data, or remove FHairStrandClusterData

	uint32 GroupIndex = 0;
	uint32 RestPointCount = 0;
	uint32 RestCurveCount = 0;
	uint32 ClusterCount = 0;
	float ClusterScale = 0;

	bool bSupportVoxelization = true;

	/* CPU LOD selection. Hair LOD selection can be done by CPU or GPU. If bUseCPULODSelection is true, 
	   CPU LOD selection is enabled otherwise the GPU selection is used. CPU LOD selection use the CPU 
	   bounding box, which might not be as accurate as the GPU ones*/
	TArray<bool> LODVisibilities;
	TArray<float>LODScreenSizes;
	TArray<bool> LODSimulations;
	TArray<bool> LODGlobalInterpolations;
	TArray<EHairGeometryType> LODGeometryTypes;
	bool bIsDeformationEnable = false;
	bool bIsSimulationCacheEnable = false;

	TArray<EHairBindingType> BindingTypes;

	// The minimum mesh LOD that this data can support
	int32 BindingMinMeshLOD = -1;

	// Data change every frame by the groom proxy based on views data
	float MeshLODIndex = -1;
	float LODIndex = -1;		// Current LOD used for all views
	float LODBias = 0;			// Current LOD bias
	bool bLODVisibility = true; // Enable/disable hair rendering for this component
	bool bAutoLOD = false;
	float AutoLODBias = 0;

	// Active/used point/curved based on select continuous LOD
	uint32 ContinuousLODPointCount = 0;
	uint32 ContinuousLODCurveCount = 0;
	uint32 ContinuousLODPreviousPointCount = 0;
	uint32 ContinuousLODPreviousCurveCount = 0;
	float ContinuousLODScreenSize = 1.f;
	float ContinuousLODCoverageScale = 1.f;
	float ContinuousLODRadiusScale = 1.f;
	FVector2f ContinuousLODScreenPos = FVector2f(0,0);
	FBoxSphereBounds ContinuousLODBounds; 	//used by Continuous LOD

	// Debug
	bool  bDebugDrawLODInfo = false; // Enable/disable hair LOD info
	float DebugScreenSize = 0.f;
	FLinearColor DebugGroupColor;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Transient resources 
// Resources are indexed by hair instance's registered index
struct FHairTransientResources
{
	inline uint32 GetClusterOffset(uint32 InRegisterIndex) const { return ClusterAABBOffetAndCounts.IsValidIndex(InRegisterIndex) ? ClusterAABBOffetAndCounts[InRegisterIndex].X : 0u; }
	inline uint32 GetClusterCount(uint32 InRegisterIndex)  const { return ClusterAABBOffetAndCounts.IsValidIndex(InRegisterIndex) ? ClusterAABBOffetAndCounts[InRegisterIndex].Y : 0u; }
	inline bool IsClusterAABBValid(uint32 InRegisterIndex) const { return GetClusterCount(InRegisterIndex) > 0; }

	// Group AABB
	TBitArray<> bIsGroupAABBValid;
	FRDGBufferRef    GroupAABBBuffer = nullptr;
	FRDGBufferSRVRef GroupAABBSRV = nullptr;
	FVector3d TranslatedWorldOffsetView0 = FVector3d::ZeroVector; // View[0]'s Translated World Offset, as all ClusterAABB & GroupAABB are expressed relative to this
	FVector3f GetTranslatedWorldOffsetCorrection(const FViewInfo& View) const;

	// Cluster AABB
	TArray<FUintVector2> ClusterAABBOffetAndCounts;
	FRDGBufferRef    ClusterAABBBuffer = nullptr;
	FRDGBufferSRVRef ClusterAABBSRV = nullptr;

	// Indirect dispatch args
	FRDGBufferRef    IndirectDispatchArgsBuffer = nullptr;
	FRDGBufferSRVRef IndirectDispatchArgsSRV = nullptr;

	// Mesh data
	TArray<FCachedGeometry> SimMeshDatas;
	TArray<FCachedGeometry> RenMeshDatas;
	const FCachedGeometry& GetMeshLODData(uint32 InRegisteredIndex, bool bSim) const { if (bSim) { check(SimMeshDatas.IsValidIndex(InRegisteredIndex)); return SimMeshDatas[InRegisteredIndex]; } else {check(RenMeshDatas.IsValidIndex(InRegisteredIndex)); return RenMeshDatas[InRegisteredIndex];} }
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FHairInstanceCullingResults

enum class EHairInstanceVisibilityType : uint8
{
	NotVisible = 0,
	StrandsPrimaryView = 1,
	StrandsShadowView = 2,
	CardsOrMeshesPrimaryView = 3,
	CardsOrMeshesShadowView = 4,
	Count
};

struct FHairInstanceCullingResults
{
	// List of instances visible in any views. 
	// This is used to know what instance needs to run interpolation
	FHairStrandsInstances VisibleInAnyViews_Strands; // Primary & Shadow
	FHairStrandsInstances VisibleInAnyViews_CardsOrMeshes_Primary;
	FHairStrandsInstances VisibleInAnyViews_CardsOrMeshes_Shadow;

	// Visibility per instance per view 
	// Store up to 8 view culling result per uint
	TArray<uint32> InstancesVisibilityType;

	void InitVisibility(uint32 InstanceCount);
	void Add(const FViewInfo& View, const FHairStrandsInstance& Instance, EHairInstanceVisibilityType Flag);
	bool IsVisibleInAnyViews(const FHairStrandsInstance& Instance) const;
	bool Is(const FViewInfo& View, const FHairStrandsInstance& Instance, EHairInstanceVisibilityType Flag) const;
	EHairInstanceVisibilityType Is(const FViewInfo& View, const FHairStrandsInstance& Instance) const;
	RENDERER_API EHairInstanceVisibilityType Is(const FSceneView& View, const FHairStrandsInstance& Instance) const;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// Helpers
struct FPointPerCurveDispatchInfo
{
	// From the asset
	uint32 SourceCurveCount = 0;
	uint32 SourcePoinPerCurve = 0;

	// Derived
	uint32 CurvePerGroup = 0;
	uint32 PointPerCurve = 0;
	uint32 GroupSize = 0;
	FIntVector DispatchCount = FIntVector::ZeroValue;
};

// Compute the dispatch information for pass dispatching work per curve
RENDERER_API FPointPerCurveDispatchInfo GetPointPerCurveDispatchInfo(uint32 InAssetMaxPointPerCurve, uint32 InAssetCurveCount, uint32 InGroupSize);

////////////////////////////////////////////////////////////////////////////////////////////////////
// API for enabling/disabling the various geometry representation
enum class EHairStrandsShaderType
{
	Strands,
	Cards,
	Meshes,
	Tool,
	All
};

RENDERER_API bool IsHairStrandsSupported(EHairStrandsShaderType Type, EShaderPlatform Platform);
RENDERER_API bool IsHairStrandsEnabled(EHairStrandsShaderType Type, EShaderPlatform Platform = EShaderPlatform::SP_NumPlatforms);

RENDERER_API bool IsHairRayTracingEnabled();

// Return true if hair simulation is enabled.
RENDERER_API bool IsHairStrandsSimulationEnable();

// Return true if hair binding is enabled (i.e., hair can be attached to skeletal mesh)
RENDERER_API bool IsHairStrandsBindingEnable();

// Return true if strand reordering for compute raster continuous LOD is enabled
RENDERER_API bool IsHairStrandContinuousDecimationReorderingEnabled();

// Return true if continuous LOD is enabled - implies computer raster and continuous decimation reordering is true
RENDERER_API bool IsHairVisibilityComputeRasterContinuousLODEnabled();

// Return true if Linear Swept Spheres enabled for hair strands
RENDERER_API bool IsHairStrandsLinearSweptSpheresEnabled();

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

typedef TArray<FRHIUnorderedAccessView*> FBufferTransitionQueue;
RENDERER_API void TransitBufferToReadable(FRDGBuilder& GraphBuilder, FBufferTransitionQueue& BuffersToTransit);

/// Return the hair coverage for a certain hair count and normalized avg hair radius (i.e, [0..1])
RENDERER_API float GetHairCoverage(uint32 HairCount, float AverageHairRadius);

/// Return the average hair normalized radius for a given hair count and a given coverage value
RENDERER_API float GetHairAvgRadius(uint32 InCount, float InCoverage);


//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// HairStrands Bookmark API
enum class EHairStrandsBookmark : uint8
{
	ProcessTasks,
	ProcessLODSelection,
	ProcessBindingSurfaceUpdate,
	ProcessGuideInterpolation,
	ProcessCardsAndMeshesInterpolation_PrimaryView,
	ProcessCardsAndMeshesInterpolation_ShadowView,
	ProcessStrandsInterpolation,
	ProcessDebug,
	ProcessEndOfFrame,
	ProcessGuideDeformation
};

struct FHairStrandsBookmarkParameters
{
	FShaderPrintData* ShaderPrintData = nullptr;
	class FGlobalShaderMap* ShaderMap = nullptr;

	uint32 ViewUniqueID = ~0; // View 0
	FIntRect ViewRect; // View 0

	FHairInstanceCullingResults CullingResults;
	FHairTransientResources* TransientResources = nullptr;

	FHairStrandsInstances* Instances = nullptr;
	const FSceneView* View = nullptr;// // View 0
	FSceneInterface* Scene = nullptr;
	TArray<const FSceneView*> AllViews;
	FRDGTextureRef SceneColorTexture = nullptr;
	FRDGTextureRef SceneDepthTexture = nullptr; 

	inline bool HasInstances() const { return Instances != nullptr && Instances->Num() > 0; }
};

typedef void (*THairStrandsBookmarkFunction)(FRDGBuilder* GraphBuilder, EHairStrandsBookmark Bookmark, FHairStrandsBookmarkParameters& Parameters);
RENDERER_API void RegisterBookmarkFunction(THairStrandsBookmarkFunction Bookmark);

////////////////////////////////////////////////////////////////////////////////////////////////////
// Render Curve (Experimental)

struct FRenderCurveResourceData
{
	struct FHeader
	{
		FGuid Id;
		uint32 ClusterCount = 0;
		uint32 MaxClusterStrideInBytes = 0;
	} Header;

	struct FData
	{
		FByteBulkData BulkData;
	} Data;

	void Serialize(FArchive& Ar, UObject* Owner)
	{
		Ar << Header.Id;
		Ar << Header.ClusterCount;
		Ar << Header.MaxClusterStrideInBytes;
		Data.BulkData.Serialize(Ar, Owner, 0/*ChunkIndex*/, false /*bAttemptFileMapping*/);
	}
};

RENDERER_API bool IsRenderCurveEnabled();