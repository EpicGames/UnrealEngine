// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "HAL/Platform.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "Serialization/MemoryLayout.h"
#include "Shader.h"
#include "ShaderCore.h"
#include "ShaderParameterUtils.h"
#include "ShaderParameters.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "StereoRenderUtils.h"

class FPointerTableBase;

class FHdrCustomResolveVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FHdrCustomResolveVS,Global);
public:
	FHdrCustomResolveVS() {}
	FHdrCustomResolveVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader( Initializer )
	{
	}
};

class FHdrCustomResolveArrayVS : public FHdrCustomResolveVS
{
	DECLARE_SHADER_TYPE(FHdrCustomResolveArrayVS, Global);
public:
	FHdrCustomResolveArrayVS() {}
	FHdrCustomResolveArrayVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FHdrCustomResolveVS(Initializer)
	{
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (FHdrCustomResolveVS::ShouldCompilePermutation(Parameters))
		{
			UE::StereoRenderUtils::FStereoShaderAspects Aspects(Parameters.Platform);
			return Aspects.IsMobileMultiViewEnabled();
		}
		return false;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FHdrCustomResolveVS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_TEXTUREARRAY"), 1);
	}
};

// --- regular shaders ---

class FHdrCustomResolve2xPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FHdrCustomResolve2xPS,Global);
public:
	FHdrCustomResolve2xPS() {}
	FHdrCustomResolve2xPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader( Initializer )
	{
		Tex.Bind(Initializer.ParameterMap, TEXT("Tex"), SPF_Mandatory);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHITexture* Texture2DMS)
	{
		SetTextureParameter(BatchedParameters, Tex, Texture2DMS);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_2X"), 1);
	}

protected:
	LAYOUT_FIELD(FShaderResourceParameter, Tex);
};

class FHdrCustomResolve4xPS : public FHdrCustomResolve2xPS
{
	DECLARE_SHADER_TYPE(FHdrCustomResolve4xPS,Global);
public:
	FHdrCustomResolve4xPS() {}
	FHdrCustomResolve4xPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FHdrCustomResolve2xPS( Initializer )
	{
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// skip parent because it sets 2X macro
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_4X"), 1);
	}
};


class FHdrCustomResolve8xPS : public FHdrCustomResolve2xPS
{
	DECLARE_SHADER_TYPE(FHdrCustomResolve8xPS,Global);
public:
	FHdrCustomResolve8xPS() {}
	FHdrCustomResolve8xPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FHdrCustomResolve2xPS( Initializer )
	{
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// skip parent because it sets 2X macro
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_8X"), 1);
	}
};

// --- array shaders ---

class FHdrCustomResolveArray2xPS : public FHdrCustomResolve2xPS
{
	DECLARE_SHADER_TYPE(FHdrCustomResolveArray2xPS, Global);
public:
	FHdrCustomResolveArray2xPS() {}
	FHdrCustomResolveArray2xPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FHdrCustomResolve2xPS(Initializer)
	{
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (FHdrCustomResolve2xPS::ShouldCompilePermutation(Parameters))
		{
			UE::StereoRenderUtils::FStereoShaderAspects Aspects(Parameters.Platform);
			return Aspects.IsMobileMultiViewEnabled();
		}
		return false;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FHdrCustomResolve2xPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_TEXTUREARRAY"), 1);
	}
};

class FHdrCustomResolveArray4xPS : public FHdrCustomResolveArray2xPS
{
	DECLARE_SHADER_TYPE(FHdrCustomResolveArray4xPS, Global);
public:
	FHdrCustomResolveArray4xPS() {}
	FHdrCustomResolveArray4xPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FHdrCustomResolveArray2xPS(Initializer)
	{
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// skip parent because it sets 2X macro
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_4X"), 1);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_TEXTUREARRAY"), 1);
	}
};


class FHdrCustomResolveArray8xPS : public FHdrCustomResolveArray2xPS
{
	DECLARE_SHADER_TYPE(FHdrCustomResolveArray8xPS, Global);
public:
	FHdrCustomResolveArray8xPS() {}
	FHdrCustomResolveArray8xPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FHdrCustomResolveArray2xPS(Initializer)
	{
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// skip parent because it sets 2X macro
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_8X"), 1);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_TEXTUREARRAY"), 1);
	}
};

// --- FMask shaders ---

class FHdrCustomResolveFMask2xPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FHdrCustomResolveFMask2xPS, Global);
public:
	FHdrCustomResolveFMask2xPS() {}
	FHdrCustomResolveFMask2xPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		Tex.Bind(Initializer.ParameterMap, TEXT("Tex"), SPF_Mandatory);
		FMaskTex.Bind(Initializer.ParameterMap, TEXT("FMaskTex"), SPF_Optional);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHITexture* Texture2DMS, FRHIShaderResourceView* FMaskSRV)
	{
		SetTextureParameter(BatchedParameters, Tex, Texture2DMS);
		SetSRVParameter(BatchedParameters, FMaskTex, FMaskSRV);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HDR_RESOLVE_NUM_SAMPLES"), 2);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_USES_FMASK"), 1);
	}

protected:
	LAYOUT_FIELD(FShaderResourceParameter, Tex);
	LAYOUT_FIELD(FShaderResourceParameter, FMaskTex);
};

class FHdrCustomResolveFMask4xPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FHdrCustomResolveFMask4xPS, Global);
public:
	FHdrCustomResolveFMask4xPS() {}
	FHdrCustomResolveFMask4xPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		Tex.Bind(Initializer.ParameterMap, TEXT("Tex"), SPF_Mandatory);
		FMaskTex.Bind(Initializer.ParameterMap, TEXT("FMaskTex"), SPF_Optional);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHITexture* Texture2DMS, FRHIShaderResourceView* FMaskSRV)
	{
		SetTextureParameter(BatchedParameters, Tex, Texture2DMS);
		SetSRVParameter(BatchedParameters, FMaskTex, FMaskSRV);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HDR_RESOLVE_NUM_SAMPLES"), 4);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_USES_FMASK"), 1);
	}

protected:
	LAYOUT_FIELD(FShaderResourceParameter, Tex);
	LAYOUT_FIELD(FShaderResourceParameter, FMaskTex);
};


class FHdrCustomResolveFMask8xPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FHdrCustomResolveFMask8xPS, Global);
public:
	FHdrCustomResolveFMask8xPS() {}
	FHdrCustomResolveFMask8xPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		Tex.Bind(Initializer.ParameterMap, TEXT("Tex"), SPF_Mandatory);
		FMaskTex.Bind(Initializer.ParameterMap, TEXT("FMaskTex"), SPF_Optional);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHITexture* Texture2DMS, FRHIShaderResourceView* FMaskSRV)
	{
		SetTextureParameter(BatchedParameters, Tex, Texture2DMS);
		SetSRVParameter(BatchedParameters, FMaskTex, FMaskSRV);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HDR_RESOLVE_NUM_SAMPLES"), 8);
		OutEnvironment.SetDefine(TEXT("HDR_CUSTOM_RESOLVE_USES_FMASK"), 1);
	}

protected:
	LAYOUT_FIELD(FShaderResourceParameter, Tex);
	LAYOUT_FIELD(FShaderResourceParameter, FMaskTex);
};
