// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPassProcessor.h"
#include "RHI.h"
#include "RenderGraphResources.h"

struct FInstanceCullingResult;
class FGPUScene;
class FInstanceCullingManager;
class FInstanceCullingDrawParams;
class FScene;
class FGPUScenePrimitiveCollector;
class FInstanceCullingDeferredContext;
struct FMeshDrawCommandPassStats;
class FSceneCullingRendererHierarchicalCullingQuery;
class FGlobalShaderMap;

DECLARE_UNIFORM_BUFFER_STRUCT(FSceneUniformParameters, RENDERER_API)

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FInstanceCullingGlobalUniforms, RENDERER_API)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, InstanceIdsBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PageInfoBuffer)
	SHADER_PARAMETER(uint32, BufferCapacity)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FBatchedPrimitiveParameters,RENDERER_API)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, Data)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FInstanceCullingDrawParams, )
	RDG_BUFFER_ACCESS(DrawIndirectArgsBuffer, ERHIAccess::IndirectArgs)
	RDG_BUFFER_ACCESS(InstanceIdOffsetBuffer, ERHIAccess::VertexOrIndexBuffer)
	SHADER_PARAMETER(uint32, InstanceDataByteOffset) // offset into per-instance buffer
	SHADER_PARAMETER(uint32, IndirectArgsByteOffset) // offset into indirect args buffer
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FInstanceCullingGlobalUniforms, InstanceCulling)	
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FBatchedPrimitiveParameters, BatchedPrimitive)
END_SHADER_PARAMETER_STRUCT()

FMeshDrawCommandOverrideArgs GetMeshDrawCommandOverrideArgs(const FInstanceCullingDrawParams& InstanceCullingDrawParams);

enum class EInstanceCullingMode
{
	Normal,
	Stereo,
};

enum class EInstanceCullingFlags : uint8
{
	None 							= 0u,
	NoInstanceOrderPreservation		= 1u << 0,
	DrawOnlySelected				= 1u << 1,
};
ENUM_CLASS_FLAGS(EInstanceCullingFlags)

// Enumeration of the specialized command processing variants
enum class EBatchProcessingMode : uint32
{
	// Generic processing mode, handles all the features.
	Generic,
	// General work batches that need load balancing, either instance runs or primitive id ranges (auto instanced) but culling is disabled
	// may have multi-view (but probably not used for that path)
	UnCulled,

	Num,
};

class FInstanceProcessingGPULoadBalancer;

/**
 */
class FInstanceCullingContext
{
public:
	enum class EInstanceFlags : uint8
	{
		None 						= 0,			
		DynamicInstanceDataOffset 	= 1 << 0,
		ForceInstanceCulling 		= 1 << 1,
		PreserveInstanceOrder		= 1 << 2
	};

	static constexpr uint32 UniformViewInstanceStride[2] = 
	{	// One for each BatchProcessingMode
		BATCHED_INSTANCE_DATA_STRIDE, 
		BATCHED_PRIMITIVE_DATA_STRIDE 
	};

	static constexpr uint32 IndirectArgsNumWords = 5;
	static constexpr uint32 CompactionBlockNumInstances = 64;
	RENDERER_API static uint32 GetInstanceIdBufferStride(EShaderPlatform ShaderPlatform);
	RENDERER_API static FUniformBufferStaticSlot GetStaticUniformBufferSlot(EShaderPlatform ShaderPlatform);
	
	FInstanceCullingContext() {}

	/**
	 * Create an instance culling context to process draw commands that can be culled using GPU-Scene.
	 * @param InPrevHZB if non-null enables HZB-occlusion culling for the context (if r.InstanceCulling.OcclusionCull is enabled),
	 *                  NOTE: only one PrevHZB target is allowed accross all passes currently, so either must be atlased or otherwise the same.
	 */
	RENDERER_API FInstanceCullingContext(
		const TCHAR* PassName,
		EShaderPlatform ShaderPlatform,
		FInstanceCullingManager* InInstanceCullingManager, 
		TArrayView<const int32> InViewIds, 
		const TRefCountPtr<IPooledRenderTarget>& InPrevHZB,
		EInstanceCullingMode InInstanceCullingMode = EInstanceCullingMode::Normal,
		EInstanceCullingFlags InFlags = EInstanceCullingFlags::None,
		EBatchProcessingMode InSingleInstanceProcessingMode = EBatchProcessingMode::UnCulled,
		FSceneCullingRendererHierarchicalCullingQuery* InCullingQuery = nullptr
	);
	RENDERER_API ~FInstanceCullingContext();

	static RENDERER_API const TRDGUniformBufferRef<FInstanceCullingGlobalUniforms> CreateDummyInstanceCullingUniformBuffer(FRDGBuilder& GraphBuilder);

	static bool IsGPUCullingEnabled();
	static bool IsOcclusionCullingEnabled();

	/**
	 * Call to empty out the culling commands & other culling data.
	 */
	void ResetCommands(int32 MaxNumCommands);

	bool IsEnabled() const { return bIsEnabled; }

	bool IsInstanceOrderPreservationEnabled() const;

	/**
	 * Add command to cull a range of instances for the given mesh draw command index.
	 * Multiple commands may add to the same slot, ordering is not preserved.
	 */
	void AddInstancesToDrawCommand(uint32 IndirectArgsOffset, int32 InstanceDataOffset, uint32 RunOffset, uint32 NumInstances, EInstanceFlags InstanceFlags);
	void AddInstancesToDrawCommand(uint32 IndirectArgsOffset, int32 InstanceDataOffset, uint32 RunOffset, uint32 NumInstances, EInstanceFlags InstanceFlags, uint32 MaxBatchSize);
	
	/**
	 * Command that is executed in the per-view, post-cull pass to gather up the instances belonging to this primitive.
	 * Multiple commands may add to the same slot, ordering is not preserved.
	 */
	void AddInstanceRunsToDrawCommand(uint32 IndirectArgsOffset, int32 InstanceDataOffset, const uint32* Runs, uint32 NumRuns, EInstanceFlags InstanceFlags, uint32 MaxBatchSize);

	/*
	 * Allocate space for indirect draw call argumens for a given MeshDrawCommand and initialize with draw command data.
	 * TODO: support cached pre-allocated commands.
	 */
	uint32 AllocateIndirectArgs(const FMeshDrawCommand* MeshDrawCommand);

	/*
	* Computes instance data byte offset for a next draw command taking into account platform specifics
	*/
	uint32 StepInstanceDataOffsetBytes(uint32 NumStepDraws) const;
	uint32 GetInstanceIdNumElements() const;

	using SyncPrerequisitesFuncType = TFunction<void (FInstanceCullingContext &InstanceCullingContext)>;
	
	/**
	 * Set up the context to track an async setup process, or some deferred setup work.
	 * The supplied function should do two things, apart from any other processing needed.
	 *   1. wait for the async setup task
	 *   2. Call SetDynamicPrimitiveInstanceOffsets (unless that is achieved somehow else).
	 */
	void BeginAsyncSetup(SyncPrerequisitesFuncType&& InSyncPrerequisitesFunc);

	/**
	 * Calls the sync function passed tp BeginAsyncSetup to ensure the setup processing is completed.
	 */
	void WaitForSetupTask();

	/**
	 */
	void SetDynamicPrimitiveInstanceOffsets(int32 InDynamicInstanceIdOffset, int32 InDynamicInstanceIdNum);

	/**
	 * This version is never deferred, nor async, calling BeginAsyncSetup before this is an error.
	 */
	void BuildRenderingCommands(
		FRDGBuilder& GraphBuilder,
		const FGPUScene& GPUScene,
		int32 InDynamicInstanceIdOffset,
		int32 InDynamicInstanceIdNum,
		FInstanceCullingResult& Results);

	/**
	 * This BuildRenderingCommands operation may be deferred and merged into a global pass when possible.
	 * Note: InstanceCullingDrawParams is captured by the deferred culling passes and must therefore have a RDG-lifetime.
	 * If BeginAsyncSetup has been called prior to this, the WaitForSetupTask is deferred as long as possible. 
	 * If BeginAsyncSetup was not called, then SetDynamicPrimitiveInstanceOffsets must be called before this.
	 */
	void BuildRenderingCommands(FRDGBuilder& GraphBuilder, const FGPUScene& GPUScene, FInstanceCullingDrawParams* InstanceCullingDrawParams);

	/**
	 * Returns true if there are any instances in this context needing to be rendered. Must not be called before WaitForSetupTask if BeginAsyncSetup was called.
	 */
	bool HasCullingCommands() const;

	EInstanceCullingMode GetInstanceCullingMode() const { return InstanceCullingMode; }

	/**
	 * Add a batched BuildRenderingCommands pass. Each batch represents a BuildRenderingCommands call from a mesh pass.
	 * Batches are collected as we walk through the main render setup and are executed when RDG Execute or Drain is called.
	 * This implicitly ends the deferred context, so if Drain is used, it should be paired with a new call to BeginDeferredCulling.
	 */
	static FInstanceCullingDeferredContext* CreateDeferredContext(
		FRDGBuilder& GraphBuilder,
		const FGPUScene& GPUScene,
		FInstanceCullingManager& InstanceCullingManager);

	static void SetDeferredContextCullingParams(const FInstanceCullingDeferredContext& DeferredContext, FInstanceCullingDrawParams& Parameters);

	/**
	 * Helper function to add a pass to zero the instance count in the indirect args.
	 */
	static void AddClearIndirectArgInstanceCountPass(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGBufferRef DrawIndirectArgsBuffer, TFunction<int32()> NumIndirectArgsCallback = TFunction<int32()>());

	void SetupDrawCommands(
		FMeshCommandOneFrameArray& VisibleMeshDrawCommandsInOut,
		bool bCompactIdenticalCommands,
		const FScene *Scene,
		// Stats
		int32& MaxInstancesOut,
		int32& VisibleMeshDrawCommandsNumOut,
		int32& NewPassVisibleMeshDrawCommandsNumOut);

	void SubmitDrawCommands(
		const FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
		const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
		const FMeshDrawCommandOverrideArgs& OverrideArgs,
		int32 StartIndex,
		int32 NumMeshDrawCommands,
		uint32 InstanceFactor,
		FRHICommandList& RHICmdList) const;

	FInstanceCullingManager* InstanceCullingManager = nullptr;
	EShaderPlatform ShaderPlatform = SP_NumPlatforms;
	TArray<int32, TInlineAllocator<6, SceneRenderingAllocator>> ViewIds;
	TRefCountPtr<IPooledRenderTarget> PrevHZB = nullptr;
	// Note: bIsEnabled is immutable after construction and is accessed from multiple threads. To avoid false race detection, it should not be made into a bit-field.
	bool bIsEnabled = false;
	EInstanceCullingMode InstanceCullingMode = EInstanceCullingMode::Normal;
	EInstanceCullingFlags Flags = EInstanceCullingFlags::None;

	uint32 TotalInstances = 0U;

	int32 DynamicInstanceIdOffset = -1;
	int32 DynamicInstanceIdNum = -1;

	SyncPrerequisitesFuncType SyncPrerequisitesFunc;
protected:
	// This is set during SetupDrawCommands to indicate if there are draw commands using indirect args, as this requires setting up the relevant buffers (even if the instance count is zero).
	bool bHasAnyIndirectDraw:1 = false;
public:

	enum class EAsyncProcessingMode
	{
		DeferredOrAsync,
		Synchronous,
	};

	void BuildRenderingCommandsInternal(FRDGBuilder& GraphBuilder, const FGPUScene& GPUScene, EAsyncProcessingMode AsyncProcessingMode, FInstanceCullingDrawParams* InstanceCullingDrawParams);

	// Auxiliary info for each mesh draw command that needs submitting.
	struct FMeshDrawCommandInfo
	{
		// flag to indicate if using indirect or not.
		uint32 bUseIndirect : 1U;
		// stores either the offset (in bytes) to the indirect args or the number of instances
		uint32 IndirectArgsOffsetOrNumInstances : 31U;
		// offset into per-instance buffer
		uint32 InstanceDataByteOffset;
		//
		uint32 NumBatches : 15u;
		uint32 BatchDataStride : 17u;
	};

	struct FPayloadData
	{
		uint32 bDynamicInstanceDataOffset_IndirectArgsIndex;
		uint32 InstanceDataOffset;
		uint32 RunInstanceOffset;
		uint32 CompactionDataIndex;

		FPayloadData() = default;
		FPayloadData(
			bool bInDynamicInstanceDataOffset,
			uint32 InIndirectArgsIndex,
			uint32 InInstanceDataOffset,
			uint32 InRunInstanceOffset,
			uint32 InCompactionDataIndex)
			: bDynamicInstanceDataOffset_IndirectArgsIndex(InIndirectArgsIndex | (bInDynamicInstanceDataOffset ? (1u << 31u) : 0u))
			, InstanceDataOffset(InInstanceDataOffset)
			, RunInstanceOffset(InRunInstanceOffset)
			, CompactionDataIndex(InCompactionDataIndex)
		{
			checkSlow(InIndirectArgsIndex < (1u << 31u));
		}
	};

	struct FCompactionData
	{
		static const uint32 NumViewBits = 8;

		uint32 NumInstances_NumViews;
		uint32 BlockOffset;
		uint32 IndirectArgsIndex;
		uint32 SrcInstanceIdOffset;
		uint32 DestInstanceIdOffset;

		FCompactionData() = default;
		FCompactionData(
			uint32 InNumInstances,
			uint32 InNumViews,
			uint32 InBlockOffset,
			uint32 InIndirectArgsIndex,
			uint32 InSrcInstanceIdOffset,
			uint32 InDestInstanceIdOffset)
			: NumInstances_NumViews(InNumViews | (InNumInstances << NumViewBits))
			, BlockOffset(InBlockOffset)
			, IndirectArgsIndex(InIndirectArgsIndex)
			, SrcInstanceIdOffset(InSrcInstanceIdOffset)
			, DestInstanceIdOffset(InDestInstanceIdOffset)
		{
			checkSlow(InNumViews < (1u << NumViewBits));
			checkSlow(InNumInstances < (1u << (32 - NumViewBits)));
		}
	};

	TArray<FMeshDrawCommandInfo, SceneRenderingAllocator> MeshDrawCommandInfos;
	TArray<FRHIDrawIndexedIndirectParameters, SceneRenderingAllocator> IndirectArgs;
	TArray<FUintVector2, SceneRenderingAllocator> DrawCommandDescs;
	TArray<FPayloadData, SceneRenderingAllocator> PayloadData;
	TArray<uint32, SceneRenderingAllocator> InstanceIdOffsets;

	TArray<FCompactionData, SceneRenderingAllocator> DrawCommandCompactionData;
	TArray<uint32, SceneRenderingAllocator> CompactionBlockDataIndices;
	uint32 NumCompactionInstances = 0U;

	using LoadBalancerArray = TStaticArray<FInstanceProcessingGPULoadBalancer*, static_cast<uint32>(EBatchProcessingMode::Num)>;
	// Driver for collecting items using one mode of processing
	LoadBalancerArray LoadBalancers = LoadBalancerArray(InPlace, nullptr);

	// Set of specialized batches that collect items with different properties each context may have only a subset.
	//TStaticArray<FBatchProcessor, EBatchProcessingMode::Num> Batches;

	// Processing mode to use for single-instance primitives, default to skip culling, as this is already done on CPU. 
	EBatchProcessingMode SingleInstanceProcessingMode = EBatchProcessingMode::UnCulled;

	// Static uniform buffer slot for InstanceCulling/BatchedPrimitive UB. Only valid for a mobile renderer, which needs to handle cases with a manualy merged render-passes
	FUniformBufferStaticSlot InstanceCullingStaticSlot;
	// Whether current platform uses 'Uniform Buffer View' path
	bool bUsesUniformBufferView;
private:
	FSceneCullingRendererHierarchicalCullingQuery* CullingQuery = nullptr;

#if MESH_DRAW_COMMAND_STATS
public:
	// Optional pass stats
	FMeshDrawCommandPassStats* MeshDrawCommandPassStats = nullptr;
#endif
};

ENUM_CLASS_FLAGS(FInstanceCullingContext::EInstanceFlags)

