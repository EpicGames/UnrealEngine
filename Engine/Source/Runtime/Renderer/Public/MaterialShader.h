// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MaterialShader.h: Shader base classes
=============================================================================*/

#pragma once

#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Shader.h"
#include "ShaderParameterUtils.h"
#include "RHIFwd.h"
#include "Serialization/MemoryLayout.h"

class FScene;
class FUniformExpressionSet;
class FViewUniformShaderParameters;

template<typename TBufferStruct> class TUniformBufferRef;

/**
 * Debug information related to uniform expression sets.
 */
class FDebugUniformExpressionSet
{
	DECLARE_TYPE_LAYOUT(FDebugUniformExpressionSet, NonVirtual);
public:
	FDebugUniformExpressionSet();

	explicit FDebugUniformExpressionSet(const FUniformExpressionSet& InUniformExpressionSet);

	/** Initialize from a uniform expression set. */
	void InitFromExpressionSet(const FUniformExpressionSet& InUniformExpressionSet);

	/** Returns true if the number of uniform expressions matches those with which the debug set was initialized. */
	bool Matches(const FUniformExpressionSet& InUniformExpressionSet) const;
	
	/** The number of each type of expression contained in the set. */
	LAYOUT_FIELD(int32, NumPreshaders);
	LAYOUT_ARRAY(int32, NumTextureExpressions, NumMaterialTextureParameterTypes);
};

struct FMaterialShaderPermutationParameters : public FShaderPermutationParameters
{
	FMaterialShaderParameters MaterialParameters;

	FMaterialShaderPermutationParameters(EShaderPlatform InPlatform, const FMaterialShaderParameters& InMaterialParameters, int32 InPermutationId, EShaderPermutationFlags InFlags)
		: FShaderPermutationParameters(InPlatform, InPermutationId, InFlags)
		, MaterialParameters(InMaterialParameters)
	{}
};

/** Base class of all shaders that need material parameters. */
class FMaterialShader : public FShader
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FMaterialShader, RENDERER_API, NonVirtual);
public:
	using FPermutationParameters = FMaterialShaderPermutationParameters;
	using ShaderMetaType = FMaterialShaderType;

	static RENDERER_API FName UniformBufferLayoutName;

	RENDERER_API FMaterialShader();

	RENDERER_API FMaterialShader(const FMaterialShaderType::CompiledShaderInitializerType& Initializer);

	RENDERER_API FRHIUniformBuffer* GetParameterCollectionBuffer(const FGuid& Id, const FSceneInterface* SceneInterface) const;

	RENDERER_API void SetViewParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer);

	template<typename ShaderRHIParamRef, typename TRHICommandList>
	inline void SetViewParameters(TRHICommandList& RHICmdList, const ShaderRHIParamRef ShaderRHI, const FSceneView& View, const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer)
	{
		FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
		SetViewParameters(BatchedParameters, View, ViewUniformBuffer);
		RHICmdList.SetBatchedShaderParameters(ShaderRHI, BatchedParameters);
	}

	/** Sets pixel parameters that are material specific but not FMeshBatch specific. */
	RENDERER_API void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FMaterialRenderProxy* MaterialRenderProxy,
		const FMaterial& Material,
		const FSceneView& View);
	
	/** Sets pixel parameters that are material specific but not FMeshBatch specific. */
	RENDERER_API void SetParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FMaterialRenderProxy* MaterialRenderProxy, 
		const FMaterial& Material,
		const FSceneInterface* Scene);

	/** Sets pixel parameters that are material specific but not FMeshBatch specific. */
	template<typename TRHIShader, typename TRHICommandList>
	inline void SetParameters(
		TRHICommandList& RHICmdList,
		TRHIShader* ShaderRHI,
		const FMaterialRenderProxy* MaterialRenderProxy,
		const FMaterial& Material,
		const FSceneView& View)
	{
		FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
		SetParameters(BatchedParameters, MaterialRenderProxy, Material, View);
		RHICmdList.SetBatchedShaderParameters(ShaderRHI, BatchedParameters);
	}

	RENDERER_API void GetShaderBindings(
		const FSceneInterface* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		FMeshDrawSingleShaderBindings& ShaderBindings) const;

private:
#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING || !WITH_EDITOR)
	RENDERER_API void VerifyExpressionAndShaderMaps(const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial& Material, const FUniformExpressionCache* UniformExpressionCache) const;
#endif

	LAYOUT_FIELD(TMemoryImageArray<FShaderUniformBufferParameter>, ParameterCollectionUniformBuffers);

	LAYOUT_FIELD(FShaderUniformBufferParameter, MaterialUniformBuffer);
	LAYOUT_FIELD(FShaderUniformBufferParameter, MaterialUniformBufferVS);

	// Only needed to avoid unbound parameter error
	// This texture is bound as an UAV (RWTexture) and so it must be bound together with any RT. So it actually bound but not as part of the material
	LAYOUT_FIELD(FShaderResourceParameter, VTFeedbackBuffer);

protected:
	LAYOUT_FIELD_EDITORONLY(FDebugUniformExpressionSet, DebugUniformExpressionSet);
	LAYOUT_FIELD_EDITORONLY(FRHIUniformBufferLayoutInitializer, DebugUniformExpressionUBLayout);
	LAYOUT_FIELD_EDITORONLY(FMemoryImageString, DebugDescription);
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "SceneView.h"
#endif
