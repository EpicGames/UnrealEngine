// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshDrawShaderBindings.h: 
=============================================================================*/

#pragma once

#include "Shader.h"
#include "UniformBuffer.h"

// Whether to assert when mesh command shader bindings were not set by the pass processor.
// Enabled by default in debug
#define VALIDATE_MESH_COMMAND_BINDINGS DO_GUARD_SLOW

/** Stores the number of each resource type that will need to be bound to a single shader, computed during shader reflection. */
class FMeshDrawShaderBindingsLayout
{
protected:
	const FShaderParameterMapInfo& ParameterMapInfo;

public:

	FMeshDrawShaderBindingsLayout(const TShaderRef<FShader>& Shader)
		: ParameterMapInfo(Shader->ParameterMapInfo)
	{
		check(Shader.IsValid());
	}

#if DO_GUARD_SLOW
	const FShaderParameterMapInfo& GetParameterMapInfo()
	{
		return ParameterMapInfo;
	}
#endif

	bool operator==(const FMeshDrawShaderBindingsLayout& Rhs) const
	{
		// Since 4.25, FShader (the owner of this memory) is no longer shared across the shadermaps and gets deleted at the same time as the owning shadermap.
		// To prevent crashes when a singe mesh draw command is shared across multiple MICs with compatible shaders, consider shader bindings belonging to different FShaders different.
		return &ParameterMapInfo == &Rhs.ParameterMapInfo;
	}

	inline uint32 GetLooseDataSizeBytes() const
	{
		uint32 DataSize = 0;
		for (const FShaderLooseParameterBufferInfo& Info : ParameterMapInfo.LooseParameterBuffers)
		{
			DataSize += Info.Size;
		}
		return DataSize;
	}

	inline uint32 GetDataSizeBytes() const
	{
		uint32 DataSize = sizeof(void*) * 
			(ParameterMapInfo.UniformBuffers.Num() 
			+ ParameterMapInfo.TextureSamplers.Num() 
			+ ParameterMapInfo.SRVs.Num());

		// Allocate a bit for each SRV tracking whether it is a FRHITexture* or FRHIShaderResourceView*
		DataSize += FMath::DivideAndRoundUp(ParameterMapInfo.SRVs.Num(), 8);

		DataSize += GetLooseDataSizeBytes();

		// Align to pointer size so subsequent packed shader bindings will have their pointers aligned
		return Align(DataSize, sizeof(void*));
	}

protected:

	// Note: pointers first in layout, so they stay aligned
	inline uint32 GetUniformBufferOffset() const
	{
		return 0;
	}

	inline uint32 GetSamplerOffset() const
	{
		return ParameterMapInfo.UniformBuffers.Num() * sizeof(FRHIUniformBuffer*);
	}

	inline uint32 GetSRVOffset() const
	{
		return GetSamplerOffset() 
			+ ParameterMapInfo.TextureSamplers.Num() * sizeof(FRHISamplerState*);
	}

	inline uint32 GetSRVTypeOffset() const
	{
		return GetSRVOffset() 
			+ ParameterMapInfo.SRVs.Num() * sizeof(FRHIShaderResourceView*);
	}

	inline uint32 GetLooseDataOffset() const
	{
		return GetSRVTypeOffset()
			+ FMath::DivideAndRoundUp(ParameterMapInfo.SRVs.Num(), 8);
	}

	friend class FMeshDrawShaderBindings;
};

class FMeshDrawSingleShaderBindings : public FMeshDrawShaderBindingsLayout
{
	friend class FReadOnlyMeshDrawSingleShaderBindings;
public:
	FMeshDrawSingleShaderBindings(const FMeshDrawShaderBindingsLayout& InLayout, uint8* InData) :
		FMeshDrawShaderBindingsLayout(InLayout)
	{
		Data = InData;
	}

	template<typename UniformBufferStructType>
	void Add(const TShaderUniformBufferParameter<UniformBufferStructType>& Parameter, const TUniformBufferRef<UniformBufferStructType>& Value)
	{
		checkfSlow(Parameter.IsInitialized(), TEXT("Parameter was not serialized"));

		if (Parameter.IsBound())
		{
			checkf(Value.GetReference(), TEXT("Attempted to set null uniform buffer for type %s"), UniformBufferStructType::FTypeInfo::GetStructMetadata()->GetStructTypeName());
			WriteBindingUniformBuffer(Value.GetReference(), Parameter.GetBaseIndex());
		}
	}

	template<typename UniformBufferStructType>
	void Add(const TShaderUniformBufferParameter<UniformBufferStructType>& Parameter, const TUniformBuffer<UniformBufferStructType>& Value)
	{
		checkfSlow(Parameter.IsInitialized(), TEXT("Parameter was not serialized"));

		if (Parameter.IsBound())
		{
			checkf(Value.GetUniformBufferRHI(), TEXT("Attempted to set null uniform buffer for type %s"), UniformBufferStructType::FTypeInfo::GetStructMetadata()->GetStructTypeName());
			WriteBindingUniformBuffer(Value.GetUniformBufferRHI(), Parameter.GetBaseIndex());
		}
	}

	void Add(const FShaderUniformBufferParameter& Parameter, const FRHIUniformBuffer* Value)
	{
		checkfSlow(Parameter.IsInitialized(), TEXT("Parameter was not serialized"));

		if (Parameter.IsBound())
		{
			checkf(Value, TEXT("Attempted to set null uniform buffer"));
			WriteBindingUniformBuffer(Value, Parameter.GetBaseIndex());
		}
	}

	void Add(FShaderResourceParameter Parameter, FRHIShaderResourceView* Value)
	{
		checkfSlow(Parameter.IsInitialized(), TEXT("Parameter was not serialized"));

		if (Parameter.IsBound())
		{
			checkf(Value, TEXT("Attempted to set null SRV on slot %u"), Parameter.GetBaseIndex());
			WriteBindingSRV(Value, Parameter.GetBaseIndex());
		}
	}

	void Add(FShaderResourceParameter SamplerParameter, FRHISamplerState* SamplerStateRHI)
	{
		checkfSlow(SamplerParameter.IsInitialized(), TEXT("Parameter was not serialized"));

		if (SamplerParameter.IsBound())
		{
			checkf(SamplerStateRHI, TEXT("Attempted to set null Sampler on slot %u"), SamplerParameter.GetBaseIndex());
			WriteBindingSampler(SamplerStateRHI, SamplerParameter.GetBaseIndex());
		}
	}

	void Add(FShaderResourceParameter TextureParameter, FRHITexture* TextureRHI)
	{
		checkfSlow(TextureParameter.IsInitialized(), TEXT("Parameter was not serialized"));

		if (TextureParameter.IsBound())
		{
			checkf(TextureRHI, TEXT("Attempted to set null Texture on slot %u"), TextureParameter.GetBaseIndex());
			WriteBindingTexture(TextureRHI, TextureParameter.GetBaseIndex());
		}
	}

	void AddTexture(
		FShaderResourceParameter TextureParameter,
		FShaderResourceParameter SamplerParameter,
		FRHISamplerState* SamplerStateRHI,
		FRHITexture* TextureRHI)
	{
		Add(TextureParameter, TextureRHI);
		Add(SamplerParameter, SamplerStateRHI);
	}

	template<class ParameterType>
	void Add(FShaderParameter Parameter, const ParameterType& Value)
	{
		static_assert(!TIsUECoreVariant<ParameterType, double>::Value, "FMeshDrawSingleShaderBindings cannot Add double core variants! Switch to float variant.");
		static_assert(!TIsUECoreVariant<typename std::remove_pointer<typename TDecay<ParameterType>::Type>::type, double>::Value, "FMeshDrawSingleShaderBindings cannot Add double core variants! Switch to float variant.");
		static_assert(!TIsPointer<ParameterType>::Value, "Passing by pointer is not valid.");
		checkfSlow(Parameter.IsInitialized(), TEXT("Parameter was not serialized"));

		if (Parameter.IsBound())
		{
			bool bFoundParameter = false;
			uint8* LooseDataOffset = GetLooseDataStart();

			TArrayView<const FShaderLooseParameterBufferInfo> LooseParameterBuffers(ParameterMapInfo.LooseParameterBuffers);
			for (int32 LooseBufferIndex = 0; LooseBufferIndex < LooseParameterBuffers.Num(); LooseBufferIndex++)
			{
				const FShaderLooseParameterBufferInfo& LooseParameterBuffer = LooseParameterBuffers[LooseBufferIndex];

				if (LooseParameterBuffer.BaseIndex == Parameter.GetBufferIndex())
				{
					TArrayView<const FShaderLooseParameterInfo> Parameters(LooseParameterBuffer.Parameters);
					for (int32 LooseParameterIndex = 0; LooseParameterIndex < Parameters.Num(); LooseParameterIndex++)
					{
						const FShaderLooseParameterInfo LooseParameter = Parameters[LooseParameterIndex];

						if (Parameter.GetBaseIndex() == LooseParameter.BaseIndex)
						{
							checkSlow(Parameter.GetNumBytes() == LooseParameter.Size);
							ensureMsgf(sizeof(ParameterType) == Parameter.GetNumBytes(), TEXT("Size mismatch between the value being set and the size the shader requires.  Setting %" SIZE_T_FMT " bytes on loose parameter at BaseIndex %u, which requires %u bytes.  This can cause GPU hangs, depending on usage."), sizeof(ParameterType), Parameter.GetBaseIndex(), Parameter.GetNumBytes());
							const int32 NumBytesToSet = FMath::Min<int32>(sizeof(ParameterType), Parameter.GetNumBytes());
							FMemory::Memcpy(LooseDataOffset, &Value, NumBytesToSet);
							const int32 NumBytesToClear = FMath::Min<int32>(0, Parameter.GetNumBytes() - NumBytesToSet);
							FMemory::Memset(LooseDataOffset + NumBytesToSet, 0x00, NumBytesToClear);
							bFoundParameter = true;
							break;
						}

						LooseDataOffset += LooseParameter.Size;
					}
					break;
				}

				LooseDataOffset += LooseParameterBuffer.Size;
			}

			checkfSlow(bFoundParameter, TEXT("Attempted to set loose parameter at BaseIndex %u, Size %u which was never in the shader's parameter map."), Parameter.GetBaseIndex(), Parameter.GetNumBytes());
		}
	}

private:
	uint8* Data;

	inline const FRHIUniformBuffer** GetUniformBufferStart() const
	{
		return (const FRHIUniformBuffer**)(Data + GetUniformBufferOffset());
	}

	inline FRHISamplerState** GetSamplerStart() const
	{
		uint8* SamplerDataStart = Data + GetSamplerOffset();
		return (FRHISamplerState**)SamplerDataStart;
	}

	inline FRHIResource** GetSRVStart() const
	{
		uint8* SRVDataStart = Data + GetSRVOffset();
		checkfSlow(Align(*SRVDataStart, sizeof(void*)) == *SRVDataStart, TEXT("FMeshDrawSingleShaderBindings should have been laid out so that stored pointers are aligned"));
		return (FRHIResource**)SRVDataStart;
	}

	inline uint8* GetSRVTypeStart() const
	{
		uint8* SRVTypeDataStart = Data + GetSRVTypeOffset();
		return SRVTypeDataStart;
	}

	inline uint8* GetLooseDataStart() const
	{
		uint8* LooseDataStart = Data + GetLooseDataOffset();
		return LooseDataStart;
	}
	
	template<typename ElementType>
	inline int FindSortedArrayBaseIndex(const TConstArrayView<ElementType>& Array, uint32 BaseIndex)
	{
		int Index = 0; 
		int Size = Array.Num();

		//start with binary search for larger lists
		while (Size > 8)
		{
			const int LeftoverSize = Size % 2;
			Size = Size / 2;

			const int CheckIndex = Index + Size;
			const int IndexIfLess = CheckIndex + LeftoverSize;

			Index = Array[CheckIndex].BaseIndex < BaseIndex ? IndexIfLess : Index;
		}

		//small size array optimization
		const int ArrayEnd = FMath::Min(Index + Size + 1, Array.Num());
		while (Index < ArrayEnd)
		{
			if (Array[Index].BaseIndex == BaseIndex)
			{
				return Index;
			}
			Index++;
		}

		return -1;
	}

	inline void WriteBindingUniformBuffer(const FRHIUniformBuffer* Value, uint32 BaseIndex)
	{
		int32 FoundIndex = FindSortedArrayBaseIndex(MakeArrayView(ParameterMapInfo.UniformBuffers), BaseIndex);
		if (FoundIndex >= 0)
		{
			GetUniformBufferStart()[FoundIndex] = Value;
		}

		checkfSlow(FoundIndex >= 0, TEXT("Attempted to set a uniform buffer at BaseIndex %u which was never in the shader's parameter map."), BaseIndex);
	}

	inline void WriteBindingSampler(FRHISamplerState* Value, uint32 BaseIndex)
	{
		int32 FoundIndex = FindSortedArrayBaseIndex(MakeArrayView(ParameterMapInfo.TextureSamplers), BaseIndex);
		if (FoundIndex >= 0)
		{
			GetSamplerStart()[FoundIndex] = Value;
		}

		checkfSlow(FoundIndex >= 0, TEXT("Attempted to set a texture sampler at BaseIndex %u which was never in the shader's parameter map."), BaseIndex);
	}

	inline void WriteBindingSRV(FRHIShaderResourceView* Value, uint32 BaseIndex)
	{
		int32 FoundIndex = FindSortedArrayBaseIndex(MakeArrayView(ParameterMapInfo.SRVs), BaseIndex);
		if (FoundIndex >= 0)
		{
			uint32 TypeByteIndex = FoundIndex / 8;
			uint32 TypeBitIndex = FoundIndex % 8;
			GetSRVTypeStart()[TypeByteIndex] |= 1 << TypeBitIndex;
			GetSRVStart()[FoundIndex] = Value;
		}

		checkfSlow(FoundIndex >= 0, TEXT("Attempted to set SRV at BaseIndex %u which was never in the shader's parameter map."), BaseIndex);
	}

	inline void WriteBindingTexture(FRHITexture* Value, uint32 BaseIndex)
	{
		int32 FoundIndex = FindSortedArrayBaseIndex(MakeArrayView(ParameterMapInfo.SRVs), BaseIndex);
		if (FoundIndex >= 0)
		{
			GetSRVStart()[FoundIndex] = Value;
		}

		checkfSlow(FoundIndex >= 0, TEXT("Attempted to set Texture at BaseIndex %u which was never in the shader's parameter map."), BaseIndex);
	}
};