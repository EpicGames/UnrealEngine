// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/Function.h"

class FSceneView;
class FSceneViewFamily;

struct FMeshEdgesViewSettings
{
	// Opacity of the wireframe blended with the shaded view.
	float Opacity = 1.0;
};

// Retrieve settings for how MeshEdges should be rendered for the specified view.
// The returned object reference is owned by the ViewFamily and may be invalidated
// when the number of views changes. Do not cache.
RENDERER_API const FMeshEdgesViewSettings& GetMeshEdgesViewSettings(const FSceneView& View);
RENDERER_API FMeshEdgesViewSettings& GetMeshEdgesViewSettings(FSceneView& View);

struct FMeshEdgesViewFamilySettings
{
	TFunction<void(FSceneViewFamily& WireframeViewFamily)> OnBeforeWireframeRender = [](auto&){};
};

RENDERER_API const FMeshEdgesViewFamilySettings& GetMeshEdgesViewFamilySettings(const FSceneViewFamily& ViewFamily);
RENDERER_API FMeshEdgesViewFamilySettings& GetMeshEdgesViewFamilySettings(FSceneViewFamily& ViewFamily);
