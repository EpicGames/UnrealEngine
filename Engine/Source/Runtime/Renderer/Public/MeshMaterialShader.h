// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshMaterialShader.h: Shader base classes
=============================================================================*/

#pragma once

#include "MeshMaterialShaderType.h"
#include "MaterialShader.h"

class FPrimitiveSceneProxy;
class FSceneView;
struct FMeshBatch;
struct FMeshBatchElement;
struct FMeshDrawingRenderState;
struct FMeshPassProcessorRenderState;

template<typename TBufferStruct> class TUniformBufferRef;

class FMeshMaterialShaderElementData
{
public:
	FRHIUniformBuffer* FadeUniformBuffer = nullptr;
	FRHIUniformBuffer* DitherUniformBuffer = nullptr;

	RENDERER_API void InitializeMeshMaterialData();
	RENDERER_API void InitializeMeshMaterialData(const FSceneView* SceneView, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, const FMeshBatch& RESTRICT MeshBatch, int32 StaticMeshId, bool bAllowStencilDither);
	RENDERER_API void InitializeMeshMaterialData(const FSceneView* SceneView, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, bool bDitheredLODTransition, bool bAllowStencilDither);
};

struct FMeshMaterialShaderPermutationParameters : public FMaterialShaderPermutationParameters
{
	// Type of vertex factory to compile.
	const FVertexFactoryType* VertexFactoryType;

	FMeshMaterialShaderPermutationParameters(EShaderPlatform InPlatform, const FMaterialShaderParameters& InMaterialParameters, const FVertexFactoryType* InVertexFactoryType, int32 InPermutationId, EShaderPermutationFlags InFlags)
		: FMaterialShaderPermutationParameters(InPlatform, InMaterialParameters, InPermutationId, InFlags)
		, VertexFactoryType(InVertexFactoryType)
	{}
};

struct FVertexFactoryShaderPermutationParameters
{
	const FShaderType* ShaderType;
	const FVertexFactoryType* VertexFactoryType;
	FMaterialShaderParameters MaterialParameters;
	EShaderPlatform Platform;
	EShaderPermutationFlags Flags;

	FVertexFactoryShaderPermutationParameters(
		EShaderPlatform InPlatform,
		const FMaterialShaderParameters& InMaterialParameters, 
		const FVertexFactoryType* InVertexFactoryType,
		const FShaderType* InShaderType,
		EShaderPermutationFlags InFlags
		)
		: ShaderType(InShaderType)
		, VertexFactoryType(InVertexFactoryType)
		, MaterialParameters(InMaterialParameters)
		, Platform(InPlatform)
		, Flags(InFlags)
	{}
};

/** Base class of all shaders that need material and vertex factory parameters. */
class FMeshMaterialShader : public FMaterialShader
{
	DECLARE_EXPORTED_TYPE_LAYOUT(FMeshMaterialShader, RENDERER_API, NonVirtual);
public:
	using FPermutationParameters = FMeshMaterialShaderPermutationParameters;
	using ShaderMetaType = FMeshMaterialShaderType;

	FMeshMaterialShader() {}

	RENDERER_API FMeshMaterialShader(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer);

	RENDERER_API void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FMeshMaterialShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const;

	RENDERER_API void GetElementShaderBindings(
		const FShaderMapPointerTable& PointerTable,
		const FScene* Scene, 
		const FSceneView* ViewIfDynamicMeshCommand, 
		const FVertexFactory* VertexFactory,
		const EVertexInputStreamType InputStreamType,
		const FStaticFeatureLevel FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMeshBatch& MeshBatch,
		const FMeshBatchElement& BatchElement, 
		const FMeshMaterialShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const;

	template<typename ShaderType, typename PointerTableType, typename ShaderElementDataType>
	static inline void GetElementShaderBindings(const TShaderRefBase<ShaderType, PointerTableType>& Shader,
		const FScene* Scene,
		const FSceneView* ViewIfDynamicMeshCommand,
		const FVertexFactory* VertexFactory,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMeshBatch& MeshBatch,
		const FMeshBatchElement& BatchElement,
		const ShaderElementDataType& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams)
	{
		Shader->GetElementShaderBindings(Shader.GetPointerTable(), Scene, ViewIfDynamicMeshCommand, VertexFactory, InputStreamType, FeatureLevel, PrimitiveSceneProxy, MeshBatch, BatchElement, ShaderElementData, ShaderBindings, VertexStreams);
	}

private:
	RENDERER_API void WriteFrozenVertexFactoryParameters(FMemoryImageWriter& Writer, const TMemoryImagePtr<FVertexFactoryShaderParameters>& InVertexFactoryParameters) const;
	LAYOUT_FIELD_WITH_WRITER(TMemoryImagePtr<FVertexFactoryShaderParameters>, VertexFactoryParameters, WriteFrozenVertexFactoryParameters);

protected:
	LAYOUT_FIELD(FShaderUniformBufferParameter, PassUniformBuffer);
};

