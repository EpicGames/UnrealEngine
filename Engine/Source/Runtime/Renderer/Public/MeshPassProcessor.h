// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshPassProcessor.h
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Math/MathFwd.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "RHI.h"
#include "RHIResources.h"
#include "Shader.h"
#include "PSOPrecache.h"
#include "PSOPrecacheMaterial.h"
#include "PipelineStateCache.h"
#include "Misc/ScopeRWLock.h"
#include "MeshDrawCommandStatsDefines.h"
#include "VertexFactory.h"
#include "MeshDrawShaderBindings.h"
#include "Misc/AssertionMacros.h"
#include "SceneRenderingAllocator.h"
#include "MeshBatch.h"
#include "SceneUtils.h"
#include "GBufferInfo.h"
#include "Hash/CityHash.h"
#include "DebugViewModeHelpers.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include <atomic>

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "MeshDrawCommandStatsDefines.h"
#include "RendererInterface.h"
#include "MeshMaterialShader.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#define MESH_DRAW_COMMAND_DEBUG_DATA ((!UE_BUILD_SHIPPING && !UE_BUILD_TEST) || VALIDATE_MESH_COMMAND_BINDINGS || WANTS_DRAW_MESH_EVENTS || WITH_DEBUG_VIEW_MODES)

class FScene;
class FPrimitiveSceneInfo;
class FPrimitiveSceneProxy;
class FMaterialRenderProxy;
struct FMeshBatch;

class FGPUScene;
class FInstanceCullingDrawParams;

class FRayTracingLocalShaderBindingWriter;

struct FMeshProcessorShaders;
struct FSceneTexturesConfig;
struct FVisibleMeshDrawCommandStatsData;

/** Mesh pass types supported. */
namespace EMeshPass
{
	enum Type : uint8
	{
		DepthPass,
		SecondStageDepthPass,
		BasePass,
		AnisotropyPass,
		SkyPass,
		SingleLayerWaterPass,
		SingleLayerWaterDepthPrepass,
		CSMShadowDepth,
		VSMShadowDepth,
		OnePassPointLightShadowDepth,
		Distortion,
		Velocity,
		TranslucentVelocity,
		TranslucentVelocityClippedDepth, /**  A pass to handle pixels with depth below the opaque threshold*/
		TranslucencyStandard,
		TranslucencyStandardModulate,
		TranslucencyAfterDOF,
		TranslucencyAfterDOFModulate,
		TranslucencyAfterMotionBlur,
		TranslucencyHoldout, /** A standalone pass to render all translucency for holdout, inferring the background visibility*/
		TranslucencyAll, /** Drawing all translucency, regardless of separate or standard.  Used when drawing translucency outside of the main renderer, eg FRendererModule::DrawTile. */
		LightmapDensity,
		DebugViewMode, /** Any of EDebugViewShaderMode */
		CustomDepth,
		MobileBasePassCSM,  /** Mobile base pass with CSM shading enabled */
		VirtualTexture,
		LumenCardCapture,
		LumenCardNanite,
		LumenTranslucencyRadianceCacheMark,
		FrontLayerTranslucencyGBuffer,
		DitheredLODFadingOutMaskPass, /** A mini depth pass used to mark pixels with dithered LOD fading out. Currently only used by ray tracing shadows. */
		NaniteMeshPass,
		MeshDecal_DBuffer,
		MeshDecal_SceneColorAndGBuffer,
		MeshDecal_SceneColorAndGBufferNoNormal,
		MeshDecal_SceneColor,
		MeshDecal_AmbientOcclusion,
		WaterInfoTextureDepthPass,
		WaterInfoTexturePass,

#if WITH_EDITOR
		HitProxy,
		HitProxyOpaqueOnly,
		EditorLevelInstance,
		EditorSelection,
#endif

		Num,
		NumBits = 6,
	};
}
static_assert(EMeshPass::Num <= (1 << EMeshPass::NumBits), "EMeshPass::Num will not fit in EMeshPass::NumBits");
static_assert(EMeshPass::NumBits <= sizeof(EMeshPass::Type) * 8, "EMeshPass::Type storage is too small");

inline const TCHAR* GetMeshPassName(EMeshPass::Type MeshPass)
{
	switch (MeshPass)
	{
	case EMeshPass::DepthPass: return TEXT("DepthPass");
	case EMeshPass::SecondStageDepthPass: return TEXT("SecondStageDepthPass");
	case EMeshPass::BasePass: return TEXT("BasePass");
	case EMeshPass::AnisotropyPass: return TEXT("AnisotropyPass");
	case EMeshPass::SkyPass: return TEXT("SkyPass");
	case EMeshPass::SingleLayerWaterPass: return TEXT("SingleLayerWaterPass");
	case EMeshPass::SingleLayerWaterDepthPrepass: return TEXT("SingleLayerWaterDepthPrepass");
	case EMeshPass::CSMShadowDepth: return TEXT("CSMShadowDepth");
	case EMeshPass::VSMShadowDepth: return TEXT("VSMShadowDepth");
	case EMeshPass::OnePassPointLightShadowDepth: return TEXT("OnePassPointLightShadowDepth");
	case EMeshPass::Distortion: return TEXT("Distortion");
	case EMeshPass::Velocity: return TEXT("Velocity");
	case EMeshPass::TranslucentVelocity: return TEXT("TranslucentVelocity");
	case EMeshPass::TranslucentVelocityClippedDepth: return TEXT("TranslucentVelocityClippedDepth");
	case EMeshPass::TranslucencyStandard: return TEXT("TranslucencyStandard");
	case EMeshPass::TranslucencyStandardModulate: return TEXT("TranslucencyStandardModulate");
	case EMeshPass::TranslucencyAfterDOF: return TEXT("TranslucencyAfterDOF");
	case EMeshPass::TranslucencyAfterDOFModulate: return TEXT("TranslucencyAfterDOFModulate");
	case EMeshPass::TranslucencyAfterMotionBlur: return TEXT("TranslucencyAfterMotionBlur");
	case EMeshPass::TranslucencyHoldout: return TEXT("TranslucencyHoldout");
	case EMeshPass::TranslucencyAll: return TEXT("TranslucencyAll");
	case EMeshPass::LightmapDensity: return TEXT("LightmapDensity");
	case EMeshPass::DebugViewMode: return TEXT("DebugViewMode");
	case EMeshPass::CustomDepth: return TEXT("CustomDepth");
	case EMeshPass::MobileBasePassCSM: return TEXT("MobileBasePassCSM");
	case EMeshPass::VirtualTexture: return TEXT("VirtualTexture");
	case EMeshPass::LumenCardCapture: return TEXT("LumenCardCapture");
	case EMeshPass::LumenCardNanite: return TEXT("LumenCardNanite");
	case EMeshPass::LumenTranslucencyRadianceCacheMark: return TEXT("LumenTranslucencyRadianceCacheMark");
	case EMeshPass::FrontLayerTranslucencyGBuffer: return TEXT("FrontLayerTranslucencyGBuffer");
	case EMeshPass::DitheredLODFadingOutMaskPass: return TEXT("DitheredLODFadingOutMaskPass");
	case EMeshPass::NaniteMeshPass: return TEXT("NaniteMeshPass");
	case EMeshPass::MeshDecal_DBuffer: return TEXT("MeshDecal_DBuffer");
	case EMeshPass::MeshDecal_SceneColorAndGBuffer: return TEXT("MeshDecal_SceneColorAndGBuffer");
	case EMeshPass::MeshDecal_SceneColorAndGBufferNoNormal: return TEXT("MeshDecal_SceneColorAndGBufferNoNormal");
	case EMeshPass::MeshDecal_SceneColor: return TEXT("MeshDecal_SceneColor");
	case EMeshPass::MeshDecal_AmbientOcclusion: return TEXT("MeshDecal_AmbientOcclusion");
	case EMeshPass::WaterInfoTextureDepthPass: return TEXT("WaterInfoTextureDepthPass");
	case EMeshPass::WaterInfoTexturePass: return TEXT("WaterInfoTexturePass");
#if WITH_EDITOR
	case EMeshPass::HitProxy: return TEXT("HitProxy");
	case EMeshPass::HitProxyOpaqueOnly: return TEXT("HitProxyOpaqueOnly");
	case EMeshPass::EditorLevelInstance: return TEXT("EditorLevelInstance");
	case EMeshPass::EditorSelection: return TEXT("EditorSelection");
#endif
	}

#if WITH_EDITOR
	static_assert(EMeshPass::Num == 39 + 4, "Need to update switch(MeshPass) after changing EMeshPass"); // GUID to prevent incorrect auto-resolves, please change when changing the expression: {674D7D62-CFD8-4971-9A8D-CD91E5612CD8}
#else
	static_assert(EMeshPass::Num == 39, "Need to update switch(MeshPass) after changing EMeshPass"); // GUID to prevent incorrect auto-resolves, please change when changing the expression: {674D7D62-CFD8-4971-9A8D-CD91E5612CD8}
#endif

	checkf(0, TEXT("Missing case for EMeshPass %u"), (uint32)MeshPass);
	return nullptr;
}

/** Mesh pass mask - stores one bit per mesh pass. */
class FMeshPassMask
{
public:
	FMeshPassMask()
		: Data(0)
	{
	}

	void Set(EMeshPass::Type Pass) 
	{ 
		Data |= (uint64(1) << Pass); 
	}

	bool Get(EMeshPass::Type Pass) const 
	{ 
		return !!(Data & (uint64(1) << Pass)); 
	}

	EMeshPass::Type SkipEmpty(EMeshPass::Type Pass) const 
	{
		uint64 Mask = 0xFFffFFffFFffFFffULL << Pass;
		return EMeshPass::Type(FMath::Min<uint64>(EMeshPass::Num, FMath::CountTrailingZeros64(Data & Mask)));
	}

	int GetNum() 
	{ 
		return FMath::CountBits64(Data); 
	}

	void AppendTo(FMeshPassMask& Mask) const 
	{ 
		Mask.Data |= Data; 
	}

	void Reset() 
	{ 
		Data = 0; 
	}

	bool IsEmpty() const 
	{ 
		return Data == 0; 
	}

	uint64 Data;
};

struct FMinimalBoundShaderStateInput
{
	inline FMinimalBoundShaderStateInput() {}

	FMinimalBoundShaderStateInput(const FMinimalBoundShaderStateInput & Other)
		: VertexDeclarationRHI(Other.VertexDeclarationRHI)
		, CachedVertexShader(Other.CachedVertexShader.load(std::memory_order_relaxed))
		, CachedPixelShader(Other.CachedPixelShader)
		, bAllShaderAreLoaded(Other.bAllShaderAreLoaded)
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		, CachedGeometryShader(Other.CachedGeometryShader)
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
		, CachedMeshShader(Other.CachedMeshShader)
#endif
		, VertexShaderResource(Other.VertexShaderResource)
		, PixelShaderResource(Other.PixelShaderResource)
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		, GeometryShaderResource(Other.GeometryShaderResource)
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
		, MeshShaderResource(Other.MeshShaderResource)
#endif
		, VertexShaderIndex(Other.VertexShaderIndex)
		, PixelShaderIndex(Other.PixelShaderIndex)
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		, GeometryShaderIndex(Other.GeometryShaderIndex)
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
		, MeshShaderIndex(Other.MeshShaderIndex)
#endif
	{}

	FBoundShaderStateInput AsBoundShaderState(bool bRequired = true) const
	{
		bool bLocalAllShaderAreLoaded = true;
		bool bCanSkipShader = AllowSkipUnloadedShaders();
		const bool bShaderRequired = bRequired && !bCanSkipShader;

		auto GetShaderResource = [&bLocalAllShaderAreLoaded](FRHIShader* Shader) -> FRHIShader* {
			if (Shader == nullptr)
			{
				bLocalAllShaderAreLoaded &= false;
			}
			return Shader;
		};

		FRHIVertexShader* LocalCachedVertexShader = CachedVertexShader.load(std::memory_order_acquire);
		if (!LocalCachedVertexShader || !bAllShaderAreLoaded)
		{
			CachedPixelShader = PixelShaderResource ? static_cast<FRHIPixelShader*>(GetShaderResource(PixelShaderResource->GetShader(PixelShaderIndex, bShaderRequired))) : nullptr;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			CachedGeometryShader = GeometryShaderResource ? static_cast<FRHIGeometryShader*>(GetShaderResource(GeometryShaderResource->GetShader(GeometryShaderIndex, bShaderRequired))) : nullptr;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
			CachedMeshShader = MeshShaderResource ? static_cast<FRHIMeshShader*>(GetShaderResource(MeshShaderResource->GetShader(MeshShaderIndex, bShaderRequired))) : nullptr;
#endif
			LocalCachedVertexShader = VertexShaderResource ? static_cast<FRHIVertexShader*>(GetShaderResource(VertexShaderResource->GetShader(VertexShaderIndex, bShaderRequired))) : nullptr;

			bAllShaderAreLoaded = bLocalAllShaderAreLoaded;
			CachedVertexShader.store(LocalCachedVertexShader, std::memory_order_release);
		}

#if PLATFORM_SUPPORTS_MESH_SHADERS
		if (CachedMeshShader)
		{
			return FBoundShaderStateInput(CachedMeshShader, nullptr, CachedPixelShader);
		}
		else
#endif
		{
			return FBoundShaderStateInput(VertexDeclarationRHI
				, LocalCachedVertexShader
				, CachedPixelShader
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
				, CachedGeometryShader
#endif
			);
		}
	}

	RENDERER_API bool AllowSkipUnloadedShaders() const;

	bool IsShaderAllLoaded() const { return bAllShaderAreLoaded; }

	void ForceShaderReload() const 
	{
		FRHIVertexShader* LocalCachedVertexShader = CachedVertexShader.load(std::memory_order_acquire);
		if (!LocalCachedVertexShader || !bAllShaderAreLoaded)
		{
			CachedPixelShader = PixelShaderResource ? static_cast<FRHIPixelShader*>(PixelShaderResource->GetShader(PixelShaderIndex)) : nullptr;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			CachedGeometryShader = GeometryShaderResource ? static_cast<FRHIGeometryShader*>(GeometryShaderResource->GetShader(GeometryShaderIndex)) : nullptr;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
			CachedMeshShader = MeshShaderResource ? static_cast<FRHIMeshShader*>(MeshShaderResource->GetShader(MeshShaderIndex)) : nullptr;
#endif
			LocalCachedVertexShader = VertexShaderResource ? static_cast<FRHIVertexShader*>(VertexShaderResource->GetShader(VertexShaderIndex)) : nullptr;

			bAllShaderAreLoaded = true;

			CachedVertexShader.store(LocalCachedVertexShader, std::memory_order_release);
		}
	}

	void LazilyInitShaders() const
	{
		AsBoundShaderState(); // querying shaders will initialize on demand
	}

	bool NeedsShaderInitialisation() const
	{
		if (VertexShaderResource && !VertexShaderResource->HasShader(VertexShaderIndex))
		{
			return true;
		}
		if (PixelShaderResource && !PixelShaderResource->HasShader(PixelShaderIndex))
		{
			return true;
		}
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		if (GeometryShaderResource && !GeometryShaderResource->HasShader(GeometryShaderIndex))
		{
			return true;
		}
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
		if (MeshShaderResource && !MeshShaderResource->HasShader(MeshShaderIndex))
		{
			return true;
		}
#endif
		return false;
	}

	FRHIVertexDeclaration* VertexDeclarationRHI = nullptr;
	mutable std::atomic<FRHIVertexShader*> CachedVertexShader = nullptr;
	mutable FRHIPixelShader* CachedPixelShader = nullptr;
	mutable bool bAllShaderAreLoaded = true;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	mutable FRHIGeometryShader* CachedGeometryShader = nullptr;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
	mutable FRHIMeshShader* CachedMeshShader = nullptr;
#endif
	TRefCountPtr<FShaderMapResource> VertexShaderResource;
	TRefCountPtr<FShaderMapResource> PixelShaderResource;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	TRefCountPtr<FShaderMapResource> GeometryShaderResource;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
	TRefCountPtr<FShaderMapResource> MeshShaderResource;
#endif
	int32 VertexShaderIndex = INDEX_NONE;
	int32 PixelShaderIndex = INDEX_NONE;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	int32 GeometryShaderIndex = INDEX_NONE;
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
	int32 MeshShaderIndex = INDEX_NONE;
#endif
};


/**
 * Pipeline state without render target state
 * Useful for mesh passes where the render target state is not changing between draws.
 * Note: the size of this class affects rendering mesh pass traversal performance.
 */
class FGraphicsMinimalPipelineStateInitializer
{
public:
	// Can't use TEnumByte<EPixelFormat> as it changes the struct to be non trivially constructible, breaking memset
	using TRenderTargetFormats = TStaticArray<EPixelFormat, MaxSimultaneousRenderTargets>;
	using TRenderTargetFlags = TStaticArray<uint32/*ETextureCreateFlags*/, MaxSimultaneousRenderTargets>;

	FGraphicsMinimalPipelineStateInitializer()
		: BlendState(nullptr)
		, RasterizerState(nullptr)
		, DepthStencilState(nullptr)
		, PrimitiveType(PT_Num)
	{}

	FGraphicsMinimalPipelineStateInitializer(
		FMinimalBoundShaderStateInput	InBoundShaderState,
		FRHIBlendState*					InBlendState,
		FRHIRasterizerState*			InRasterizerState,
		FRHIDepthStencilState*			InDepthStencilState,
		EPrimitiveType					InPrimitiveType
	)
		: BoundShaderState(InBoundShaderState)
		, BlendState(InBlendState)
		, RasterizerState(InRasterizerState)
		, DepthStencilState(InDepthStencilState)
		, PrimitiveType(InPrimitiveType)
	{
	}

	FGraphicsMinimalPipelineStateInitializer(const FGraphicsMinimalPipelineStateInitializer& InMinimalState)
		: BoundShaderState(InMinimalState.BoundShaderState)
		, BlendState(InMinimalState.BlendState)
		, RasterizerState(InMinimalState.RasterizerState)
		, DepthStencilState(InMinimalState.DepthStencilState)
		, bDepthBounds(InMinimalState.bDepthBounds)
		, bAllowVariableRateShading(InMinimalState.bAllowVariableRateShading)
		, DrawShadingRate(InMinimalState.DrawShadingRate)
		, PrimitiveType(InMinimalState.PrimitiveType)
		, StatePrecachePSOHash(InMinimalState.StatePrecachePSOHash)
		, PSOPrecacheState(InMinimalState.PSOPrecacheState)
	{
	}

	RENDERER_API void SetupBoundShaderState(FRHIVertexDeclaration* VertexDeclaration, const FMeshProcessorShaders& Shaders);

 	RENDERER_API void ComputeStatePrecachePSOHash();

	FGraphicsPipelineStateInitializer AsGraphicsPipelineStateInitializer(bool bRequired = true) const
	{
		FGraphicsPipelineStateInitializer result(
			BoundShaderState.AsBoundShaderState(bRequired)
			, BlendState
			, RasterizerState
			, DepthStencilState
			, PrimitiveType
			, 0
			, FGraphicsPipelineStateInitializer::TRenderTargetFormats(InPlace, PF_Unknown)
			, FGraphicsPipelineStateInitializer::TRenderTargetFlags(InPlace, TexCreate_None)
			, PF_Unknown
			, TexCreate_None
			, ERenderTargetLoadAction::ENoAction
			, ERenderTargetStoreAction::ENoAction
			, ERenderTargetLoadAction::ENoAction
			, ERenderTargetStoreAction::ENoAction
			, FExclusiveDepthStencil::DepthNop
			, 0
			, ESubpassHint::None
			, 0
			, EConservativeRasterization::Disabled
			, 0
			, bDepthBounds
			, 0
			, false
			, bAllowVariableRateShading
			, DrawShadingRate
		);

		if (PipelineStateCache::IsPSOPrecachingEnabled() )
		{
			checkSlow(StatePrecachePSOHash == 0 || RHIComputeStatePrecachePSOHash(result) == StatePrecachePSOHash);
			result.StatePrecachePSOHash = StatePrecachePSOHash == 0 ? RHIComputeStatePrecachePSOHash(result) : StatePrecachePSOHash;			
		}

		return result;
	}

	inline bool operator==(const FGraphicsMinimalPipelineStateInitializer& rhs) const
	{
		if (BoundShaderState.VertexDeclarationRHI != rhs.BoundShaderState.VertexDeclarationRHI ||
			BoundShaderState.VertexShaderResource != rhs.BoundShaderState.VertexShaderResource ||
			BoundShaderState.PixelShaderResource != rhs.BoundShaderState.PixelShaderResource ||
			BoundShaderState.VertexShaderIndex != rhs.BoundShaderState.VertexShaderIndex ||
			BoundShaderState.PixelShaderIndex != rhs.BoundShaderState.PixelShaderIndex ||
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			BoundShaderState.GeometryShaderResource != rhs.BoundShaderState.GeometryShaderResource ||
			BoundShaderState.GeometryShaderIndex != rhs.BoundShaderState.GeometryShaderIndex ||
#endif
			BlendState != rhs.BlendState ||
			RasterizerState != rhs.RasterizerState ||
			DepthStencilState != rhs.DepthStencilState ||
			bDepthBounds != rhs.bDepthBounds ||
			bAllowVariableRateShading != rhs.bAllowVariableRateShading ||
			DrawShadingRate != rhs.DrawShadingRate ||
			PrimitiveType != rhs.PrimitiveType)
		{
			return false;
		}

		return true;
	}

	inline bool operator!=(const FGraphicsMinimalPipelineStateInitializer& rhs) const
	{
		return !(*this == rhs);
	}

	inline friend uint32 GetTypeHash(const FGraphicsMinimalPipelineStateInitializer& Initializer)
	{
		//add and initialize any leftover padding within the struct to avoid unstable key
		struct FHashKey
		{
			uint32 VertexDeclaration;
			uint32 VertexShader;
			uint32 PixelShader;
			uint32 RasterizerState;
		} HashKey;
		HashKey.VertexDeclaration = PointerHash(Initializer.BoundShaderState.VertexDeclarationRHI);
		HashKey.VertexShader = GetTypeHash(Initializer.BoundShaderState.VertexShaderIndex);
		HashKey.PixelShader = GetTypeHash(Initializer.BoundShaderState.PixelShaderIndex);
		HashKey.RasterizerState = PointerHash(Initializer.RasterizerState);

		return uint32(CityHash64((const char*)&HashKey, sizeof(FHashKey)));
	}

#define COMPARE_FIELD_BEGIN(Field) \
		if (Field != rhs.Field) \
		{ return Field COMPARE_OP rhs.Field; }

#define COMPARE_FIELD(Field) \
		else if (Field != rhs.Field) \
		{ return Field COMPARE_OP rhs.Field; }

#define COMPARE_FIELD_END \
		else { return false; }

	bool operator<(const FGraphicsMinimalPipelineStateInitializer& rhs) const
	{
#define COMPARE_OP <

		COMPARE_FIELD_BEGIN(BoundShaderState.VertexDeclarationRHI)
			COMPARE_FIELD(BoundShaderState.VertexShaderIndex)
			COMPARE_FIELD(BoundShaderState.PixelShaderIndex)
			COMPARE_FIELD(BoundShaderState.VertexShaderResource)
			COMPARE_FIELD(BoundShaderState.PixelShaderResource)
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			COMPARE_FIELD(BoundShaderState.GeometryShaderIndex)
			COMPARE_FIELD(BoundShaderState.GeometryShaderResource)
#endif
			COMPARE_FIELD(BlendState)
			COMPARE_FIELD(RasterizerState)
			COMPARE_FIELD(DepthStencilState)
			COMPARE_FIELD(bDepthBounds)
			COMPARE_FIELD(bAllowVariableRateShading)
			COMPARE_FIELD(DrawShadingRate)
			COMPARE_FIELD(PrimitiveType)
		COMPARE_FIELD_END;

#undef COMPARE_OP
	}

	bool operator>(const FGraphicsMinimalPipelineStateInitializer& rhs) const
	{
#define COMPARE_OP >

		COMPARE_FIELD_BEGIN(BoundShaderState.VertexDeclarationRHI)
			COMPARE_FIELD(BoundShaderState.VertexShaderIndex)
			COMPARE_FIELD(BoundShaderState.PixelShaderIndex)
			COMPARE_FIELD(BoundShaderState.VertexShaderResource)
			COMPARE_FIELD(BoundShaderState.PixelShaderResource)
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
			COMPARE_FIELD(BoundShaderState.GeometryShaderIndex)
			COMPARE_FIELD(BoundShaderState.GeometryShaderResource)
#endif
			COMPARE_FIELD(BlendState)
			COMPARE_FIELD(RasterizerState)
			COMPARE_FIELD(DepthStencilState)
			COMPARE_FIELD(bDepthBounds)
			COMPARE_FIELD(bAllowVariableRateShading)
			COMPARE_FIELD(DrawShadingRate)
			COMPARE_FIELD(PrimitiveType)
			COMPARE_FIELD_END;

#undef COMPARE_OP
	}

#undef COMPARE_FIELD_BEGIN
#undef COMPARE_FIELD
#undef COMPARE_FIELD_END

	// TODO: [PSO API] - As we migrate reuse existing API objects, but eventually we can move to the direct initializers. 
	// When we do that work, move this to RHI.h as its more appropriate there, but here for now since dependent typdefs are here.
	FMinimalBoundShaderStateInput	BoundShaderState;
	FRHIBlendState*					BlendState;
	FRHIRasterizerState*			RasterizerState;
	FRHIDepthStencilState*			DepthStencilState;

	// Note: FGraphicsMinimalPipelineStateInitializer is 8-byte aligned and can't have any implicit padding,
	// as it is sometimes hashed and compared as raw bytes. Explicit padding is therefore required between
	// all data members and at the end of the structure.
	bool							bDepthBounds = false;
	bool							bAllowVariableRateShading = true;
	EVRSShadingRate					DrawShadingRate  = EVRSShadingRate::VRSSR_1x1;

	EPrimitiveType					PrimitiveType;
		
	// Data hash of the minimal PSO which is used to optimize the computation of the full PSO
	uint64							StatePrecachePSOHash = 0;
	// The PSO precache state - updated at draw time and can be used to skip draw when still precaching
	mutable EPSOPrecacheResult		PSOPrecacheState = EPSOPrecacheResult::Unknown;
};

static_assert(sizeof(FMeshPassMask::Data) * 8 >= EMeshPass::Num, "FMeshPassMask::Data is too small to fit all mesh passes.");

/** Set of FGraphicsMinimalPipelineStateInitializer unique per MeshDrawCommandsPassContext */
typedef Experimental::TRobinHoodHashSet< FGraphicsMinimalPipelineStateInitializer > FGraphicsMinimalPipelineStateSet;

/** Uniquely represents a FGraphicsMinimalPipelineStateInitializer for fast compares. */
class FGraphicsMinimalPipelineStateId
{
public:
	FORCEINLINE_DEBUGGABLE uint32 GetId() const
	{
		checkSlow(IsValid());
		return PackedId;
	}

	inline bool IsValid() const 
	{
		return bValid != 0;
	}


	inline bool operator==(const FGraphicsMinimalPipelineStateId& rhs) const
	{
		return PackedId == rhs.PackedId;
	}

	inline bool operator!=(const FGraphicsMinimalPipelineStateId& rhs) const
	{
		return !(*this == rhs);
	}
	
	inline const FGraphicsMinimalPipelineStateInitializer& GetPipelineState(const FGraphicsMinimalPipelineStateSet& InPipelineSet) const
	{
		if (bComesFromLocalPipelineStateSet)
		{
			return InPipelineSet.GetByElementId(SetElementIndex);
		}

		{
			FRWScopeLock ReadLock(PersistentIdTableLock, SLT_ReadOnly);
			const auto& Result = PersistentIdTable.GetByElementId(SetElementIndex);
#if MESH_DRAW_COMMAND_DEBUG_DATA
			checkf(Result.Value.RefNum && Result.Value.DebugSalt == DebugSalt, TEXT("Pipeline state ID used after release.  Call 'AddRefPersistentId' to ensure pipeline state doesn't get released while in use."));
#endif
			return Result.Key;
		}
	}

	static void InitializePersistentIds();
	/**
	 * Get a ref counted persistent pipeline id, which needs to manually released.
	 */
	static FGraphicsMinimalPipelineStateId GetPersistentId(const FGraphicsMinimalPipelineStateInitializer& InPipelineState);

	/**
	 * Removes a persistent pipeline Id from the global persistent Id table.
	 */
	static void RemovePersistentId(FGraphicsMinimalPipelineStateId Id);
	
	/**
	 * Get a pipeline state id in this order: global persistent Id table. If not found, will lookup in PassSet argument. If not found in PassSet argument, create a blank pipeline set id and add it PassSet argument
	 */
	RENDERER_API static FGraphicsMinimalPipelineStateId GetPipelineStateId(const FGraphicsMinimalPipelineStateInitializer& InPipelineState, FGraphicsMinimalPipelineStateSet& InOutPassSet, bool& NeedsShaderInitialisation);

	static int32 GetLocalPipelineIdTableSize() 
	{
#if MESH_DRAW_COMMAND_DEBUG_DATA
		return LocalPipelineIdTableSize;
#else
		return 0;
#endif //MESH_DRAW_COMMAND_DEBUG_DATA
	}
	static void ResetLocalPipelineIdTableSize();
	static void AddSizeToLocalPipelineIdTableSize(SIZE_T Size);

	static SIZE_T GetPersistentIdTableSize() 
	{ 
		FRWScopeLock ReadLock(PersistentIdTableLock, SLT_ReadOnly);
		return PersistentIdTable.GetAllocatedSize();
	}
	static int32 GetPersistentIdNum() 
	{ 
		FRWScopeLock ReadLock(PersistentIdTableLock, SLT_ReadOnly);
		return PersistentIdTable.Num();
	}

	/**
	  * When ID table is frozen, items with a zero ref count aren't actually released, so if the same item gets used again,
	  * its ID doesn't change.  Useful when re-creating draw commands in the middle of the frame, where you want to prevent
	  * existing pipeline state IDs from changing, which may be referenced by in flight tasks.  Unfreezing the table will
	  * clean up anything with a zero ref count.
	  */
	static void FreezeIdTable(bool bEnable);

private:
	union
	{
		uint32 PackedId = 0;

		struct
		{
			uint32 SetElementIndex				   : 30;
			uint32 bComesFromLocalPipelineStateSet : 1;
			uint32 bValid						   : 1;
		};
	};

	struct FRefCountedGraphicsMinimalPipelineState
	{
		FRefCountedGraphicsMinimalPipelineState() : RefNum(0)
		{
#if MESH_DRAW_COMMAND_DEBUG_DATA
			DebugSalt = DebugSaltAllocationIndex++;
#endif
		}

		FRefCountedGraphicsMinimalPipelineState(const FRefCountedGraphicsMinimalPipelineState&& Other) :
			RefNum(Other.RefNum.load())
		{
#if MESH_DRAW_COMMAND_DEBUG_DATA
			DebugSalt = Other.DebugSalt;
#endif
		}

		std::atomic<uint32> RefNum;
#if MESH_DRAW_COMMAND_DEBUG_DATA
		int32 DebugSalt;
#endif
	};

	static FRWLock PersistentIdTableLock;
	using PersistentTableType = Experimental::TRobinHoodHashMap<FGraphicsMinimalPipelineStateInitializer, FRefCountedGraphicsMinimalPipelineState>;
	static PersistentTableType PersistentIdTable;
	static bool NeedsShaderInitialisation;

	static bool bIsIdTableFrozen;
	static std::atomic<int32> ReffedItemCount;	// Number of items with a non-zero reference count

#if MESH_DRAW_COMMAND_DEBUG_DATA
	int32 DebugSalt;
	static std::atomic<int32> DebugSaltAllocationIndex;
	static std::atomic<int32> LocalPipelineIdTableSize;
	static std::atomic<int32> CurrentLocalPipelineIdTableSize;
#endif //MESH_DRAW_COMMAND_DEBUG_DATA
};

class FShaderBindingState
{
public:
	enum { MAX_UNIFORM_BUFFERS_PER_STAGE = 14 };

	int32 MaxUniformBufferUsed = -1;
	FRHIUniformBuffer* UniformBuffers[MAX_UNIFORM_BUFFERS_PER_STAGE] = {};
};

struct FMeshProcessorShaders
{
	TShaderRef<FShader> MeshShader;
	TShaderRef<FShader> VertexShader;
	TShaderRef<FShader> PixelShader;
	TShaderRef<FShader> GeometryShader;
	TShaderRef<FShader> ComputeShader;
	TShaderRef<FShader> WorkGraphShader;
#if RHI_RAYTRACING
	TShaderRef<FShader> RayTracingShader;
#endif

	TShaderRef<FShader> GetShader(EShaderFrequency Frequency) const
	{
		if (Frequency == SF_Vertex)
		{
			return VertexShader;
		}
		if (Frequency == SF_Mesh)
		{
			return MeshShader;
		}
		if (Frequency == SF_Pixel)
		{
			return PixelShader;
		}
		if (Frequency == SF_Geometry)
		{
			return GeometryShader;
		}
		if (Frequency == SF_Compute)
		{
			return ComputeShader;
		}
		if (Frequency == SF_WorkGraphComputeNode)
		{
			return WorkGraphShader;
		}
#if RHI_RAYTRACING
		if (Frequency == SF_RayHitGroup || Frequency == SF_RayCallable || Frequency == SF_RayMiss)
		{
			if (RayTracingShader.IsValid() && Frequency != RayTracingShader->GetFrequency())
			{
				checkf(0, TEXT("Requested raytracing shader frequency (%d) doesn't match assigned shader frequency (%d)."), Frequency, RayTracingShader->GetFrequency());
				return TShaderRef<FShader>();
			}
			return RayTracingShader;
		}
#endif // RHI_RAYTRACING

		checkf(0, TEXT("Unhandled shader frequency"));
		return TShaderRef<FShader>();
	}

	TArray<TShaderRef<FShader>, TInlineAllocator<3>> GetValidShaders() const
	{
		TArray<TShaderRef<FShader>, TInlineAllocator<3>> Shaders;
		if (VertexShader.IsValid())
		{
			Shaders.Add(VertexShader);
		}
		if (MeshShader.IsValid())
		{
			Shaders.Add(MeshShader);
		}
		if (PixelShader.IsValid())
		{
			Shaders.Add(PixelShader);
		}
		if (GeometryShader.IsValid())
		{
			Shaders.Add(GeometryShader);
		}
		if (ComputeShader.IsValid())
		{
			Shaders.Add(ComputeShader);
		}
#if RHI_RAYTRACING
		if (RayTracingShader.IsValid())
		{
			Shaders.Add(RayTracingShader);
		}
#endif
		return Shaders;
	}
};

/** 
 * Number of resource bindings to allocate inline within a FMeshDrawCommand.
 * This is tweaked so that the bindings for BasePass shaders of an average material using a FLocalVertexFactory fit into the inline storage.
 * Overflow of the inline storage will cause a heap allocation per draw (and corresponding cache miss on traversal)
 */
inline const int32 NumInlineShaderBindings = 10;

/**
* Debug only data for being able to backtrack the origin of given FMeshDrawCommand.
*/
struct FMeshDrawCommandDebugData
{
#if MESH_DRAW_COMMAND_DEBUG_DATA
	// sorted from larger to smaller to minimize padding
	TShaderRef<FShader> VertexShader;
	TShaderRef<FShader> PixelShader;
	// Warning: the following members may not be valid if state buckets are being used (ie if GPUScene is enabled); they can point to invalid or even deleted resources
	// This is because multiple proxies can share the same MDC. In the case of MaterialRenderProxies, it's rare for them to differ, but it can happen
	const FPrimitiveSceneProxy* PrimitiveSceneProxyIfNotUsingStateBuckets;
	const FMaterialRenderProxy* MaterialRenderProxyIfNotUsingStateBuckets;

	// This could in theory be inaccurate when bucketing is active, but it should be correct the vast majority of the time (even if the Proxy ptr changes), so it's good enough for draw events
	FString MaterialRenderProxyName;

	const FMaterial* Material;
#if PSO_PRECACHING_VALIDATE
	// so far these are only used for PSO precaching validation
	const FVertexFactory* VertexFactory;
	const FVertexFactoryType* VertexFactoryType;
#endif
	FName ResourceName;
#if PSO_PRECACHING_VALIDATE
	uint32 PSOCollectorIndex;
#endif
	int8 LODIndex;
	int32 SegmentIndex;
#endif
};

/** Data needed to resolve mesh draw command stats. */
struct FMeshDrawCommandStatsData
{
#if MESH_DRAW_COMMAND_STATS
	/** ID used to retrieve the component data during stat collection. */
	FName CategoryName;
	/** Total number of vertices for the mesh draw command */
	uint32 NumVertices;
#endif
};

class FMeshDrawCommandStateCache
{
public:

	uint32 PipelineId;
	uint32 StencilRef;
	FShaderBindingState ShaderBindings[SF_NumStandardFrequencies];
	FVertexInputStream VertexStreams[MaxVertexElementCount];

	FMeshDrawCommandStateCache()
	{
		// Must init to impossible values to avoid filtering the first draw's state
		PipelineId = -1;
		StencilRef = -1;
	}

	inline void SetPipelineState(int32 NewPipelineId)
	{
		PipelineId = NewPipelineId;
		StencilRef = -1;

		// Vertex streams must be reset if PSO changes.
		for (int32 VertexStreamIndex = 0; VertexStreamIndex < UE_ARRAY_COUNT(VertexStreams); ++VertexStreamIndex)
		{
			VertexStreams[VertexStreamIndex].VertexBuffer = nullptr;
		}

		// Shader bindings must be reset if PSO changes
		for (int32 FrequencyIndex = 0; FrequencyIndex < UE_ARRAY_COUNT(ShaderBindings); FrequencyIndex++)
		{
			FShaderBindingState& RESTRICT ShaderBinding = ShaderBindings[FrequencyIndex];

			for (int32 SlotIndex = 0; SlotIndex <= ShaderBinding.MaxUniformBufferUsed; SlotIndex++)
			{
				ShaderBinding.UniformBuffers[SlotIndex] = nullptr;
			}

			ShaderBinding.MaxUniformBufferUsed = -1;
		}
	}

	void InvalidateUniformBuffer(const FRHIUniformBuffer* UniformBuffer)
	{
		for (int32 FrequencyIndex = 0; FrequencyIndex < UE_ARRAY_COUNT(ShaderBindings); FrequencyIndex++)
		{
			FShaderBindingState& ShaderBinding = ShaderBindings[FrequencyIndex];
			for (int32 SlotIndex = 0; SlotIndex <= ShaderBinding.MaxUniformBufferUsed; SlotIndex++)
			{
				if (ShaderBinding.UniformBuffers[SlotIndex] == UniformBuffer)
				{
					ShaderBinding.UniformBuffers[SlotIndex] = nullptr;
				}
			}
		}
	}
};


/**
 * Encapsulates shader bindings for a single FMeshDrawCommand.
 */
class FMeshDrawShaderBindings
{
public:
	FMeshDrawShaderBindings()
	{
		static_assert(sizeof(ShaderFrequencyBits) * 8 > SF_NumFrequencies, "Please increase ShaderFrequencyBits size");
	}

	FMeshDrawShaderBindings(FMeshDrawShaderBindings&& Other)
	{
		if (!UsesInlineStorage())
		{
			delete[] Data.GetHeapData();
		}
		Size = Other.Size;
		ShaderFrequencyBits = Other.ShaderFrequencyBits;
		ShaderLayouts = MoveTemp(Other.ShaderLayouts);
		if (Other.UsesInlineStorage())
		{
			Data = MoveTemp(Other.Data);
		}
		else
		{
			Data.SetHeapData(Other.Data.GetHeapData());
			Other.Data.SetHeapData(nullptr);
		}
		Other.Size = 0;
	}

	FMeshDrawShaderBindings(const FMeshDrawShaderBindings& Other)
	{
		CopyFrom(Other);
	}

	RENDERER_API ~FMeshDrawShaderBindings();

	FMeshDrawShaderBindings& operator=(const FMeshDrawShaderBindings& Other)
	{
		CopyFrom(Other);
		return *this;
	}

	FMeshDrawShaderBindings& operator=(FMeshDrawShaderBindings&& Other)
	{
		if (!UsesInlineStorage())
		{
			delete[] Data.GetHeapData();
		}
		Size = Other.Size;
		ShaderFrequencyBits = Other.ShaderFrequencyBits;
		ShaderLayouts = MoveTemp(Other.ShaderLayouts);
		if (Other.UsesInlineStorage())
		{
			Data = MoveTemp(Other.Data);
		}
		else
		{	
			Data.SetHeapData(Other.Data.GetHeapData());
			Other.Data.SetHeapData(nullptr);
		}
		Other.Size = 0;
		return *this;
	}

	/** Allocates space for the bindings of all shaders. */
	RENDERER_API void Initialize(const FMeshProcessorShaders& Shaders);
	RENDERER_API void Initialize(const TShaderRef<FShader>& Shader);

	/** Called once binding setup is complete. */
	RENDERER_API void Finalize(const FMeshProcessorShaders* ShadersForDebugging);

	FORCEINLINE FMeshDrawSingleShaderBindings GetSingleShaderBindings(EShaderFrequency Frequency, int32& DataOffset)
	{
		int FrequencyIndex = FPlatformMath::CountBits(ShaderFrequencyBits & ((1 << (Frequency + 1)) - 1)) - 1;

#if DO_CHECK && !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
		int32 CheckedDataOffset = 0;
		for (int32 BindingIndex = 0; BindingIndex < FrequencyIndex; BindingIndex++)
		{
			CheckedDataOffset += ShaderLayouts[BindingIndex].GetDataSizeBytes();
		}
		checkf(CheckedDataOffset == DataOffset, TEXT("GetSingleShaderBindings was not called in the order of ShaderFrequencies"));
#endif
		if (FrequencyIndex >= 0)
		{
			int32 StartDataOffset = DataOffset;
			DataOffset += ShaderLayouts[FrequencyIndex].GetDataSizeBytes();
			return FMeshDrawSingleShaderBindings(ShaderLayouts[FrequencyIndex], GetData() + StartDataOffset);
		}

		checkf(0, TEXT("Invalid shader binding frequency requested"));
		return FMeshDrawSingleShaderBindings(FMeshDrawShaderBindingsLayout(TShaderRef<FShader>()), nullptr);
	}

	FORCEINLINE FMeshDrawSingleShaderBindings GetSingleShaderBindings(EShaderFrequency Frequency)
	{
		int32 DataOffset = 0;
		return GetSingleShaderBindings(Frequency, DataOffset);
	}

	/** Set shader bindings on the commandlist, filtered by state cache. */
	RENDERER_API void SetOnCommandList(FRHICommandList& RHICmdList, const FBoundShaderStateInput& Shaders, class FShaderBindingState* StateCacheShaderBindings) const;

	RENDERER_API void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, class FShaderBindingState* StateCacheShaderBindings = nullptr) const;
	RENDERER_API void SetOnCommandList(FRHIComputeCommandList& RHICmdList, FRHIComputeShader* Shader, class FShaderBindingState* StateCacheShaderBindings = nullptr) const;

#if RHI_RAYTRACING
	RENDERER_API FRayTracingLocalShaderBindings* SetRayTracingShaderBindings(FRayTracingLocalShaderBindingWriter* BindingWriter, uint32 ShaderIndexInPipeline, uint32 RecordIndex, uint32 UserData, ERayTracingLocalShaderBindingType BindingType) const;
	RENDERER_API FRayTracingLocalShaderBindings* SetRayTracingShaderBindingsForHitGroup(FRayTracingLocalShaderBindingWriter* BindingWriter, uint32 RecordIndex, const FRHIRayTracingGeometry* Geometry, uint32 GeometrySegmentIndex, uint32 HitGroupIndexInPipeline, ERayTracingLocalShaderBindingType BindingType) const;

	// TODO: should these move to a binding writer too? should we introduce a different class to do these bindings since they aren't mesh related? rename this class entirely?
	void SetRayTracingShaderBindingsForMissShader(FRHICommandList& RHICmdList, FRHIShaderBindingTable* SBT, uint32 RecordIndex, FRayTracingPipelineState* Pipeline, uint32 ShaderIndexInPipeline) const;
#endif // RHI_RAYTRACING

	/** Returns whether this set of shader bindings can be merged into an instanced draw call with another. */
	bool RENDERER_API MatchesForDynamicInstancing(const FMeshDrawShaderBindings& Rhs) const;

	uint32 RENDERER_API GetDynamicInstancingHash() const;

	SIZE_T GetAllocatedSize() const
	{
		SIZE_T Bytes = ShaderLayouts.GetAllocatedSize();
		if (!UsesInlineStorage())
		{
			Bytes += Size;
		}

		return Bytes;
	}

	void GetShaderFrequencies(TArray<EShaderFrequency, TInlineAllocator<SF_NumFrequencies>>& OutShaderFrequencies) const
	{
		OutShaderFrequencies.Empty(ShaderLayouts.Num());

		for (int32 BindingIndex = 0; BindingIndex < SF_NumFrequencies; BindingIndex++)
		{
			if ((ShaderFrequencyBits & (1 << BindingIndex)) != 0)
			{
				OutShaderFrequencies.Add(EShaderFrequency(BindingIndex));
			}
		}
	}

	inline int32 GetDataSize() const { return Size; }

private:
	TArray<FMeshDrawShaderBindingsLayout, TInlineAllocator<2>> ShaderLayouts;
	struct FData
	{
		uint8* InlineStorage[NumInlineShaderBindings] = {};
		uint8* GetHeapData()
		{
			return InlineStorage[0];
		}
		const uint8* GetHeapData() const
		{
			return InlineStorage[0];
		}
		void SetHeapData(uint8* HeapData)
		{
			InlineStorage[0] = HeapData;
		}
	} Data = {};
	uint16 ShaderFrequencyBits = 0;
	uint16 Size = 0;

	void Allocate(uint16 InSize)
	{
		check(Size == 0 && Data.GetHeapData() == nullptr);

		Size = InSize;

		if (InSize > sizeof(FData))
		{
			Data.SetHeapData(new uint8[InSize]);
		}
	}

	void AllocateZeroed(uint16 InSize) 
	{
		Allocate(InSize);

		// Verify no type overflow
		check(Size == InSize);

		if (!UsesInlineStorage())
		{
			FPlatformMemory::Memzero(GetData(), InSize);
		}
	}

	inline bool UsesInlineStorage() const
	{
		return Size <= sizeof(FData);
	}

	uint8* GetData()
	{
		return UsesInlineStorage() ? reinterpret_cast<uint8*>(&Data.InlineStorage[0]) : Data.GetHeapData();
	}

	const uint8* GetData() const
	{
		return UsesInlineStorage() ? reinterpret_cast<const uint8*>(&Data.InlineStorage[0]) : Data.GetHeapData();
	}

	RENDERER_API void CopyFrom(const FMeshDrawShaderBindings& Other);

	RENDERER_API void Release();
};

class FReadOnlyMeshDrawSingleShaderBindings : public FMeshDrawShaderBindingsLayout
{
public:
	static RENDERER_API void SetShaderBindings(
		FRHIBatchedShaderParameters& BatchedParameters,
		const class FReadOnlyMeshDrawSingleShaderBindings& RESTRICT SingleShaderBindings,
		FShaderBindingState& RESTRICT ShaderBindingState);

	static RENDERER_API void SetShaderBindings(
		FRHIBatchedShaderParameters& BatchedParameters,
		const class FReadOnlyMeshDrawSingleShaderBindings& RESTRICT SingleShaderBindings);
	
	explicit FReadOnlyMeshDrawSingleShaderBindings(const FMeshDrawSingleShaderBindings& Bindings) :
		FMeshDrawShaderBindingsLayout(Bindings)
	{
		Data = Bindings.Data;
	}

	FReadOnlyMeshDrawSingleShaderBindings(const FMeshDrawShaderBindingsLayout& InLayout, const uint8* InData) :
		FMeshDrawShaderBindingsLayout(InLayout)
	{
		Data = InData;
	}

	inline FRHIUniformBuffer*const* GetUniformBufferStart() const
	{
		return (FRHIUniformBuffer**)(Data + GetUniformBufferOffset());
	}

	inline FRHISamplerState** GetSamplerStart() const
	{
		const uint8* SamplerDataStart = Data + GetSamplerOffset();
		return (FRHISamplerState**)SamplerDataStart;
	}

	inline FRHIResource** GetSRVStart() const
	{
		const uint8* SRVDataStart = Data + GetSRVOffset();
		return (FRHIResource**)SRVDataStart;
	}

	inline const uint8* GetSRVTypeStart() const
	{
		const uint8* SRVTypeDataStart = Data + GetSRVTypeOffset();
		return SRVTypeDataStart;
	}

	inline const uint8* GetLooseDataStart() const
	{
		const uint8* LooseDataStart = Data + GetLooseDataOffset();
		return LooseDataStart;
	}

private:
	const uint8* Data;
};

struct FMeshDrawCommandOverrideArgs
{
	FRHIBuffer* InstanceBuffer;
	FRHIBuffer* IndirectArgsBuffer;
	FRHIUniformBuffer* InstanceCullingStaticUB;
	uint32 InstanceDataByteOffset;
	uint32 IndirectArgsByteOffset;
	TOptional<FUintVector4> RootConstants;
	
	FMeshDrawCommandOverrideArgs()
	{
		InstanceBuffer = nullptr;
		IndirectArgsBuffer = nullptr;
		InstanceCullingStaticUB = nullptr;
		InstanceDataByteOffset = 0u;
		IndirectArgsByteOffset = 0u;
	}
};

struct FMeshDrawCommandSceneArgs
{
	FRHIBuffer* PrimitiveIdsBuffer;
	FRHIBuffer* IndirectArgsBuffer;
	uint32 PrimitiveIdOffset;
	uint32 IndirectArgsByteOffset;
	FUniformBufferStaticSlot BatchedPrimitiveSlot;
	TOptional<FUintVector4> RootConstants;

	FMeshDrawCommandSceneArgs()
	{
		PrimitiveIdsBuffer = nullptr;
		IndirectArgsBuffer = nullptr;
		PrimitiveIdOffset = 0u;
		IndirectArgsByteOffset = 0u;
		BatchedPrimitiveSlot = MAX_UNIFORM_BUFFER_STATIC_SLOTS;
	}
};

/** 
 * FMeshDrawCommand fully describes a mesh pass draw call, captured just above the RHI.  
		FMeshDrawCommand should contain only data needed to draw.  For InitViews payloads, use FVisibleMeshDrawCommand.
 * FMeshDrawCommands are cached at Primitive AddToScene time for vertex factories that support it (no per-frame or per-view shader binding changes).
 * Dynamic Instancing operates at the FMeshDrawCommand level for robustness.  
		Adding per-command shader bindings will reduce the efficiency of Dynamic Instancing, but rendering will always be correct.
 * Any resources referenced by a command must be kept alive for the lifetime of the command.  FMeshDrawCommand is not responsible for lifetime management of resources.
		For uniform buffers referenced by cached FMeshDrawCommand's, RHIUpdateUniformBuffer makes it possible to access per-frame data in the shader without changing bindings.
 */
class FMeshDrawCommand
{
public:
	
	/**
	 * Resource bindings
	 */
	FMeshDrawShaderBindings ShaderBindings;
	FVertexInputStreamArray VertexStreams;
	FRHIBuffer* IndexBuffer;

	/**
	 * PSO
	 */
	FGraphicsMinimalPipelineStateId CachedPipelineId;
	
	/**
	 * Draw command parameters
	 */
	uint32 FirstIndex;
	uint32 NumPrimitives;
	uint32 NumInstances;

	union
	{
		struct 
		{
			uint32 BaseVertexIndex;
			uint32 NumVertices;
		} VertexParams;
		
		struct  
		{
			FRHIBuffer* Buffer;
			uint32 Offset;
		} IndirectArgs;
	};

	int8 PrimitiveIdStreamIndex;

	/** Non-pipeline state */
	uint8 StencilRef;

	/** Redundant as already present in CachedPipelineId, but need access for dynamic instancing on GPU. */
	EPrimitiveType PrimitiveType : PT_NumBits;

	FMeshDrawCommand() {};
	FMeshDrawCommand(FMeshDrawCommand&& Other) = default;
	FMeshDrawCommand(const FMeshDrawCommand& Other) = default;
	FMeshDrawCommand& operator=(const FMeshDrawCommand& Other) = default;
	FMeshDrawCommand& operator=(FMeshDrawCommand&& Other) = default; 

	bool MatchesForDynamicInstancing(const FMeshDrawCommand& Rhs) const
	{
		return CachedPipelineId == Rhs.CachedPipelineId
			&& StencilRef == Rhs.StencilRef
			&& ShaderBindings.MatchesForDynamicInstancing(Rhs.ShaderBindings)
			&& VertexStreams == Rhs.VertexStreams
			&& PrimitiveIdStreamIndex == Rhs.PrimitiveIdStreamIndex
			&& IndexBuffer == Rhs.IndexBuffer
			&& FirstIndex == Rhs.FirstIndex
			&& NumPrimitives == Rhs.NumPrimitives
			&& NumInstances == Rhs.NumInstances
			&& ((NumPrimitives > 0 && VertexParams.BaseVertexIndex == Rhs.VertexParams.BaseVertexIndex && VertexParams.NumVertices == Rhs.VertexParams.NumVertices)
				|| (NumPrimitives == 0 && IndirectArgs.Buffer == Rhs.IndirectArgs.Buffer && IndirectArgs.Offset == Rhs.IndirectArgs.Offset));
	}

	uint32 GetDynamicInstancingHash() const
	{
		//add and initialize any leftover padding within the struct to avoid unstable keys
		struct FHashKey
		{
			uint32 IndexBuffer;
			uint32 VertexBuffers = 0;
		    uint32 VertexStreams = 0;
			uint32 PipelineId;
			uint32 DynamicInstancingHash;
			uint32 FirstIndex;
			uint32 NumPrimitives;
			uint32 NumInstances;
			uint32 IndirectArgsBufferOrBaseVertexIndex;
			uint32 NumVertices;
			uint32 StencilRefAndPrimitiveIdStreamIndex;

			static inline uint32 PointerHash(const void* Key)
			{
				// Ignoring the lower 4 bits since they are likely zero anyway.
				// Higher bits are more significant in 64 bit builds.
				return static_cast<uint32>(reinterpret_cast<UPTRINT>(Key) >> 4);
			};

			static inline uint32 HashCombine(uint32 A, uint32 B)
			{
				return A ^ (B + 0x9e3779b9 + (A << 6) + (A >> 2));
			}
		} HashKey;

		HashKey.PipelineId = CachedPipelineId.GetId();
		HashKey.StencilRefAndPrimitiveIdStreamIndex = StencilRef | (PrimitiveIdStreamIndex << 8);
		HashKey.DynamicInstancingHash = ShaderBindings.GetDynamicInstancingHash();

		for (int index = 0; index < VertexStreams.Num(); index++)
		{
			const FVertexInputStream& VertexInputStream = VertexStreams[index];
			const uint32 StreamIndex = VertexInputStream.StreamIndex;
			const uint32 Offset = VertexInputStream.Offset;

			uint32 Packed = (StreamIndex << 28) | Offset;
			HashKey.VertexStreams = FHashKey::HashCombine(HashKey.VertexStreams, Packed);
			HashKey.VertexBuffers = FHashKey::HashCombine(HashKey.VertexBuffers, FHashKey::PointerHash(VertexInputStream.Pointer));
		}

		HashKey.IndexBuffer = FHashKey::PointerHash(IndexBuffer);
		HashKey.FirstIndex = FirstIndex;
		HashKey.NumPrimitives = NumPrimitives;
		HashKey.NumInstances = NumInstances;

		if (NumPrimitives > 0)
		{
			HashKey.IndirectArgsBufferOrBaseVertexIndex = VertexParams.BaseVertexIndex;
			HashKey.NumVertices = VertexParams.NumVertices;
		}
		else
		{
			HashKey.IndirectArgsBufferOrBaseVertexIndex = FHashKey::PointerHash(IndirectArgs.Buffer);
			HashKey.NumVertices = IndirectArgs.Offset;
		}		

		return uint32(CityHash64((char*)&HashKey, sizeof(FHashKey)));
	}

	/** Allocates room for the shader bindings. */
	inline void InitializeShaderBindings(const FMeshProcessorShaders& Shaders)
	{
		ShaderBindings.Initialize(Shaders);
	}

	inline void SetStencilRef(uint32 InStencilRef)
	{
		StencilRef = static_cast<uint8>(InStencilRef);
		// Verify no overflow
		checkSlow((uint32)StencilRef == InStencilRef);
	}

	/** Called when the mesh draw command is complete. */
	RENDERER_API void SetDrawParametersAndFinalize(
		const FMeshBatch& MeshBatch, 
		int32 BatchElementIndex,
		FGraphicsMinimalPipelineStateId PipelineId,
		const FMeshProcessorShaders* ShadersForDebugging);

	void Finalize(FGraphicsMinimalPipelineStateId PipelineId, const FMeshProcessorShaders* ShadersForDebugging)
	{
		CachedPipelineId = PipelineId;
		ShaderBindings.Finalize(ShadersForDebugging);	
	}

	/** 
	 * Submits the state and shader bindings to the RHI command list, but does not invoke the draw. 	 
	 * If function returns false, then draw can be skipped because the PSO used by the draw command is not compiled yet (still precaching)
	 */
	static bool SubmitDrawBegin(
		const FMeshDrawCommand& RESTRICT MeshDrawCommand,
		const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
		const FMeshDrawCommandSceneArgs& SceneArgs,
		uint32 InstanceFactor,
		FRHICommandList& RHICmdList,
		FMeshDrawCommandStateCache& RESTRICT StateCache,
		bool bAllowSkipDrawCommand);

	/** Submits just the draw primitive portion of the draw command. */
	static void SubmitDrawEnd(const FMeshDrawCommand& MeshDrawCommand, const FMeshDrawCommandSceneArgs& SceneArgs, uint32 InstanceFactor, FRHICommandList& RHICmdList);

	/** 	 
	 * Submits the state and shader bindings to the RHI command list, but does not invoke the draw indirect. 
	 * If function returns false, then draw will be skipped because PSO used by the draw command is not compiled yet (still precaching)
	 */
	static bool SubmitDrawIndirectBegin(
		const FMeshDrawCommand& RESTRICT MeshDrawCommand,
		const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
		const FMeshDrawCommandSceneArgs& SceneArgs,
		uint32 InstanceFactor,
		FRHICommandList& RHICmdList,
		FMeshDrawCommandStateCache& RESTRICT StateCache,
		bool bAllowSkipDrawCommand);

	/** Submits just the draw indirect primitive portion of the draw command. */
	static void SubmitDrawIndirectEnd(const FMeshDrawCommand& MeshDrawCommand, const FMeshDrawCommandSceneArgs& SceneArgs, uint32 InstanceFactor, FRHICommandList& RHICmdList);

	/** Submits commands to the RHI Commandlist to draw the MeshDrawCommand. */
	static bool SubmitDraw(
		const FMeshDrawCommand& RESTRICT MeshDrawCommand,
		const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
		const FMeshDrawCommandSceneArgs& SceneArgs,
		uint32 InstanceFactor,
		FRHICommandList& CommandList,
		class FMeshDrawCommandStateCache& RESTRICT StateCache);

	/** Returns the pipeline state sort key, which can be used for sorting material draws to reduce context switches. */
	uint64 GetPipelineStateSortingKey(FRHICommandList& RHICmdList, const FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo) const;

	/** Returns the pipeline state sort key, which can be used for sorting material draws to reduce context switches. This variant will not attempt to create a PSO if it doesn't exist yet and will just return the fallback. */
	uint64 GetPipelineStateSortingKey(const FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo) const;

	FORCENOINLINE friend uint32 GetTypeHash( const FMeshDrawCommand& Other )
	{
		return Other.CachedPipelineId.GetId();
	}
#if MESH_DRAW_COMMAND_DEBUG_DATA
	RENDERER_API void SetDebugData(const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FMaterial* Material, const FMaterialRenderProxy* MaterialRenderProxy, const FMeshProcessorShaders& UntypedShaders, const FVertexFactory* VertexFactory, const FMeshBatch& MeshBatch, int32 PSOCollectorIndex);
#else
	void SetDebugData(const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FMaterial* Material, const FMaterialRenderProxy* MaterialRenderProxy, const FMeshProcessorShaders& UntypedShaders, const FVertexFactory* VertexFactory, const FMeshBatch& MeshBatch, int32 PSOCollectorIndex) {}
#endif

	SIZE_T GetAllocatedSize() const
	{
		return ShaderBindings.GetAllocatedSize() + VertexStreams.GetAllocatedSize();
	}

	SIZE_T GetDebugDataSize() const
	{
#if MESH_DRAW_COMMAND_DEBUG_DATA
		return sizeof(DebugData);
#else
		return 0;
#endif
	}

#if MESH_DRAW_COMMAND_DEBUG_DATA
	void ClearDebugPrimitiveSceneProxy() const
	{
		DebugData.PrimitiveSceneProxyIfNotUsingStateBuckets = nullptr;
	}

	const FMeshDrawCommandDebugData& GetDebugData() const
	{
		return DebugData;
	}
private:
	mutable FMeshDrawCommandDebugData DebugData;
#endif

#if MESH_DRAW_COMMAND_STATS
private:
	FMeshDrawCommandStatsData StatsData;
public:
	RENDERER_API void SetStatsData(const FPrimitiveSceneProxy* PrimitiveSceneProxy);
	void GetStatsData(FVisibleMeshDrawCommandStatsData& OutVisibleStatsData) const;
#else
public:
	void SetStatsData(const FPrimitiveSceneProxy* PrimitiveSceneProxy) {}
#endif
};

/** FVisibleMeshDrawCommand sort key. */
class FMeshDrawCommandSortKey
{
public:
	union 
	{
		uint64 PackedData;

		struct
		{
			uint64 VertexShaderHash		: 16; // Order by vertex shader's hash.
			uint64 PixelShaderHash		: 32; // Order by pixel shader's hash.
			uint64 Background			: 1;
			uint64 Masked				: 15; // First order by masked.
		} BasePass;

		struct
		{
			uint64 MeshIdInPrimitive	: 16; // Order meshes belonging to the same primitive by a stable id.
			uint64 Distance				: 32; // Order by distance.
			uint64 Priority				: 16; // First order by priority.
		} Translucent;

		struct 
		{
			uint64 VertexShaderHash : 32;	// Order by vertex shader's hash.
			uint64 PixelShaderHash : 32;	// First order by pixel shader's hash.
		} Generic;
	};

	FORCEINLINE bool operator!=(FMeshDrawCommandSortKey B) const
	{
		return PackedData != B.PackedData;
	}

	FORCEINLINE bool operator<(FMeshDrawCommandSortKey B) const
	{
		return PackedData < B.PackedData;
	}

	static RENDERER_API const FMeshDrawCommandSortKey Default;
};


/** Flags that may be passed along with a visible mesh draw command OR stored with a cached one. */
enum class EFVisibleMeshDrawCommandFlags : uint8
{
	Default = 0U,
	/** If set, the FMaterial::MaterialUsesWorldPositionOffset_RenderThread() indicates that WPO is active for the given material. */
	MaterialUsesWorldPositionOffset = 1U << 0U,

	/**
	 * If set, the FMaterial::ShouldAlwaysEvaluateWorldPositionOffset() indicates that WPO is ALWAYS active for the given material.
	 * NOTE: This flag is only set if MaterialUsesWorldPositionOffset is also set.
	 */
	MaterialAlwaysEvaluatesWorldPositionOffset = 1U << 1U,

	/** If set, the mesh draw command supports primitive ID steam (required for dynamic instancing and GPU-Scene instance culling). */
	HasPrimitiveIdStreamIndex = 1U << 2U,

	/** If set, forces individual instances to always be culled independently from the primitive */
	ForceInstanceCulling = 1U << 3U,

	/** If set, requires that instances preserve their original draw order in the draw command */
	PreserveInstanceOrder = 1U << 4U,

	/** If set, the instance culling machinery will pull the instance count from the primitive in FScene, making it possible to bypass MDC re-caching */
	FetchInstanceCountFromScene = 1U << 5U,

	All = MaterialUsesWorldPositionOffset | HasPrimitiveIdStreamIndex | ForceInstanceCulling | PreserveInstanceOrder | FetchInstanceCountFromScene,
	NumBits = 6U
};
ENUM_CLASS_FLAGS(EFVisibleMeshDrawCommandFlags);
static_assert(uint32(EFVisibleMeshDrawCommandFlags::All) < (1U << uint32(EFVisibleMeshDrawCommandFlags::NumBits)), "EFVisibleMeshDrawCommandFlags::NumBits too small to represent all flags in EFVisibleMeshDrawCommandFlags.");

/**
 * Wrapper to make it harder to confuse the packed and persistent index when used as arguments etc.
 */
struct FPersistentPrimitiveIndex
{
	bool IsValid() const { return Index != INDEX_NONE; }
	int32 Index = INDEX_NONE;

	FORCEINLINE bool operator == (FPersistentPrimitiveIndex B) const
	{
		return Index == B.Index;
	}

	friend uint32 GetTypeHash(FPersistentPrimitiveIndex PersistentPrimitiveIndex)
	{
		return GetTypeHash(PersistentPrimitiveIndex.Index);
	}
};

/**
 * Container for primtive ID info that needs to be passed around, in the future will likely be condensed to just the instance ID.
 */
struct FMeshDrawCommandPrimitiveIdInfo
{
	FORCEINLINE FMeshDrawCommandPrimitiveIdInfo() {};

	// Use this ctor for scene primitves
	FORCEINLINE FMeshDrawCommandPrimitiveIdInfo(int32 InScenePrimitiveId, FPersistentPrimitiveIndex InDrawPrimitiveId, int32 InInstanceSceneDataOffset) :
		DrawPrimitiveId(InDrawPrimitiveId.Index),
		ScenePrimitiveId(InScenePrimitiveId),
		InstanceSceneDataOffset(InInstanceSceneDataOffset),
		bIsDynamicPrimitive(0)
	{
	}

	UE_DEPRECATED(5.4, "Use the above Ctor instead.")
	// Use this ctor when for dynamic primitives like editor widgets
	FORCEINLINE FMeshDrawCommandPrimitiveIdInfo(int32 InDrawPrimitiveId, int32 InScenePrimitiveId, int32 InInstanceSceneDataOffset) :
		DrawPrimitiveId(InDrawPrimitiveId),
		ScenePrimitiveId(InScenePrimitiveId),
		InstanceSceneDataOffset(InInstanceSceneDataOffset),
		bIsDynamicPrimitive(0)
	{
	}

	// Draw PrimitiveId this draw command is associated with - used by the shader to fetch primitive data from the PrimitiveSceneData SRV.
	// If it's < Scene->Primitives.Num() then it's a valid Scene PrimitiveIndex and can be used to backtrack to the FPrimitiveSceneInfo.
	int32 DrawPrimitiveId;

	// Scene PrimitiveId that generated this draw command, or -1 if no FPrimitiveSceneInfo. Can be used to backtrack to the FPrimitiveSceneInfo.
	int32 ScenePrimitiveId;

	// Offset to the first instance belonging to the primitive in GPU scene
	int32 InstanceSceneDataOffset : 31;
	
	// Set to true if the primitive ID and instance data offset is a dynamic ID, which means it needs to be translated before use.
	uint32 bIsDynamicPrimitive : 1;
};

/**
 * Container for mesh command data required by instance culling.
 */
struct FMeshDrawCommandCullingPayload
{
	union
	{
		uint32 PackedData = 0;
		struct
		{
			uint32 LodIndex : 4;
			uint32 MinScreenSize : 13;
			uint32 MaxScreenSize : 13;
		};
	};

	int32 DynamicMeshBoundsIndex = -1;

	// Pack screen sizes into fixed point 3.10
	static uint32 PackScreenSize(float ScreenSize)
	{
		constexpr float MaxExpressibleValue = (float)(0x1FFFU) / (float)(1 << 10);
		ScreenSize = FMath::Clamp(ScreenSize, 0.f, MaxExpressibleValue);
		return (uint32)(ScreenSize * (1 << 10)) & 0x1FFF;
	}

	static float UnpackScreenSize(uint32 PackedScreenSize)
	{
		return (float)(PackedScreenSize & 0x1FFFU) / (float)(1 << 10);
	}
};

static FMeshDrawCommandCullingPayload CreateCullingPayload(FMeshBatch const& MeshBatch, FMeshBatchElement const& MeshBatchElement)
{
	FMeshDrawCommandCullingPayload CullingPayload;
	CullingPayload.LodIndex = MeshBatch.LODIndex;
	CullingPayload.MinScreenSize = FMeshDrawCommandCullingPayload::PackScreenSize(MeshBatchElement.MinScreenSize);
	CullingPayload.MaxScreenSize = FMeshDrawCommandCullingPayload::PackScreenSize(MeshBatchElement.MaxScreenSize);
	CullingPayload.DynamicMeshBoundsIndex = MeshBatch.DynamicMeshBoundsIndex;
	return CullingPayload;
}

/**
 * Flags stored on visible mesh commands to indicate how to interpret the culling payload.
 */
enum class EMeshDrawCommandCullingPayloadFlags : uint8
{
	Default = 0U,
	MinScreenSizeCull = 1U, // Cull when below a minimum screen space size.
	MaxScreenSizeCull = 2U, // Cull when above a maximum screen space size.
	
	NoScreenSizeCull = Default,
	All = MinScreenSizeCull | MaxScreenSizeCull,
	NumBits = 2U
};
ENUM_CLASS_FLAGS(EMeshDrawCommandCullingPayloadFlags);
static_assert(uint32(EMeshDrawCommandCullingPayloadFlags::All) < (1U << uint32(EMeshDrawCommandCullingPayloadFlags::NumBits)), "EMeshDrawCommandCullingPayloadFlags::NumBits too small to represent all flags in EMeshDrawCommandCullingPayloadFlags.");


/** Interface for the different types of draw lists. */
class FMeshPassDrawListContext
{
public:

	virtual ~FMeshPassDrawListContext() = default;

	virtual FMeshDrawCommand& AddCommand(FMeshDrawCommand& Initializer, uint32 NumElements) = 0;

	virtual void FinalizeCommand(
		const FMeshBatch& MeshBatch, 
		int32 BatchElementIndex,
		const FMeshDrawCommandPrimitiveIdInfo& IdInfo,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		FMeshDrawCommandSortKey SortKey,
		EFVisibleMeshDrawCommandFlags Flags,
		const FGraphicsMinimalPipelineStateInitializer& PipelineState,
		const FMeshProcessorShaders* ShadersForDebugging,
		FMeshDrawCommand& MeshDrawCommand) = 0;
};

/** Storage for Mesh Draw Commands built every frame. */
class FDynamicMeshDrawCommandStorage
{
public:
	// Using TChunkedArray to support growing without moving FMeshDrawCommand, since FVisibleMeshDrawCommand stores a pointer to these
	TChunkedArray<FMeshDrawCommand> MeshDrawCommands;
};

/** 
 * Stores information about a mesh draw command that has been determined to be visible, for further visibility processing. 
 * This class should only store data needed by InitViews operations (visibility, sorting) and not data needed for draw submission, which belongs in FMeshDrawCommand.
 */
class FVisibleMeshDrawCommand
{
public:

	// Note: no ctor as TChunkedArray::CopyToLinearArray requires POD types

	FORCEINLINE_DEBUGGABLE void Setup(
		const FMeshDrawCommand* InMeshDrawCommand,
		const FMeshDrawCommandPrimitiveIdInfo& InPrimitiveIdInfo,
		int32 InStateBucketId,
		ERasterizerFillMode InMeshFillMode,
		ERasterizerCullMode InMeshCullMode,
		EFVisibleMeshDrawCommandFlags InFlags,
		FMeshDrawCommandSortKey InSortKey,
		FMeshDrawCommandCullingPayload InCullingPayload,
		EMeshDrawCommandCullingPayloadFlags InCullingPayloadFlags,
		const uint32* InRunArray = nullptr,
		int32 InNumRuns = 0)
	{
		MeshDrawCommand = InMeshDrawCommand;
		PrimitiveIdInfo = InPrimitiveIdInfo;
		PrimitiveIdBufferOffset = -1;
		StateBucketId = InStateBucketId;
		MeshFillMode = InMeshFillMode;
		MeshCullMode = InMeshCullMode;
		SortKey = InSortKey;
		Flags = InFlags;
		CullingPayload = InCullingPayload;
		CullingPayloadFlags = InCullingPayloadFlags;
		RunArray = InRunArray;
		NumRuns = InNumRuns;
	}

	// Mesh Draw Command stored separately to avoid fetching its data during sorting
	const FMeshDrawCommand* MeshDrawCommand;

	// Sort key for non state based sorting (e.g. sort translucent draws by depth).
	FMeshDrawCommandSortKey SortKey;

	FMeshDrawCommandPrimitiveIdInfo PrimitiveIdInfo;

	// Offset into the buffer of PrimitiveIds built for this pass, in int32's.
	int32 PrimitiveIdBufferOffset;

	// Dynamic instancing state bucket ID.  
	// Any commands with the same StateBucketId can be merged into one draw call with instancing.
	// A value of -1 means the draw is not in any state bucket and should be sorted by other factors instead.
	int32 StateBucketId;
	
	// Used for passing sub-selection of instances through to the culling
	const uint32* RunArray;
	int32 NumRuns;

	// Used for passing LOD info to the culling
	FMeshDrawCommandCullingPayload CullingPayload;
	EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags : uint32(EMeshDrawCommandCullingPayloadFlags::NumBits);

	// Needed for view overrides
	ERasterizerFillMode MeshFillMode : ERasterizerFillMode_NumBits + 1;
	ERasterizerCullMode MeshCullMode : ERasterizerCullMode_NumBits + 1;

	EFVisibleMeshDrawCommandFlags Flags : uint32(EFVisibleMeshDrawCommandFlags::NumBits);
};

struct FCompareFMeshDrawCommands
{
	FORCEINLINE bool operator() (const FVisibleMeshDrawCommand& A, const FVisibleMeshDrawCommand& B) const
	{
		// First order by a sort key.
		if (A.SortKey != B.SortKey)
		{
			return A.SortKey < B.SortKey;
		}

		// Next order by instancing bucket.
		if (A.StateBucketId != B.StateBucketId)
		{
			return A.StateBucketId < B.StateBucketId;
		}

		return false;
	}
};

template <>
struct TUseBitwiseSwap<FVisibleMeshDrawCommand>
{
	// Prevent Memcpy call overhead during FVisibleMeshDrawCommand sorting
	enum { Value = false };
};

typedef TArray<FVisibleMeshDrawCommand, SceneRenderingAllocator> FMeshCommandOneFrameArray;
typedef TMap<int32, FUniformBufferRHIRef, SceneRenderingSetAllocator> FTranslucentSelfShadowUniformBufferMap;

/** Context used when building FMeshDrawCommands for one frame only. */
class FDynamicPassMeshDrawListContext : public FMeshPassDrawListContext
{
public:
	FDynamicPassMeshDrawListContext
	(
		FDynamicMeshDrawCommandStorage& InDrawListStorage, 
		FMeshCommandOneFrameArray& InDrawList,
		FGraphicsMinimalPipelineStateSet& InPipelineStateSet,
		bool& InNeedsShaderInitialisation
	) :
		DrawListStorage(InDrawListStorage),
		DrawList(InDrawList),
		GraphicsMinimalPipelineStateSet(InPipelineStateSet),
		NeedsShaderInitialisation(InNeedsShaderInitialisation)
	{}

	virtual FMeshDrawCommand& AddCommand(FMeshDrawCommand& Initializer, uint32 NumElements) override final
	{
		const int32 Index = DrawListStorage.MeshDrawCommands.AddElement(Initializer);
		FMeshDrawCommand& NewCommand = DrawListStorage.MeshDrawCommands[Index];
		return NewCommand;
	}

	virtual void FinalizeCommand(
		const FMeshBatch& MeshBatch, 
		int32 BatchElementIndex,
		const FMeshDrawCommandPrimitiveIdInfo &IdInfo,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		FMeshDrawCommandSortKey SortKey,
		EFVisibleMeshDrawCommandFlags Flags,
		const FGraphicsMinimalPipelineStateInitializer& PipelineState,
		const FMeshProcessorShaders* ShadersForDebugging,
		FMeshDrawCommand& MeshDrawCommand) override final
	{
		FGraphicsMinimalPipelineStateId PipelineId = FGraphicsMinimalPipelineStateId::GetPipelineStateId(PipelineState, GraphicsMinimalPipelineStateSet, NeedsShaderInitialisation);

		MeshDrawCommand.SetDrawParametersAndFinalize(MeshBatch, BatchElementIndex, PipelineId, ShadersForDebugging);

		FVisibleMeshDrawCommand NewVisibleMeshDrawCommand;
		//@todo MeshCommandPipeline - assign usable state ID for dynamic path draws
		// Currently dynamic path draws will not get dynamic instancing, but they will be roughly sorted by state
		const FMeshBatchElement& MeshBatchElement = MeshBatch.Elements[BatchElementIndex];
		NewVisibleMeshDrawCommand.Setup(&MeshDrawCommand, IdInfo, -1, MeshFillMode, MeshCullMode, Flags, SortKey,
			CreateCullingPayload(MeshBatch, MeshBatchElement),
			EMeshDrawCommandCullingPayloadFlags::NoScreenSizeCull,
			MeshBatchElement.bIsInstanceRuns ? MeshBatchElement.InstanceRuns : nullptr,
			MeshBatchElement.bIsInstanceRuns ? MeshBatchElement.NumInstances : 0
			);
		DrawList.Add(NewVisibleMeshDrawCommand);
	}

	/**
	 * Use to add pre-built (cached) draw commands to a dynamic context. 
	 */
	FORCEINLINE void AddVisibleMeshDrawCommand(const FVisibleMeshDrawCommand& VisibleMeshDrawCommand)
	{
		DrawList.Add(VisibleMeshDrawCommand);
	}

private:
	FDynamicMeshDrawCommandStorage& DrawListStorage;
	FMeshCommandOneFrameArray& DrawList;
	FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet;
	bool& NeedsShaderInitialisation;
};

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack (push,4)
#endif

/** 
 * Stores information about a mesh draw command which is cached in the scene. 
 * This is stored separately from the cached FMeshDrawCommand so that InitViews does not have to load the FMeshDrawCommand into cache.
 */
class FCachedMeshDrawCommandInfo
{
public:
	FCachedMeshDrawCommandInfo() : FCachedMeshDrawCommandInfo(EMeshPass::Num)
	{}

	explicit FCachedMeshDrawCommandInfo(EMeshPass::Type InMeshPass) :
		SortKey(FMeshDrawCommandSortKey::Default),
		CommandIndex(INDEX_NONE),
		StateBucketId(INDEX_NONE),
		MeshPass(InMeshPass),
		MeshFillMode(ERasterizerFillMode_Num),
		MeshCullMode(ERasterizerCullMode_Num),
		Flags(EFVisibleMeshDrawCommandFlags::Default)
	{}

	FMeshDrawCommandSortKey SortKey;

	// Stores the index into FScene::CachedDrawLists of the corresponding FMeshDrawCommand, or -1 if not stored there
	int32 CommandIndex;

	// Stores the index into FScene::CachedMeshDrawCommandStateBuckets of the corresponding FMeshDrawCommand, or -1 if not stored there
	int32 StateBucketId;

	// Used for passing LOD info to the culling
	FMeshDrawCommandCullingPayload CullingPayload;

	// Needed for easier debugging and faster removal of cached mesh draw commands.
	EMeshPass::Type MeshPass : EMeshPass::NumBits + 1;

	// Needed for view overrides
	ERasterizerFillMode MeshFillMode : ERasterizerFillMode_NumBits + 1;
	ERasterizerCullMode MeshCullMode : ERasterizerCullMode_NumBits + 1;

	EFVisibleMeshDrawCommandFlags Flags : uint32(EFVisibleMeshDrawCommandFlags::NumBits);
};

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack (pop)
#endif

class FCachedPassMeshDrawList
{
public:

	FCachedPassMeshDrawList() :
		LowestFreeIndexSearchStart(0)
	{}

	/** Indices held by FStaticMeshBatch::CachedMeshDrawCommands must be stable */
	TSparseArray<FMeshDrawCommand> MeshDrawCommands;
	int32 LowestFreeIndexSearchStart;
};

struct FMeshDrawCommandCount 
{
	FMeshDrawCommandCount() :
		Num(0)
	{

	}

	FMeshDrawCommandCount(FMeshDrawCommandCount&& Other) :
		Num(Other.Num.load())
	{

	}

	std::atomic<uint32> Num;
};

struct MeshDrawCommandKeyFuncs : TDefaultMapHashableKeyFuncs<FMeshDrawCommand, FMeshDrawCommandCount, false>
{
	/**
	 * @return True if the keys match.
	 */
	static inline bool Matches(KeyInitType A, KeyInitType B)
	{
		return A.MatchesForDynamicInstancing(B);
	}

	/** Calculates a hash index for a key. */
	static inline uint32 GetKeyHash(KeyInitType Key)
	{
		return Key.GetDynamicInstancingHash();
	}
};

using FDrawCommandIndices = TArray<int32, TInlineAllocator<5>>;
using FStateBucketMap = Experimental::TRobinHoodHashMap<FMeshDrawCommand, FMeshDrawCommandCount, MeshDrawCommandKeyFuncs>;

class FCachedPassMeshDrawListContext : public FMeshPassDrawListContext
{
public:
	struct FMeshPassScope
	{
		FMeshPassScope(const FMeshPassScope&) = delete;
		FMeshPassScope& operator=(const FMeshPassScope&) = delete;

		inline FMeshPassScope(FCachedPassMeshDrawListContext& InContext, EMeshPass::Type MeshPass)
			: Context(InContext)
		{
			Context.BeginMeshPass(MeshPass);
		}

		inline ~FMeshPassScope()
		{
			Context.EndMeshPass();
		}
		
	private:
		FCachedPassMeshDrawListContext& Context;
	};

	FCachedPassMeshDrawListContext(FScene& InScene);

	virtual void ReserveMemoryForCommands(int32 MaxCommandsExpected) {};
	virtual FMeshDrawCommand& AddCommand(FMeshDrawCommand& Initializer, uint32 NumElements) override final;

	void BeginMeshPass(EMeshPass::Type MeshPass);
	void EndMeshPass();

	void BeginMesh(int32 SceneInfoIndex, int32 MeshIndex);
	void EndMesh();

	FCachedMeshDrawCommandInfo GetCommandInfoAndReset();
	bool HasAnyLooseParameterBuffers() const { return bAnyLooseParameterBuffers; }	

protected:
	void FinalizeCommandCommon(
		const FMeshBatch& MeshBatch, 
		int32 BatchElementIndex,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		FMeshDrawCommandSortKey SortKey,
		EFVisibleMeshDrawCommandFlags Flags,
		const FGraphicsMinimalPipelineStateInitializer& PipelineState,
		const FMeshProcessorShaders* ShadersForDebugging,
		FMeshDrawCommand& MeshDrawCommand);

	FScene& Scene;
	FMeshDrawCommand MeshDrawCommandForStateBucketing;
	FCachedMeshDrawCommandInfo CommandInfo;
	EMeshPass::Type CurrMeshPass = EMeshPass::Num;
	bool bUseGPUScene = false;
	bool bAnyLooseParameterBuffers = false;
};

class FCachedPassMeshDrawListContextImmediate : public FCachedPassMeshDrawListContext
{
public:
	FCachedPassMeshDrawListContextImmediate(FScene& InScene) : FCachedPassMeshDrawListContext(InScene) {}

	virtual void FinalizeCommand(
		const FMeshBatch& MeshBatch, 
		int32 BatchElementIndex,
		const FMeshDrawCommandPrimitiveIdInfo& IdInfo,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		FMeshDrawCommandSortKey SortKey,
		EFVisibleMeshDrawCommandFlags Flags,
		const FGraphicsMinimalPipelineStateInitializer& PipelineState,
		const FMeshProcessorShaders* ShadersForDebugging,
		FMeshDrawCommand& MeshDrawCommand) override final;
};

class FCachedPassMeshDrawListContextDeferred : public FCachedPassMeshDrawListContext
{
public:
	FCachedPassMeshDrawListContextDeferred(FScene& InScene) : FCachedPassMeshDrawListContext(InScene) {}

	virtual void ReserveMemoryForCommands(int32 MaxCommandsExpected) override final
	{
		if (LIKELY(MaxCommandsExpected))
		{
			DeferredCommands.Reserve(MaxCommandsExpected);
			if (LIKELY(bUseGPUScene))
			{
				DeferredCommandHashes.Reserve(DeferredCommands.Max());
			}
		}
	}

	virtual void FinalizeCommand(
		const FMeshBatch& MeshBatch, 
		int32 BatchElementIndex,
		const FMeshDrawCommandPrimitiveIdInfo& IdInfo,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		FMeshDrawCommandSortKey SortKey,
		EFVisibleMeshDrawCommandFlags Flags,
		const FGraphicsMinimalPipelineStateInitializer& PipelineState,
		const FMeshProcessorShaders* ShadersForDebugging,
		FMeshDrawCommand& MeshDrawCommand) override final;

	void DeferredFinalizeMeshDrawCommands(const TArrayView<FPrimitiveSceneInfo*>& SceneInfos, int32 Start, int32 End);

private:
	TArray<FMeshDrawCommand> DeferredCommands;
	TArray<Experimental::FHashType> DeferredCommandHashes;
};

template<typename VertexType, typename PixelType, typename GeometryType = FMeshMaterialShader, typename RayTracingType = FMeshMaterialShader, typename ComputeType = FMeshMaterialShader>
struct TMeshProcessorShaders
{
	TShaderRef<VertexType> VertexShader;
	TShaderRef<PixelType> PixelShader;
	TShaderRef<GeometryType> GeometryShader;

	UE_DEPRECATED(5.4, "Compute shader mesh passes should use FMeshProcessorShaders directly instead of using TMeshProcessorShaders")
	TShaderRef<ComputeType> ComputeShader;
#if RHI_RAYTRACING
	UE_DEPRECATED(5.4, "RayTracing shader mesh passes should use FMeshProcessorShaders directly instead of using TMeshProcessorShaders")
	TShaderRef<RayTracingType> RayTracingShader;
#endif

	TMeshProcessorShaders() = default;

	FMeshProcessorShaders GetUntypedShaders() const
	{
		FMeshProcessorShaders Shaders;
		Shaders.VertexShader = VertexShader;
		Shaders.PixelShader = PixelShader;
		Shaders.GeometryShader = GeometryShader;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Shaders.ComputeShader = ComputeShader;
#if RHI_RAYTRACING
		Shaders.RayTracingShader = RayTracingShader;
#endif
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return Shaders;
	}
};

enum class EMeshPassFeatures
{
	Default = 0,
	PositionOnly = 1 << 0,
	PositionAndNormalOnly = 1 << 1,
};
ENUM_CLASS_FLAGS(EMeshPassFeatures);

/**
 * A set of render state overrides passed into a Mesh Pass Processor, so it can be configured from the outside.
 */
struct FMeshPassProcessorRenderState
{
	FMeshPassProcessorRenderState() = default;
	FMeshPassProcessorRenderState(const FMeshPassProcessorRenderState& DrawRenderState) = default;
	~FMeshPassProcessorRenderState() = default;

public:
	FORCEINLINE_DEBUGGABLE void SetBlendState(FRHIBlendState* InBlendState)
	{
		BlendState = InBlendState;
	}

	FORCEINLINE_DEBUGGABLE FRHIBlendState* GetBlendState() const
	{
		return BlendState;
	}

	FORCEINLINE_DEBUGGABLE void SetDepthStencilState(FRHIDepthStencilState* InDepthStencilState)
	{
		DepthStencilState = InDepthStencilState;
		StencilRef = 0;
	}

	FORCEINLINE_DEBUGGABLE void SetStencilRef(uint32 InStencilRef)
	{
		StencilRef = InStencilRef;
	}

	FORCEINLINE_DEBUGGABLE FRHIDepthStencilState* GetDepthStencilState() const
	{
		return DepthStencilState;
	}

	FORCEINLINE_DEBUGGABLE void SetDepthStencilAccess(FExclusiveDepthStencil::Type InDepthStencilAccess)
	{
		DepthStencilAccess = InDepthStencilAccess;
	}

	FORCEINLINE_DEBUGGABLE FExclusiveDepthStencil::Type GetDepthStencilAccess() const
	{
		return DepthStencilAccess;
	}

	FORCEINLINE_DEBUGGABLE uint32 GetStencilRef() const
	{
		return StencilRef;
	}

	FORCEINLINE_DEBUGGABLE void ApplyToPSO(FGraphicsPipelineStateInitializer& GraphicsPSOInit) const
	{
		GraphicsPSOInit.BlendState = BlendState;
		GraphicsPSOInit.DepthStencilState = DepthStencilState;
	}

private:
	FRHIBlendState*					BlendState =			nullptr;
	FRHIDepthStencilState*			DepthStencilState =		nullptr;
	FExclusiveDepthStencil::Type	DepthStencilAccess	=	FExclusiveDepthStencil::DepthRead_StencilRead;;

	FRHIUniformBuffer*				ViewUniformBuffer = nullptr;
	FRHIUniformBuffer*				InstancedViewUniformBuffer = nullptr;

	FRHIUniformBuffer*				PassUniformBuffer = nullptr;
	FRHIUniformBuffer*				NaniteUniformBuffer = nullptr;

	uint32							StencilRef = 0;
};

enum class EDrawingPolicyOverrideFlags
{
	None = 0,
	TwoSided = 1 << 0,
	DitheredLODTransition = 1 << 1,
	Wireframe = 1 << 2,
	ReverseCullMode = 1 << 3,
};
ENUM_CLASS_FLAGS(EDrawingPolicyOverrideFlags);

/** 
 * Base class of mesh processors, whose job is to transform FMeshBatch draw descriptions received from scene proxy implementations into FMeshDrawCommands ready for the RHI command list
 */
class FMeshPassProcessor : public IPSOCollector
{
public:

	EMeshPass::Type MeshPassType;
	const FScene* RESTRICT Scene;
	ERHIFeatureLevel::Type FeatureLevel;
	const FSceneView* ViewIfDynamicMeshCommand;
	FMeshPassDrawListContext* DrawListContext;
		
	UE_DEPRECATED(5.4, "Use the below Ctor instead which provides either EMeshPass::Type or the name of the mesh pass directly.")
	FMeshPassProcessor(const FScene* InScene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext) :
		FMeshPassProcessor(EMeshPass::Num, InScene, InFeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext) { }

	RENDERER_API FMeshPassProcessor(EMeshPass::Type InMeshPassType, const FScene* InScene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);
	RENDERER_API FMeshPassProcessor(const TCHAR* InMeshPassName, const FScene* InScene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);

	virtual ~FMeshPassProcessor() = default;

	void SetDrawListContext(FMeshPassDrawListContext* InDrawListContext)
	{
		DrawListContext = InDrawListContext;
	}

	// FMeshPassProcessor interface
	// Add a FMeshBatch to the pass
	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) = 0;
		
	// By default no PSOs collected 
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override {}

	static FORCEINLINE_DEBUGGABLE ERasterizerCullMode InverseCullMode(ERasterizerCullMode CullMode)
	{
		return CullMode == CM_None ? CM_None : (CullMode == CM_CCW ? CM_CW : CM_CCW);
	}

	struct FMeshDrawingPolicyOverrideSettings
	{
		EDrawingPolicyOverrideFlags	MeshOverrideFlags = EDrawingPolicyOverrideFlags::None;
		EPrimitiveType				MeshPrimitiveType = PT_TriangleList;
	};

	RENDERER_API static FMeshDrawingPolicyOverrideSettings ComputeMeshOverrideSettings(const FPSOPrecacheParams& PrecachePSOParams);
	RENDERER_API static FMeshDrawingPolicyOverrideSettings ComputeMeshOverrideSettings(const FMeshBatch& Mesh);

	RENDERER_API static ERasterizerFillMode ComputeMeshFillMode(const FMaterial& InMaterialResource, const FMeshDrawingPolicyOverrideSettings& InOverrideSettings);
	RENDERER_API static ERasterizerCullMode ComputeMeshCullMode(const FMaterial& InMaterialResource, const FMeshDrawingPolicyOverrideSettings& InOverrideSettings);

	template<typename PassShadersType, typename ShaderElementDataType>
	void BuildMeshDrawCommands(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		const FMeshPassProcessorRenderState& RESTRICT DrawRenderState,
		const PassShadersType& PassShaders,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		FMeshDrawCommandSortKey SortKey,
		EMeshPassFeatures MeshPassFeatures,
		const ShaderElementDataType& ShaderElementData);

	template<typename PassShadersType>
	void AddGraphicsPipelineStateInitializer(
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		const FMeshPassProcessorRenderState& RESTRICT DrawRenderState,
		const FGraphicsPipelineRenderTargetsInfo& RESTRICT RenderTargetsInfo,
		const PassShadersType& PassShaders,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		EPrimitiveType PrimitiveType,
		EMeshPassFeatures MeshPassFeatures,
		bool bRequired,
		TArray<FPSOPrecacheData>& PSOInitializers);

	template<typename PassShadersType>
	static void AddGraphicsPipelineStateInitializer(
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		const FMeshPassProcessorRenderState& RESTRICT DrawRenderState,
		const FGraphicsPipelineRenderTargetsInfo& RESTRICT RenderTargetsInfo,
		const PassShadersType& PassShaders,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		EPrimitiveType PrimitiveType,
		EMeshPassFeatures MeshPassFeatures,
		ESubpassHint SubpassHint,
		uint8 SubpassIndex,
		bool bRequired,
		int32 PSOCollectorIndex,
		TArray<FPSOPrecacheData>& PSOInitializers);

protected:
	RENDERER_API FMeshDrawCommandPrimitiveIdInfo GetDrawCommandPrimitiveId(
		const FPrimitiveSceneInfo* RESTRICT PrimitiveSceneInfo,
		const FMeshBatchElement& BatchElement) const;

	RENDERER_API bool ShouldSkipMeshDrawCommand(
		const FMeshBatch& RESTRICT MeshBatch,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy
	) const;

	RENDERER_API bool PipelineVariableRateShadingEnabled() const;
	RENDERER_API bool HardwareVariableRateShadingSupportedByScene() const;
};

#if PSO_PRECACHING_VALIDATE

namespace PSOCollectorStats
{
	/**
	 * Create a shaders-only initializer where all state except for shaders is removed from the given initializer.
	 */
	RENDERER_API extern FGraphicsMinimalPipelineStateInitializer GetShadersOnlyInitializer(const FGraphicsMinimalPipelineStateInitializer& Initializer);

	/**
	 * Apply a custom mask to the given initializer removing some state.
	 */
	RENDERER_API extern FGraphicsMinimalPipelineStateInitializer PatchMinimalPipelineStateToCheck(const FGraphicsMinimalPipelineStateInitializer& Initializer);

	/**
	 * Compute the hash of a minimal graphics PSO initializer to be used by PSO precaching validation.
	 */
	RENDERER_API extern uint64 GetPSOPrecacheHash(const FGraphicsMinimalPipelineStateInitializer& Initializer);

	/**
	 * Check if ShaderOnly PSO is precached and if not log information
	 */
	RENDERER_API extern void CheckShaderOnlyStateInCache(const FGraphicsMinimalPipelineStateInitializer& Initializer, const FMaterial& Material, const FVertexFactoryType* VFType, const FPrimitiveSceneProxy* PrimitiveSceneProxy, int32 PSOCollectorIndex);

	/**
	 * Check if minimal graphics PSO is precached and if not log information
	 */
	RENDERER_API extern void CheckMinimalPipelineStateInCache(const FGraphicsMinimalPipelineStateInitializer& Initializer, const FMaterial& Material, const FVertexFactoryType* VFType, const FPrimitiveSceneProxy* PrimitiveSceneProxy, int32 PSOCollectorIndex);
}

#endif // PSO_PRECACHING_VALIDATE

typedef FMeshPassProcessor* (*DeprecatedPassProcessorCreateFunction)(const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);
typedef FMeshPassProcessor* (*PassProcessorCreateFunction)(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);

enum class EMeshPassFlags
{
	None = 0,
	CachedMeshCommands = 1 << 0,
	MainView = 1 << 1
};
ENUM_CLASS_FLAGS(EMeshPassFlags);

class FPassProcessorManager
{
public:

	static FMeshPassProcessor* CreateMeshPassProcessor(EShadingPath ShadingPath, EMeshPass::Type PassType, ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
	{
		check(ShadingPath < EShadingPath::Num&& PassType < EMeshPass::Num);
		uint32 ShadingPathIdx = (uint32)ShadingPath;
		checkf(JumpTable[ShadingPathIdx][PassType] || DeprecatedJumpTable[ShadingPathIdx][PassType], TEXT("Pass type %u create function was never registered for shading path %u.  Use a FRegisterPassProcessorCreateFunction to register a create function for this enum value."), (uint32)PassType, ShadingPathIdx);
		if (JumpTable[ShadingPathIdx][PassType])
		{
			return JumpTable[ShadingPathIdx][PassType](FeatureLevel, Scene, InViewIfDynamicMeshCommand, InDrawListContext);
		}
		else
		{
			return DeprecatedJumpTable[ShadingPathIdx][PassType](Scene, InViewIfDynamicMeshCommand, InDrawListContext);
		}
	}

	static EMeshPassFlags GetPassFlags(EShadingPath ShadingPath, EMeshPass::Type PassType)
	{
		check(ShadingPath < EShadingPath::Num && PassType < EMeshPass::Num);
		uint32 ShadingPathIdx = (uint32)ShadingPath;
		return Flags[ShadingPathIdx][PassType];
	}

	static int32 GetPSOCollectorIndex(EShadingPath ShadingPath, EMeshPass::Type PassType)
	{
		if (PassType == EMeshPass::Num)
		{
			return INDEX_NONE;
		}

		check(ShadingPath < EShadingPath::Num);
		uint32 ShadingPathIdx = (uint32)ShadingPath;
		return PSOCollectorIndex[ShadingPathIdx][PassType];
	}

	/** Only call on the game thread. Heavy weight. Flush rendering commands and recreate all component render states. */
	static void SetPassFlags(EShadingPath ShadingPath, EMeshPass::Type PassType, EMeshPassFlags NewFlags);

private:
	RENDERER_API static PassProcessorCreateFunction JumpTable[(uint32)EShadingPath::Num][EMeshPass::Num];
	RENDERER_API static DeprecatedPassProcessorCreateFunction DeprecatedJumpTable[(uint32)EShadingPath::Num][EMeshPass::Num];
	RENDERER_API static EMeshPassFlags Flags[(uint32)EShadingPath::Num][EMeshPass::Num];
	RENDERER_API static int32 PSOCollectorIndex[(uint32)EShadingPath::Num][EMeshPass::Num];
	friend class FRegisterPassProcessorCreateFunction;
};

class FRegisterPassProcessorCreateFunction
{
public:
	FRegisterPassProcessorCreateFunction(PassProcessorCreateFunction CreateFunction, EShadingPath InShadingPath, EMeshPass::Type InPassType, EMeshPassFlags PassFlags, int32 PSOCollectorIndex = INDEX_NONE) 
		: ShadingPath(InShadingPath)
		, PassType(InPassType)
	{
		uint32 ShadingPathIdx = (uint32)ShadingPath;
		FPassProcessorManager::JumpTable[ShadingPathIdx][PassType] = CreateFunction;
		FPassProcessorManager::Flags[ShadingPathIdx][PassType] = PassFlags;
		FPassProcessorManager::PSOCollectorIndex[ShadingPathIdx][PassType] = PSOCollectorIndex;
	}

	~FRegisterPassProcessorCreateFunction()
	{
		uint32 ShadingPathIdx = (uint32)ShadingPath;
		FPassProcessorManager::JumpTable[ShadingPathIdx][PassType] = nullptr;
		FPassProcessorManager::Flags[ShadingPathIdx][PassType] = EMeshPassFlags::None;
		FPassProcessorManager::PSOCollectorIndex[ShadingPathIdx][PassType] = INDEX_NONE;
	}

private:
	EShadingPath ShadingPath;
	EMeshPass::Type PassType;
};

// Helper marco to register the mesh pass processor to both the FPassProcessorManager & the FPSOCollectorCreateManager
#define REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(Name, MeshPassProcessorCreateFunction, ShadingPath, MeshPass, MeshPassFlags) \
	IPSOCollector* CreatePSOCollector##Name(ERHIFeatureLevel::Type FeatureLevel) \
	{ \
		return MeshPassProcessorCreateFunction(FeatureLevel, nullptr, nullptr, nullptr); \
	} \
	FRegisterPSOCollectorCreateFunction RegisterPSOCollector##Name(&CreatePSOCollector##Name, ShadingPath, GetMeshPassName(MeshPass)); \
	FRegisterPassProcessorCreateFunction RegisterMeshPassProcesser##Name(&MeshPassProcessorCreateFunction, ShadingPath, MeshPass, MeshPassFlags, RegisterPSOCollector##Name.GetIndex());


RENDERER_API extern void SubmitMeshDrawCommands(
	const FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet, 
	const FMeshDrawCommandSceneArgs& SceneArgs,
	uint32 PrimitiveIdBufferStride,
	bool bDynamicInstancing,
	uint32 InstanceFactor,
	FRHICommandList& RHICmdList);

extern void SubmitMeshDrawCommandsRange(
	const FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	const FMeshDrawCommandSceneArgs& SceneArgs,
	uint32 PrimitiveIdBufferStride,
	bool bDynamicInstancing,
	int32 StartIndex,
	int32 NumMeshDrawCommands,
	uint32 InstanceFactor,
	FRHICommandList& RHICmdList);

extern void ApplyViewOverridesToMeshDrawCommands(
	const FSceneView& View,
	FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	FDynamicMeshDrawCommandStorage& DynamicMeshDrawCommandStorage,
	FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	bool& NeedsShaderInitialisation);

RENDERER_API extern void DrawDynamicMeshPassPrivate(
	const FSceneView& View,
	FRHICommandList& RHICmdList,
	FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	FDynamicMeshDrawCommandStorage& DynamicMeshDrawCommandStorage,
	FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	bool& InNeedsShaderInitialisation,
	uint32 InstanceFactor);

RENDERER_API extern FMeshDrawCommandSortKey CalculateMeshStaticSortKey(const FMeshMaterialShader* VertexShader, const FMeshMaterialShader* PixelShader);

RENDERER_API extern void AddRenderTargetInfo(EPixelFormat PixelFormat, ETextureCreateFlags CreateFlags, FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo);
RENDERER_API extern void SetupDepthStencilInfo(EPixelFormat DepthStencilFormat, ETextureCreateFlags DepthStencilCreateFlags, ERenderTargetLoadAction DepthTargetLoadAction, ERenderTargetLoadAction StencilTargetLoadAction, FExclusiveDepthStencil DepthStencilAccess, FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo);
RENDERER_API extern void SetupGBufferRenderTargetInfo(const FSceneTexturesConfig& SceneTexturesConfig, FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo, bool bSetupDepthStencil, EGBufferLayout Layout = GBL_Default);
RENDERER_API extern void ApplyTargetsInfo(FGraphicsPipelineStateInitializer& GraphicsPSOInit, const FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo);
RENDERER_API extern ESubpassHint GetSubpassHint(const FStaticShaderPlatform Platform, bool bIsUsingGBuffers, bool bMultiViewRendering, uint32 NumSamples);

inline FMeshDrawCommandSortKey CalculateMeshStaticSortKey(const TShaderRef<FMeshMaterialShader>& VertexShader, const TShaderRef<FMeshMaterialShader>& PixelShader)
{
	return CalculateMeshStaticSortKey(VertexShader.GetShader(), PixelShader.GetShader());
}
