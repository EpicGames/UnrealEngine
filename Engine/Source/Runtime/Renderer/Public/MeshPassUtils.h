// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPassProcessor.h"
#include "RenderGraphUtils.h"
#include "MeshMaterialShader.h"

namespace UE::MeshPassUtils
{
	namespace Private
	{
		template<typename TShaderClass>
		inline void PrepareDispatch(
			FRHIComputeCommandList& RHICmdList,
			const TShaderRef<TShaderClass>& ComputeShader,
			const FMeshDrawShaderBindings& ShaderBindings,
			const typename TShaderClass::FParameters& PassParameters)
		{
			FRHIComputeShader* ShaderRHI = ComputeShader.GetComputeShader();

			FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
			ShaderBindings.SetParameters(BatchedParameters);
			SetShaderParameters(BatchedParameters, ComputeShader, PassParameters);

			SetComputePipelineState(RHICmdList, ShaderRHI);
			RHICmdList.SetBatchedShaderParameters(ShaderRHI, BatchedParameters);
		}

		template<typename TShaderClass>
		inline void AfterDispatch(FRHIComputeCommandList& RHICmdList, const TShaderRef<TShaderClass>& ComputeShader)
		{
			UnsetShaderUAVs(RHICmdList, ComputeShader, ComputeShader.GetComputeShader());
		}
	}

	/** Dispatch a compute shader to RHI Command List with its parameters and mesh shader bindings. */
	template<typename TShaderClass>
	inline void Dispatch(
		FRHIComputeCommandList& RHICmdList,
		const TShaderRef<TShaderClass>& ComputeShader,
		const FMeshDrawShaderBindings& ShaderBindings,
		const typename TShaderClass::FParameters& PassParameters,
		FIntVector GroupCount)
	{
		FComputeShaderUtils::ValidateGroupCount(GroupCount);

		Private::PrepareDispatch(RHICmdList, ComputeShader, ShaderBindings, PassParameters);
		RHICmdList.DispatchComputeShader(GroupCount.X, GroupCount.Y, GroupCount.Z);
		Private::AfterDispatch(RHICmdList, ComputeShader);
	}

	/** Indirect dispatch a compute shader to RHI Command List with its parameters and mesh shader bindings. */
	template<typename TShaderClass>
	inline void DispatchIndirect(
		FRHIComputeCommandList& RHICmdList,
		const TShaderRef<TShaderClass>& ComputeShader,
		const FMeshDrawShaderBindings& ShaderBindings,
		const typename TShaderClass::FParameters& PassParameters,
		FRHIBuffer* IndirectArgsBuffer,
		uint32 IndirectArgOffset)
	{
		FComputeShaderUtils::ValidateIndirectArgsBuffer(IndirectArgsBuffer->GetSize(), IndirectArgOffset);

		Private::PrepareDispatch(RHICmdList, ComputeShader, ShaderBindings, PassParameters);
		RHICmdList.DispatchIndirectComputeShader(IndirectArgsBuffer, IndirectArgOffset);
		Private::AfterDispatch(RHICmdList, ComputeShader);
	}

	template<typename TShaderClass, typename TShaderElementData>
	inline void SetupComputeBindings(
		const TShaderRef<TShaderClass>& Shader,
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const TShaderElementData& ShaderElementData,
		FMeshDrawShaderBindings& ShaderBindings
	)
	{
		FMeshProcessorShaders MeshProcessorShaders;

		EShaderFrequency Frequency = Shader->GetFrequency();
		if (Frequency == SF_Compute)
		{
			MeshProcessorShaders.ComputeShader = Shader;
		}
		else if (Frequency == SF_WorkGraphComputeNode)
		{
			MeshProcessorShaders.WorkGraphShader = Shader;
		}

		ShaderBindings.Initialize(MeshProcessorShaders);

		if (Shader.IsValid())
		{
			FMeshDrawSingleShaderBindings SingleShaderBindings = ShaderBindings.GetSingleShaderBindings(Frequency);
			Shader->GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, SingleShaderBindings);
		}

		ShaderBindings.Finalize(&MeshProcessorShaders);
	}

	template<typename TShaderClass>
	inline void SetupComputeBindings(
		const TShaderRef<TShaderClass>& Shader,
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		FMeshDrawShaderBindings& ShaderBindings
	)
	{
		FMeshMaterialShaderElementData ShaderElementData;
		ShaderElementData.InitializeMeshMaterialData();

		SetupComputeBindings(Shader, Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
	}
}
