// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MultiGPU.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"

class FSceneView;
class FRDGBuilder;
class FRDGTexture;
class FRHICommandListImmediate;

using FRDGTextureRef = FRDGTexture*;

namespace UE::Renderer::Private
{

/** Interface for implementing third party path tracing spatial denoiser. */
class IPathTracingDenoiser
{
public:

	/** Inputs of the path tracing denoiser. */
	struct FInputs
	{
		FRDGTextureRef ColorTex;
		FRDGTextureRef AlbedoTex;
		FRDGTextureRef NormalTex;
		FRDGTextureRef OutputTex;
	};

	virtual ~IPathTracingDenoiser() = default;

	virtual bool NeedTextureCreateExtraFlags() const { return false; }

	/** Adds the necessary passes into RDG for denoising. */
	virtual void AddPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FInputs& Inputs) const = 0;
};

/** Interface for implementing third party path tracing spatial temporal denoiser. */
class IPathTracingSpatialTemporalDenoiser
{
public:

	/** Ref counted history to be saved in the history. */
	class IHistory : public FRefCountedObject
	{
	public:
		virtual ~IHistory() = default;

		/** Debug name of the history. Must exactly point to the same const TCHAR* as IPathTracingSpatialTemporalDenoiser::GetDebugName().
		 * This is used to ensure IHistory is fed to a compatible IPathTracingSpatialTemporalDenoiser.
		 */
		virtual const TCHAR* GetDebugName() const = 0;
	};

	/** Inputs of the path tracing denoiser. */
	struct FInputs
	{
		FRDGTextureRef ColorTex;
		FRDGTextureRef AlbedoTex;
		FRDGTextureRef NormalTex;
		FRDGTextureRef DepthTex;
		FRDGTextureRef VarianceTex;
		FRDGTextureRef OutputTex;

		FRDGTextureRef FlowTex;
		FRDGTextureRef PreviousOutputTex;

		int DenoisingFrameId = -1;
		bool bForceSpatialDenoiserOnly = false;

		/** The history of the previous frame set by FOutputs::NewHistory. PrevHistory->GetDebugName() is guarentee to match the IPathTracingSpatialTemporalDenoiser. */
		TRefCountPtr<IHistory> PrevHistory;
	};

	struct FMotionVectorInputs
	{
		FRDGTextureRef InputFrameTex;
		FRDGTextureRef ReferenceFrameTex;
		FRDGTextureRef OutputTex;
		float PreExposure = 1.0f;
	};

	/** Outputs of the third party path tracing spacial temporal denoiser. */
	struct FOutputs
	{
		/** New history to be kept alive for next frame. NewHistory->GetDebugName() must exactly point to the same const TCHAR* as IPathTracingSpatialTemporalDenoiser::GetDebugName(). */
		TRefCountPtr<IHistory> NewHistory;
	};

	virtual ~IPathTracingSpatialTemporalDenoiser() = default;

	/** Debug name of the history. Must exactly point to the same const TCHAR* as IPathTracingSpatialTemporalDenoiser::IHistory::GetDebugName(). */
	virtual const TCHAR* GetDebugName() const = 0;

	virtual bool NeedTextureCreateExtraFlags() const { return false; }

	virtual bool NeedVarianceTexture() const { return false;}

	/** Adds the necessary passes into RDG for denoising. */
	virtual FOutputs AddPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FInputs& Inputs) const = 0;

	/**  */
	virtual void AddMotionVectorPass(FRDGBuilder& GraphBuilder, const FSceneView& View, const FMotionVectorInputs& Inputs) const = 0;
};

} // namespace UE::Renderer::Private

/** Register a spatial denoiser*/
extern RENDERER_API void RegisterSpatialDenoiser(TUniquePtr<UE::Renderer::Private::IPathTracingDenoiser> PathTracingDenoiser, FString Name);

/** Register a spatial temporal denoiser*/
extern RENDERER_API void RegisterSpatialTemporalDenoiser(TUniquePtr<UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser> PathTracingDenoiser, FString Name);

/** Unregister all denoisers with the name Name*/
extern RENDERER_API void UnregisterDenoiser(FString Name);

/** Return true if there is any temporal denoiser plugin registered*/
extern RENDERER_API bool HasTemporalDenoiser();
