// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "RendererInterface.h"
#include "RHIFwd.h"

class FRHICommandList;
struct FRHIBatchedShaderParameters;
class FSceneView;
class FShaderMapPointerTable;

template<typename ShaderType, typename PointerTableType> class TShaderRefBase;
template<typename ShaderType> using TShaderRef = TShaderRefBase<ShaderType, FShaderMapPointerTable>;

namespace UE::Renderer::PostProcess
{
	RENDERER_API void SetDrawRectangleParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FShader* VertexShader,
		float X,
		float Y,
		float SizeX,
		float SizeY,
		float U,
		float V,
		float SizeU,
		float SizeV,
		FIntPoint TargetSize,
		FIntPoint TextureSize);

	RENDERER_API void SetDrawRectangleParameters(FRHIBatchedShaderParameters& BatchedParameters, const FShader* VertexShader, const FIntPoint& ViewSize);
	RENDERER_API void SetDrawRectangleParameters(FRHIBatchedShaderParameters& BatchedParameters, const FShader* VertexShader, const FSceneView& View);

	/**
	 * Draws a quad with the given vertex positions and UVs in denormalized pixel/texel coordinates.
	 * The platform-dependent mapping from pixels to texels is done automatically.
	 * Note that the positions are affected by the current viewport.
	 * NOTE: DrawRectangle should be used in the vertex shader to calculate the correct position and uv for vertices.
	 * NOTE2: Assumes previously set PSO has PrimitiveType = PT_TriangleList
	 *
	 * X, Y							Position in screen pixels of the top left corner of the quad
	 * SizeX, SizeY					Size in screen pixels of the quad
	 * U, V							Position in texels of the top left corner of the quad's UV's
	 * SizeU, SizeV					Size in texels of the quad's UV's
	 * TargetSizeX, TargetSizeY		Size in screen pixels of the target surface
	 * TextureSize                  Size in texels of the source texture
	 * VertexShader					The vertex shader used for rendering
	 * Flags						see EDrawRectangleFlags
	 * InstanceCount				Number of instances of rectangle
	 */
	RENDERER_API void DrawRectangle(
		FRHICommandList& RHICmdList,
		const TShaderRef<FShader>& VertexShader,
		float X,
		float Y,
		float SizeX,
		float SizeY,
		float U,
		float V,
		float SizeU,
		float SizeV,
		FIntPoint TargetSize,
		FIntPoint TextureSize,
		EDrawRectangleFlags Flags = EDRF_Default,
		uint32 InstanceCount = 1
	);

	RENDERER_API void DrawRectangle(
		FRHICommandList& RHICmdList,
		const TShaderRef<FShader>& VertexShader,
		const FSceneView& View,
		EDrawRectangleFlags Flags = EDRF_Default,
		uint32 InstanceCount = 1
	);

	// NOTE: Assumes previously set PSO has PrimitiveType = PT_TriangleList
	RENDERER_API void DrawPostProcessPass(
		FRHICommandList& RHICmdList,
		const TShaderRef<FShader>& VertexShader,
		float X,
		float Y,
		float SizeX,
		float SizeY,
		float U,
		float V,
		float SizeU,
		float SizeV,
		FIntPoint TargetSize,
		FIntPoint TextureSize,
		int32 StereoViewIndex,
		bool bHasCustomMesh,
		EDrawRectangleFlags Flags = EDRF_Default
	);
}
