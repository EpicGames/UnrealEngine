// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

/** The vertex data used to filter a texture. */
struct FDistortionVertex
{
	FVector2f	Position;
	FVector2f	TexR;
	FVector2f	TexG;
	FVector2f	TexB;
	float		VignetteFactor;
	float		TimewarpFactor;
};

struct FHMDDistortionInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] The input scene color and view rect.
	FScreenPassTexture SceneColor;
};

FScreenPassTexture AddDefaultHMDDistortionPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FHMDDistortionInputs& Inputs);

FScreenPassTexture AddHMDDistortionPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FHMDDistortionInputs& Inputs);