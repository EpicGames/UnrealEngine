// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneTexturesConfig.h"
#include "ScreenPass.h"
#include "Engine/BlendableInterface.h"
#include "RenderGraphFwd.h"

struct FSceneWithoutWaterTextures;
class FSceneView;
struct FPostProcessMaterialInputs;
class UMaterialInterface;

const uint32 kPostProcessMaterialInputCountMax = 5;
const uint32 kPathTracingPostProcessMaterialInputCountMax = 5;

/** Named post process material slots. Inputs are aliased and have different semantics
 *  based on the post process material blend point, which is documented with the input.
 */
enum class EPostProcessMaterialInput : uint32
{
	// Always Active. Color from the previous stage of the post process chain.
	SceneColor = 0,

	// Always Active.
	SeparateTranslucency = 1,

	// Replace Tonemap Only. Half resolution combined bloom input.
	CombinedBloom = 2,

	// Buffer Visualization Only.
	PreTonemapHDRColor = 2,
	PostTonemapHDRColor = 3,

	// Active if separate velocity pass is used--i.e. not part of base pass; Not active during Replace Tonemap.
	Velocity = 4
};

enum class EPathTracingPostProcessMaterialInput : uint32
{
	Radiance = 0,
	DenoisedRadiance = 1,
	Albedo = 2,
	Normal = 3,
	Variance = 4
};

struct FPostProcessMaterialInputs
{
	FPostProcessMaterialInputs()
	{
		FMemory::Memset(bUserSceneTexturesSet, 0);
	}

	inline void SetInput(FRDGBuilder& GraphBuilder, EPostProcessMaterialInput Input, FScreenPassTexture Texture)
	{
		SetInput(Input, FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, Texture));
	}

	inline void SetInput(EPostProcessMaterialInput Input, FScreenPassTextureSlice Texture)
	{
		check((uint32)Input < kPostProcessMaterialInputCountMax);

		Textures[(uint32)Input] = Texture;
	}

	inline void SetUserSceneTextureInput(EPostProcessMaterialInput Input, FScreenPassTextureSlice Texture)
	{
		check((uint32)Input < kPostProcessMaterialInputCountMax);

		UserSceneTextures[(uint32)Input] = Texture;
		bUserSceneTexturesSet[(uint32)Input] = true;
	}

	inline FScreenPassTextureSlice GetInput(EPostProcessMaterialInput Input) const
	{
		return bUserSceneTexturesSet[(uint32)Input] ? UserSceneTextures[(uint32)Input] : Textures[(uint32)Input];
	}

	inline FScreenPassTextureSlice GetSceneColorOutput(EBlendableLocation BlendableLocation) const
	{
		return Textures[(uint32)(BlendableLocation == BL_TranslucencyAfterDOF ? EPostProcessMaterialInput::SeparateTranslucency : EPostProcessMaterialInput::SceneColor)];
	}

	inline void SetPathTracingInput(EPathTracingPostProcessMaterialInput Input, FScreenPassTexture Texture)
	{
		PathTracingTextures[(uint32)Input] = Texture;
	}

	inline FScreenPassTexture GetPathTracingInput(EPathTracingPostProcessMaterialInput Input) const
	{
		return PathTracingTextures[(uint32)Input];
	}

	inline void Validate() const
	{
		ValidateInputExists(EPostProcessMaterialInput::SceneColor);
		
		// TODO:  Is separate translucency always guaranteeed to be present?  A previous version of the code appeared to be attempting to validate this, but
		//		  due to a bug (ValidateInputExists ignoring its Input argument and instead validating EPostProcessMaterialInput::SceneColor regardless of
		//		  what was passed in), didn't actually do so.  I'm afraid to enable this assert now, as I don't know if it will randomly crash some project.
		// ValidateInputExists(EPostProcessMaterialInput::SeparateTranslucency);

		// Either override output format is valid or the override output texture is; not both.
		if (OutputFormat != PF_Unknown)
		{
			check(OverrideOutput.Texture == nullptr);
		}
		if (OverrideOutput.Texture)
		{
			check(OutputFormat == PF_Unknown);
		}

		check(SceneTextures.SceneTextures || SceneTextures.MobileSceneTextures);
	}

	inline void ValidateInputExists(EPostProcessMaterialInput Input) const
	{
		check(Textures[(int32)Input].IsValid());
	}

	/**
	 * Returns the input scene color as a 2D texture output. This WILL perform a GPU copy if the override output is active or the input scene color was a 2D texture array slice.
	 */
	RENDERER_API FScreenPassTexture ReturnUntouchedSceneColorForPostProcessing(FRDGBuilder& GraphBuilder) const;

	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	/** Array of input textures bound to the material. The first element represents the output from
	 *  the previous post process and is required. All other inputs are optional.
	 */
	TStaticArray<FScreenPassTextureSlice, kPostProcessMaterialInputCountMax> Textures;

	/**
	 * UserSceneTexture inputs, which take precedence over Textures above if set.  The reason for separating these from "Textures" above
	 * is because "Textures" is also where the output SceneColor is fetched from, when OverrideOutput isn't set (see "GetSceneColorOutput"
	 * function).  The separate bools are needed to track which inputs are UserSceneTextures, as opposed to checking IsValid(), because
	 * the entry can be invalid when an input being missing -- dummy black is substituted for those downstream.
	 */
	TStaticArray<FScreenPassTextureSlice, kPostProcessMaterialInputCountMax> UserSceneTextures;
	TStaticArray<bool, kPostProcessMaterialInputCountMax> bUserSceneTexturesSet;

	/**
	*	Array of input textures bound to the material from path tracing. All inputs are optional
	*/
	TStaticArray<FScreenPassTexture, kPathTracingPostProcessMaterialInputCountMax> PathTracingTextures;

	/** The output texture format to use if a new texture is created. Uses the input format if left unknown. */
	EPixelFormat OutputFormat = PF_Unknown;

	/** Whether or not the stencil test must be done in the pixel shader rather than rasterizer state. */
	bool bManualStencilTest = false;

	/** Custom depth/stencil used for stencil operations. */
	FRDGTextureRef CustomDepthTexture = nullptr;

	/** The uniform buffer containing all scene textures. */
	FSceneTextureShaderParameters SceneTextures;

	/** Depth and color textures of the scene without single layer water. May be nullptr if not available. */
	const FSceneWithoutWaterTextures* SceneWithoutWaterTextures = nullptr;

	/** Allows (but doesn't guarantee) an optimization where, if possible, the scene color input is reused as
	 *  the output. This can elide a copy in certain circumstances; for example, when the scene color input isn't
	 *  actually used by the post process material and no special depth-stencil / blend composition is required.
	 *  Set this to false when you need to guarantee creation of a dedicated output texture.
	 */
	bool bAllowSceneColorInputAsOutput = true;

	bool bMetalMSAAHDRDecode = false;

	bool bUserSceneTextureOutput = false;
	bool bUserSceneTextureFirstRender = false;
	uint32 UserSceneTextureSceneColorInput = INDEX_NONE;

	/**
	 * When true, the AddPostProcessMaterialPass can execute outside of the regular render post-processing stack.
	 * This offers a leaner alternative to DrawMaterialToRenderTarget, where bindings can be assigned using regular
	 * post-process inputs instead of named texture parameters.
	 * 
	 * Other bindings that require a fully initialized rendering pipeline are skipped.
	 * The caller remains responsible for providing valid (dummy) SceneTextures & ViewUniformBuffer.
	 */
	bool bStandalonePass = false;
};

class UMaterialInterface;

FScreenPassTexture RENDERER_API AddPostProcessMaterialPass(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	const FPostProcessMaterialInputs& Inputs,
	const UMaterialInterface* MaterialInterface);
