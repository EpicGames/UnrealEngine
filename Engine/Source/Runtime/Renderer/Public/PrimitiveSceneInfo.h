// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PrimitiveSceneInfo.h: Primitive scene info definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "RenderDeferredCleanup.h"
#include "HitProxies.h"
#include "Math/GenericOctreePublic.h"
#include "PrimitiveComponentId.h"
#include "PrimitiveDirtyState.h"
#include "MeshPassProcessor.h"
#include "ShaderParameterMacros.h"
#include "RayTracingMeshDrawCommands.h"
#include "RayTracingGeometryInstance.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "MeshBatch.h"
#include "RendererInterface.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

enum class ERayTracingPrimitiveFlags : uint8;

class FIndirectLightingCacheUniformParameters;
class FPlanarReflectionSceneProxy;
class FPrimitiveSceneInfo;
class FPrimitiveSceneProxy;
class FRayTracingGeometry;
class FReflectionCaptureProxy;
class FScene;
class FViewInfo;
class UPrimitiveComponent;
class IPrimitiveComponent;
struct FPrimitiveSceneInfoAdapter;
struct FPrimitiveSceneDesc;
struct FPrimitiveSceneInfoData;
struct FInstanceDataBufferHeader;
class FInstanceSceneDataBuffers;
class FInstanceDataUpdateTaskInfo;
struct FMeshBatch;

struct FNaniteMaterialSlot;
struct FNaniteRasterBin;
struct FNaniteShadingBin;
struct FRayTracingInstance;
struct FRayTracingSBTAllocation;
struct FRayTracingMaskAndFlags;

template<typename ElementType,typename OctreeSemantics> class TOctree2;

namespace Nanite
{
	using CoarseMeshStreamingHandle = int16;
}

/** Data used to track a primitive's allocation in the volume texture atlas that stores indirect lighting. */
class FIndirectLightingCacheAllocation
{
public:

	FIndirectLightingCacheAllocation() :
		Add(FVector(0, 0, 0)),
		Scale(FVector(0, 0, 0)),
		MinUV(FVector(0, 0, 0)),
		MaxUV(FVector(0, 0, 0)),
		MinTexel(FIntVector(-1, -1, -1)),
		AllocationTexelSize(0),
		TargetPosition(FVector(0, 0, 0)),
		TargetDirectionalShadowing(1),
		TargetSkyBentNormal(FVector4f(0, 0, 1, 1)),
		SingleSamplePosition(FVector(0, 0, 0)),
		CurrentDirectionalShadowing(1),
		CurrentSkyBentNormal(FVector4f(0, 0, 1, 1)),
		bHasEverUpdatedSingleSample(false),
		bPointSample(true),
		bIsDirty(false),
		bUnbuiltPreview(false)
	{
		for (int32 VectorIndex = 0; VectorIndex < 3; VectorIndex++) // RGB
		{
			TargetSamplePacked0[VectorIndex] = FVector4f(0, 0, 0, 0);
			SingleSamplePacked0[VectorIndex] = FVector4f(0, 0, 0, 0);
			TargetSamplePacked1[VectorIndex] = FVector4f(0, 0, 0, 0);
			SingleSamplePacked1[VectorIndex] = FVector4f(0, 0, 0, 0);
		}
		TargetSamplePacked2 = FVector4f(0, 0, 0, 0);
		SingleSamplePacked2 = FVector4f(0, 0, 0, 0);
	}

	/** Add factor for calculating UVs from position. */
	FVector Add;

	/** Scale factor for calculating UVs from position. */
	FVector Scale;

	/** Used to clamp lookup UV to a valid range for pixels outside the object's bounding box. */
	FVector MinUV;

	/** Used to clamp lookup UV to a valid range for pixels outside the object's bounding box. */
	FVector MaxUV;

	/** Block index in the volume texture atlas, can represent unallocated. */
	FIntVector MinTexel;

	/** Size in texels of the allocation into the volume texture atlas. */
	int32 AllocationTexelSize;

	/** Position at the new single lighting sample. Used for interpolation over time. */
	FVector TargetPosition;

	/** SH sample at the new single lighting sample position. Used for interpolation over time. */
	FVector4f TargetSamplePacked0[3];	// { { R.C0, R.C1, R.C2, R.C3 }, { G.C0, G.C1, G.C2, G.C3 }, { B.C0, B.C1, B.C2, B.C3 } }
	FVector4f TargetSamplePacked1[3];	// { { R.C4, R.C5, R.C6, R.C7 }, { G.C4, G.C5, G.C6, G.C7 }, { B.C4, B.C5, B.C6, B.C7 } }
	FVector4f TargetSamplePacked2;		// { R.C8, R.C8, R.C8, R.C8 }

	/** Target shadowing of the stationary directional light. */
	float TargetDirectionalShadowing;

	/** Target directional occlusion of the sky. */
	FVector4f TargetSkyBentNormal;

	/** Current position of the single lighting sample.  Used for interpolation over time. */
	FVector SingleSamplePosition;

	/** Current SH sample used when lighting the entire object with one sample. */
	FVector4f SingleSamplePacked0[3];	// { { R.C0, R.C1, R.C2, R.C3 }, { G.C0, G.C1, G.C2, G.C3 }, { B.C0, B.C1, B.C2, B.C3 } }
	FVector4f SingleSamplePacked1[3];	// { { R.C4, R.C5, R.C6, R.C7 }, { G.C4, G.C5, G.C6, G.C7 }, { B.C4, B.C5, B.C6, B.C7 } }
	FVector4f SingleSamplePacked2;		// { R.C8, R.C8, R.C8, R.C8 }

	/** Current shadowing of the stationary directional light. */
	float CurrentDirectionalShadowing;

	/** Current directional occlusion of the sky. */
	FVector4f CurrentSkyBentNormal;

	/** Whether SingleSamplePacked has ever been populated with valid results, used to initialize. */
	bool bHasEverUpdatedSingleSample;

	/** Whether this allocation is a point sample and therefore was not put into the volume texture atlas. */
	bool bPointSample;

	/** Whether the primitive allocation is dirty and should be updated regardless of having moved. */
	bool bIsDirty;

	bool bUnbuiltPreview;

	void SetDirty() 
	{ 
		bIsDirty = true; 
	}

	bool IsValid() const
	{
		return MinTexel.X >= 0 && MinTexel.Y >= 0 && MinTexel.Z >= 0 && AllocationTexelSize > 0;
	}

	void SetParameters(FIntVector InMinTexel, int32 InAllocationTexelSize, FVector InScale, FVector InAdd, FVector InMinUV, FVector InMaxUV, bool bInPointSample, bool bInUnbuiltPreview)
	{
		checkf(InAllocationTexelSize > 1 || bInPointSample, TEXT("%i, %i"), InAllocationTexelSize, bInPointSample ? 1 : 0);
		Add = InAdd;
		Scale = InScale;
		MinUV = InMinUV;
		MaxUV = InMaxUV;
		MinTexel = InMinTexel;
		AllocationTexelSize = InAllocationTexelSize;
		bIsDirty = false;
		bPointSample = bInPointSample;
		bUnbuiltPreview = bInUnbuiltPreview;
	}
};

/** Flags needed for shadow culling.  These are pulled out of the FPrimitiveSceneProxy so we can do rough culling before dereferencing the proxy. */
struct FPrimitiveFlagsCompact
{
	/** True if the primitive casts dynamic shadows. */
	uint8 bCastDynamicShadow : 1;

	/** True if the primitive will cache static lighting. */
	uint8 bStaticLighting : 1;

	/** True if the primitive casts static shadows. */
	uint8 bCastStaticShadow : 1;

	/** True if the primitive is a Nanite mesh. */
	uint8 bIsNaniteMesh : 1;

	/** True if the primitive is always visible. */
	uint8 bIsAlwaysVisible : 1;

	/** True if the primitive draws only meshes that support GPU-Scene. */
	uint8 bSupportsGPUScene : 1;

	/** True if the proxy is force hidden. */
	uint8 bIsForceHidden : 1;

	FPrimitiveFlagsCompact(const FPrimitiveSceneProxy* Proxy);
};

/** The information needed to determine whether a primitive is visible. */
class FPrimitiveSceneInfoCompact
{
public:
	FPrimitiveSceneInfo* PrimitiveSceneInfo;
	FPrimitiveSceneProxy* Proxy;
	FCompactBoxSphereBounds Bounds;
	float MinDrawDistance;
	float MaxDrawDistance;
	/** Used for precomputed visibility */
	int32 VisibilityId;
	FPrimitiveFlagsCompact PrimitiveFlagsCompact;

	/** Initialization constructor. */
	FPrimitiveSceneInfoCompact(FPrimitiveSceneInfo* InPrimitiveSceneInfo);
};

/** Lod data used for runtime virtual texture page rendering. Packed to reduce memory overhead since one of these is allocated per primitive. */
struct FPrimitiveRuntimeVirtualTextureLodInfo
{
	/** LodBias is in range [-7,8] so is stored with this offset. */
	enum { LodBiasOffset = 7 };

	/** Minimum Lod for primitive in the runtime virtual texture. */
	uint16 MinLod : 4 = 0;
	/** Maximum Lod for primitive in the runtime virtual texture. */
	uint16 MaxLod : 4 = 0;
	/** Bias to use for Lod calculation in the runtime virtual texture. */
	uint16 LodBias : 4 = 0;
	/** 
	 * Culling method used to remove the primitive from low mips of the runtime virtual texture.
	 * 0: CullValue is the number of low mips for which we cull the primitive from the runtime virtual texture.
	 * 1: CullValue is the pixel coverage threshold at which we cull the primitive from the runtime virtual texture. 
	 */
	uint16 CullMethod : 1 = 0;
	/** Value used according to the CullMethod. */
	uint16 CullValue : 3 = 0;
};

/** The type of the octree used by FScene to find primitives. */
typedef TOctree2<FPrimitiveSceneInfoCompact,struct FPrimitiveOctreeSemantics> FScenePrimitiveOctree;

/** Nanite mesh pass types. */
namespace ENaniteMeshPass
{
	enum Type
	{
		BasePass,
		LumenCardCapture,
		MaterialCache,
		Num,
	};
}

enum class EUpdateStaticMeshFlags : uint8
{
	RasterCommands		= (1U << 1U),
	RayTracingCommands	= (1U << 2U),

	AllCommands			= RasterCommands | RayTracingCommands,
};
ENUM_CLASS_FLAGS(EUpdateStaticMeshFlags);

/**
 * The renderer's internal state for a single UPrimitiveComponent.  This has a one to one mapping with FPrimitiveSceneProxy, which is in the engine module.
 */
class FPrimitiveSceneInfo : public FDeferredCleanupInterface
{
	friend class FSceneRenderer;
	friend struct FViewDebugInfo;
public:
	using FPersistentId = FPersistentPrimitiveIndex;

	/** The render proxy for the primitive. */
	FPrimitiveSceneProxy* Proxy;

	/** 
	 * Id for the component this primitive belongs to.
	 * This will stay the same for the lifetime of the component, so it can be used to identify the component across re-registers.
	 */
	FPrimitiveComponentId PrimitiveComponentId;

	/** 
	 * The root attachment component id for use with lighting, if valid.
	 * If the root id is not valid, this is a parent primitive.
	 */
	FPrimitiveComponentId LightingAttachmentRoot;

	/** 
	 * The component id of the LOD parent if valid.
	 */
	FPrimitiveComponentId LODParentComponentId;

	/** The primitive's cached mesh draw commands infos for all static meshes. Kept separately from StaticMeshes for cache efficiency inside InitViews. */
	TArray<class FCachedMeshDrawCommandInfo> StaticMeshCommandInfos;

	/** The primitive's static mesh relevances. Must be in sync with StaticMeshes. Kept separately from StaticMeshes for cache efficiency inside InitViews. */
	TArray<class FStaticMeshBatchRelevance> StaticMeshRelevances;

	/** The primitive's static meshes. */
	TArray<class FStaticMeshBatch> StaticMeshes;

	TArray<FNaniteRasterBin> NaniteRasterBins[ENaniteMeshPass::Num];
	TArray<FNaniteShadingBin> NaniteShadingBins[ENaniteMeshPass::Num];
	TArray<FNaniteMaterialSlot> NaniteMaterialSlots[ENaniteMeshPass::Num];

	/** The identifier for the primitive in Scene->PrimitiveOctree. */
	FOctreeElementId2 OctreeId;

	/** 
	 * Caches the primitive's indirect lighting cache allocation.
	 * Note: This is only valid during the rendering of a frame, not just once the primitive is attached. 
	 */
	const FIndirectLightingCacheAllocation* IndirectLightingCacheAllocation;

	/** 
	 * The uniform buffer holding precomputed lighting parameters for the indirect lighting cache allocation.
	 * WARNING : This can hold buffer valid for a single frame only, don't cache anywhere. 
	 * See FPrimitiveSceneInfo::UpdateIndirectLightingCacheBuffer()
	 */
	TUniformBufferRef<FIndirectLightingCacheUniformParameters> IndirectLightingCacheUniformBuffer;

	/** 
	 * Planar reflection that was closest to this primitive, used for forward reflections.
	 */
	const FPlanarReflectionSceneProxy* CachedPlanarReflectionProxy;

	/** 
	 * Reflection capture proxy that was closest to this primitive, used for the forward shading rendering path. 
	 */
	const FReflectionCaptureProxy* CachedReflectionCaptureProxy;

	/** Mapping from instance index in this primitive to index in the global distance field object buffers. */
	TArray<int32, TInlineAllocator<1>> DistanceFieldInstanceIndices;

	/** Mapping from instance index in this primitive to index in the LumenPrimitiveGroup array. */
	TArray<int32, TInlineAllocator<1>> LumenPrimitiveGroupIndices;

	static const uint32 MaxCachedReflectionCaptureProxies = 3;
	UE_DEPRECATED(5.8, "CachedReflectionCaptureProxies is no longer initialized, and will be removed in a future release.  Use CachedReflectionCaptureProxy.")
	const FReflectionCaptureProxy* CachedReflectionCaptureProxies[MaxCachedReflectionCaptureProxies];
	
	/** The hit proxies used by the primitive. */
	TArray<TRefCountPtr<HHitProxy> > HitProxies;

	/** The hit proxy which is used to represent the primitive's dynamic elements. */
	HHitProxy* DefaultDynamicHitProxy;

	/** The ID of the hit proxy which is used to represent the primitive's dynamic elements. */
	FHitProxyId DefaultDynamicHitProxyId;

	/** Last render time in seconds since level started play. */
	float LastRenderTime;

	/** The list of lights affecting this primitive. */
	class FLightPrimitiveInteraction* LightList;

	/** The scene the primitive is in. */
	FScene* Scene;

	/** The number of local lights with dynamic lighting for mobile */
	int32 NumMobileDynamicLocalLights;

	/** The sphere radius to use for per instance GPU Lodding. Will be 0.f if GPU LOD isn't enabled on this primitive. */
	float GpuLodInstanceRadius;

	/** Initialization constructor. */
	FPrimitiveSceneInfo(UPrimitiveComponent* InPrimitive,FScene* InScene);
	FPrimitiveSceneInfo(FPrimitiveSceneDesc* InPrimitiveSceneDesc, FScene* InScene);

	/** Destructor. */
	~FPrimitiveSceneInfo();

	/** Adds the primitive to the scene. */
	static void AddToScene(FScene* Scene, TArrayView<FPrimitiveSceneInfo*> SceneInfos);

	/** Removes the primitive from the scene. */
	void RemoveFromScene(bool bUpdateStaticDrawLists);

	/** Allocate/Free slots for instance data in GPU-Scene */
	static void AllocateGPUSceneInstances(FScene* Scene, const TArrayView<FPrimitiveSceneInfo*>& SceneInfos);
	void FreeGPUSceneInstances();

	/** return true if we need to call LazyUpdateForRendering */
	inline bool NeedsIndirectLightingCacheBufferUpdate()
	{
		return bIndirectLightingCacheBufferDirty;
	}

	/** Updates the primitive's static meshes in the scene. */
	static void UpdateStaticMeshes(FScene* Scene, TArrayView<FPrimitiveSceneInfo*> SceneInfos, EUpdateStaticMeshFlags UpdateFlags, bool bReAddToDrawLists = true);

	/** Sets a flag to update the primitive's static meshes before it is next rendered. */
	void RequestStaticMeshUpdate();

	/** Sets a flag to update the primitive's uniform buffer before it is next rendered. */
	RENDERER_API bool RequestUniformBufferUpdate();

	/** Adds the primitive's static meshes to the scene. */
	static void AddStaticMeshes(FRHICommandListBase& RHICmdList, FScene* Scene, TArrayView<FPrimitiveSceneInfo*> SceneInfos, bool bCacheMeshDrawCommands = true);

	/** Removes the primitive's static meshes from the scene. */
	void RemoveStaticMeshes();

	/** Set LOD Parent primitive information to the scene. */
	void LinkLODParentComponent();

	/** clear LOD parent primitive information from the scene. */
	void UnlinkLODParentComponent();

	/** Adds the primitive to the scene's attachment groups. */
	void LinkAttachmentGroup();

	/** Removes the primitive from the scene's attachment groups. */
	void UnlinkAttachmentGroup();

	/** Adds a request to update GPU scene representation. */
	RENDERER_API bool RequestGPUSceneUpdate(EPrimitiveDirtyState PrimitiveDirtyState = EPrimitiveDirtyState::ChangedAll);

	/** Marks the primitive UB as needing updated and requests a GPU scene update */
	void MarkGPUStateDirty(EPrimitiveDirtyState PrimitiveDirtyState = EPrimitiveDirtyState::ChangedAll)
	{
		RequestUniformBufferUpdate();
		RequestGPUSceneUpdate(PrimitiveDirtyState);
	}

	/** Refreshes a primitive's references to raster bins. To be called after changes that might have invalidated them. */
	RENDERER_API void RefreshNaniteRasterBins();

	/** Update the occlusion flags for this primitive. */
	void UpdateOcclusionFlags();

	/** 
	 * Builds an array of all primitive scene info's in this primitive's attachment group. 
	 * This only works on potential parents (!LightingAttachmentRoot.IsValid()) and will include the current primitive in the output array.
	 */
	void GatherLightingAttachmentGroupPrimitives(TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator>& OutChildSceneInfos);
	void GatherLightingAttachmentGroupPrimitives(TArray<const FPrimitiveSceneInfo*, SceneRenderingAllocator>& OutChildSceneInfos) const;

	/** 
	 * Builds a cumulative bounding box of this primitive and all the primitives in the same attachment group. 
	 * This only works on potential parents (!LightingAttachmentRoot.IsValid()).
	 */
	FBoxSphereBounds GetAttachmentGroupBounds() const;

	/** Size this class uses in bytes */
	uint32 GetMemoryFootprint();

	/** 
	 * Retrieves the index of the primitive in the scene's primitives array.
	 * This index is only valid until a primitive is added to or removed from
	 * the scene!
	 */
	inline int32 GetIndex() const { return PackedIndex; }
	/** 
	 * Retrieves the address of the primitives index into in the scene's primitives array.
	 * This address is only for reference purposes
	 */
	inline const int32* GetIndexAddress() const { return &PackedIndex; }

	/** Simple comparison against the invalid values used before/after scene add/remove. */
	inline bool IsIndexValid() const { return PackedIndex != INDEX_NONE && PackedIndex != MAX_int32; }

	/**
	 * Persistent index of the primitive in the range [0, FScene::GetMaxPersistentPrimitiveIndex() ). Where the max is never higher than the high watermark of the primitives in the scene.
	 * This index remains stable for the life-time of the primitive in the scene (i.e., same as PrimitiveSceneInfo and Proxy - between FScene::AddPrimitive / FScene::RemovePrimitive). 
	 * The intended use is to enable tracking primitive data over time in renderer systems using direct indexing, e.g., for efficiently storing a bit per primitive. 
	 * Direct indexing also facilitates easy GPU-reflection and access of the persistent data, where it can be accessed as e.g., GetPrimitiveData(...).PersistentPrimitiveIndex.
	 * It is allocated using FScene::PersistentPrimitiveIdAllocator from a range with holes that is kept as compact as possible up to the high-water mark of the scene primitives. 
	 * Due to persistence this maximum can be substantially larger than the current number of Primitives at times, but never worse than the high-watermark.
	 * In the future, the index will likely be refactored to persist for the lifetime of the component (to facilitate tracking data past proxy re-creates).
	 * Note: It is not currently used to index any of the current FScene primitive arrays (use PackedIndex), though this is intended to change.
	 */
	inline FPersistentPrimitiveIndex GetPersistentIndex() const { return PersistentIndex; }

	/**
	 * Shifts primitive position and all relevant data by an arbitrary delta.
	 * Called on world origin changes
	 * @param InOffset - The delta to shift by
	 */
	void ApplyWorldOffset(FRHICommandListBase& RHICmdList, FVector InOffset);

	inline void MarkIndirectLightingCacheBufferDirty()
	{
		if (!bIndirectLightingCacheBufferDirty)
		{
			bIndirectLightingCacheBufferDirty = true;
		}
	}

	void UpdateIndirectLightingCacheBuffer(FRHICommandListBase& RHICmdList);

	/** Will output the LOD ranges of the static meshes used with this primitive. */
	RENDERER_API void GetStaticMeshesLODRange(int8& OutMinLOD, int8& OutMaxLOD) const;

	/** Will output the FMeshBatch associated with the specified LODIndex. */
	RENDERER_API const FMeshBatch* GetMeshBatch(int8 InLODIndex) const;

	int32 GetInstanceSceneDataOffset() const { return InstanceSceneDataOffset; }
	int32 GetNumInstanceSceneDataEntries() const { return NumInstanceSceneDataEntries; }

	int32 GetInstancePayloadDataOffset() const { return InstancePayloadDataOffset; }
	int32 GetInstancePayloadDataStride() const { return InstancePayloadDataStride; }

	int32 GetLightmapDataOffset() const { return LightmapDataOffset; }
	int32 GetNumLightmapDataEntries() const { return NumLightmapDataEntries; }

	/** Returns whether the primitive needs to call CacheReflectionCaptures. */
	bool NeedsReflectionCaptureUpdate() const;

	/** Cache per-primitive reflection captures used for mobile/forward rendering.  Returns true if the cached reflection captures changed. */
	bool CacheReflectionCaptures();

	/** Nulls out the cached per-primitive reflection capture, if it matches ChangedProxy or ChangedProxy is null. */
	void RemoveCachedReflectionCapture(const FReflectionCaptureProxy* ChangedProxy = nullptr);

	/** Nulls out the cached per-primitive planar reflection, if it matches ChangedProxy or ChangedProxy is null. */
	void RemoveCachedPlanarReflection(const FPlanarReflectionSceneProxy* ChangedProxy = nullptr);

	/** Helper function for writing out to the last render times to the game thread */
	void UpdateComponentLastRenderTime(float CurrentWorldTime, bool bUpdateLastRenderTimeOnScreen);

	/** Updates static lighting uniform buffer, returns the number of entries needed for GPUScene */
	RENDERER_API int32 UpdateStaticLightingBuffer();

	/** Get the packed LodInfo. */
	FPrimitiveRuntimeVirtualTextureLodInfo const& GetRuntimeVirtualTextureLodInfo() const { return RuntimeVirtualTextureLodInfo; }

	/** Mark the runtime virtual textures covered by this primitive as dirty. */
	void FlushRuntimeVirtualTexture();

#if RHI_RAYTRACING
	static void UpdateCachedRaytracingData(FScene* Scene, const TArrayView<FPrimitiveSceneInfo*>& SceneInfos);

	bool IsCachedRayTracingGeometryValid(int8 LODIndex, int32 InstanceIndex) const;

	UE_DEPRECATED(5.8, "Please specify both LODIndex and InstanceIndex.")
	bool IsCachedRayTracingGeometryValid(int8 LODIndex) const
	{
		return IsCachedRayTracingGeometryValid(LODIndex, 0);
	}

	UE_DEPRECATED(5.8, "Use GetRayTracingLODData() instead.")
	RENDERER_API const FRayTracingGeometry* GetStaticRayTracingGeometry(int8 LODIndex) const;
	UE_DEPRECATED(5.8, "Use GetRayTracingLODData() instead.")
	RENDERER_API const FRayTracingGeometry* GetValidStaticRayTracingGeometry(int8& InOutLODIndex) const;

	int32 GetStaticRayTracingGeometryNum() const { return RayTracingLODData.Num(); }

	UE_DEPRECATED(5.7, "Use GetRayTracingLODData() instead.")
	const FRayTracingGeometry* GetCachedRayTracingGeometry() const { return nullptr; }
#endif

	/** Return primitive fullname (for debugging only). */
	FString GetFullnameForDebuggingOnly() const;
	/** Return primitive Owner actor name (for debugging only). */
	FString GetOwnerActorNameOrLabelForDebuggingOnly() const;
	UE_DEPRECATED(5.7, "Unused")
	inline bool ShouldCacheShadowAsStatic() const
	{
		return false;
	}

	inline FMeshDrawCommandPrimitiveIdInfo GetMDCIdInfo() const { return FMeshDrawCommandPrimitiveIdInfo(PackedIndex, PersistentIndex, InstanceSceneDataOffset);}

	UPrimitiveComponent* GetComponentForDebugOnly() const;
	IPrimitiveComponent* GetComponentInterfaceForDebugOnly() const;

	bool HasInstanceDataBuffers() const { return InstanceSceneDataBuffersInternal != nullptr; }
	
	/**
	 * Waits for (potential) instance update to produce the data, to avoid a sync, use GetInstanceDataHeader().
	 */
	RENDERER_API const FInstanceSceneDataBuffers* GetInstanceSceneDataBuffers() const;

	/**
	 * Returns the updated header data in the InstanceDataUpdateTaskInfo without blocking. 
	 * For a primitive without instances, it returns a header with an instance count of one.
	 */
	FInstanceDataBufferHeader GetInstanceDataHeader() const;

	/** Returns the primitive scene data for this proxy. */
	const FPrimitiveSceneInfoData* GetSceneData() const { return SceneData; }

private:
	
	FPrimitiveSceneInfo(const FPrimitiveSceneInfoAdapter& InAdapter, FScene* InScene);

	/** Let FScene have direct access to the Id. */
	friend class FScene;

	/**
	 * The index of the primitive in the scene's packed arrays. This value may
	 * change as primitives are added and removed from the scene.
	 */
	int32 PackedIndex;

	/**
	 * See GetPersistentIndex()
	 */
	FPersistentPrimitiveIndex PersistentIndex;

	/** 
	 * The IPrimitiveComponentInterface this scene info is for, useful for quickly inspecting properties on the corresponding component while debugging.
	 * This should not be dereferenced on the rendering thread.  The game thread can be modifying UObject members at any time.
	 * Use PrimitiveComponentId instead when a component identifier is needed.
	 * 	
	 */	
	IPrimitiveComponent* PrimitiveComponentInterfaceForDebuggingOnly;

	/** 
	 * Ptr to the FPrimitiveSceneInfoData for this prim, this is used for shared data between the primitive and the component that created the primitive. 
	 */
	FPrimitiveSceneInfoData* SceneData;

	/** Packed LOD info used for selecting LOD when rendering to a runtime virtual texture. */
	FPrimitiveRuntimeVirtualTextureLodInfo RuntimeVirtualTextureLodInfo;

	/** If this is TRUE, this primitive's uniform buffer needs to be updated before it can be rendered. */
	bool bNeedsUniformBufferUpdate : 1;

	/** If this is TRUE, this primitive's indirect lighting cache buffer needs to be updated before it can be rendered. */
	bool bIndirectLightingCacheBufferDirty : 1;

	/** If this is TRUE, this primitive has registered with the virtual texture system for a callback on virtual texture changes. */
	bool bRegisteredLightmapVirtualTextureProducerCallback : 1;

	/** True if the primitive registered with velocity data and needs to remove itself when being removed from the scene. */
	bool bRegisteredWithVelocityData : 1;

	/** True if the Nanite raster bins were registered with custom depth enabled */
	bool bNaniteRasterBinsRenderCustomDepth : 1;

	/** True if the primitive is queued for add. */
	bool bPendingAddToScene : 1;

	/** True if the primitive is queued to have static meshes built. */
	bool bPendingAddStaticMeshes : 1;

	/** True if the primitive is queued for deletion. */
	bool bPendingDelete : 1 = false;

	/** True if the primitive is queued to have its runtime virtual texture flushed. */
	bool bPendingFlushRuntimeVirtualTexture : 1;

public:
	/** Set to true for the primitive to be rendered in the main pass to be visible in a view. */
	bool bShouldRenderInMainPass : 1;

	/** Set to true for the primitive to be rendered into the real-time sky light reflection capture. */
	bool bVisibleInRealTimeSkyCapture : 1;

	/** True if the primitive can write to runtime virtual texture. */
	bool bWritesRuntimeVirtualTexture : 1;

	/** Whether the primitive is newly registered or moved and CachedReflectionCaptureProxy needs to be updated on the next render. Not a bitfield to avoid parallel access with other bits. */
	bool bNeedsCachedReflectionCaptureUpdate = true;

#if RHI_RAYTRACING
	bool : 0; // force separate storage unit to avoid race conditions with tasks modifying other non-raytracing bits.
	bool bHasRayTracingRepresentation : 1 = false;
	bool bIsRelevantToPathTracing : 1;
	bool bIsCachedRayTracingInstanceValid : 1;
	UE_DEPRECATED(5.7, "Use DynamicRayTracingInstanceCachedData or GetRayTracingLODData(...) instead.")
	bool bCachedRayTracingInstanceMaskAndFlagsDirty : 1;
	UE_DEPRECATED(5.7, "Use DynamicRayTracingInstanceCachedData or GetRayTracingLODData(...) instead.")
	bool bCachedRayTracingInstanceAnySegmentsDecal : 1;
	UE_DEPRECATED(5.7, "Use DynamicRayTracingInstanceCachedData or GetRayTracingLODData(...) instead.")
	bool bCachedRayTracingInstanceAllSegmentsDecal : 1;
	UE_DEPRECATED(5.7, "Use DynamicRayTracingInstanceCachedData or GetRayTracingLODData(...) instead.")
	bool bCachedRayTracingInstanceAllSegmentsTranslucent : 1;
	bool : 0; // force separate storage unit to avoid race condition modifying bIsCachedRayTracingInstanceValid and bDynamicRayTracingInstanceCachedDataDirty.
	bool bDynamicRayTracingInstanceCachedDataDirty : 1 = true;

	struct FDynamicRayTracingInstanceCachedData
	{
		uint8 Mask = 0;
		uint8 bForceOpaque : 1 = false;
		uint8 bDoubleSided : 1 = false;
		uint8 bReverseCulling : 1 = false;
		uint8 bAnySegmentsDecal : 1 = false;
		uint8 bAllSegmentsDecal : 1 = false;
		uint8 bAllSegmentsTranslucent : 1 = false;
	};
	FDynamicRayTracingInstanceCachedData DynamicRayTracingInstanceCachedData;

	struct FRayTracingInstanceData
	{
		const FRayTracingGeometry* RayTracingGeometry = nullptr;

		FRayTracingSBTAllocation* SBTAllocation = nullptr;
		FRayTracingCachedMeshCommandFlags CachedMeshCommandFlags;
		TArray<FRayTracingShaderBindingData> CachedShaderBindingDataBase;
		TArray<FRayTracingShaderBindingData> CachedShaderBindingDataDecal;
		int32 BaseCachedMeshCommandIndex = INDEX_NONE;
		int32 NumCachedMeshCommands = 0;
		int32 SBTAllocationUniqueId = -1;
		int32 InstanceContributionToHitGroupIndexBase = INDEX_NONE;
		int32 InstanceContributionToHitGroupIndexDecal = INDEX_NONE;

		uint32 RayTracingInstanceIndexMain = UINT32_MAX;
		uint32 RayTracingInstanceIndexDecal = UINT32_MAX;
	};

	struct FRayTracingLODData
	{
		// for cached static instances (hot path), we only cache data for one LOD, so inline allocator avoids cache misses.
		TArray<FRayTracingInstanceData, TInlineAllocator<1>> Instances;
	};

	// TODO: rename to GetStaticRayTracingLODNum ?
	uint32 GetRayTracingLODDataNum() const
	{
		return RayTracingLODData.Num();
	}

	const FRayTracingLODData& GetRayTracingLODData(uint32 Index) const
	{
		return RayTracingLODData[Index];
	}

	UE_DEPRECATED(5.7, "Cached Ray Tracing Instances are stored directly in FRayTracingScene. Use GetBaseRayTracingInstanceIndex()/GetDecalRayTracingInstanceIndex() instead.")
	const FRayTracingGeometryInstance GetCachedRayTracingInstance() const
	{
		return {};
	}

	UE_DEPRECATED(5.8, "Get index for specific instances via FRayTracingLODData instead.")
	const uint32 GetMainRayTracingInstanceIndex() const
	{
		if (RayTracingLODData.IsEmpty() || RayTracingLODData[0].Instances.IsEmpty())
		{
			return UINT32_MAX;
		}

		return RayTracingLODData[0].Instances[0].RayTracingInstanceIndexMain;
	}

	UE_DEPRECATED(5.8, "Get index for specific instances via FRayTracingLODData instead.")
	const uint32 GetDecalRayTracingInstanceIndex() const
	{
		if (RayTracingLODData.IsEmpty() || RayTracingLODData[0].Instances.IsEmpty())
		{
			return UINT32_MAX;
		}

		return RayTracingLODData[0].Instances[0].RayTracingInstanceIndexDecal;
	}

	// TODO: rename to SetStaticRayTracingLODData ?
	void SetRayTracingLODData(TArray<FRayTracingLODData> LODData)
	{
		RayTracingLODData = MoveTemp(LODData);
	}

	void SetCachedRayTracingInstanceGeometryRHI(FRHIRayTracingGeometry* GeometryRHI, uint32 SegmentCount);

	UE_DEPRECATED(5.7, "Please provide SegmentCount.")
	void SetCachedRayTracingInstanceGeometryRHI(FRHIRayTracingGeometry* GeometryRHI)
	{
		SetCachedRayTracingInstanceGeometryRHI(GeometryRHI, GeometryRHI->GetInitializer().Segments.Num());
	}

	UE_DEPRECATED(5.7, "No longer necessary.")
	void UpdateCachedRayTracingInstanceMaskAndFlags(FRayTracingMaskAndFlags& InstanceMaskAndFlags) {}

private:

	void AllocateRayTracingSBT(FRayTracingInstanceData& InstanceData, const FRHIRayTracingGeometry* CachedRayTracingInstanceGeometryRHI, uint32 CachedRayTracingInstanceSegmentCount);

	void CacheRayTracingShaderBindingData(FRayTracingInstanceData& InstanceData, const FRHIRayTracingGeometry* CachedRayTracingInstanceGeometryRHI);
	void CacheRayTracingInstance(FRayTracingInstanceData& InstanceData, FRayTracingGeometryInstance Instance);

	// for cached static instances (hot path), we only cache data for one LOD, so inline allocator avoids cache misses.
	TArray<FRayTracingLODData, TInlineAllocator<1>> RayTracingLODData;
#endif // RHI_RAYTRACING

private:
	// Don't access this directly, even internally unless you are sure what you're up to. Use GetInstanceSceneDataBuffers() which handles thread safety.
	const FInstanceSceneDataBuffers *InstanceSceneDataBuffersInternal = nullptr;
	FInstanceDataUpdateTaskInfo *InstanceDataUpdateTaskInfo = nullptr;

	/** Index into the scene's PrimitivesNeedingLevelUpdateNotification array for this primitive scene info level. */
	int32 LevelUpdateNotificationIndex;

	/** Offset into the scene's instance scene data buffer, when GPUScene is enabled. */
	int32 InstanceSceneDataOffset;

	/** Number of entries in the scene's instance scene data buffer. */
	int32 NumInstanceSceneDataEntries;

	/** Offset into the scene's instance payload data buffer, when GPUScene is enabled. */
	int32 InstancePayloadDataOffset;

	/** Number of float4 payload data values per instance */
	int32 InstancePayloadDataStride;

	/** Offset into the scene's lightmap data buffer, when GPUScene is enabled. */
	int32 LightmapDataOffset;

	/** Number of entries in the scene's lightmap data buffer. */
	int32 NumLightmapDataEntries;

	void UpdateIndirectLightingCacheBuffer(
		FRHICommandListBase& RHICmdList,
		const class FIndirectLightingCache* LightingCache,
		const class FIndirectLightingCacheAllocation* LightingAllocation,
		FVector VolumetricLightmapLookupPosition,
		uint32 SceneFrameNumber,
		class FVolumetricLightmapSceneData* VolumetricLightmapSceneData);

	/** Creates cached mesh draw commands for all meshes. */
	static void CacheMeshDrawCommands(FScene* Scene, TArrayView<FPrimitiveSceneInfo*> SceneInfos);

	/** Removes cached mesh draw commands for all meshes. */
	void RemoveCachedMeshDrawCommands();

	/** Constructs Nanite raster and shading bin information for unique material instances found within the scene. */
	static void CacheNaniteMaterialBins(FScene* Scene, const TArrayView<FPrimitiveSceneInfo*>& SceneInfos);

	/** Removes Nanite raster and shading bin information from the scene. */
	void RemoveCachedNaniteMaterialBins();

#if RHI_RAYTRACING
	/** Creates cached ray tracing representations for all meshes. */
	static void CacheRayTracingPrimitives(FScene* Scene, const TArrayView<FPrimitiveSceneInfo*>& SceneInfos);

	/** Removes cached ray tracing representations for all meshes. */
	void RemoveCachedRayTracingPrimitives();

	static void SetupCachedRayTracingInstance(FPrimitiveSceneInfo* SceneInfo, const FRayTracingInstance& RayTracingInstance, FRayTracingGeometryInstance& OutCachedRayTracingInstance, uint32& OutSegmentCount);
#endif

public:
	DECLARE_MULTICAST_DELEGATE(FPrimitiveSceneInfoEvent);
	RENDERER_API static FPrimitiveSceneInfoEvent OnGPUSceneInstancesAllocated;
	RENDERER_API static FPrimitiveSceneInfoEvent OnGPUSceneInstancesFreed;
};

/** Defines how the primitive is stored in the scene's primitive octree. */
struct FPrimitiveOctreeSemantics
{
	/** Note: this is coupled to shadow gather task granularity, see r.ParallelGatherShadowPrimitives. */
	enum { MaxElementsPerLeaf = 256 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef FDefaultAllocator ElementAllocator;

	inline static const FCompactBoxSphereBounds& GetBoundingBox(const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact)
	{
		return PrimitiveSceneInfoCompact.Bounds;
	}

	inline static bool AreElementsEqual(const FPrimitiveSceneInfoCompact& A,const FPrimitiveSceneInfoCompact& B)
	{
		return A.PrimitiveSceneInfo == B.PrimitiveSceneInfo;
	}

	inline static void SetElementId(const FPrimitiveSceneInfoCompact& Element,FOctreeElementId2 Id)
	{
		Element.PrimitiveSceneInfo->OctreeId = Id;
		SetOctreeNodeIndex(Element, Id);
	}

	inline static void ApplyOffset(FPrimitiveSceneInfoCompact& Element, FVector Offset)
	{
		Element.Bounds.Origin+= Offset;
	}

	static void SetOctreeNodeIndex(const FPrimitiveSceneInfoCompact& Element, FOctreeElementId2 Id);
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MeshPassProcessor.h"
#endif
