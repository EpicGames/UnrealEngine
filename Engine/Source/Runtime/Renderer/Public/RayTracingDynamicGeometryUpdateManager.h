// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"

#if RHI_RAYTRACING

#include "RHI.h"
#include "RHIUtilities.h"
#include "RenderGraphDefinitions.h"
#include "MeshPassProcessor.h"
#include "SceneManagement.h"

class FPrimitiveSceneProxy;
class FScene;
class FSceneView;
class FViewInfo;
class FRDGBuilder;
class FRayTracingGeometry;
struct FMeshComputeDispatchCommand;

DECLARE_UNIFORM_BUFFER_STRUCT(FSceneUniformParameters, RENDERER_API)

class FRayTracingDynamicGeometryUpdateManager
{
public:

	RENDERER_API FRayTracingDynamicGeometryUpdateManager();
	virtual RENDERER_API ~FRayTracingDynamicGeometryUpdateManager();
	
	/** Add dynamic geometry to update including CS shader to deform the vertices */
	RENDERER_API void AddDynamicGeometryToUpdate(
		FRHICommandListBase& RHICmdList,
		const FScene* Scene,
		const FSceneView* View,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		FRayTracingDynamicGeometryUpdateParams Params,
		uint32 PrimitiveId
	);

	/** Starts an update batch and returns the current shared buffer generation ID which is used for validation. */
	RENDERER_API int64 BeginUpdate();

	RENDERER_API void ScheduleUpdates(FRDGBuilder& GraphBuilder, bool bUseTracingFeedback);

	/** Add RDG pass which will dispatch all dynamic geometry vertex updates and then request BLAS build and update for all pending requests */
	RENDERER_API void AddDynamicGeometryUpdatePass(
		FRDGBuilder& GraphBuilder,
		ERDGPassFlags ComputePassFlags,
		const TRDGUniformBufferRef<FSceneUniformParameters>& SceneUB,
		ERHIPipeline ResourceAccessPipelines,
		FRDGBufferRef& OutDynamicGeometryScratchBuffer);

	UE_DEPRECATED(5.8, "Use the version that doesn't take bUseTracingFeedback parameter and manually call ScheduleUpdates() instead.")
	void AddDynamicGeometryUpdatePass(
		FRDGBuilder& GraphBuilder,
		ERDGPassFlags ComputePassFlags,
		const TRDGUniformBufferRef<FSceneUniformParameters>& SceneUB,
		bool bUseTracingFeedback,
		ERHIPipeline ResourceAccessPipelines,
		FRDGBufferRef& OutDynamicGeometryScratchBuffer)
	{
		ScheduleUpdates(GraphBuilder, bUseTracingFeedback);

		AddDynamicGeometryUpdatePass(GraphBuilder, ComputePassFlags, SceneUB, ResourceAccessPipelines, OutDynamicGeometryScratchBuffer);
	}

	/** Clears the working arrays to not hold any references. */
	RENDERER_API void Clear();

private:

	struct FUpdateRequest
	{
		const FScene* Scene = nullptr;
		const FSceneView* View = nullptr;
		const FPrimitiveSceneProxy* PrimitiveSceneProxy = nullptr;
		FRayTracingDynamicGeometryUpdateParams UpdateParams;
		uint32 PrimitiveId = UINT32_MAX;
		float Priority = 0.0f;
		bool bRecreateGeometry = false;
		bool bSkipped = false;
		EAccelerationStructureBuildMode ScheduledBuildMode = EAccelerationStructureBuildMode::Build;
	};

	static void DispatchUpdates(FRHICommandList& RHICmdList, TConstArrayView<FMeshComputeDispatchCommand> DispatchCommands, ERHIPipeline SrcResourceAccessPipelines = ERHIPipeline::Graphics, ERHIPipeline DstResourceAccessPipelines = ERHIPipeline::Graphics);

	void EndUpdate();

	void ClassifyRequests(bool bUseTracingFeedback, bool bVertexBufferRequired);
	void AddRequestsToBuildList(FRDGBuilder& GraphBuilder);

	void AddDispatchCommands(
		FRHICommandListBase& RHICmdList,
		const FScene* Scene,
		const FSceneView* View,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FRayTracingDynamicGeometryUpdateParams& Params,
		uint32 PrimitiveId,
		FRWBuffer* RWBuffer,
		uint32 VertexBufferOffset,
		TArray<FMeshComputeDispatchCommand, SceneRenderingAllocator>& OutDispatchCommands);

	FRWBuffer* AllocateSharedBuffer(FRHICommandListBase& RHICmdList, uint32 VertexBufferSize, uint32& OutVertexBufferOffset);

	TArray<FUpdateRequest> BuildRequests;
	TArray<FUpdateRequest> UpdateRequests;

	// Group dispatch commands per view uniform buffer since it is specified when creating the RDG passes
	TMap<FRHIUniformBuffer*, TArray<FMeshComputeDispatchCommand, SceneRenderingAllocator>*> DispatchCommandsPerView;
	TMap<FRHIUniformBuffer*, FRHIUniformBuffer*> InstancedViewUniformBuffers;
	TArray<FRayTracingGeometryBuildParams, SceneRenderingAllocator>* BuildParams = nullptr;

	struct FSharedBuffer
	{
		FRWBuffer RWBuffer;
		uint32 UsedSize = 0;
		uint32 LastUsedGenerationID = 0;
		bool bPinned = false;
	};

	FSharedBuffer* FindVertexPositionBuffer(FRHIBuffer* Buffer) const;
	bool PinSharedBuffer(FRayTracingGeometry& Geometry);

	TArray<FSharedBuffer*> VertexPositionBuffers;

	// Any uniform buffers that must be kept alive until EndUpdate (after DispatchUpdates is called)
	TSet<FUniformBufferRHIRef> ReferencedUniformBuffers;

	// Generation ID when the shared vertex buffers have been reset. The current generation ID is stored in the FRayTracingGeometry to keep track
	// if the vertex buffer data is still valid for that frame - validated before generation the TLAS
	int64 SharedBufferGenerationID = 0;

	uint32 ScratchBufferSize = 0;
};

#endif // RHI_RAYTRACING