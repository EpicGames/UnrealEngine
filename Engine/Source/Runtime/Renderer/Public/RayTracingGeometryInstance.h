// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

class FRHIRayTracingGeometry;

enum class ERayTracingInstanceFlags : uint8
{
	None = 0,
	TriangleCullDisable = 1 << 1, // No back face culling. Triangle is visible from both sides.
	TriangleCullReverse = 1 << 2, // Makes triangle front-facing if its vertices are counterclockwise from ray origin.
	ForceOpaque = 1 << 3, // Disable any-hit shader invocation for this instance.
	ForceNonOpaque = 1 << 4, // Force any-hit shader invocation even if geometries inside the instance were marked opaque.
	NaniteRayTracing = 1 << 5, // Instances using Nanite Ray Tracing that will get their BLAS address from NaniteBLASData buffer in RayTracingBuildInstanceBufferCS
};
ENUM_CLASS_FLAGS(ERayTracingInstanceFlags);

/**
* High level descriptor of one or more instances of a mesh in a ray tracing scene.
* All instances covered by this descriptor will share shader bindings, but may have different transforms and user data.
*/
struct FRayTracingGeometryInstance
{
	FRHIRayTracingGeometry* GeometryRHI = nullptr;

	int32 InstanceContributionToHitGroupIndex = INDEX_NONE;

	// A single physical mesh may be duplicated many times in the scene with different transforms and user data.
	// All copies share the same shader binding table entries and therefore will have the same material and shader resources.
	TArrayView<const FMatrix> Transforms;

	// Offsets into the scene's instance scene data buffer used to get instance transforms from GPUScene
	// If BaseInstanceSceneDataOffset != -1, instances are assumed to be continuous.
	int32 BaseInstanceSceneDataOffset = -1;
	TArrayView<const uint32> InstanceSceneDataOffsets;

	// Conservative number of instances. Some of the actual instances may be made inactive if GPU transforms are used.
	// Must be less or equal to number of entries in Transforms view if CPU transform data is used.
	uint32 NumTransforms = 0;

	// Each geometry copy can receive a user-provided integer, which can be used to retrieve extra shader parameters or customize appearance.
	// This data can be retrieved using GetInstanceUserData() in closest/any hit shaders.
	// If UserData view is empty, then DefaultUserData value will be used for all instances.
	// If UserData view is used, then it must have the same number of entries as NumInstances.
	uint32 DefaultUserData = 0;
	TArrayView<const uint32> UserData;

	// Whether local bounds scale and center translation should be applied to the instance transform.
	bool bApplyLocalBoundsTransform : 1 = false;
	// Whether to increment UserData for each instance of this geometry (only applied when using DefaultUserData)
	bool bIncrementUserDataPerInstance : 1 = false;

	uint8 LightingChannelMask : 3 = GetDefaultLightingChannelMask();

	// Mask that will be tested against one provided to TraceRay() in shader code.
	// If binary AND of instance mask with ray mask is zero, then the instance is considered not intersected / invisible.
	uint8 Mask = 0xFF;

	// Flags to control triangle back face culling, whether to allow any-hit shaders, etc.
	ERayTracingInstanceFlags Flags = ERayTracingInstanceFlags::None;
};

static_assert(sizeof(FRayTracingGeometryInstance) <= 104,
	"Ray tracing instance descriptor is expected to be no more than 104 bytes, as there may be a very large number of them.");
