// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Math/MathFwd.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "RenderResource.h"
#include "MeshBatch.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

class FRayTracingGeometry;
struct FMeshBatch;

enum class ERayTracingInstanceLayer : uint8
{
	NearField,
	FarField,
};

struct FRayTracingInstance
{
	/** The underlying geometry of this instance specification. */
	const FRayTracingGeometry* Geometry = nullptr;

	/**
	 * Materials for each segment, in the form of mesh batches. We will check whether every segment of the geometry has been assigned a material.
	 * Unlike the raster path, mesh batches assigned here are considered transient and will be discarded immediately upon we finished gathering for the current scene proxy.
	 */
	TArray<FMeshBatch> Materials;

	/** Similar to Materials, but memory is owned by someone else (i.g. FPrimitiveSceneProxy). */
	TConstArrayView<FMeshBatch> MaterialsView;

	bool OwnsMaterials() const
	{
		return Materials.Num() != 0;
	}

	TConstArrayView<FMeshBatch> GetMaterials() const
	{
		if (OwnsMaterials())
		{
			check(MaterialsView.Num() == 0);
			return TConstArrayView<FMeshBatch>(Materials);
		}
		else
		{
			check(Materials.Num() == 0);
			return MaterialsView;
		}
	}

	/** Whether local bounds scale and center translation should be applied to the instance transform. */
	bool bApplyLocalBoundsTransform = false;

	/** Whether the instance is thin geometry (e.g., Hair strands)*/
	bool bThinGeometry = false;

	/** Mark InstanceMaskAndFlags dirty to be automatically updated in the renderer module (dirty by default).
	* If caching is used, clean the dirty state by setting it to false so no duplicate update will be performed in the renderer module.
	*/
	bool bInstanceMaskAndFlagsDirty = true;

	/** 
	* Transforms count. When NumTransforms == 1 we create a single instance. 
	* When it's more than one we create multiple identical instances with different transforms. 
	* NumTransforms should be less or equal to the number of GPUScene instances or InstanceTransforms.Num() 
	*/
	uint32 NumTransforms = 0;

	// Indices of primitive instances to be included in ray tracing scene
	TArray<uint32> PrimitiveInstanceIndices;

	/** Similar to PrimitiveInstanceIndices, but memory is owned by someone else (i.g. FPrimitiveSceneProxy). */
	TConstArrayView<uint32> PrimitiveInstanceIndicesView;

	bool OwnsPrimitiveInstanceIndices() const
	{
		return PrimitiveInstanceIndices.Num() != 0;
	}

	TConstArrayView<uint32> GetPrimitiveInstanceIndices() const
	{
		if (OwnsPrimitiveInstanceIndices())
		{
			check(PrimitiveInstanceIndicesView.Num() == 0);
			return TConstArrayView<uint32>(PrimitiveInstanceIndices);
		}
		else
		{
			check(PrimitiveInstanceIndices.Num() == 0);
			return PrimitiveInstanceIndicesView;
		}
	}

	/** Instance transforms. */
	TArray<FMatrix> InstanceTransforms;

	/** Similar to InstanceTransforms, but memory is owned by someone else (i.g. FPrimitiveSceneProxy). */
	TConstArrayView<FMatrix> InstanceTransformsView;

	bool OwnsTransforms() const
	{
		return InstanceTransforms.Num() != 0;
	}

	TConstArrayView<FMatrix> GetTransforms() const
	{
		if (OwnsTransforms())
		{
			check(InstanceTransformsView.Num() == 0);
			return TConstArrayView<FMatrix>(InstanceTransforms);
		}
		else
		{
			check(InstanceTransforms.Num() == 0);
			return InstanceTransformsView;
		}
	}

	//disable deprecation warnings for default constructors
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRayTracingInstance() = default;
	FRayTracingInstance(const FRayTracingInstance&) = default;
	FRayTracingInstance& operator=(const FRayTracingInstance&) = default;
	FRayTracingInstance(FRayTracingInstance&&) = default;
	FRayTracingInstance& operator=(FRayTracingInstance&&) = default;
	~FRayTracingInstance() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

struct FStaticRayTracingInstancesLOD
{
	TArray<FRayTracingInstance, TInlineAllocator<1>> Instances;
};

struct FStaticRayTracingInstances
{
	TArray<FStaticRayTracingInstancesLOD, TInlineAllocator<1>> LODs;
};
