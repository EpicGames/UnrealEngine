// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphDefinitions.h"
#include "RHIUtilities.h"
#include "CoreMinimal.h"

#if RHI_RAYTRACING

class FGPUScene;
class FGPUSceneResourceParametersRHI;
struct FRayTracingCullingParameters;
struct FDFVector3;

struct FRayTracingInstanceGroup;
struct FRayTracingInstanceDescriptor;
struct FRayTracingGeometryInstance;

struct FRayTracingInstanceBufferBuilderInitializer
{
	TConstArrayView<FRayTracingGeometryInstance> Instances;
	TBitArray<> VisibleInstances;
	FVector PreViewTranslation;
	bool bUseLightingChannels = false;
	bool bForceOpaque = false;
	bool bDisableTriangleCull = false;
};

class FRayTracingInstanceBufferBuilder
{
public:

	RENDERER_API void Init(TConstArrayView<FRayTracingGeometryInstance> InInstances, FVector InPreViewTranslation);
	RENDERER_API void Init(FRayTracingInstanceBufferBuilderInitializer Initializer);

	RENDERER_API void FillRayTracingInstanceUploadBuffer(FRHICommandList& RHICmdList);
	RENDERER_API void FillAccelerationStructureAddressesBuffer(FRHICommandList& RHICmdList);

	UE_DEPRECATED(5.8, "Use the version that takes FGPUSceneResourceParametersRHI* and (optionally) provides NaniteBLASDataSRV instead.")
	RENDERER_API void BuildRayTracingInstanceBuffer(
		FRHICommandList& RHICmdList,
		const FGPUScene* GPUScene,
		const FRayTracingCullingParameters* CullingParameters,
		FRHIUnorderedAccessView* InstancesUAV,
		FRHIUnorderedAccessView* HitGroupContributionsUAV,
		uint32 MaxNumInstances,
		bool bCompactOutput,
		FRHIUnorderedAccessView* OutputStatsUAV,
		uint32 OutputStatsOffset,
		FRHIUnorderedAccessView* InstanceExtraDataUAV);

	RENDERER_API void BuildRayTracingInstanceBuffer(
		FRHICommandList& RHICmdList,
		const FGPUScene* GPUScene,
		const FGPUSceneResourceParametersRHI* GPUSceneParameters,
		const FRayTracingCullingParameters* CullingParameters,
		FRHIUnorderedAccessView* InstancesUAV,
		FRHIUnorderedAccessView* HitGroupContributionsUAV,
		uint32 MaxNumInstances,
		FRHIShaderResourceView* NaniteBLASDataSRV,
		bool bCompactOutput,
		FRHIUnorderedAccessView* OutputStatsUAV,
		uint32 OutputStatsOffset,
		FRHIUnorderedAccessView* InstanceExtraDataUAV);

	uint32 GetMaxNumInstances() const { return Data.NumNativeGPUSceneInstances + Data.NumNativeCPUInstances; }

	TConstArrayView<FRHIRayTracingGeometry*> GetReferencedGeometries() const { return Data.ReferencedGeometries; }
	TConstArrayView<uint32> GetInstanceGeometryIndices() const { return Data.InstanceGeometryIndices; }
	TConstArrayView<uint32> GetBaseInstancePrefixSum() const { return Data.BaseInstancePrefixSum; }

private:

	struct FInstanceGroupEntryRef
	{
		uint32 GroupIndex;
		uint32 BaseIndexInGroup;
	};

	struct FInitializationData
	{
		uint32 NumGPUInstanceGroups;
		uint32 NumCPUInstanceGroups;
		uint32 NumGPUInstanceDescriptors;
		uint32 NumCPUInstanceDescriptors;
		uint32 NumNativeGPUSceneInstances;
		uint32 NumNativeCPUInstances;

		// index of each instance geometry in ReferencedGeometries
		TArray<uint32> InstanceGeometryIndices;
		// base offset of each instance entries in the instance upload buffer
		TArray<uint32> BaseUploadBufferOffsets;
		// prefix sum of `Instance.NumTransforms` for all instances in this scene
		TArray<uint32> BaseInstancePrefixSum;
		// reference to corresponding instance group entry
		TArray<FInstanceGroupEntryRef> InstanceGroupEntryRefs;

		// Unique list of geometries referenced by all instances in this scene.
		// Any referenced geometry is kept alive while the scene is alive.
		TArray<FRHIRayTracingGeometry*> ReferencedGeometries;
	};

	void BuildRayTracingSceneInitializationData();

	static void FillRayTracingInstanceUploadBuffer(
		FVector PreViewTranslation,
		TConstArrayView<FRayTracingGeometryInstance> Instances,
		const TBitArray<>& VisibleInstances,
		bool bUseLightingChannels,
		bool bForceOpaque,
		bool bDisableTriangleCull,
		TConstArrayView<uint32> InstanceGeometryIndices,
		TConstArrayView<uint32> BaseUploadBufferOffsets,
		TConstArrayView<uint32> BaseInstancePrefixSum,
		TConstArrayView<FInstanceGroupEntryRef> InstanceGroupEntryRefs,
		uint32 NumGPUInstanceGroups,
		uint32 NumCPUInstanceGroups,
		uint32 NumGPUInstanceDescriptors,
		uint32 NumCPUInstanceDescriptors,
		TArrayView<FRayTracingInstanceGroup> OutInstanceGroupUploadData,
		TArrayView<FRayTracingInstanceDescriptor> OutInstanceUploadData,
		TArrayView<FVector4f> OutTransformData);

	TConstArrayView<FRayTracingGeometryInstance> Instances;
	TBitArray<> VisibleInstances;
	FVector PreViewTranslation;
	bool bUseLightingChannels = false;
	bool bForceOpaque = false;
	bool bDisableTriangleCull = false;

	FInitializationData Data;

	FBufferRHIRef InstanceGroupUploadBuffer;
	FShaderResourceViewRHIRef InstanceGroupUploadSRV;

	FBufferRHIRef InstanceUploadBuffer;
	FShaderResourceViewRHIRef InstanceUploadSRV;

	FBufferRHIRef TransformUploadBuffer;
	FShaderResourceViewRHIRef TransformUploadSRV;

	FByteAddressBuffer AccelerationStructureAddressesBuffer;
};

#endif // RHI_RAYTRACING
