// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshPassProcessor.h
=============================================================================*/

#pragma once

#include "MeshPassProcessor.h"
#include "SpanAllocator.h"

class FRayTracingGeometry;
struct FRayTracingSBTAllocation;
class FViewUniformShaderParameters;

struct FRayTracingCachedMeshCommandFlags
{
	FRayTracingCachedMeshCommandFlags()
	{	
		Data = 0;

		bAllSegmentsOpaque = true;
		bAllSegmentsCastShadow = true;
		bAnySegmentsCastShadow = false;
		bAnySegmentsDecal = false;
		bAllSegmentsDecal = true;
		bTwoSided = false;
		bIsSky = false;
		bAllSegmentsTranslucent = true;
		bAllSegmentsReverseCulling = true;
	}

	bool operator==(const FRayTracingCachedMeshCommandFlags& Other) const
	{
		return CachedMeshCommandHash == Other.CachedMeshCommandHash &&
			Data == Other.Data;
	}

	bool operator!=(const FRayTracingCachedMeshCommandFlags& Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FRayTracingCachedMeshCommandFlags& Key)
	{
		return HashCombine(GetTypeHash(Key.CachedMeshCommandHash), Key.Data);
	}

	uint64 CachedMeshCommandHash = 0;
	union
	{
		struct
		{
			uint8 InstanceMask;
			bool bAllSegmentsOpaque : 1;
			bool bAllSegmentsCastShadow : 1;
			bool bAnySegmentsCastShadow : 1;
			bool bAnySegmentsDecal : 1;
			bool bAllSegmentsDecal : 1;
			bool bTwoSided : 1;
			bool bIsSky : 1;
			bool bAllSegmentsTranslucent : 1;
			bool bAllSegmentsReverseCulling : 1;
		};
		uint32 Data;
	};
};

class FRayTracingMeshCommand
{
public:
	FMeshDrawShaderBindings ShaderBindings;
	FRHIRayTracingShader* MaterialShader = nullptr;

	uint32 MaterialShaderIndex = UINT_MAX;
	uint32 GeometrySegmentIndex = UINT_MAX;
	uint8 InstanceMask = 0xFF;

	bool bCastRayTracedShadows : 1 = true;
	bool bOpaque : 1 = true;
	bool bAlphaMasked : 1 = false;
	bool bDecal : 1 = false;
	bool bIsSky : 1 = false;
	bool bIsTranslucent : 1 = false;
	bool bTwoSided : 1 = false;
	bool bReverseCulling : 1 = false;
	bool bNaniteRayTracing : 1 = false;
	bool bCanBeCached : 1 = false;

	RENDERER_API void SetRayTracingShaderBindingsForHitGroup(
		FRayTracingLocalShaderBindingWriter* BindingWriter,
		const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
		FRHIUniformBuffer* SceneUniformBuffer,
		FRHIUniformBuffer* NaniteUniformBuffer,
		uint32 RecordIndex,
		const FRHIRayTracingGeometry* RayTracingGeometry,
		uint32 SegmentIndex,
		uint32 HitGroupIndexInPipeline,
		ERayTracingLocalShaderBindingType BindingType) const;

	/** Sets ray hit group shaders on the mesh command and allocates room for the shader bindings. */
	RENDERER_API void SetShader(const TShaderRef<FShader>& Shader);

	RENDERER_API bool IsUsingNaniteRayTracing() const;

	RENDERER_API void UpdateFlags(FRayTracingCachedMeshCommandFlags& Flags) const;

	bool HasGlobalUniformBufferBindings() const
	{
		return ViewUniformBufferParameter.IsBound() ||
			SceneUniformBufferParameter.IsBound() ||
			NaniteUniformBufferParameter.IsBound();
	}

private:
	FShaderUniformBufferParameter ViewUniformBufferParameter;
	FShaderUniformBufferParameter SceneUniformBufferParameter;
	FShaderUniformBufferParameter NaniteUniformBufferParameter;
};

class FRayTracingMeshCommandStorage
{
public:
	int32 Allocate(int32 Num)
	{
		const int32 StartOffset = Allocator.Allocate(Num);

		{
			const int32 MaxSize = Allocator.GetMaxSize();

			if (MaxSize > Array.Num())
			{
				Array.SetNum(MaxSize);
			}
		}

		return StartOffset;
	}

	void Free(int32 StartOffset, int32 Num)
	{
		Allocator.Free(StartOffset, Num);
	}

	FRayTracingMeshCommand& operator[](int32 Index)
	{
		return Array[Index];
	}

	const FRayTracingMeshCommand& operator[](int32 Index) const
	{
		return Array[Index];
	}

	bool IsEmpty() const
	{
		return Allocator.GetSparselyAllocatedSize() == 0;
	}

private:
	TArray<FRayTracingMeshCommand> Array;
	FSpanAllocator Allocator;
};

class FRayTracingShaderBindingData
{
public:

	// TODO: Deprecate and only support RayTracingMeshCommandIndex
	FRayTracingShaderBindingData(
		const FRayTracingMeshCommand* InRayTracingMeshCommand, 
		const FRHIRayTracingGeometry* InRayTracingGeometry,
		bool bInDynamicRayTracingGeometry,
		uint32 InSBTRecordIndex, 
		ERayTracingLocalShaderBindingType InBindingType, 
		bool bInHidden)
		: RayTracingMeshCommand(InRayTracingMeshCommand)
		, RayTracingGeometry(InRayTracingGeometry)
		, RayTracingMeshCommandIndex(INDEX_NONE)
		, SBTRecordIndex(InSBTRecordIndex)
		, BindingType(InBindingType)
		, bHidden(bInHidden)
		, bDynamicRayTracingGeometry(bInDynamicRayTracingGeometry)
	{
	}

	UE_DEPRECATED(5.8, "Please provide bInDynamicRayTracingGeometry.")
	FRayTracingShaderBindingData(
		const FRayTracingMeshCommand* InRayTracingMeshCommand,
		const FRHIRayTracingGeometry* InRayTracingGeometry,
		uint32 InSBTRecordIndex,
		ERayTracingLocalShaderBindingType InBindingType,
		bool bInHidden)
		: FRayTracingShaderBindingData(InRayTracingMeshCommand, InRayTracingGeometry, InRayTracingGeometry->GetInitializer().bAllowUpdate, InSBTRecordIndex, InBindingType, bInHidden)
	{
	}

	FRayTracingShaderBindingData(
		uint32 InRayTracingMeshCommandIndex,
		const FRHIRayTracingGeometry* InRayTracingGeometry,
		bool bInDynamicRayTracingGeometry,
		uint32 InSBTRecordIndex,
		ERayTracingLocalShaderBindingType InBindingType,
		bool bInHidden)
		: RayTracingMeshCommand(nullptr)
		, RayTracingGeometry(InRayTracingGeometry)
		, RayTracingMeshCommandIndex(InRayTracingMeshCommandIndex)
		, SBTRecordIndex(InSBTRecordIndex)
		, BindingType(InBindingType)
		, bHidden(bInHidden)
		, bDynamicRayTracingGeometry(bInDynamicRayTracingGeometry)
	{
	}

	UE_DEPRECATED(5.8, "Please provide bInDynamicRayTracingGeometry.")
	FRayTracingShaderBindingData(
		uint32 InRayTracingMeshCommandIndex, 
		const FRHIRayTracingGeometry* InRayTracingGeometry, 
		uint32 InSBTRecordIndex, 
		ERayTracingLocalShaderBindingType InBindingType, 
		bool bInHidden)
		: FRayTracingShaderBindingData(InRayTracingMeshCommandIndex, InRayTracingGeometry, InRayTracingGeometry->GetInitializer().bAllowUpdate, InSBTRecordIndex, InBindingType, bInHidden)
	{
	}

	const FRayTracingMeshCommand& GetRayTracingMeshCommand(const FRayTracingMeshCommandStorage& Storage) const
	{
		return RayTracingMeshCommand ? *RayTracingMeshCommand : Storage[RayTracingMeshCommandIndex];
	}

	const FRayTracingMeshCommand* RayTracingMeshCommand; // TODO: Deprecate and only support RayTracingMeshCommandIndex
	const FRHIRayTracingGeometry* RayTracingGeometry;
	uint32 RayTracingMeshCommandIndex;
	uint32 SBTRecordIndex;
	ERayTracingLocalShaderBindingType BindingType;
	bool bHidden : 1;
	bool bDynamicRayTracingGeometry : 1;
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRayTracingShaderBindingData(const FRayTracingShaderBindingData&) = default;
	FRayTracingShaderBindingData& operator=(const FRayTracingShaderBindingData&) = default;
	FRayTracingShaderBindingData(FRayTracingShaderBindingData&&) = default;
	FRayTracingShaderBindingData& operator=(FRayTracingShaderBindingData&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

template <>
struct TUseBitwiseSwap<FRayTracingShaderBindingData>
{
	// Prevent Memcpy call overhead during FRayTracingShaderBindingData sorting
	enum { Value = false };
};

typedef TArray<FRayTracingShaderBindingData, SceneRenderingAllocator> FRayTracingShaderBindingDataOneFrameArray;

class FRayTracingMeshCommandContext
{
public:

	virtual ~FRayTracingMeshCommandContext() = default;

	virtual FRayTracingMeshCommand& AddCommand(const FRayTracingMeshCommand& Initializer) = 0;

	virtual void FinalizeCommand(FRayTracingMeshCommand& RayTracingMeshCommand) = 0;
};

using FTempRayTracingMeshCommandStorage = TArray<FRayTracingMeshCommand>;

using FCachedRayTracingMeshCommandStorage UE_DEPRECATED(5.7, "Use FRayTracingMeshCommandStorage instead.") = TSparseArray<FRayTracingMeshCommand>;

using FDynamicRayTracingMeshCommandStorage = TChunkedArray<FRayTracingMeshCommand>;

template<class T>
class FCachedRayTracingMeshCommandContext : public FRayTracingMeshCommandContext
{
public:
	FCachedRayTracingMeshCommandContext(T& InDrawListStorage) : DrawListStorage(InDrawListStorage) {}

	virtual FRayTracingMeshCommand& AddCommand(const FRayTracingMeshCommand& Initializer) override final
	{
		CommandIndex = DrawListStorage.Add(Initializer);
		return DrawListStorage[CommandIndex];
	}

	virtual void FinalizeCommand(FRayTracingMeshCommand& RayTracingMeshCommand) override final {}

	int32 CommandIndex = -1;

private:
	T& DrawListStorage;
};

class FDynamicRayTracingMeshCommandContext : public FRayTracingMeshCommandContext
{
public:
	FDynamicRayTracingMeshCommandContext
	(
		FDynamicRayTracingMeshCommandStorage& InDynamicCommandStorage,
		FRayTracingShaderBindingDataOneFrameArray& InShaderBindings,
		const FRHIRayTracingGeometry* InRayTracingGeometry,
		uint32 InGeometrySegmentIndex,
		bool bInDynamicRayTracingGeometry,
		FRayTracingSBTAllocation* InSBTAllocation
	)
		: DynamicCommandStorage(InDynamicCommandStorage)
		, ShaderBindings(InShaderBindings)
		, RayTracingGeometry(InRayTracingGeometry)
		, GeometrySegmentIndex(InGeometrySegmentIndex)
		, bDynamicRayTracingGeometry(bInDynamicRayTracingGeometry)
		, SBTAllocation(InSBTAllocation)
	{}

	UE_DEPRECATED(5.8, "Please provide bInDynamicRayTracingGeometry.")
	FDynamicRayTracingMeshCommandContext
	(
		FDynamicRayTracingMeshCommandStorage & InDynamicCommandStorage,
		FRayTracingShaderBindingDataOneFrameArray & InShaderBindings,
		const FRHIRayTracingGeometry * InRayTracingGeometry,
		uint32 InGeometrySegmentIndex,
		FRayTracingSBTAllocation * InSBTAllocation
	) : FDynamicRayTracingMeshCommandContext(InDynamicCommandStorage, InShaderBindings, InRayTracingGeometry, InGeometrySegmentIndex, InRayTracingGeometry->GetInitializer().bAllowUpdate, InSBTAllocation)
	{
	}

	virtual FRayTracingMeshCommand& AddCommand(const FRayTracingMeshCommand& Initializer) override final
	{
		const int32 Index = DynamicCommandStorage.AddElement(Initializer);
		FRayTracingMeshCommand& NewCommand = DynamicCommandStorage[Index];
		NewCommand.GeometrySegmentIndex = GeometrySegmentIndex;
		return NewCommand;
	}

	RENDERER_API virtual void FinalizeCommand(FRayTracingMeshCommand& RayTracingMeshCommand) override final;

private:
	FDynamicRayTracingMeshCommandStorage& DynamicCommandStorage;
	FRayTracingShaderBindingDataOneFrameArray& ShaderBindings;

	const FRHIRayTracingGeometry* RayTracingGeometry;
	uint32 GeometrySegmentIndex;
	bool bDynamicRayTracingGeometry;

	FRayTracingSBTAllocation* SBTAllocation;
};

class FRayTracingShaderCommand
{
public:
	FMeshDrawShaderBindings ShaderBindings;
	FRHIRayTracingShader* Shader = nullptr;

	uint32 ShaderIndex = UINT_MAX;
	uint32 SlotInScene = UINT_MAX;

	RENDERER_API void SetRayTracingShaderBindings(
		FRayTracingLocalShaderBindingWriter* BindingWriter,
		const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
		FRHIUniformBuffer* SceneUniformBuffer,
		FRHIUniformBuffer* NaniteUniformBuffer,
		uint32 ShaderIndexInPipeline,
		uint32 ShaderSlot) const;

	/** Sets ray tracing shader on the command and allocates room for the shader bindings. */
	RENDERER_API void SetShader(const TShaderRef<FShader>& Shader);

private:
	FShaderUniformBufferParameter ViewUniformBufferParameter;
	FShaderUniformBufferParameter SceneUniformBufferParameter;
	FShaderUniformBufferParameter NaniteUniformBufferParameter;
};
