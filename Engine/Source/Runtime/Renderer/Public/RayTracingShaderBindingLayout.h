// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"

#if RHI_RAYTRACING

#include "RHICommandList.h"
#include "RHIResources.h"

class FShaderBindingLayout;
class FViewInfo;

namespace RayTracing
{
	// Get shader resource table desc used for all raytracing shaders which is shared between all shaders in the RTPSO
	RENDERER_API const FShaderBindingLayout* GetShaderBindingLayout(EShaderPlatform ShaderPlatform);

	// Setup the runtime static uniform buffer bindings on the command list if enabled
	RENDERER_API TOptional<FScopedUniformBufferStaticBindings> BindStaticUniformBufferBindings(const FViewInfo& View, FRHIUniformBuffer* SceneUniformBuffer, FRHIUniformBuffer* NaniteRayTracingUniformBuffer, FRHICommandList& RHICmdList);
}

#endif // RHI_RAYTRACING