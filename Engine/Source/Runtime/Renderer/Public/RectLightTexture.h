// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class UTexture;

namespace RectLightAtlas
{

// Scope for invalidating a particular texture 
// This ensures the atlas contains the latest version of the texture and filter it
struct FAtlasTextureInvalidationScope
{
	RENDERER_API FAtlasTextureInvalidationScope(const UTexture* In);
	RENDERER_API ~FAtlasTextureInvalidationScope();
	FAtlasTextureInvalidationScope(const FAtlasTextureInvalidationScope&) = delete;
	FAtlasTextureInvalidationScope& operator=(const FAtlasTextureInvalidationScope&) = delete;
	bool bLocked = false;
};

} 
