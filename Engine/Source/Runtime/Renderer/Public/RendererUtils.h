// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "GlobalShader.h"
#include "HAL/Platform.h"
#include "RHIDefinitions.h"
#include "RenderGraph.h"
#include "RenderGraphDefinitions.h"
#include "ShaderParameterMacros.h"
#include "RenderGraphFwd.h"

struct FSubstrateSceneData;
class FScene;
class FGlobalShaderMap;
class FRDGBuilder;
class FRHICommandListImmediate;
class FScene;
struct IPooledRenderTarget;
template <typename ReferencedType> class TRefCountPtr;

class FRenderTargetWriteMask
{
public:
	static RENDERER_API void Decode(
		FRHICommandListImmediate& RHICmdList,
		FGlobalShaderMap* ShaderMap,
		TArrayView<IPooledRenderTarget* const> InRenderTargets,
		TRefCountPtr<IPooledRenderTarget>& OutRTWriteMask,
		ETextureCreateFlags RTWriteMaskFastVRamConfig,
		const TCHAR* RTWriteMaskDebugName);

	static RENDERER_API void Decode(
		FRDGBuilder& GraphBuilder,
		FGlobalShaderMap* ShaderMap,
		TArrayView<FRDGTextureRef const> InRenderTargets,
		FRDGTextureRef& OutRTWriteMask,
		ETextureCreateFlags RTWriteMaskFastVRamConfig,
		const TCHAR* RTWriteMaskDebugName);
};

class FDepthBounds
{
public:

	struct FDepthBoundsValues
	{
		float MinDepth;
		float MaxDepth;
	};

	static RENDERER_API FDepthBoundsValues CalculateNearFarDepthExcludingSky();
};

// A minimal uniform struct providing necessary access for external systems to Substrate parameters.
DECLARE_UNIFORM_BUFFER_STRUCT(FSubstratePublicGlobalUniformParameters, RENDERER_API)

namespace Substrate
{
	UE_DEPRECATED(5.8, "Use the version that takes FSceneTextures instead")
	RENDERER_API TRDGUniformBufferRef<FSubstratePublicGlobalUniformParameters> GetPublicGlobalUniformBuffer(FRDGBuilder& GraphBuilder, FScene& Scene);

	RENDERER_API TRDGUniformBufferRef<FSubstratePublicGlobalUniformParameters> GetPublicGlobalUniformBuffer(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);
}

// A minimal uniform struct providing necessary access for external systems to Nanite Ray Tracing parameters.
DECLARE_UNIFORM_BUFFER_STRUCT(FNaniteRayTracingUniformParameters, RENDERER_API)

namespace Nanite
{
	RENDERER_API TRDGUniformBufferRef<FNaniteRayTracingUniformParameters> GetPublicGlobalRayTracingUniformBuffer();
}