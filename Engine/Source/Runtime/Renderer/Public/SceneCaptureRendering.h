// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneCaptureRendering.h: SceneCaptureRendering definitions.
=============================================================================*/

#pragma once

#include "Containers/ContainersFwd.h"
#include "RenderGraphFwd.h"

class FSceneView;
class FSceneViewFamily;
class FViewInfo;
struct FMinimalSceneTextures;

RENDERER_API void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef ViewFamilyTexture,
	FRDGTextureRef ViewFamilyDepthTexture,
	const FSceneViewFamily& ViewFamily,
	const TArray<const FViewInfo*>& Views);

RENDERER_API void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef ViewFamilyTexture,
	FRDGTextureRef ViewFamilyDepthTexture,
	const FSceneViewFamily& ViewFamily,
	TConstArrayView<FViewInfo> Views);

RENDERER_API void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef ViewFamilyTexture,
	FRDGTextureRef ViewFamilyDepthTexture,
	const FSceneViewFamily& ViewFamily,
	TConstStridedView<FSceneView> Views);


RENDERER_API void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef ViewFamilyTexture,
	const FSceneViewFamily& ViewFamily,
	const TArray<const FViewInfo*>& Views);

RENDERER_API void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef ViewFamilyTexture,
	const FSceneViewFamily& ViewFamily,
	TConstArrayView<FViewInfo> Views);

RENDERER_API void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef ViewFamilyTexture,
	const FSceneViewFamily& ViewFamily,
	TConstStridedView<FSceneView> Views);
