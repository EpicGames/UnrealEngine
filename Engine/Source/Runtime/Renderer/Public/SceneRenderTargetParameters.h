// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Misc/EnumClassFlags.h"
#include "RenderGraphFwd.h"
#include "RenderResource.h"
#include "RHIFeatureLevel.h"
#include "ShaderParameterMacros.h"
#include "SceneTexturesConfig.h"
#include "PooledRenderTarget.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "RendererInterface.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

struct FSceneTextures;
class FSceneTextureUniformParameters;
class FSceneView;
class FViewInfo;
class FMobileSceneTextureUniformParameters;
class FSceneTextureShaderParameters;

enum class ESceneTexture
{
	Color,
	Depth,
	SmallDepth,
	Velocity,
	GBufferA,
	GBufferB,
	GBufferC,
	GBufferD,
	GBufferE,
	GBufferF,
	GBufferSGGX,
	SSAO,
	CustomDepth,
};

RENDERER_API FRDGTextureRef GetSceneTexture(const FSceneTextures& SceneTextures, ESceneTexture InSceneTexture);

enum class ESceneTextureSetupMode : uint32
{
	None			= 0,
	SceneColor		= 1 << 0,
	SceneDepth		= 1 << 1,
	SceneVelocity	= 1 << 2,
	GBufferA		= 1 << 3,
	GBufferB		= 1 << 4,
	GBufferC		= 1 << 5,
	GBufferD		= 1 << 6,
	GBufferE		= 1 << 7,
	GBufferF		= 1 << 8,
	SSAO			= 1 << 9,
	CustomDepth		= 1 << 10,
	SGGX			= 1 << 11,
	GBuffers		= GBufferA | GBufferB | GBufferC | GBufferD | GBufferE | GBufferF | SGGX,
	All				= SceneColor | SceneDepth | SceneVelocity | GBuffers | SSAO | CustomDepth
};
ENUM_CLASS_FLAGS(ESceneTextureSetupMode);

/** Fills the shader parameter struct. */
extern RENDERER_API void SetupSceneTextureUniformParameters(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures* SceneTextures,
	ERHIFeatureLevel::Type FeatureLevel,
	ESceneTextureSetupMode SetupMode,
	FSceneTextureUniformParameters& OutParameters);

/** Returns RDG scene texture uniform buffer. */
extern RENDERER_API TRDGUniformBufferRef<FSceneTextureUniformParameters> CreateSceneTextureUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures* SceneTextures,
	ERHIFeatureLevel::Type FeatureLevel,
	ESceneTextureSetupMode SetupMode = ESceneTextureSetupMode::All);

/** Returns RDG scene texture uniform buffer for a specified View. */
extern RENDERER_API TRDGUniformBufferRef<FSceneTextureUniformParameters> CreateSceneTextureUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	ESceneTextureSetupMode SetupMode = ESceneTextureSetupMode::All);

enum class EMobileSceneTextureSetupMode : uint32
{
	None			= 0,
	SceneColor		= 1 << 0,
	SceneDepth		= 1 << 1,
	CustomDepth		= 1 << 2,
	GBufferA		= 1 << 3,
	GBufferB		= 1 << 4,
	GBufferC		= 1 << 5,
	GBufferD		= 1 << 6,
	SceneDepthAux	= 1 << 7,
	SceneVelocity	= 1 << 8,
	GBuffers		= GBufferA | GBufferB | GBufferC | GBufferD | SceneDepthAux,
	All				= SceneColor | SceneDepth | CustomDepth | GBuffers | SceneVelocity
};
ENUM_CLASS_FLAGS(EMobileSceneTextureSetupMode);

/** Fills the scene texture uniform buffer struct. */
extern RENDERER_API void SetupMobileSceneTextureUniformParameters(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures* SceneTextures,
	EMobileSceneTextureSetupMode SetupMode,
	FMobileSceneTextureUniformParameters& SceneTextureParameters);

/** Creates the RDG mobile scene texture uniform buffer. */
extern RENDERER_API TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> CreateMobileSceneTextureUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures* SceneTextures,
	EMobileSceneTextureSetupMode SetupMode = EMobileSceneTextureSetupMode::All);

/** Creates the RDG mobile scene texture uniform buffer. */
extern RENDERER_API TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> CreateMobileSceneTextureUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	EMobileSceneTextureSetupMode SetupMode = EMobileSceneTextureSetupMode::All);

/** Returns scene texture shader parameters containing the RDG uniform buffer for either mobile or deferred shading. */
extern RENDERER_API FSceneTextureShaderParameters CreateSceneTextureShaderParameters(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures* SceneTextures,
	ERHIFeatureLevel::Type FeatureLevel,
	ESceneTextureSetupMode SetupMode = ESceneTextureSetupMode::All);

/** Returns scene texture shader parameters containing the RDG uniform buffer for either mobile or deferred shading. */
extern RENDERER_API FSceneTextureShaderParameters CreateSceneTextureShaderParameters(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	ESceneTextureSetupMode SetupMode = ESceneTextureSetupMode::All);

extern RENDERER_API FSceneTextureShaderParameters GetSceneTextureShaderParameters(const FSceneView& View);

BEGIN_SHADER_PARAMETER_STRUCT(FSceneTextureExtractsParameters, )
	SHADER_PARAMETER_STRUCT_REF(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_REF(FMobileSceneTextureUniformParameters, MobileSceneTextures)
END_SHADER_PARAMETER_STRUCT()

/** Struct containing references to extracted RHI resources after RDG execution. All textures are
 *  left in an SRV read state, so they can safely be used for read without being re-imported into
 *  RDG. Likewise, the uniform buffer is non-RDG and can be used as is.
 */
class FSceneTextureExtracts : public FRenderResource
{
public:
	FRHIUniformBuffer* GetUniformBuffer() const
	{
		return UniformBuffer.IsValid() ? UniformBuffer.GetReference() : MobileUniformBuffer.GetReference();
	}

	TUniformBufferRef<FSceneTextureUniformParameters> GetUniformBufferRef() const
	{
		return UniformBuffer;
	}

	TUniformBufferRef<FMobileSceneTextureUniformParameters> GetMobileUniformBufferRef() const
	{
		return MobileUniformBuffer;
	}

	FSceneTextureExtractsParameters GetShaderParameters() const
	{
		FSceneTextureExtractsParameters Parameters;
		Parameters.SceneTextures = UniformBuffer;
		Parameters.MobileSceneTextures = MobileUniformBuffer;
		return Parameters;
	}

	RENDERER_API FRHITexture* GetDepthTexture() const;

	RENDERER_API void QueueExtractions(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);

private:
	RENDERER_API void Release();
	void ReleaseRHI() override { Release(); }

	// Contains the resolved scene depth target.
	TRefCountPtr<IPooledRenderTarget> Depth;

	// Contains the resolved scene depth target.
	TRefCountPtr<IPooledRenderTarget> PartialDepth;

	// Contains the custom depth targets.
	TRefCountPtr<IPooledRenderTarget> CustomDepth;

	// Contains RHI scene texture uniform buffers referencing the extracted textures.
	TUniformBufferRef<FSceneTextureUniformParameters> UniformBuffer;
	TUniformBufferRef<FMobileSceneTextureUniformParameters> MobileUniformBuffer;
};

/** Returns the global scene texture extracts struct. */
const RENDERER_API FSceneTextureExtracts& GetSceneTextureExtracts();

/** Pass through to View.GetSceneTexturesConfig().Extent, useful in headers where the FViewInfo structure isn't exposed. */
extern RENDERER_API FIntPoint GetSceneTextureExtentFromView(const FViewInfo& View);

/** Resets the scene texture extent history. Call this method after rendering with very large render
 *  targets. The next scene render will create them at the requested size.
 */
extern RENDERER_API void ResetSceneTextureExtentHistory();

/** Registers system textures into RDG. */
extern RENDERER_API void CreateSystemTextures(FRDGBuilder& GraphBuilder);


/** Returns whether scene textures have been initialized. */
UE_DEPRECATED(5.1, "Single pass multiple view family rendering makes this obsolete.  Use ViewFamily.SceneTexturesConfig.IsValid() instead.")
extern RENDERER_API bool IsSceneTexturesValid();

/** Returns the full-resolution scene texture extent. */
UE_DEPRECATED(5.1, "Single pass multiple view family rendering makes this obsolete.  Use ViewFamily.SceneTexturesConfig.Extent instead.")
extern RENDERER_API FIntPoint GetSceneTextureExtent();

/** Returns the feature level being used by the renderer. */
UE_DEPRECATED(5.1, "Single pass multiple view family rendering makes this obsolete.  Use ViewFamily.SceneTexturesConfig.FeatureLevel instead.")
extern RENDERER_API ERHIFeatureLevel::Type GetSceneTextureFeatureLevel();
