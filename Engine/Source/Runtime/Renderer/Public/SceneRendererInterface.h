// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShaderParameterStructDeclaration.h"
#include "RenderGraphFwd.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "SceneView.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

class FScene;
class FSceneUniformBuffer;
class FPrimitiveSceneInfo;
struct FPersistentPrimitiveIndex;
class FSceneView;

DECLARE_UNIFORM_BUFFER_STRUCT(FSceneUniformParameters, RENDERER_API)

namespace UE::Renderer::Private
{

/** 
 * Experimental interface to invalidate (VSM) shadows by marking instances during a scene render. The collected objects are invalidated at the next scene 
 * update time, before GPU-Scene update, so this can't be used to trigger invalidation for the current frame.
 * NOTE: marked as Renderer::Private as it is experimental and may be subject to change.
 */
class IShadowInvalidatingInstances
{
public:
	virtual ~IShadowInvalidatingInstances() = default;

	/**
	 * Add primitive for invalidation. Will add all the instances to the queue. 
	 * Should be called from the render thread.
	 */
	virtual void AddPrimitive(const FPrimitiveSceneInfo *PrimitiveSceneInfo) = 0;

	/**
	 * Add a range of instances for invalidation. 
	 * Should be called from the render thread.
	 */
	virtual void AddInstanceRange(FPersistentPrimitiveIndex PersistentPrimitiveIndex, uint32 InstanceSceneDataOffset, uint32 NumInstanceSceneDataEntries) = 0;
};

}

/**
 * Public interface into FSceneRenderer. Used as the scope for scene rendering functions.
 */
class ISceneRenderer
{
public:
	virtual ~ISceneRenderer() = default;

	virtual FScene* GetScene() = 0;
	virtual const FSceneUniformBuffer& GetSceneUniforms() const = 0;
	virtual FSceneUniformBuffer& GetSceneUniforms() = 0;

	virtual TRDGUniformBufferRef<FSceneUniformParameters> GetSceneUniformBufferRef(FRDGBuilder& GraphBuilder) = 0;

	/** 
	 * If supported, will return an interface to mark primitives as invalidating the shadows (which will be effected post-frame).
	 * The returned interface will mark primitives for view-dependent shadows according to the supplied SceneView.
	 * Returns null if there is no need to invalidate anything.
	 */
	virtual UE::Renderer::Private::IShadowInvalidatingInstances *GetShadowInvalidatingInstancesInterface(const FSceneView *SceneView) { return nullptr; }
};

RENDERER_API TRDGUniformBufferRef<FSceneUniformParameters> GetSceneUniformBufferRef(FRDGBuilder& GraphBuilder, const FSceneView& View);
