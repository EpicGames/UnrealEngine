// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExtensibleUniformBuffer.h"

/**
 * RDG shader parameter struct containing data for FSceneUniformBuffer.
 */
DECLARE_EXTENSIBLE_UNIFORM_BUFFER_PARAMETER_STRUCT(RENDERER_API, FSceneUniformParameters, Scene);

/**
 * Used to declare individual UB members.
 */
template<typename TMember>
using TSceneUniformBufferMemberRegistration = TExtensibleUniformBufferMemberRegistration<TMember, FSceneUniformParameters>;

#define DECLARE_SCENE_UB_STRUCT(StructType, FieldName, PrefixKeywords)						\
	namespace SceneUB {																		\
		PrefixKeywords extern TSceneUniformBufferMemberRegistration<StructType> FieldName;	\
	}

#define IMPLEMENT_SCENE_UB_STRUCT(StructType, FieldName, DefaultValueFactoryType) \
	TSceneUniformBufferMemberRegistration<StructType> SceneUB::FieldName { TEXT(#FieldName), DefaultValueFactoryType } 

/**
 * Holds scene-scoped parameters and stores these in uniform (constant) buffers for access on GPU.
 */
class FSceneUniformBuffer : public TExtensibleUniformBuffer<FSceneUniformParameters, TSceneUniformBufferMemberRegistration>
{ };