// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CanvasTypes.h"
#include "CommonRenderResources.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "SceneView.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

struct FScreenPassTexture;
struct FScreenPassTextureSlice;

class FSceneView;

// Returns whether a HMD hidden area mask is being used for VR.
RENDERER_API bool IsHMDHiddenAreaMaskActive();

// Returns the global engine mini font texture.
FRHITexture* GetMiniFontTexture();

// Creates and returns an RDG texture for the view family output. Returns null if no RHI texture exists.
FRDGTextureRef RENDERER_API TryCreateViewFamilyTexture(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily);
FRDGTextureRef RENDERER_API TryCreateViewFamilyDepthTexture(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily);

// The vertex shader used by DrawScreenPass to draw a rectangle.
class FScreenPassVS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FScreenPassVS, RENDERER_API);

	FScreenPassVS() = default;
	FScreenPassVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

// Describes a texture with a paired viewport rect.
struct FScreenPassTexture
{
	FScreenPassTexture() {}

	explicit FScreenPassTexture(FRDGTextureRef InTexture);
	explicit FScreenPassTexture(const FScreenPassTextureSlice& ScreenTexture);

	FScreenPassTexture(FRDGTextureRef InTexture, FIntRect InViewRect);

	// Returns a 2D texture from the input slice. A copy is used if the input is a slice of a 2D texture array, or the override output is active.
	// If a MultiViewCount is set, that number of slices will be copied. This mode of use is incompatible with TSR, which outputs a Texture2DArray to represent SceneColor history rather than multi-view slices.
	RENDERER_API static FScreenPassTexture CopyFromSlice(FRDGBuilder& GraphBuilder, const FScreenPassTextureSlice& ScreenTextureSlice, FScreenPassTexture OverrideOutput = FScreenPassTexture(), uint32 MultiViewCount = 0);

	bool IsValid() const;

	bool operator==(FScreenPassTexture Other) const;
	bool operator!=(FScreenPassTexture Other) const;

	void UpdateVisualizeTextureExtent();

	FRDGTextureRef Texture = nullptr;
	FIntRect ViewRect;
};

// Describes a Texture2D or a slice within a texture array with a paired viewport rect.
struct FScreenPassTextureSlice
{
	FScreenPassTextureSlice() = default;

	FScreenPassTextureSlice(FRDGTextureSRVRef InTextureSRV, FIntRect InViewRect);

	RENDERER_API static FScreenPassTextureSlice CreateFromScreenPassTexture(FRDGBuilder& GraphBuilder, const FScreenPassTexture& ScreenTexture);

	bool IsValid() const;

	bool operator==(FScreenPassTextureSlice Other) const;
	bool operator!=(FScreenPassTextureSlice Other) const;

	FRDGTextureSRVRef TextureSRV = nullptr;
	FIntRect ViewRect;
};

// Describes a texture with a load action for usage as a render target.
struct FScreenPassRenderTarget : public FScreenPassTexture
{
	RENDERER_API static FScreenPassRenderTarget CreateFromInput(
		FRDGBuilder& GraphBuilder,
		FScreenPassTexture Input,
		ERenderTargetLoadAction OutputLoadAction,
		const TCHAR* OutputName);

	RENDERER_API static FScreenPassRenderTarget CreateFromInput(
		FRDGBuilder& GraphBuilder,
		FRDGTexture* InputTexture,
		FIntPoint Extent,
		ERenderTargetLoadAction OutputLoadAction,
		const TCHAR* OutputName);

	static FScreenPassRenderTarget CreateViewFamilyOutput(FRDGTextureRef ViewFamilyTexture, const FViewInfo& View);

	FScreenPassRenderTarget() = default;

	explicit FScreenPassRenderTarget(FScreenPassTexture InTexture)
		: FScreenPassTexture(InTexture)
	{
		// Derive the load action based on whether we are rendering to the entire texture or not.
		if (ViewRect.Min != FIntPoint::ZeroValue || (Texture && Texture->Desc.Extent != ViewRect.Max))
		{
			LoadAction = ERenderTargetLoadAction::ELoad;
		}
	}

	FScreenPassRenderTarget(FScreenPassTexture InTexture, ERenderTargetLoadAction InLoadAction)
		: FScreenPassTexture(InTexture)
		, LoadAction(InLoadAction)
	{}

	FScreenPassRenderTarget(FRDGTextureRef InTexture, ERenderTargetLoadAction InLoadAction)
		: FScreenPassTexture(InTexture)
		, LoadAction(InLoadAction)
	{}

	FScreenPassRenderTarget(FRDGTextureRef InTexture, FIntRect InViewRect, ERenderTargetLoadAction InLoadAction)
		: FScreenPassTexture(InTexture, InViewRect)
		, LoadAction(InLoadAction)
	{}

	bool operator==(FScreenPassRenderTarget Other) const;
	bool operator!=(FScreenPassRenderTarget Other) const;

	FRenderTargetBinding GetRenderTargetBinding() const;

	ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ENoAction;
};

// Describes a view rect contained within the extent of a texture. Used to derive texture coordinate transformations.
class FScreenPassTextureViewport
{
public:
	FScreenPassTextureViewport() = default;

	explicit FScreenPassTextureViewport(FIntRect InRect)
		: Extent(InRect.Max)
		, Rect(InRect)
	{}

	explicit FScreenPassTextureViewport(FScreenPassTexture InTexture);
	explicit FScreenPassTextureViewport(FScreenPassTextureSlice InTexture);

	explicit FScreenPassTextureViewport(FRDGTextureRef InTexture)
		: FScreenPassTextureViewport(FScreenPassTexture(InTexture))
	{}

	explicit FScreenPassTextureViewport(FIntPoint InExtent)
		: Extent(InExtent)
		, Rect(FIntPoint::ZeroValue, InExtent)
	{}

	FScreenPassTextureViewport(FIntPoint InExtent, FIntRect InRect)
		: Extent(InExtent)
		, Rect(InRect)
	{}

	FScreenPassTextureViewport(FRDGTextureRef InTexture, FIntRect InRect)
		: FScreenPassTextureViewport(FScreenPassTexture(InTexture, InRect))
	{}

	bool operator==(const FScreenPassTextureViewport& Other) const;
	bool operator!=(const FScreenPassTextureViewport& Other) const;

	// Returns whether the viewport contains an empty viewport or extent.
	bool IsEmpty() const;

	// Returns whether the viewport covers the full extent of the texture.
	bool IsFullscreen() const;

	// Returns the ratio of rect size to extent along each axis.
	FVector2D GetRectToExtentRatio() const;

	// The texture extent, in pixels; defines a super-set [0, 0]x(Extent, Extent).
	FIntPoint Extent = FIntPoint::ZeroValue;

	// The viewport rect, in pixels; defines a sub-set within [0, 0]x(Extent, Extent).
	FIntRect Rect;
};

// Returns an extent downscaled by a multiple of the integer divisor (and clamped to 1).
FIntPoint GetDownscaledExtent(FIntPoint Extent, FIntPoint Divisor);

// Returns an extent scaled by the floating point scale factor.
FIntPoint GetScaledExtent(FIntPoint Extent, FVector2D Multiplier);
FIntPoint GetScaledExtent(FIntPoint Extent, float Multiplier);

// Returns a rect downscaled by a multiple of the integer divisor.
FIntRect GetDownscaledRect(FIntRect Rect, FIntPoint Divisor);

// Returns a rect scaled by the floating point scale factor.
FIntRect GetScaledRect(FIntRect Rect, FVector2D Multiplier);
FIntRect GetScaledRect(FIntRect Rect, float Multiplier);

// Returns the texture viewport downscaled by an integer divisor.
FScreenPassTextureViewport GetDownscaledViewport(FScreenPassTextureViewport Viewport, FIntPoint Divisor);

// Returns the texture viewport scaled by a float multiplier.
FScreenPassTextureViewport GetScaledViewport(FScreenPassTextureViewport Viewport, FVector2D Multiplier);

// Returns a rect with the min point at the origin and the max point at Extent.
FIntRect GetRectFromExtent(FIntPoint Extent);

// Describes the set of shader parameters for a screen pass texture viewport.
BEGIN_SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, )
	// Texture extent in pixels.
	SHADER_PARAMETER(FVector2f, Extent)
	SHADER_PARAMETER(FVector2f, ExtentInverse)

	// Scale / Bias factor to convert from [-1, 1] to [ViewportMin, ViewportMax]
	SHADER_PARAMETER(FVector2f, ScreenPosToViewportScale)
	SHADER_PARAMETER(FVector2f, ScreenPosToViewportBias)

	// Texture viewport min / max in pixels.
	SHADER_PARAMETER(FIntPoint, ViewportMin)
	SHADER_PARAMETER(FIntPoint, ViewportMax)

	// Texture viewport size in pixels.
	SHADER_PARAMETER(FVector2f, ViewportSize)
	SHADER_PARAMETER(FVector2f, ViewportSizeInverse)

	// Texture viewport min / max in normalized UV coordinates, with respect to the texture extent.
	SHADER_PARAMETER(FVector2f, UVViewportMin)
	SHADER_PARAMETER(FVector2f, UVViewportMax)

	// Texture viewport size in normalized UV coordinates, with respect to the texture extent.
	SHADER_PARAMETER(FVector2f, UVViewportSize)
	SHADER_PARAMETER(FVector2f, UVViewportSizeInverse)

	// Texture viewport min / max in normalized UV coordinates, with respect to the texture extent,
	// adjusted by a half pixel offset for bilinear filtering. Useful for clamping to avoid sampling
	// pixels on viewport edges; e.g. clamp(UV, UVViewportBilinearMin, UVViewportBilinearMax);
	SHADER_PARAMETER(FVector2f, UVViewportBilinearMin)
	SHADER_PARAMETER(FVector2f, UVViewportBilinearMax)
END_SHADER_PARAMETER_STRUCT()

FScreenPassTextureViewportParameters RENDERER_API GetScreenPassTextureViewportParameters(const FScreenPassTextureViewport& InViewport);

/** Generic affine 2D texture coordinate transformation x * S + B.
 *
 * Construct:
 *		FVector2f PointInA = ...;
 *		FVector2f PointInB = PointInA * Scale0 + Bias0;
 * 
 *		FScreenTransform AToB(Scale0, Bias0);
 *		FVector2f PointInB = PointInA * AToB;
 * 
 * Associativity:
 *		FVector2f PointInA = ...;
 *		FScreenTransform AToB = ...;
 *		FScreenTransform BToC = ...;
 *		FVector2f PointInC = (PointInA * AToB) * BToC;
 *
 *		FScreenTransform AToC = AToB * BToC;
 *		FVector2f PointInC = PointInA * AToC;
 * 
 * Explicit construction by factorization:
 *		FVector2f PointInA = ...;
 *		FVector2f PointInB = PointInA * Scale0 + Bias0;
 *		FVector2f PointInC = PointInB * Scale1 + Bias1;
 * 
 *		FScreenTransform AToC = (FScreenTransform::Identity * Scale0 + Bias0) * Scale1 + Bias1;
 *		FVector2f PointInC = PointInA * AToC;
 *
 * Shader code:
 *		#include "/Engine/Private/ScreenPass.ush"
 * 
 *		FScreenTransform AToC; // shader parameter in global scope
 * 
 *		{
 *			float2 PointInA = ...;
 *			float2 PointInC = ApplyScreenTransform(PointInA, AToC);
 *		}
 * 
 */
struct FScreenTransform
{
	FVector2f Scale;
	FVector2f Bias;

	inline FScreenTransform()
	{ }

	inline FScreenTransform(const FVector2f& InScale, const FVector2f& InBias)
		: Scale(InScale)
		, Bias(InBias)
	{ }


	// A * FScreenTransform::Identity = A
	static RENDERER_API const FScreenTransform Identity;

	// Transforms ScreenPos to/from ViewportUV
	static RENDERER_API const FScreenTransform ScreenPosToViewportUV;
	static RENDERER_API const FScreenTransform ViewportUVToScreenPos;


	/** Invert a transformation AToB to BToA. */
	static inline FScreenTransform Invert(const FScreenTransform& AToB);

	/** Change of coordinate to map from a rectangle to another. */
	static FScreenTransform ChangeRectFromTo(
		FVector2f SourceOffset, FVector2f SourceExtent,
		FVector2f DestinationOffset, FVector2f DestinationExtent);
	static FScreenTransform ChangeRectFromTo(const FIntRect& SrcViewport, const FIntRect& DestViewport);

	/** Different texture coordinate basis. */
	enum class ETextureBasis
	{
		// Viewport maps [-1.0,1.0] on X, ]1.0, -1.0[ on Y.
		ScreenPosition,

		// Viewport maps [0.0,1.0]
		ViewportUV,

		// Viewport maps [Viewport.Min,Viewport.Max] in pixel coordinate in the texture
		// Used for instance for MyTexture[uint(TexelPosition)];
		TexelPosition,

		// Viewport maps [Viewport.Min / TextureExtent,Viewport.Max / TextureExtent]
		// Used for MyTexture.SampleLevel(MySampler, TextureUV, 0);
		TextureUV,
	};

	/** Change of basis for texture coordinate. */
	static inline FScreenTransform ChangeTextureBasisFromTo(
		const FIntPoint& TextureExtent, const FIntRect& TextureViewport,
		ETextureBasis SrcBasis, ETextureBasis DestBasis);

	static inline FScreenTransform ChangeTextureBasisFromTo(
		const FScreenPassTextureViewport& TextureViewport,
		ETextureBasis SrcBasis, ETextureBasis DestBasis)
	{
		return ChangeTextureBasisFromTo(TextureViewport.Extent, TextureViewport.Rect, SrcBasis, DestBasis);
	}

	static inline FScreenTransform ChangeTextureBasisFromTo(
		const FScreenPassTexture& Texture,
		ETextureBasis SrcBasis, ETextureBasis DestBasis)
	{
		return ChangeTextureBasisFromTo(FScreenPassTextureViewport(Texture.Texture, Texture.ViewRect), SrcBasis, DestBasis);
	}

	/** Change TextureUV coordinate from one texture to another, taking into account change in texture extent too. */
	static FScreenTransform ChangeTextureUVCoordinateFromTo(
		const FScreenPassTextureViewport& SrcViewport,
		const FScreenPassTextureViewport& DestViewport);

	static FScreenTransform SvPositionToViewportUV(const FIntRect& SrcViewport);
	static FScreenTransform DispatchThreadIdToViewportUV(const FIntRect& SrcViewport);
}; // FScreenTransform

// A utility shader parameter struct containing the viewport, texture, and sampler for a unique texture input to a shader.
BEGIN_SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FScreenPassTextureViewportParameters, Viewport)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Texture)
	SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FScreenPassTextureSliceInput, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FScreenPassTextureViewportParameters, Viewport)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Texture)
	SHADER_PARAMETER_SAMPLER(SamplerState, Sampler)
END_SHADER_PARAMETER_STRUCT()

FScreenPassTextureInput GetScreenPassTextureInput(FScreenPassTexture Input, FRHISamplerState* Sampler);
FScreenPassTextureSliceInput GetScreenPassTextureInput(FScreenPassTextureSlice Input, FRHISamplerState* Sampler);

/** Draw information for the more advanced DrawScreenPass variant. Allows customizing the blend / depth stencil state,
 *  providing a custom vertex shader, and more fine-grained control of the underlying draw call.
 */
struct FScreenPassPipelineState
{
	using FDefaultBlendState = TStaticBlendState<>;
	using FDefaultDepthStencilState = TStaticDepthStencilState<false, CF_Always>;

	FScreenPassPipelineState() = default;

	FScreenPassPipelineState(
		const TShaderRef<FShader>& InVertexShader,
		const TShaderRef<FShader>& InPixelShader,
		FRHIBlendState* InBlendState = FDefaultBlendState::GetRHI(),
		FRHIDepthStencilState* InDepthStencilState = FDefaultDepthStencilState::GetRHI(),
		uint8 InStencilRef = 0,
		FRHIVertexDeclaration* InVertexDeclaration = GFilterVertexDeclaration.VertexDeclarationRHI)
		: VertexShader(InVertexShader)
		, PixelShader(InPixelShader)
		, BlendState(InBlendState)
		, DepthStencilState(InDepthStencilState)
		, VertexDeclaration(InVertexDeclaration)
		, StencilRef(InStencilRef)
	{}

	void Validate() const
	{
		check(VertexShader.IsValid());
		check(PixelShader.IsValid());
		check(BlendState);
		check(DepthStencilState);
		check(VertexDeclaration);
	}

	TShaderRef<FShader> VertexShader;
	TShaderRef<FShader> PixelShader;
	FRHIBlendState* BlendState = nullptr;
	FRHIDepthStencilState* DepthStencilState = nullptr;
	FRHIVertexDeclaration* VertexDeclaration = nullptr;
	uint8 StencilRef{};
};

// Helper function which sets the pipeline state object on the command list prior to invoking a screen pass.
void RENDERER_API SetScreenPassPipelineState(FRHICommandList& RHICmdList, const FScreenPassPipelineState& ScreenPassDraw);

enum class EScreenPassDrawFlags : uint8
{
	None,

	// Allows the screen pass to use a HMD hidden area mask if one is available. Used for VR.
	AllowHMDHiddenAreaMask = 0x2
};
ENUM_CLASS_FLAGS(EScreenPassDrawFlags);

/** Type used to carry the limited amount of data we need from a FSceneView. */
struct FScreenPassViewInfo
{
	const ERHIFeatureLevel::Type FeatureLevel;
	const int32 StereoViewIndex;
	const int32 InstanceCount;
	const bool bHMDHiddenAreaMaskActive;

	FScreenPassViewInfo()
		: FScreenPassViewInfo(GMaxRHIFeatureLevel)
	{
	}

	FScreenPassViewInfo(ERHIFeatureLevel::Type InFeatureLevel)
		: FeatureLevel(InFeatureLevel)
		, StereoViewIndex(INDEX_NONE)
		, InstanceCount(1)
		, bHMDHiddenAreaMaskActive(IsHMDHiddenAreaMaskActive())
	{
	}

	RENDERER_API FScreenPassViewInfo(const FSceneView& View);

	RENDERER_API FScreenPassViewInfo(const FSceneView& View, const uint32 OverrideInstanceCount);
};

RENDERER_API void DrawScreenPass_PostSetup(
	FRHICommandList& RHICmdList,
	const FScreenPassViewInfo& ViewInfo,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& InputViewport,
	const FScreenPassPipelineState& PipelineState,
	EScreenPassDrawFlags Flags);

/** More advanced variant of screen pass drawing. Supports overriding blend / depth stencil
 *  pipeline state, and providing a custom vertex shader. Shader parameters are not bound by
 *  this method, instead the user provides a setup function that is called prior to draw, but
 *  after setting the PSO. This setup function should assign shader parameters.
 */
template<typename TSetupFunction>
void DrawScreenPass(
	FRHICommandList& RHICmdList,
	FScreenPassViewInfo ViewInfo,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& InputViewport,
	const FScreenPassPipelineState& PipelineState,
	EScreenPassDrawFlags Flags,
	TSetupFunction SetupFunction)
{
	PipelineState.Validate();

	const FIntRect OutputRect = OutputViewport.Rect;

	RHICmdList.SetViewport(OutputRect.Min.X, OutputRect.Min.Y, 0.0f, OutputRect.Max.X, OutputRect.Max.Y, 1.0f);

	SetScreenPassPipelineState(RHICmdList, PipelineState);

	SetupFunction(RHICmdList);

	DrawScreenPass_PostSetup(RHICmdList, ViewInfo, OutputViewport, InputViewport, PipelineState, Flags);
}

/** Render graph variant of simpler DrawScreenPass function. Clears graph resources unused by the
 *  pixel shader prior to adding the pass.
 */
template <typename PixelShaderType>
inline void AddDrawScreenPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& PassName,
	FScreenPassViewInfo ViewInfo,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& InputViewport,
	const TShaderRef<FShader>& VertexShader,
	const TShaderRef<PixelShaderType>& PixelShader,
	FRHIBlendState* BlendState,
	FRHIDepthStencilState* DepthStencilState,
	typename PixelShaderType::FParameters* PixelShaderParameters,
	EScreenPassDrawFlags Flags = EScreenPassDrawFlags::None)
{
	check(VertexShader.IsValid());
	check(PixelShader.IsValid());
	check(PixelShaderParameters);

	ClearUnusedGraphResources(PixelShader, PixelShaderParameters);

	const FScreenPassPipelineState PipelineState(VertexShader, PixelShader, BlendState, DepthStencilState);

	GraphBuilder.AddPass(
		Forward<FRDGEventName&&>(PassName),
		PixelShaderParameters,
		ERDGPassFlags::Raster,
		[ViewInfo, OutputViewport, InputViewport, PipelineState, PixelShader, PixelShaderParameters, Flags](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		DrawScreenPass(RHICmdList, ViewInfo, OutputViewport, InputViewport, PipelineState, Flags, [&](FRHICommandList&)
		{
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PixelShaderParameters);
		});
	});
}

template <typename PixelShaderType>
inline void AddDrawScreenPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& PassName,
	FScreenPassViewInfo ViewInfo,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& InputViewport,
	const TShaderRef<FShader>& VertexShader,
	const TShaderRef<PixelShaderType>& PixelShader,
	typename PixelShaderType::FParameters* PixelShaderParameters,
	EScreenPassDrawFlags Flags = EScreenPassDrawFlags::None)
{
	FRHIBlendState* BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
	FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();
	AddDrawScreenPass(GraphBuilder, Forward<FRDGEventName&&>(PassName), ViewInfo, OutputViewport, InputViewport, VertexShader, PixelShader, BlendState, DepthStencilState, PixelShaderParameters, Flags);
}

template <typename PixelShaderType>
inline void AddDrawScreenPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& PassName,
	FScreenPassViewInfo ViewInfo,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& InputViewport,
	const TShaderRef<FShader>& VertexShader,
	const TShaderRef<PixelShaderType>& PixelShader,
	FRHIBlendState* BlendState,
	typename PixelShaderType::FParameters* PixelShaderParameters,
	EScreenPassDrawFlags Flags = EScreenPassDrawFlags::None)
{
	FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();
	AddDrawScreenPass(GraphBuilder, Forward<FRDGEventName&&>(PassName), ViewInfo, OutputViewport, InputViewport, VertexShader, PixelShader, BlendState, DepthStencilState, PixelShaderParameters, Flags);
}

template <typename PixelShaderType>
inline void AddDrawScreenPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& PassName,
	FScreenPassViewInfo ViewInfo,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& InputViewport,
	const TShaderRef<FShader>& VertexShader,
	const TShaderRef<PixelShaderType>& PixelShader,
	FRHIDepthStencilState* DepthStencilState,
	typename PixelShaderType::FParameters* PixelShaderParameters,
	EScreenPassDrawFlags Flags = EScreenPassDrawFlags::None)
{
	FRHIBlendState* BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
	AddDrawScreenPass(GraphBuilder, Forward<FRDGEventName&&>(PassName), ViewInfo, OutputViewport, InputViewport, VertexShader, PixelShader, BlendState, DepthStencilState, PixelShaderParameters, Flags);
}

template <typename PixelShaderType>
inline void AddDrawScreenPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& PassName,
	FScreenPassViewInfo ViewInfo,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& InputViewport,
	const TShaderRef<PixelShaderType>& PixelShader,
	typename PixelShaderType::FParameters* PixelShaderParameters,
	EScreenPassDrawFlags Flags = EScreenPassDrawFlags::None)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(ViewInfo.FeatureLevel);
	TShaderMapRef<FScreenPassVS> VertexShader(ShaderMap);
	FRHIBlendState* BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
	FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();
	AddDrawScreenPass(GraphBuilder, Forward<FRDGEventName&&>(PassName), ViewInfo, OutputViewport, InputViewport, VertexShader, PixelShader, BlendState, DepthStencilState, PixelShaderParameters, Flags);
}

/** Render graph variant of more advanced DrawScreenPass function. Does *not* clear unused graph
 *  resources, since the parameters might be shared between the vertex and pixel shaders.
 */
template <typename TSetupFunction, typename TPassParameterStruct>
inline void AddDrawScreenPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& PassName,
	FScreenPassViewInfo ViewInfo,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& InputViewport,
	const FScreenPassPipelineState& PipelineState,
	TPassParameterStruct* PassParameterStruct,
	EScreenPassDrawFlags Flags,
	TSetupFunction SetupFunction)
{
	PipelineState.Validate();
	check(PassParameterStruct);

	GraphBuilder.AddPass(
		Forward<FRDGEventName&&>(PassName),
		PassParameterStruct,
		ERDGPassFlags::Raster,
		[ViewInfo, OutputViewport, InputViewport, PipelineState, SetupFunction, Flags] (FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		DrawScreenPass(RHICmdList, ViewInfo, OutputViewport, InputViewport, PipelineState, Flags, SetupFunction);
	});
}

template <typename TSetupFunction, typename TPassParameterStruct>
inline void AddDrawScreenPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& PassName,
	FScreenPassViewInfo ViewInfo,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& InputViewport,
	const FScreenPassPipelineState& PipelineState,
	TPassParameterStruct* PassParameterStruct,
	TSetupFunction SetupFunction)
{
	AddDrawScreenPass(GraphBuilder, Forward<FRDGEventName&&>(PassName), ViewInfo, OutputViewport, InputViewport, PipelineState, PassParameterStruct, EScreenPassDrawFlags::None, SetupFunction);
}

/** Helper function which copies a region of an input texture to a region of the output texture,
 *  with support for format conversion. If formats match, the method falls back to a simple DMA
 *  (CopyTexture); otherwise, it rasterizes using a pixel shader. Use this method if the two
 *  textures may have different formats.
 */
void RENDERER_API AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	FScreenPassViewInfo ViewInfo,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	FIntPoint InputPosition = FIntPoint::ZeroValue,
	FIntPoint OutputPosition = FIntPoint::ZeroValue,
	FIntPoint Size = FIntPoint::ZeroValue,
	FRHISamplerState* Sampler = nullptr);

void RENDERER_API AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	FScreenPassViewInfo ViewInfo,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	FIntPoint InputPosition,
	FIntPoint InputSize,
	FIntPoint OutputPosition,
	FIntPoint OutputSize,
	FRHISamplerState* Sampler = nullptr);

/** Helper variant which takes a shared viewport instead of unique input / output positions. */
inline void AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	FScreenPassViewInfo ViewInfo,
	FRDGTextureRef InputTexture,
	FRDGTextureRef OutputTexture,
	FIntRect ViewportRect)
{
	AddDrawTexturePass(GraphBuilder, ViewInfo, InputTexture, OutputTexture, ViewportRect.Min, ViewportRect.Min, ViewportRect.Size());
}

void RENDERER_API AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	FScreenPassViewInfo ViewInfo,
	FScreenPassTexture Input,
	FScreenPassRenderTarget Output,
	uint32 RTMultiviewCount = 1);

inline void AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	FScreenPassViewInfo ViewInfo,
	FScreenPassTexture Input,
	FScreenPassTexture Output)
{
	ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ELoad;
	if (Input.Texture->Desc.Extent == Output.Texture->Desc.Extent && Input.ViewRect == Output.ViewRect)
	{
		LoadAction = ERenderTargetLoadAction::ENoAction;
	}
	AddDrawTexturePass(GraphBuilder, ViewInfo, Input, FScreenPassRenderTarget(Output, LoadAction));
}

/** Variation of above that accepts a slice, which contains a texture SRV instead of a texture */
void RENDERER_API AddDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	FScreenPassViewInfo ViewInfo,
	FScreenPassTextureSlice Input,
	FScreenPassRenderTarget Output);

template <typename TFunction>
inline void AddRenderTargetPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& PassName,
	FScreenPassRenderTarget Output,
	TFunction&& Function)
{
	FRenderTargetParameters* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	GraphBuilder.AddPass(MoveTemp(PassName), PassParameters, ERDGPassFlags::Raster, MoveTemp(Function));
}

RENDERER_API FCanvas& CreateCanvasForRenderTarget(FRDGBuilder& GraphBuilder, const FSceneView& View, const FScreenPassRenderTarget& RenderTarget);

template <typename TFunction>
inline void AddDrawCanvasPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& PassName,
	const FSceneView& View,
	FScreenPassRenderTarget Output,
	TFunction Function)
{
	check(Output.IsValid());

	FCanvas& Canvas = CreateCanvasForRenderTarget(GraphBuilder, View, Output);

	Function(Canvas);

	const bool bFlush = false;
	Canvas.Flush_RenderThread(GraphBuilder, bFlush);
}

enum class EDownsampleDepthFilter
{
	// Produces a depth value that is not conservative but has consistent error (i.e. picks the sample).
	Point,

	// Produces a conservative max depth value.
	Max,

	// Produces a checkerboarded selection of min and max depth values
	Checkerboard,

	// Produces a color texture where R=min and G=max DeviceZ
	MinAndMaxDepth,

	// Produces a color texture where R=min and G=max DeviceZ from a color texture containing R=min and G=max DeviceZ
	MinAndMaxDepthFromMinAndMaxDepth
};

RENDERER_API void AddDownsampleDepthPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FScreenPassTexture Input,
	FScreenPassRenderTarget Output,
	EDownsampleDepthFilter DownsampleDepthFilter);

#include "ScreenPass.inl" // IWYU pragma: export
