// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

inline FIntPoint GetDownscaledExtent(FIntPoint Extent, FIntPoint Divisor)
{
	Extent = FIntPoint::DivideAndRoundUp(Extent, Divisor);
	Extent.X = FMath::Max(1, Extent.X);
	Extent.Y = FMath::Max(1, Extent.Y);
	return Extent;
}

inline FIntPoint GetScaledExtent(FIntPoint Extent, FVector2D Multiplier)
{
	Extent.X *= Multiplier.X;
	Extent.Y *= Multiplier.Y;
	Extent.X = FMath::Max(1, Extent.X);
	Extent.Y = FMath::Max(1, Extent.Y);
	return Extent;
}

inline FIntPoint GetScaledExtent(FIntPoint Extent, float Multiplier)
{
	return GetScaledExtent(Extent, FVector2D(Multiplier));
}

inline FIntRect GetDownscaledRect(FIntRect Rect, FIntPoint Divisor)
{
	Rect.Min /= Divisor;
	Rect.Max = GetDownscaledExtent(Rect.Max, Divisor);
	return Rect;
}

// The normal GetDownscaledRect can produce overlapping rects where multiple viewports are present, because the Min rounds down, while the
// Max rounds up.  This variation rounds up for the far edges of the viewport, but rounds down for everything else, avoiding this issue.
inline FIntRect GetDownscaledViewRect(FIntRect Rect, FIntPoint ViewExtent, FIntPoint Divisor)
{
	Rect.Min /= Divisor;
	Rect.Max.X = (Rect.Max.X == ViewExtent.X) ? FMath::DivideAndRoundUp(ViewExtent.X, Divisor.X) : Rect.Max.X / Divisor.X;
	Rect.Max.Y = (Rect.Max.Y == ViewExtent.Y) ? FMath::DivideAndRoundUp(ViewExtent.Y, Divisor.Y) : Rect.Max.Y / Divisor.Y;
	Rect.Max = Rect.Max.ComponentMax(Rect.Min + FIntPoint(1, 1));
	return Rect;
}

inline FIntRect GetScaledRect(FIntRect Rect, FVector2D Multiplier)
{
	Rect.Min.X *= Multiplier.X;
	Rect.Min.Y *= Multiplier.Y;
	Rect.Max = GetScaledExtent(Rect.Max, Multiplier);
	return Rect;
}

inline FIntRect GetScaledRect(FIntRect Rect, float Multiplier)
{
	return GetScaledRect(Rect, FVector2D(Multiplier));
}

inline FScreenPassTextureViewport GetDownscaledViewport(FScreenPassTextureViewport Viewport, FIntPoint Divisor)
{
	Viewport.Rect = GetDownscaledRect(Viewport.Rect, Divisor);
	Viewport.Extent = GetDownscaledExtent(Viewport.Extent, Divisor);
	return Viewport;
}

inline FScreenPassTextureViewport GetScaledViewport(FScreenPassTextureViewport Viewport, FVector2D Multiplier)
{
	Viewport.Rect = GetScaledRect(Viewport.Rect, Multiplier);
	Viewport.Extent = GetScaledExtent(Viewport.Extent, Multiplier);
	return Viewport;
}

inline FIntRect GetRectFromExtent(FIntPoint Extent)
{
	return FIntRect(FIntPoint::ZeroValue, Extent);
}

inline FScreenPassTexture::FScreenPassTexture(FRDGTextureRef InTexture)
	: Texture(InTexture)
{
	if (InTexture)
	{
		ViewRect.Max = InTexture->Desc.Extent;
	}
}

inline void FScreenPassTexture::UpdateVisualizeTextureExtent()
{
	if (Texture)
	{
		Texture->EncloseVisualizeExtent(ViewRect.Max);
	}
}

inline FScreenPassTexture::FScreenPassTexture(FRDGTextureRef InTexture, FIntRect InViewRect)
	: Texture(InTexture)
	, ViewRect(InViewRect)
{
	UpdateVisualizeTextureExtent();
}

inline FScreenPassTexture::FScreenPassTexture(const FScreenPassTextureSlice& ScreenTexture)
	: Texture(ScreenTexture.TextureSRV ? ScreenTexture.TextureSRV->Desc.Texture : nullptr)
	, ViewRect(ScreenTexture.ViewRect)
{
	if (Texture && Texture->Desc.IsTextureArray())
	{
		check(ScreenTexture.TextureSRV->Desc.FirstArraySlice == 0);
		check(ScreenTexture.TextureSRV->Desc.NumArraySlices == 1);
	}

	UpdateVisualizeTextureExtent();
}

inline bool FScreenPassTexture::IsValid() const
{
	return Texture != nullptr && !ViewRect.IsEmpty();
}

inline bool FScreenPassTexture::operator==(FScreenPassTexture Other) const
{
	return Texture == Other.Texture && ViewRect == Other.ViewRect;
}

inline bool FScreenPassTexture::operator!=(FScreenPassTexture Other) const
{
	return !(*this == Other);
}

inline FScreenPassTextureSlice::FScreenPassTextureSlice(FRDGTextureSRVRef InTextureSRV, FIntRect InViewRect)
	: TextureSRV(InTextureSRV)
	, ViewRect(InViewRect)
{
	if (TextureSRV && TextureSRV->Desc.Texture->Desc.IsTextureArray())
	{
		check(TextureSRV->Desc.NumArraySlices > 0);
	}
}

inline bool FScreenPassTextureSlice::IsValid() const
{
	return TextureSRV != nullptr && !ViewRect.IsEmpty();
}

inline bool FScreenPassTextureSlice::operator==(FScreenPassTextureSlice Other) const
{
	return TextureSRV == Other.TextureSRV && ViewRect == Other.ViewRect;
}

inline bool FScreenPassTextureSlice::operator!=(FScreenPassTextureSlice Other) const
{
	return !(*this == Other);
}

inline bool FScreenPassRenderTarget::operator==(FScreenPassRenderTarget Other) const
{
	return FScreenPassTexture::operator==(Other) && LoadAction == Other.LoadAction;
}

inline bool FScreenPassRenderTarget::operator!=(FScreenPassRenderTarget Other) const
{
	return !(*this == Other);
}

inline FRenderTargetBinding FScreenPassRenderTarget::GetRenderTargetBinding() const
{
	return FRenderTargetBinding(Texture, LoadAction);
}

inline FScreenPassTextureViewport::FScreenPassTextureViewport(FScreenPassTexture InTexture)
{
	check(InTexture.IsValid());
	Extent = InTexture.Texture->Desc.Extent;
	Rect = InTexture.ViewRect;
}

inline FScreenPassTextureViewport::FScreenPassTextureViewport(FScreenPassTextureSlice InTexture)
{
	check(InTexture.IsValid());
	Extent = InTexture.TextureSRV->Desc.Texture->Desc.Extent;
	Rect = InTexture.ViewRect;
}

inline bool FScreenPassTextureViewport::operator==(const FScreenPassTextureViewport& Other) const
{
	return Extent == Other.Extent && Rect == Other.Rect;
}

inline bool FScreenPassTextureViewport::operator!=(const FScreenPassTextureViewport& Other) const
{
	return Extent != Other.Extent || Rect != Other.Rect;
}

inline bool FScreenPassTextureViewport::IsEmpty() const
{
	return Extent == FIntPoint::ZeroValue || Rect.IsEmpty();
}

inline bool FScreenPassTextureViewport::IsFullscreen() const
{
	return Rect.Min == FIntPoint::ZeroValue && Rect.Max == Extent;
}

inline FVector2D FScreenPassTextureViewport::GetRectToExtentRatio() const
{
	return FVector2D((float)Rect.Width() / Extent.X, (float)Rect.Height() / Extent.Y);
}

inline FScreenTransform FScreenTransform::Invert(const FScreenTransform& AToB)
{
	ensure(!FMath::IsNearlyZero(AToB.Scale.X));
	ensure(!FMath::IsNearlyZero(AToB.Scale.Y));

	float InvX = 1.0f / AToB.Scale.X;
	float InvY = 1.0f / AToB.Scale.Y;

	return FScreenTransform(
		FVector2f(InvX, InvY),
		FVector2f(-InvX * AToB.Bias.X, -InvY * AToB.Bias.Y));
}


inline FVector2f operator * (const FVector2f& PInA, const FScreenTransform& AToB)
{
	return PInA * AToB.Scale + AToB.Bias;
}

inline FScreenTransform operator * (const FScreenTransform& AToB, const FVector2f& Scale)
{
	return FScreenTransform(AToB.Scale * Scale, AToB.Bias * Scale);
}

inline FScreenTransform operator * (const FScreenTransform& AToB, const float& Scale)
{
	return FScreenTransform(AToB.Scale * Scale, AToB.Bias * Scale);
}

inline FScreenTransform operator * (const FScreenTransform& AToB, const FIntPoint& Scale)
{
	return AToB * FVector2f(Scale.X, Scale.Y);
}

inline FScreenTransform operator * (const FScreenTransform& AToB, const FScreenTransform& BToC)
{
	return FScreenTransform(AToB.Scale * BToC.Scale, AToB.Bias * BToC.Scale + BToC.Bias);
}


inline FScreenTransform operator + (const FScreenTransform& AToB, const FVector2f& Bias)
{
	return FScreenTransform(AToB.Scale, AToB.Bias + Bias);
}

inline FScreenTransform operator + (const FScreenTransform& AToB, const float& Bias)
{
	return AToB + FVector2f(Bias, Bias);
}

inline FScreenTransform operator + (const FScreenTransform& AToB, const FIntPoint& Bias)
{
	return AToB + FVector2f(Bias.X, Bias.Y);
}


inline FScreenTransform operator - (const FScreenTransform& AToB, const FVector2f& Bias)
{
	return FScreenTransform(AToB.Scale, AToB.Bias - Bias);
}

inline FScreenTransform operator - (const FScreenTransform& AToB, const float& Bias)
{
	return AToB - FVector2f(Bias, Bias);
}

inline FScreenTransform operator - (const FScreenTransform& AToB, const FIntPoint& Bias)
{
	return AToB - FVector2f(Bias.X, Bias.Y);
}


inline FScreenTransform operator / (const FScreenTransform& AToB, const FVector2f& InvertedScale)
{
	ensure(!FMath::IsNearlyZero(InvertedScale.X));
	ensure(!FMath::IsNearlyZero(InvertedScale.Y));
	return AToB * FVector2f(1.0f / InvertedScale.X, 1.0f / InvertedScale.Y);
}

inline FScreenTransform operator / (const FScreenTransform& AToB, const float& InvertedScale)
{
	return AToB / FVector2f(InvertedScale, InvertedScale);
}

inline FScreenTransform operator / (const FScreenTransform& AToB, const FIntPoint& InvertedScale)
{
	return AToB / FVector2f(InvertedScale.X, InvertedScale.Y);
}

inline FScreenTransform operator / (const FScreenTransform& AToB, const FScreenTransform& CToB)
{
	FScreenTransform BToC = FScreenTransform::Invert(CToB);
	return AToB * BToC;
}


inline FScreenTransform FScreenTransform::ChangeTextureBasisFromTo(
	const FIntPoint& TextureExtent,
	const FIntRect& TextureViewport,
	FScreenTransform::ETextureBasis SrcBasis,
	FScreenTransform::ETextureBasis DestBasis)
{
	if (int32(SrcBasis) < int32(DestBasis))
	{
		if (int32(SrcBasis) + 1 != int32(DestBasis))
		{
			ETextureBasis IntermediaryBasis = ETextureBasis((int32(SrcBasis) + int32(DestBasis)) / 2);

			return (
				FScreenTransform::ChangeTextureBasisFromTo(TextureExtent, TextureViewport, SrcBasis, IntermediaryBasis) *
				FScreenTransform::ChangeTextureBasisFromTo(TextureExtent, TextureViewport, IntermediaryBasis, DestBasis));
		}
		else if (DestBasis == ETextureBasis::ViewportUV)
		{
			return FScreenTransform::ScreenPosToViewportUV;
		}
		else if (DestBasis == ETextureBasis::TexelPosition)
		{
			return FScreenTransform(
				FVector2f(TextureViewport.Width(), TextureViewport.Height()),
				FVector2f(TextureViewport.Min.X, TextureViewport.Min.Y));
		}
		else if (DestBasis == ETextureBasis::TextureUV)
		{
			return FScreenTransform(FVector2f(1.0f / float(TextureExtent.X), 1.0f / float(TextureExtent.Y)), FVector2f(0.0f, 0.0f));
		}
		else
		{
			check(0);
		}
	}
	else if (int32(SrcBasis) > int32(DestBasis))
	{
		if (int32(SrcBasis) != int32(DestBasis) + 1)
		{
			ETextureBasis IntermediaryBasis = ETextureBasis((int32(SrcBasis) + int32(DestBasis)) / 2);

			return (
				FScreenTransform::ChangeTextureBasisFromTo(TextureExtent, TextureViewport, SrcBasis, IntermediaryBasis) *
				FScreenTransform::ChangeTextureBasisFromTo(TextureExtent, TextureViewport, IntermediaryBasis, DestBasis));
		}
		else if (DestBasis == ETextureBasis::ScreenPosition)
		{
			return FScreenTransform::ViewportUVToScreenPos;
		}
		else if (DestBasis == ETextureBasis::ViewportUV)
		{
			float InvWidth = 1.0f / float(TextureViewport.Width());
			float InvHeight = 1.0f / float(TextureViewport.Height());

			return FScreenTransform(
				FVector2f(InvWidth, InvHeight),
				FVector2f(-InvWidth * TextureViewport.Min.X, -InvHeight * TextureViewport.Min.Y));
		}
		else if (DestBasis == ETextureBasis::TexelPosition)
		{
			return FScreenTransform(FVector2f(TextureExtent.X, TextureExtent.Y), FVector2f(0.0f, 0.0f));
		}
		else
		{
			check(0);
		}
	}

	return FScreenTransform::Identity;
}

inline FScreenTransform FScreenTransform::ChangeRectFromTo(
	FVector2f SourceOffset,
	FVector2f SourceExtent,
	FVector2f DestinationOffset,
	FVector2f DestinationExtent)
{
	FScreenTransform Transform;
	Transform.Scale = DestinationExtent / SourceExtent;
	Transform.Bias = DestinationOffset - Transform.Scale * SourceOffset;
	return Transform;
}

inline FScreenTransform FScreenTransform::ChangeRectFromTo(const FIntRect& SrcViewport, const FIntRect& DestViewport)
{
	return FScreenTransform::ChangeRectFromTo(
		FVector2f(SrcViewport.Min), FVector2f(SrcViewport.Size()),
		FVector2f(DestViewport.Min), FVector2f(DestViewport.Size()));
}

template<>
struct TShaderParameterTypeInfo<FScreenTransform>
{
	static constexpr EUniformBufferBaseType BaseType = UBMT_FLOAT32;
	static constexpr int32 NumRows = 1;
	static constexpr int32 NumColumns = 4;
	static constexpr int32 NumElements = 0;
	static constexpr int32 Alignment = 16;
	static constexpr bool bIsStoredInConstantBuffer = true;

	using TAlignedType = TAlignedTypedef<FScreenTransform, Alignment>::Type;

	static const FShaderParametersMetadata* GetStructMetadata() { return nullptr; }
};

inline FScreenPassTextureInput GetScreenPassTextureInput(FScreenPassTexture TexturePair, FRHISamplerState* Sampler)
{
	FScreenPassTextureInput Input;
	Input.Viewport = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(TexturePair));
	Input.Texture = TexturePair.Texture;
	Input.Sampler = Sampler;
	return Input;
}

inline FScreenPassTextureSliceInput GetScreenPassTextureInput(FScreenPassTextureSlice TexturePair, FRHISamplerState* Sampler)
{
	FScreenPassTextureSliceInput Input;
	Input.Viewport = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(TexturePair));
	Input.Texture = TexturePair.TextureSRV;
	Input.Sampler = Sampler;
	return Input;
}
