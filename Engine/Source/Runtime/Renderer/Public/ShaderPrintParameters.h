// Copyright Epic Games, Inc. All Rights Reserved.

// The ShaderPrint system uses a RWBuffer to capture any debug print from a shader.
// This means that the buffer needs to be bound for the shader you wish to debug.
// It would be ideal if that was automatic (maybe by having a fixed bind point for the buffer and binding it for the entire view).
// But for now you need to manually add binding information to your FShader class.
// To do this use SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters) in your FShader::FParameters declaration.
// Then call a variant of SetParameters().

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphResources.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameters.h"

class FRDGBuilder;
class FSceneView;
class FViewInfo;
class FSceneInterface;
struct FFrozenShaderPrintData;
struct FGlobalShaderPermutationParameters;
struct FShaderCompilerEnvironment;
struct FShaderPrintData;

namespace ShaderPrint
{
	// ShaderPrint uniform buffer layout
	BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FShaderPrintCommonParameters, RENDERER_API)
		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER(FIntPoint, CursorCoord)
		SHADER_PARAMETER(FVector3f, TranslatedWorldOffset)
		SHADER_PARAMETER(FVector2f, FontSize)
		SHADER_PARAMETER(FVector2f, FontSpacing)
		SHADER_PARAMETER(uint32, MaxCharacterCount)
		SHADER_PARAMETER(uint32, MaxSymbolCount)
		SHADER_PARAMETER(uint32, MaxStateCount)
		SHADER_PARAMETER(uint32, MaxLineCount)
		SHADER_PARAMETER(uint32, MaxTriangleCount)
		SHADER_PARAMETER(uint32, IsDrawLocked)
	END_GLOBAL_SHADER_PARAMETER_STRUCT()

	// ShaderPrint parameter struct declaration
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_STRUCT_REF(FShaderPrintCommonParameters, Common)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ShaderPrint_StateBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, ShaderPrint_RWEntryBuffer)
	END_SHADER_PARAMETER_STRUCT()

	// Does the platform support the ShaderPrint system?
	// Use this to create debug shader permutations only for supported platforms.
	RENDERER_API bool IsSupported(EShaderPlatform Platform);

	// Set any flags or defines needed when using ShaderPrint
	RENDERER_API void ModifyCompilationEnvironment(const EShaderPlatform Platform, FShaderCompilerEnvironment& OutEnvironment);
	RENDERER_API void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	// Have we enabled the ShaderPrint system?
	// Note that even when the ShaderPrint system is enabled, it may be disabled on any view due to platform support or view flags.
	RENDERER_API bool IsEnabled();

	// Force enable/disable the ShaderPrint system.
	RENDERER_API void SetEnabled(bool bInEnabled);

	// Returns true if the shader print data is valid for binding.
	// This should be checked before using with any ShaderPrint shader permutation.
	RENDERER_API bool IsValid(FShaderPrintData const& InShaderPrintData);

	// Returns true if the shader print data is enabled.
	// When the shader print data is valid but disabled then it can be used with any ShaderPrint shader permutation, but no data will be captured.
	// This can be checked to early out on any work that is only generating ShaderPrint data.
	RENDERER_API bool IsEnabled(FShaderPrintData const& InShaderPrintData);

	// Returns true if the default view exists and has valid shader print data.
	// This should be checked before using with any ShaderPrint shader permutation.
	RENDERER_API bool IsDefaultViewValid();

	// Returns true if the default view exists and shader print data that is enabled.
	// When the shader print data is valid but disabled then it can be used with any ShaderPrint shader permutation, but no data will be captured.
	// This can be checked to early out on any work that is only generating ShaderPrint data.
	RENDERER_API bool IsDefaultViewEnabled();

	/**
	 * Call to ensure enough space for some number of characters, is added cumulatively each frame, to make 
	 * it possible for several systems to request a certain number independently.
	 * Is used to grow the max element count for subsequent frames (as the allocation happens early in the frame).
	 * @param The number of characters requested.
	 */
	RENDERER_API void RequestSpaceForCharacters(uint32 MaxElementCount);

	/**
	* Call to ensure enough space for some number of lines segments, is added cumulatively each frame, to make 
	* it possible for several systems to request a certain number independently.
	* Is used to grow the max element count for subsequent frames (as the allocation happens early in the frame).
	* @param The number of line segments requested.
	*/
	RENDERER_API void RequestSpaceForLines(uint32 MaxElementCount);

	/**
	* Call to ensure enough space for some number of solid triangles, is added cumulatively each frame, to make 
	* it possible for several systems to request a certain number independently.
	* Is used to grow the max element count for subsequent frames (as the allocation happens early in the frame).
	* @param The number of triangles requested.
	*/
	RENDERER_API void RequestSpaceForTriangles(uint32 MaxElementCount);

	/** Structure containing setup for shader print capturing. */
	struct FShaderPrintSetup
	{
		/** Construct with shader print disabled setup. */
		FShaderPrintSetup() = default;
		/** Construct with view and system defaults. */
		RENDERER_API FShaderPrintSetup(FSceneView const& InView);
		/** Construct with view rectangle and system defaults. */
		RENDERER_API FShaderPrintSetup(FIntRect InViewRect);

		/** The shader print system's enabled state. This is set in the constructor and should't be overriden. */
		bool bEnabled = false;
		/** Expected viewport rectangle. */
		FIntRect ViewRect = FIntRect(0, 0, 1, 1);
		/** Cursor pixel position within viewport. Can be used for isolating a pixel to debug. */
		FIntPoint CursorCoord = FIntPoint(-1, -1);
		/** PreView translation used for storing line positions in translated world space. */
		FVector PreViewTranslation = FVector::ZeroVector;
		/** DPI scale to take into account when drawing font. */
		float DPIScale = 1.f;
		/** Font size in pixels. */
		FIntPoint FontSize = 1;
		/** Font spacing in pixels (not including font size). */
		FIntPoint FontSpacing = 1;
		/** Initial size of character buffer. Will also be increased by RequestSpaceForCharacters(). */
		uint32 MaxCharacterCount = 0;
		/** Initial size of widget buffer. */
		uint32 MaxStateCount = 0;
		/** Initial size of line buffer. Will also be increased by RequestSpaceForLines(). */
		uint32 MaxLineCount = 0;
		/** Initial size of triangle buffer. Will also be increased by RequestSpaceForLines(). */
		uint32 MaxTriangleCount = 0;
		// Whether current draw is locked or not. Useful to stop rendering new stuff on top of the history. */
		bool bIsDrawLocked = false;
	};

	/** Fill the ShaderPrintCommonParameters uniform buffer structure for our setup. */
	RENDERER_API void GetParameters(FShaderPrintSetup const& InSetup, FShaderPrintCommonParameters& OutParameters);

	/** Create the shader print render data. This allocates and clears the render buffers. */
	RENDERER_API FShaderPrintData CreateShaderPrintData(FRDGBuilder& GraphBuilder, FShaderPrintSetup const& InSetup);
	
	/** Make the buffers in a FShaderPrintData object external to an RDG builder. Do this for later reuse, or when submiting for later rendering. */
	RENDERER_API FFrozenShaderPrintData FreezeShaderPrintData(FRDGBuilder& GraphBuilder, FShaderPrintData& ShaderPrintData);
	/** Import the shader print buffers into an RDG builder and recreate the FShaderPrintData object. */
	RENDERER_API FShaderPrintData UnFreezeShaderPrintData(FRDGBuilder& GraphBuilder, FFrozenShaderPrintData& FrozenShaderPrintData);

	/** Submit shader print data for display in the next rendered frame. The data is displayed in views from the scene, or all views if InScene==nullptr. */
	RENDERER_API void SubmitShaderPrintData(FFrozenShaderPrintData& InData, FSceneInterface const* InScene);

	/** Fill the FShaderParameters with an explicit FShaderPrintData managed by the calling code. */
	RENDERER_API void SetParameters(FRDGBuilder& GraphBuilder, FShaderPrintData const& InData, FShaderParameters& OutParameters);
	/** Fill the FShaderParameters with the opaque FShaderPrintData from the current default view. */
	RENDERER_API void SetParameters(FRDGBuilder& GraphBuilder, FShaderParameters& OutParameters);
	
	// Experimental GPU string
	//
	// In Cpp:
	//   SHADER_PARAMETER_STRUCT(ShaderPrint::FStrings::FShaderPararameters, MyVariable)
	//   ShaderPrint::FStrings MyVariable;
	//	 MyVariable.Add(FString(...), StringId);
	//
	// In shader:
	//   FSTRINGS(MyVariable)
	//   void foo()
	//   {  
	//     InitShaderPrintContext Ctx = InitShaderPrintContext(...);
	//     PrintMyVariable(Ctx, StringId, FontWhite);
	//   }
	struct FStrings
	{
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
			SHADER_PARAMETER(uint32, InfoCount)
			SHADER_PARAMETER(uint32, CharCount)
			SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, InfoBuffer)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint8>, CharBuffer)
		END_SHADER_PARAMETER_STRUCT()

		RENDERER_API FStrings(uint32 InAvgEntryCount=128u, uint32 InAvgStringLength=32u);
		RENDERER_API void Add(const FString& In, uint32 EntryID);
		RENDERER_API void Add(const FString& In);
		RENDERER_API FShaderParameters GetParameters(FRDGBuilder& GraphBuilder);
	private:
		struct FEntryInfo
		{
			uint32 EntryID;
			uint16 Offset;
			uint8  Length;
			uint8  Pad0;
		};
		TArray<FEntryInfo> Infos;
		TArray<uint8> Chars;
	};
}

/** 
 * Structure containing shader print render data.
 * This is automatically created, setup and rendered for each view.
 * Also it is possible for client code to create and own this. 
 * If this is client managed then the client can queue for rendering by calling:
 * (i) FreezeShaderPrintData() to "freeze" the data which exports it from the current RDG builder context.
 * (ii) SubmitShaderPrintData() to submit the frozen data for later thawing and rendering.
 */
struct FShaderPrintData
{
	ShaderPrint::FShaderPrintSetup Setup;
	TUniformBufferRef<ShaderPrint::FShaderPrintCommonParameters> UniformBuffer;

	FRDGBufferRef ShaderPrintEntryBuffer = nullptr;
	FRDGBufferRef ShaderPrintStateBuffer = nullptr;
};

/**
 * Structure containing "frozen" shader print render data.
 * This is in a state so that it:
 * (i) Can be thawed by the client for continued gathering of shader print glyphs, or
 * (ii) Can be submitted for later rendering using SubmitShaderPrintData().
 */
struct FFrozenShaderPrintData
{
	ShaderPrint::FShaderPrintSetup Setup;

	TRefCountPtr<FRDGPooledBuffer> ShaderPrintEntryBuffer;
	TRefCountPtr<FRDGPooledBuffer> ShaderPrintStateBuffer;
};
