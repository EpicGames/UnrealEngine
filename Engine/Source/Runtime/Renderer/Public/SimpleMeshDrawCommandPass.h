// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SimpleMeshDrawCommandPass.h
=============================================================================*/

#pragma once

#include "MeshPassProcessor.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "RenderGraphBuilder.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "MeshMaterialShader.h"
#include "SceneUtils.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

class FGPUScene;
class FInstanceCullingDrawParams;

/**
 * Similar to the parallel one, but intended for use with simpler tasks where the overhead and complexity of parallel is not justified.
 */
class FSimpleMeshDrawCommandPass
{
public:
	/**
	 * bEnableStereo - if true will extract the stereo information from the View and set up two ViewIds for the instance culling, as well as use InstanceFactor = 2 for legacy drawing.
	 */
	RENDERER_API FSimpleMeshDrawCommandPass(const FSceneView& View, FInstanceCullingManager* InstanceCullingManager, bool bEnableStereo = false);

	/**
	 * Run post-instance culling job to create the render commands and instance ID lists and optionally vertex instance data.
	 * Needs to happen after DispatchPassSetup and before DispatchDraw, but not before global instance culling has been done.
	 */
	RENDERER_API void BuildRenderingCommands(FRDGBuilder& GraphBuilder, const FSceneView& View, const FGPUScene& GPUScene, FInstanceCullingDrawParams& OutInstanceCullingDrawParams);
	RENDERER_API void BuildRenderingCommands(FRDGBuilder& GraphBuilder, const FSceneView& View, const FScene& Scene, FInstanceCullingDrawParams& OutInstanceCullingDrawParams);
	RENDERER_API void SubmitDraw(FRHICommandList& RHICmdList, const FInstanceCullingDrawParams& InstanceCullingDrawParams) const;

	FDynamicPassMeshDrawListContext* GetDynamicPassMeshDrawListContext() { return &DynamicPassMeshDrawListContext; }

	EInstanceCullingMode GetInstanceCullingMode() const { return InstanceCullingContext.GetInstanceCullingMode(); }

private:
	FSimpleMeshDrawCommandPass() = delete;
	FSimpleMeshDrawCommandPass(const FSimpleMeshDrawCommandPass&) = delete;
	FMeshCommandOneFrameArray VisibleMeshDrawCommands;
	FInstanceCullingContext InstanceCullingContext;
	FGraphicsMinimalPipelineStateSet GraphicsMinimalPipelineStateSet;
	FDynamicMeshDrawCommandStorage DynamicMeshDrawCommandStorage;
	bool bNeedsInitialization = false;
	FDynamicPassMeshDrawListContext DynamicPassMeshDrawListContext;

	// Is set to true if and only if the BuildRenderingCommands has been called with an enabled GPU scene (which implies a valid Scene etc).
	// Is used to check that we don't submit any draw commands that require a GPU scene without supplying one.
	bool bSupportsScenePrimitives = false;

	bool bUsingStereo = false;
	bool bWasDrawCommandsSetup = false;

	uint32 InstanceFactor = 1;

	// GPUCULL_TODO: Only for legacy path
	FRHIBuffer* PrimitiveIdVertexBuffer = nullptr;
};

// GPUCULL_TODO: Write documentation
template <typename PassParametersType, typename AddMeshBatchesCallbackLambdaType, typename PassPrologueLambdaType>
void AddSimpleMeshPass(FRDGBuilder& GraphBuilder, PassParametersType* PassParametersIn, const FScene* Scene, const FSceneView &View, FInstanceCullingManager* InstanceCullingManager, FRDGEventName&& PassName,
	const ERDGPassFlags& PassFlags,
	AddMeshBatchesCallbackLambdaType AddMeshBatchesCallback,
	PassPrologueLambdaType PassPrologueCallback,
	bool bAllowIndirectArgsOverride=true)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AddSimpleMeshPass);

	// TODO: don't do this when the parameters are exclusive (optimization)
	PassParametersType* PassParameters = GraphBuilder.AllocParameters<PassParametersType>();

	// Copy parameters to ensure we can overwrite the InstanceCullingDrawParams
	*PassParameters = *PassParametersIn;

	FSimpleMeshDrawCommandPass* SimpleMeshDrawCommandPass = GraphBuilder.AllocObject<FSimpleMeshDrawCommandPass>(View, InstanceCullingManager);

	AddMeshBatchesCallback(SimpleMeshDrawCommandPass->GetDynamicPassMeshDrawListContext());

	// It is legal to render a simple mesh pass without a FScene, but then mesh draw commands must not reference primitive ID streams.
	if (Scene != nullptr)
	{
		SimpleMeshDrawCommandPass->BuildRenderingCommands(GraphBuilder, View, *Scene, PassParameters->InstanceCullingDrawParams);
	}

	if (!bAllowIndirectArgsOverride)
	{
		PassParameters->InstanceCullingDrawParams.DrawIndirectArgsBuffer = nullptr;
	}

	GraphBuilder.AddPass(
		MoveTemp(PassName),
		PassParameters,
		PassFlags,
		[SimpleMeshDrawCommandPass, PassParameters, PassPrologueCallback](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			PassPrologueCallback(RHICmdList);

			SimpleMeshDrawCommandPass->SubmitDraw(RHICmdList, PassParameters->InstanceCullingDrawParams);
		}
	);
}

// GPUCULL_TODO: Write documentation
template <typename PassParametersType, typename AddMeshBatchesCallbackLambdaType>
void AddSimpleMeshPass(FRDGBuilder& GraphBuilder, PassParametersType* PassParameters, const FScene* Scene, const FSceneView& View, FInstanceCullingManager *InstanceCullingManager, FRDGEventName&& PassName, const FIntRect& ViewPortRect,
	AddMeshBatchesCallbackLambdaType AddMeshBatchesCallback)
{
	AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, View, InstanceCullingManager, MoveTemp(PassName), ERDGPassFlags::Raster, AddMeshBatchesCallback,
		[ViewPortRect](FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(static_cast<float>(ViewPortRect.Min.X), static_cast<float>(ViewPortRect.Min.Y), 0.0f,
			                       static_cast<float>(ViewPortRect.Max.X), static_cast<float>(ViewPortRect.Max.Y), 1.0f);
		}
	);
}

template <typename PassParametersType, typename AddMeshBatchesCallbackLambdaType>
void AddSimpleMeshPass(FRDGBuilder& GraphBuilder, PassParametersType* PassParameters, const FScene* Scene, const FSceneView& View, FInstanceCullingManager *InstanceCullingManager, FRDGEventName&& PassName, const FIntRect& ViewPortRect, bool bAllowIndirectArgsOverride,
	AddMeshBatchesCallbackLambdaType AddMeshBatchesCallback)
{
	AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, View, InstanceCullingManager, MoveTemp(PassName), ERDGPassFlags::Raster, AddMeshBatchesCallback,
		[ViewPortRect](FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(static_cast<float>(ViewPortRect.Min.X), static_cast<float>(ViewPortRect.Min.Y), 0.0f,
			                       static_cast<float>(ViewPortRect.Max.X), static_cast<float>(ViewPortRect.Max.Y), 1.0f);
		},
		bAllowIndirectArgsOverride
	);
}

// GPUCULL_TODO: Write documentation
template <typename PassParametersType, typename AddMeshBatchesCallbackLambdaType>
void AddSimpleMeshPass(FRDGBuilder& GraphBuilder, PassParametersType* PassParameters, const FScene* Scene, const FSceneView& View, FInstanceCullingManager *InstanceCullingManager, FRDGEventName&& PassName, 
	const FIntRect& ViewPortRect, 
	const ERDGPassFlags &PassFlags,
	AddMeshBatchesCallbackLambdaType AddMeshBatchesCallback)
{
	AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, View, InstanceCullingManager, MoveTemp(PassName), PassFlags, AddMeshBatchesCallback,
		[ViewPortRect](FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(ViewPortRect.Min.X, ViewPortRect.Min.Y, 0.0f, ViewPortRect.Max.X, ViewPortRect.Max.Y, 1.0f);
		}
	);
}

// GPUCULL_TODO: Write documentation
template <typename PassParametersType, typename AddMeshBatchesCallbackLambdaType, typename PassPrologueLambdaType>
void AddSimpleMeshPass(FRDGBuilder& GraphBuilder, PassParametersType* PassParametersIn, const FGPUScene& GPUScene, const FSceneView& View, FInstanceCullingManager* InstanceCullingManager, FRDGEventName&& PassName,
	const ERDGPassFlags& PassFlags,
	AddMeshBatchesCallbackLambdaType AddMeshBatchesCallback,
	PassPrologueLambdaType PassPrologueCallback)
{
	// TODO: don't do this when the parameters are exclusive (optimization)
	PassParametersType* PassParameters = GraphBuilder.AllocParameters<PassParametersType>();

	// Copy parameters to ensure we can overwrite the InstanceCullingDrawParams
	*PassParameters = *PassParametersIn;

	FSimpleMeshDrawCommandPass* SimpleMeshDrawCommandPass = GraphBuilder.AllocObject<FSimpleMeshDrawCommandPass>(View, InstanceCullingManager);

	AddMeshBatchesCallback(SimpleMeshDrawCommandPass->GetDynamicPassMeshDrawListContext());

	SimpleMeshDrawCommandPass->BuildRenderingCommands(GraphBuilder, View, GPUScene, PassParameters->InstanceCullingDrawParams);

	GraphBuilder.AddPass(
		MoveTemp(PassName),
		PassParameters,
		PassFlags,
		[SimpleMeshDrawCommandPass, PassParameters, PassPrologueCallback](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			PassPrologueCallback(RHICmdList);

			SimpleMeshDrawCommandPass->SubmitDraw(RHICmdList, PassParameters->InstanceCullingDrawParams);
		}
	);
}

// GPUCULL_TODO: Write documentation
template <typename PassParametersType, typename AddMeshBatchesCallbackLambdaType>
void AddSimpleMeshPass(FRDGBuilder& GraphBuilder, PassParametersType* PassParameters, const FGPUScene& GPUScene, const FSceneView& View, FInstanceCullingManager* InstanceCullingManager, FRDGEventName&& PassName, const FIntRect& ViewPortRect,
	AddMeshBatchesCallbackLambdaType AddMeshBatchesCallback)
{
	AddSimpleMeshPass(GraphBuilder, PassParameters, GPUScene, View, InstanceCullingManager, MoveTemp(PassName), ERDGPassFlags::Raster, AddMeshBatchesCallback,
		[ViewPortRect](FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(ViewPortRect.Min.X, ViewPortRect.Min.Y, 0.0f, ViewPortRect.Max.X, ViewPortRect.Max.Y, 1.0f);
		}
	);
}

// GPUCULL_TODO: Write documentation
template <typename PassParametersType, typename AddMeshBatchesCallbackLambdaType>
void AddSimpleMeshPass(FRDGBuilder& GraphBuilder, PassParametersType* PassParameters, const FGPUScene& GPUScene, const FSceneView& View, FInstanceCullingManager* InstanceCullingManager, FRDGEventName&& PassName,
	const FIntRect& ViewPortRect,
	const ERDGPassFlags& PassFlags,
	AddMeshBatchesCallbackLambdaType AddMeshBatchesCallback)
{
	AddSimpleMeshPass(GraphBuilder, PassParameters, GPUScene, View, InstanceCullingManager, MoveTemp(PassName), PassFlags, AddMeshBatchesCallback,
		[ViewPortRect](FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(ViewPortRect.Min.X, ViewPortRect.Min.Y, 0.0f, ViewPortRect.Max.X, ViewPortRect.Max.Y, 1.0f);
		}
	);
}

#if 0

// GPUCULL_TODO: Write documentation
template <typename PassParametersType, typename AddMeshBatchesCallbackLambdaType, typename PassPrologueLambdaType>
void AddSimpleMeshPass(FRDGBuilder& GraphBuilder, PassParametersType* PassParameters, const FScene* Scene, const FSceneView& View, FRDGEventName&& PassName,
	AddMeshBatchesCallbackLambdaType AddMeshBatchesCallback)
{
	// TODO: ViewRect is only in FViewInfo
	AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, View, MoveTemp(PassName), View.ViewRect, AddMeshBatchesCallback);
}

#endif
