// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MeshBatch.h"
#include "MeshPassProcessor.h"

class FPrimitiveSceneInfo;

/**
 * A mesh which is defined by a primitive at scene segment construction time and never changed.
 * Lights are attached and detached as the segment containing the mesh is added or removed from a scene.
 */
class FStaticMeshBatch : public FMeshBatch
{
public:

	/** The render info for the primitive which created this mesh. */
	FPrimitiveSceneInfo* PrimitiveSceneInfo;

	/** The index of the mesh in the scene's static meshes array. */
	int32 Id;

	// Constructor/destructor.
	FStaticMeshBatch(FPrimitiveSceneInfo* InPrimitiveSceneInfo, const FMeshBatch& InMesh, FHitProxyId InHitProxyId);
	~FStaticMeshBatch();

private:
	/** Private copy constructor. */
	FStaticMeshBatch(const FStaticMeshBatch& InStaticMesh);
};

namespace DecalRendering { enum { DecalRenderTargetMode_NumBits = 6 }; }

/**
 * FStaticMeshBatch data which is InitViews specific. Stored separately for cache efficiency.
 */
class FStaticMeshBatchRelevance
{
public:

	FStaticMeshBatchRelevance(
		const FStaticMeshBatch& StaticMesh, 
		float InScreenSize, 
		bool InbSupportsCachingMeshDrawCommands, 
		bool InbUseSkyMaterial, 
		bool bInUseSingleLayerWaterMaterial, 
		bool bInUseAnisotropy, 
		bool bInSupportsNaniteRendering, 
		bool bInSupportsGPUScene, 
		bool bInUseForWaterInfoTextureDepth, 
		bool bInUseForLumenSceneCapture, 
		uint8 InDecalRenderTargetModeMask,
		ERHIFeatureLevel::Type FeatureLevel);

	int8 GetLODIndex() const { return bInvalidLODIndex ? -1 : int8(UnsignedLODIndex); }

	/** Starting offset into continuous array of command infos for this mesh in FPrimitiveSceneInfo::CachedMeshDrawCommandInfos. */
	FMeshPassMask CommandInfosMask;

	/** The index of the mesh in the scene's static meshes array. */
	int32 Id;

	/** The screen space size to draw this primitive at */
	float ScreenSize;

	/** Number of elements in this mesh. */
	uint16 NumElements;

	/* Every bit corresponds to one MeshPass. If bit is set, then FPrimitiveSceneInfo::CachedMeshDrawCommandInfos contains this mesh pass. */
	uint16 CommandInfosBase;

private:

	/** LOD index of the mesh, used for fading LOD transitions - unsigned and clamped for packing. */
	uint8 UnsignedLODIndex : 4;

	/** Original LOD index is negative */
	uint8 bInvalidLODIndex : 1;

public:

	/** LOD index of the mesh, used for fading LOD transitions. */
	UE_DEPRECATED(5.4, "Public LODIndex member is deprecated and doesn't contain valid data anymore! Use GetLODIndex() function instead.")
	int8 LODIndex : 1;

	/** Whether the mesh batch should apply dithered LOD. */
	uint8 bDitheredLODTransition : 1;

	/** Whether the mesh batch can be selected through editor selection, aka hit proxies. */
	uint8 bSelectable : 1;

	uint8 CastShadow : 1; // Whether it can be used in shadow renderpasses.
	uint8 bUseForMaterial : 1; // Whether it can be used in renderpasses requiring material outputs.
	uint8 bUseForDepthPass : 1; // Whether it can be used in depth pass.
	uint8 bUseAsOccluder : 1; // User hint whether it's a good occluder.
	uint8 bUseSkyMaterial : 1; // Whether this batch uses a Sky material or not.
	uint8 bUseSingleLayerWaterMaterial : 1; // Whether this batch uses a water material or not.
	uint8 bUseHairStrands : 1; // Whether it contains hair strands geometry.
	uint8 bUseAnisotropy : 1; // Whether material uses anisotropy parameter.
	uint8 bOverlayMaterial : 1; // Whether mesh uses overlay material.

	/** Whether the mesh batch can be used for rendering to a virtual texture. */
	uint8 bRenderToVirtualTexture : 1;
	/** What virtual texture material type this mesh batch should be rendered with. */
	uint8 RuntimeVirtualTextureMaterialType : RuntimeVirtualTexture::MaterialType_NumBits;

	/** Cached from vertex factory to avoid dereferencing VF in InitViews. */
	uint8 bSupportsCachingMeshDrawCommands : 1;

	/** Cached from vertex factory to avoid dereferencing VF in InitViews. */
	uint8 bSupportsNaniteRendering : 1;

	/** Cached from vertex factory to avoid dereferencing VF in shadow depth rendering. */
	uint8 bSupportsGPUScene : 1;

	/** Whether the mesh batch should be used in the depth-only passes of rendering the water info texture for the water plugin */
	uint8 bUseForWaterInfoTextureDepth : 1;

	/** Cached from lumen scene card capture */
	uint8 bUseForLumenSceneCapture : 1;

	/** What decal render target modes this mesh batch should be rendered to */
	uint8 DecalRenderTargetModeMask : DecalRendering::DecalRenderTargetMode_NumBits;

	/** Computes index of cached mesh draw command in FPrimitiveSceneInfo::CachedMeshDrawCommandInfos, for a given mesh pass. */
	int32 GetStaticMeshCommandInfoIndex(EMeshPass::Type MeshPass) const;
};
