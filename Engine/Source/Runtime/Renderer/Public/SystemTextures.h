// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SystemTextures.h: System textures definitions.
=============================================================================*/

#pragma once

#include "RenderGraph.h"

/** Contains system textures which can be registered for read-only access on an RDG pass. */
struct FRDGSystemTextures
{
	/** Call to initialize for the requested builder instance. */
	RENDERER_API static const FRDGSystemTextures& Create(FRDGBuilder& GraphBuilder);

	/** Returns the instance for the builder. Must be called after Create. */
	RENDERER_API static const FRDGSystemTextures& Get(FRDGBuilder& GraphBuilder);

	/** Returns whether the system textures have been created. */
	RENDERER_API static bool IsValid(FRDGBuilder& GraphBuilder);

	FRDGTextureRef White{};
	FRDGTextureRef Black{};
	FRDGTextureRef BlackAlphaOne{};
	FRDGTextureRef BlackArray{};
	FRDGTextureRef MaxFP16Depth{};
	FRDGTextureRef DepthDummy{};
	FRDGTextureRef StencilDummy{};
	FRDGTextureRef BlackDepthCube{};
	FRDGTextureRef Green{};
	FRDGTextureRef DefaultNormal8Bit{};
	FRDGTextureRef MidGrey{};
	FRDGTextureRef VolumetricBlack{};
	FRDGTextureRef VolumetricBlackAlphaOne{};
	FRDGTextureRef VolumetricBlackUint{};
	FRDGTextureRef CubeBlack{};
	FRDGTextureRef CubeArrayBlack{};
	FRDGTextureRef ZeroUIntDummy{};

	FRDGTextureSRVRef StencilDummySRV{};
};


struct FDefaultTextureKey
{
	uint32 ValueAsUInt[4] = { 0u,0u,0u,0u };
	EPixelFormat Format = PF_Unknown;
	ETextureDimension Dimension = ETextureDimension::Texture2D;
};
struct FDefaultTexture
{
	uint32 Hash = 0;
	FDefaultTextureKey Key;
	TRefCountPtr<IPooledRenderTarget> Texture;
};

enum class EDefaultBufferType
{
	VertexBuffer,
	StructuredBuffer,
	ByteAddressBuffer
};

struct FDefaultBufferKey
{
	uint32 ValueAsUInt[4] = { 0u,0u,0u,0u };
	uint32 NumBytePerElement = 0;
	EDefaultBufferType BufferType = EDefaultBufferType::VertexBuffer;
};
struct FDefaultBuffer
{
	uint32 Hash = 0;
	FDefaultBufferKey Key;
	TRefCountPtr<FRDGPooledBuffer> Buffer;
};

/**
 * Encapsulates the system textures used for scene rendering.
 */
class FSystemTextures : public FRenderResource
{
public:
	FSystemTextures()
		: FRenderResource()
		, FeatureLevelInitializedTo(ERHIFeatureLevel::Num)
	{}

	/**
	 * Initialize/allocate textures if not already.
	 */
	void InitializeTextures(FRHICommandListImmediate& RHICmdList, const ERHIFeatureLevel::Type InFeatureLevel);

	// FRenderResource interface.
	/**
	 * Release textures when device is lost/destroyed.
	 */
	virtual void ReleaseRHI();

	// -----------

	/**
		Any Textures added here MUST be explicitly released on ReleaseRHI()!
		Some RHIs need all their references released during destruction!
	*/

	// float4(1,1,1,1) can be used in case a light is not shadow casting
	TRefCountPtr<IPooledRenderTarget> WhiteDummy;
	// float4(0,0,0,0) can be used in additive postprocessing to avoid a shader combination
	TRefCountPtr<IPooledRenderTarget> BlackDummy;
	// float4(0,0,0,1)
	TRefCountPtr<IPooledRenderTarget> BlackAlphaOneDummy;
	// Dummy texture array with 1 black slice 
	TRefCountPtr<IPooledRenderTarget> BlackArrayDummy;
	// used by the material expression Noise
	TRefCountPtr<IPooledRenderTarget> PerlinNoiseGradient;
	// used by the material expression Noise (faster version, should replace old version), todo: move out of SceneRenderTargets
	TRefCountPtr<IPooledRenderTarget> PerlinNoise3D;
	// Sobol sampling texture, the first sample points for four sobol dimensions in RGBA
	TRefCountPtr<IPooledRenderTarget> SobolSampling;
	/** SSAO randomization */
	TRefCountPtr<IPooledRenderTarget> SSAORandomization;
	/** GTAO PreIntegrated */
	TRefCountPtr<IPooledRenderTarget> GTAOPreIntegrated;

	/** Preintegrated GF for single sample IBL */
	TRefCountPtr<IPooledRenderTarget> PreintegratedGF;
	/** Hair BSDF LUT texture */
	TRefCountPtr<IPooledRenderTarget> HairLUT0;
	TRefCountPtr<IPooledRenderTarget> HairLUT1;
	TRefCountPtr<IPooledRenderTarget> HairLUT2;
	/** GGX/Sheen Linearly Transformed Cosines LUTs */
	TRefCountPtr<IPooledRenderTarget> GGXLTCMat;
	TRefCountPtr<IPooledRenderTarget> GGXLTCAmp;
	TRefCountPtr<IPooledRenderTarget> SheenLTC;
	/** Texture that holds a single value containing the maximum depth that can be stored as FP16. */
	TRefCountPtr<IPooledRenderTarget> MaxFP16Depth;
	/** Depth texture that holds a single depth value */
	TRefCountPtr<IPooledRenderTarget> DepthDummy;
	/** Stencil texture that holds a single stencil value. */
	TRefCountPtr<IPooledRenderTarget> StencilDummy;
	TRefCountPtr<IPooledRenderTarget> BlackDepthCube;
	// float4(0,1,0,1)
	TRefCountPtr<IPooledRenderTarget> GreenDummy;
	// float4(0.5,0.5,0.5,1)
	TRefCountPtr<IPooledRenderTarget> DefaultNormal8Bit;
	// float4(0.5,0.5,0.5,0.5)
	TRefCountPtr<IPooledRenderTarget> MidGreyDummy;

	/** float4(0,0,0,0) volumetric texture. */
	TRefCountPtr<IPooledRenderTarget> VolumetricBlackDummy;
	TRefCountPtr<IPooledRenderTarget> VolumetricBlackAlphaOneDummy;
	TRefCountPtr<IPooledRenderTarget> VolumetricBlackUintDummy;

	/** float4(0,0,0,0) cube textures. */
	TRefCountPtr<IPooledRenderTarget> CubeBlackDummy;
	TRefCountPtr<IPooledRenderTarget> CubeArrayBlackDummy;

	// Dummy 0 Uint texture for RHIs that need explicit overloads
	TRefCountPtr<IPooledRenderTarget> ZeroUIntDummy;
	// Dummy 0 Uint texture for RHIs that need explicit overloads
	TRefCountPtr<IPooledRenderTarget> ZeroUIntArrayDummy;
    // Dummy 0 Uint texture for RHIs that need explicit overloads, specific version supporting atomics on Metal
    TRefCountPtr<IPooledRenderTarget> ZeroUIntArrayAtomicCompatDummy;

	// SRV for WhiteDummy Texture.
	TRefCountPtr<FRHIShaderResourceView> WhiteDummySRV;
	// SRV for StencilDummy Texture.
	TRefCountPtr<FRHIShaderResourceView> StencilDummySRV;

	// ASCII Standard character set - IBM code page 437 (character 32-127)
	TRefCountPtr<IPooledRenderTarget> AsciiTexture;

	// Space Filling Curve (used in FRandomSequence shader API)
	TRefCountPtr<IPooledRenderTarget> SpaceFillingCurve;

	// Create simple default texture
	FRDGTextureRef RENDERER_API GetWhiteDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetBlackDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetBlackArrayDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetZeroUIntDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetZeroUIntArrayDummy(FRDGBuilder& GraphBuilder) const;
    FRDGTextureRef RENDERER_API GetZeroUIntArrayAtomicCompatDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetZeroUShort4Dummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetBlackAlphaOneDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetMaxFP16Depth(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetDepthDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetStencilDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetGreenDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetDefaultNormal8Bit(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetMidGreyDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetVolumetricBlackDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetVolumetricBlackUintDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetCubeBlackDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetCubeArrayBlackDummy(FRDGBuilder& GraphBuilder) const;
	FRDGTextureRef RENDERER_API GetSpaceFillingCurveTexture(FRDGBuilder& GraphBuilder);

	// Create default 2D texture (1x1) with specific format and initialize value 
	FRDGTextureRef RENDERER_API GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, float Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, uint32 Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, const FVector3f& Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, const FVector4f& Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, const FUintVector4& Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture2D(FRDGBuilder& GraphBuilder, EPixelFormat Format, const FClearValueBinding& Value);

	// Create default 2D/3D/Cube/Array texture (1x1) with specific format and initialize value 
	FRDGTextureRef RENDERER_API GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, float Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, uint32 Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FVector2D& Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FIntPoint& Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FVector3f& Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FVector4f& Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FUintVector4& Value);
	FRDGTextureRef RENDERER_API GetDefaultTexture(FRDGBuilder& GraphBuilder, ETextureDimension Dimension, EPixelFormat Format, const FClearValueBinding& Value);

	// Create default buffer initialize to zero.
	FRDGBufferRef RENDERER_API GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement);
	FRDGBufferRef RENDERER_API GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement);
	FRDGBufferRef RENDERER_API GetDefaultByteAddressBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement);

	template <typename T>
	FRDGBufferRef GetDefaultBuffer(FRDGBuilder& GraphBuilder)
	{
		return GetDefaultBuffer(GraphBuilder, sizeof(T));
	}

	template <typename T>
	FRDGBufferRef GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder)
	{
		return GetDefaultStructuredBuffer(GraphBuilder, sizeof(T));
	}

	// Create a default buffer initialized with a reference element.
	FRDGBufferRef RENDERER_API GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, float Value);
	FRDGBufferRef RENDERER_API GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, uint32 Value);
	FRDGBufferRef RENDERER_API GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FVector3f& Value);
	FRDGBufferRef RENDERER_API GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FVector4f& Value);
	FRDGBufferRef RENDERER_API GetDefaultBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FUintVector4& Value);
	FRDGBufferRef RENDERER_API GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, float Value);
	FRDGBufferRef RENDERER_API GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, uint32 Value);
	FRDGBufferRef RENDERER_API GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FVector3f& Value);
	FRDGBufferRef RENDERER_API GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FVector4f& Value);
	FRDGBufferRef RENDERER_API GetDefaultStructuredBuffer(FRDGBuilder& GraphBuilder, uint32 NumBytePerElement, const FUintVector4& Value);


	// Experimental functions (non-public)
	void SetDefaultIfNull(FRDGBuilder& GraphBuilder, FRDGTextureRef& Out);
	void SetDefaultIfNull(FRDGBuilder& GraphBuilder, FRDGTextureUAVRef& Out);
	void SetDefaultIfNull(FRDGBuilder& GraphBuilder, TRDGBufferAccess<ERHIAccess::IndirectArgs>& Out);
	void SetDefaultIfNull(FRDGBuilder& GraphBuilder, FRDGBufferRef& Out);
	void SetDefaultIfNull(FRDGBuilder& GraphBuilder, FRDGBufferSRVRef& Out);
	void SetDefaultIfNull(FRDGBuilder& GraphBuilder, FRDGBufferUAVRef& Out);

protected:
	/** Maximum feature level that the textures have been initialized up to */
	ERHIFeatureLevel::Type FeatureLevelInitializedTo;
	bool bEngineDependentTexturesInitialized = false;

	/** Default textures allocated on-demand */
	TArray<FDefaultTexture> DefaultTextures;
	TArray<FDefaultBuffer> DefaultBuffers;
	FHashTable HashDefaultTextures;
	FHashTable HashDefaultBuffers;

	void InitializeCommonTextures(FRHICommandListImmediate& RHICmdList);
	void InitializeFeatureLevelDependentTextures(FRHICommandListImmediate& RHICmdList, const ERHIFeatureLevel::Type InFeatureLevel);
	void InitializeEngineDependentTextures(FRHICommandListImmediate& RHICmdList);
};

/** The global system textures used for scene rendering. */
RENDERER_API extern TGlobalResource<FSystemTextures> GSystemTextures;
