// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "SceneView.h"

namespace UE::Renderer::Private
{

/** Interface for implementing third party temporal upscaler. */
class ITemporalUpscaler : public ISceneViewFamilyExtention
{
public:

	/** Ref counted history to be saved in the history. */
	class IHistory : public IRefCountedObject
	{
	public:
		virtual ~IHistory() = default;

		/** Debug name of the history. Must exactly point to the same const TCHAR* as ITemporalUpscaler::GetDebugName().
		 * This is used for debugging GPU memory uses of a viewport, but also to ensure IHistory is fed to a compatible ITemporalUpscaler.
		 */
		virtual const TCHAR* GetDebugName() const = 0;

		/** Size of the history on the GPU in bytes. */
		virtual uint64 GetGPUSizeBytes() const = 0;
	};

	/** Inputs of the temporal upscaler. */
	struct FInputs
	{
		/** Outputs view rect that must be on FOutputs::FullRes::ViewRect. */
		FIntRect OutputViewRect;

		/** Pixel jitter offset of the rendering pixel. */
		FVector2f TemporalJitterPixels;

		/** Pre exposure of the SceneColor. */
		float PreExposure = 1.0f;

		/** The post-DOF and pre-MotionBlur SceneColor. */
		FScreenPassTexture SceneColor;

		/** The scene depth. */
		FScreenPassTexture SceneDepth;

		/** The scene velocity. */
		FScreenPassTexture SceneVelocity;

		/** Texture that contain eye adaptation on the red channel. */
		FRDGTextureRef EyeAdaptationTexture = nullptr;

		/** The history of the previous frame set by FOutputs::NewHistory. PrevHistory->GetDebugName() is guarentee to match the ITemporalUpscaler. */
		TRefCountPtr<IHistory> PrevHistory;
	};

	/** Outputs of the third party temporal upscaler. */
	struct FOutputs
	{
		/** Output of the temporal upscaler. FullRes.ViewRect must match FInputs::OutputViewRect. */
		FScreenPassTexture FullRes;

		/** New history to be kept alive for next frame. NewHistory->GetDebugName() must exactly point to the same const TCHAR* as ITemporalUpscaler::GetDebugName(). */
		TRefCountPtr<IHistory> NewHistory;
	};

	virtual ~ITemporalUpscaler() = default;

	/** Debug name of the history. Must exactly point to the same const TCHAR* as ITemporalUpscaler::IHistory::GetDebugName(). */
	virtual const TCHAR* GetDebugName() const = 0;

	/** Adds the necessary passes into RDG for temporal upscaling the rendering resolution to desired output res. */
	virtual FOutputs AddPasses(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FInputs& Inputs) const = 0;


	virtual float GetMinUpsampleResolutionFraction() const = 0;
	virtual float GetMaxUpsampleResolutionFraction() const = 0;

	virtual ITemporalUpscaler* Fork_GameThread(const class FSceneViewFamily& ViewFamily) const = 0;
};

} // namespace UE::Renderer::Private
