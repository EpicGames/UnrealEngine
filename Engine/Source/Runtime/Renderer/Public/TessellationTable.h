// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Math/IntVector.h"

namespace Nanite
{

class FTessellationTable
{
public:
	static constexpr uint32 BarycentricMax = 1 << 15;

	TArray< FUintVector2 >	OffsetTable;
	TArray< uint32 >		VertsAndIndexes;

public:
				FTessellationTable();
	 int32		GetPattern( FIntVector TessFactors ) const;

	uint32		GetNumVerts( int32 Pattern ) const;
	uint32		GetNumTris( int32 Pattern ) const;

private:
	FIntVector	GetBarycentrics( uint32 Vert ) const;
	void		SnapAtEdges( FIntVector& Barycentrics, const FIntVector& TessFactors ) const;

	void		AddPatch( const FIntVector& TessFactors );
	void		AddToVertsAndIndices( bool bImmediate, const FIntVector& TessFactors );

	void		ConstrainToCacheWindow();
	void		ConstrainForImmediateTessellation();

	void		WriteSVG( const FIntVector& TessFactors );

	TArray< uint32 >		Verts;
	TArray< uint32 >		Indexes;
};

inline uint32 FTessellationTable::GetNumVerts( int32 Pattern ) const
{
	return	OffsetTable[ Pattern + 1 ].X -
			OffsetTable[ Pattern ].X;
}

inline uint32 FTessellationTable::GetNumTris( int32 Pattern ) const
{
	return	OffsetTable[ Pattern + 1 ].Y -
			OffsetTable[ Pattern ].Y;
}

} // namespace Nanite