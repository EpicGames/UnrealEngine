// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalRenderResources.h"

inline FRHITexture* OrWhite2DIfNull(FRHITexture* Tex)
{
	FRHITexture* Result = Tex ? Tex : GWhiteTexture->TextureRHI.GetReference();
	check(Result);
	return Result;
}

inline FRHITexture* OrBlack2DIfNull(FRHITexture* Tex)
{
	FRHITexture* Result = Tex ? Tex : GBlackTexture->TextureRHI.GetReference();
	check(Result);
	return Result;
}

inline FRHITexture* OrBlack2DArrayIfNull(FRHITexture* Tex)
{
	FRHITexture* Result = Tex ? Tex : GBlackArrayTexture->TextureRHI.GetReference();
	check(Result);
	return Result;
}

inline FRHITexture* OrBlack3DIfNull(FRHITexture* Tex)
{
	// we fall back to 2D which are unbound mobile parameters
	return OrBlack2DIfNull(Tex ? Tex : GBlackVolumeTexture->TextureRHI.GetReference());
}

inline FRHITexture* OrBlack3DAlpha1IfNull(FRHITexture* Tex)
{
	// we fall back to 2D which are unbound mobile parameters
	return OrBlack2DIfNull(Tex ? Tex : GBlackAlpha1VolumeTexture->TextureRHI.GetReference());
}

inline FRHITexture* OrBlack3DUintIfNull(FRHITexture* Tex)
{
	// we fall back to 2D which are unbound mobile parameters
	return OrBlack2DIfNull(Tex ? Tex : GBlackUintVolumeTexture->TextureRHI.GetReference());
}

inline void SetBlack2DIfNull(FRHITexture*& Tex)
{
	if (!Tex)
	{
		Tex = GBlackTexture->TextureRHI.GetReference();
		check(Tex);
	}
}

inline void SetBlack3DIfNull(FRHITexture*& Tex)
{
	if (!Tex)
	{
		Tex = GBlackVolumeTexture->TextureRHI.GetReference();
		// we fall back to 2D which are unbound mobile parameters
		SetBlack2DIfNull(Tex);
	}
}

inline void SetBlackAlpha13DIfNull(FRHITexture*& Tex)
{
	if (!Tex)
	{
		Tex = GBlackAlpha1VolumeTexture->TextureRHI.GetReference();
		// we fall back to 2D which are unbound mobile parameters
		SetBlack2DIfNull(Tex); // This is actually a rgb=0, a=1 texture
	}
}
