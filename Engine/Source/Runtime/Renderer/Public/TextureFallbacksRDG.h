// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SystemTextures.h"

inline FRDGTextureRef OrWhite2DIfNull(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture)
{
	return Texture ? Texture : GSystemTextures.GetWhiteDummy(GraphBuilder);
}

inline FRDGTextureRef OrBlack2DIfNull(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture)
{
	return Texture ? Texture : GSystemTextures.GetBlackDummy(GraphBuilder);
}

inline FRDGTextureRef OrBlack2DArrayIfNull(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture)
{
	return Texture ? Texture : GSystemTextures.GetBlackArrayDummy(GraphBuilder);
}

inline FRDGTextureRef OrBlack3DIfNull(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture)
{
	return Texture ? Texture : GSystemTextures.GetVolumetricBlackDummy(GraphBuilder);
}

inline FRDGTextureRef OrBlack3DUintIfNull(FRDGBuilder& GraphBuilder, FRDGTextureRef Texture)
{
	return Texture ? Texture : GSystemTextures.GetVolumetricBlackUintDummy(GraphBuilder);
}

inline void SetBlack2DIfNull(FRDGBuilder& GraphBuilder, FRDGTextureRef& Texture)
{
	if (!Texture)
	{
		Texture = GSystemTextures.GetBlackDummy(GraphBuilder);
	}
}

inline void SetBlack3DIfNull(FRDGBuilder& GraphBuilder, FRDGTextureRef& Texture)
{
	if (!Texture)
	{
		Texture = GSystemTextures.GetVolumetricBlackDummy(GraphBuilder);
	}
}
