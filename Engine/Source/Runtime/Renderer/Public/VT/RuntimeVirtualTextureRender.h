// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIAccess.h"
#include "VT/RuntimeVirtualTextureEnum.h"

class FRHITexture;
class FRDGBuilder;
class FScene;
class FSceneInterface;
struct IPooledRenderTarget;
class ISceneRenderer;
class URuntimeVirtualTextureComponent;

namespace RuntimeVirtualTexture
{
	/** Enum for our maximum RenderPages() batch size. */
	enum { MaxRenderPageBatch = 8 };

	/** Structure containing a texture layer target description for a call for RenderPages(). */
	struct FRenderPageTarget
	{
		/** Physical target to render to. */
		IPooledRenderTarget* PooledRenderTarget = nullptr;

		UE_DEPRECATED(5.6, "Use PooledRenderTarget instead.")
		FRHITexture* Texture = nullptr;
		UE_DEPRECATED(5.6, "PooledRenderTarget tracks its own state.")
		ERHIAccess TextureAccessBefore = ERHIAccess::SRVMask;
		UE_DEPRECATED(5.6, "PooledRenderTarget tracks its own state.")
		ERHIAccess TextureAccessAfter = ERHIAccess::SRVMask;

		// Disable deprecation warnings for implicit copying of deprecated members.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FRenderPageTarget& operator=(FRenderPageTarget const& RHS) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	/** A single page description. Multiple of these can be placed in a single FRenderPageBatchDesc batch description. */
	struct FRenderPageDesc
	{
		/** vLevel to render at. */
		uint8 vLevel;
		/** UV range to render in virtual texture space. */
		FBox2D UVRange;
		/** Destination box to render in texel space of the target physical texture. */
		FIntRect DestRect[MaxTextureLayers];
		UE_DEPRECATED(5.6, "Use DestRect instead.")
		FBox2D DestBox[MaxTextureLayers];
	
		// Disable deprecation warnings for implicit copying of deprecated members.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FRenderPageDesc& operator=(FRenderPageDesc const& RHS) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	/** A description of a batch of pages to be rendered with a single call to RenderPages(). */
	struct FRenderPageBatchDesc
	{
		/** Scene to use when rendering the batch. */
		ISceneRenderer* SceneRenderer;
		UE_DEPRECATED(5.6, "Use SceneRenderer instead.")
		FScene* Scene;
		/** Unique object ID of the runtime virtual texture that we are rendering. */
		int32 RuntimeVirtualTextureId;
		UE_DEPRECATED(5.6, "Use RuntimeVirtualTextureId instead.")
		uint32 RuntimeVirtualTextureMask;
		/** Virtual texture UV space to world space transform. */
		FTransform UVToWorld;
		/** Virtual texture world space bounds. */
		FBox WorldBounds;
		/** Material type of the runtime virtual texture that we are rendering. */
		ERuntimeVirtualTextureMaterialType MaterialType;
		/** Max mip level of the runtime virtual texture that we are rendering. */
		uint8 MaxLevel;
		/** Set to true to clear before rendering. */
		bool bClearTextures;
		/** Set to true for thumbnail rendering. */
		bool bIsThumbnails;
		/** Fixed BaseColor to apply. Uses alpha channel to blend with material output. */
		FLinearColor FixedColor;
		/** CustomData that can be read in the material. */
		FVector4f CustomMaterialData;

		/** Physical texture targets to render to. */
		FRenderPageTarget Targets[MaxTextureLayers];
		
		/** Number of pages to render. */
		int32 NumPageDescs;
		/** Page descriptions for each page in the batch. */
		FRenderPageDesc PageDescs[MaxRenderPageBatch];

		// Disable deprecation warnings for implicit copying of deprecated members.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FRenderPageBatchDesc& operator=(FRenderPageBatchDesc const& RHS) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	/** Opaque class with context for rendering a batch of pages. */
	class FBatchRenderContext;

	/** Returns true if the scene is initialized for rendering to runtime virtual textures. Always check this before calling RenderPages(). */
	RENDERER_API bool IsSceneReadyToRender(FSceneInterface* Scene);

	/** Render a batch of pages for a runtime virtual texture. */
	RENDERER_API void RenderPages(FRDGBuilder& GraphBuilder, FRenderPageBatchDesc const& InDesc);

	/** Get a context for rendering a batch of pages. */
	FBatchRenderContext const* InitPageBatch(FRDGBuilder& GraphBuilder, FRenderPageBatchDesc const& InDesc);
	/** Do rendering for all pages in a context. */
	void RenderPageBatch(FRDGBuilder& GraphBuilder, FBatchRenderContext const& InBatch);
	/** Finalize all pages in a context to their final physical location. */
	void FinalizePageBatch(FRDGBuilder& GraphBuilder, FBatchRenderContext const& InBatch);

	UE_DEPRECATED(5.6, "Use RenderPages().")
	inline void RenderPagesStandAlone(FRDGBuilder& GraphBuilder, FRenderPageBatchDesc const& InDesc)
	{
		RenderPages(GraphBuilder, InDesc);
	}

#if WITH_EDITOR
	UE_DEPRECATED(5.6, "Use RuntimeVirtualTextureId everywhere that you would previously use a SceneIndex.")
	RENDERER_API uint32 GetRuntimeVirtualTextureSceneIndex_GameThread(URuntimeVirtualTextureComponent* InComponent);
#endif
}
