// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"

class FRDGBuffer;
class FRDGBuilder;
class FRHICommandList;

/** 
 * Description of how to interpret an RHIBuffer that is being fed to the virtual texture feedback system.
 */
struct FVirtualTextureFeedbackBufferDesc
{
	UE_DEPRECATED(5.6, "Directly set up members instead.")
	RENDERER_API void Init(int32 InBufferSize);
	UE_DEPRECATED(5.6, "All feedback buffers are 1D. Directly set up members instead.")
	RENDERER_API void Init2D(FIntPoint InBufferSize);
	UE_DEPRECATED(5.6, "All feedback buffers are 1D. Directly set up members instead.")
	RENDERER_API void Init2D(FIntPoint InUnscaledBufferSize, TArrayView<FIntRect> const& InUnscaledViewRects, int32 InBufferScale);

	/** Reserved number of feedback items in buffer. */
	uint32 BufferSize = 0;
	/** True if size of valid data in buffer is stored in the first item in the buffer. */
	bool bSizeInHeader = false;
	/** True if buffer made up of pairs of page id and count (stride 2). False if buffer is page ids only (stride 1). */
	bool bPageAndCount = false;
};

/** 
 * Submit an RHIBuffer containing virtual texture feedback data to the virtual texture system.
 * The buffer is internally copied to the CPU and parsed to determine which virtual texture pages need to be mapped.
 * RHIBuffers that are passed in are expected to have been transitioned to a state for reading.
 * Multiple buffers can be transferred per frame using this function.
 * The function can be called from the render thread only.
*/
RENDERER_API void SubmitVirtualTextureFeedbackBuffer(FRHICommandList& RHICmdList, FBufferRHIRef const& InBuffer, FVirtualTextureFeedbackBufferDesc const& InDesc);
RENDERER_API void SubmitVirtualTextureFeedbackBuffer(FRDGBuilder& GraphBuilder, FRDGBuffer* InBuffer, FVirtualTextureFeedbackBufferDesc const& InDesc);
