// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadingRateAttachment.h: Variable rate shading attachment generation class
	definitions.
=============================================================================*/

#pragma once

#include "Logging/LogMacros.h"
#include "RHI.h"
#include "RenderGraphDefinitions.h"
#include "RenderResource.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "RendererInterface.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

struct FMinimalSceneTextures; // Forward declaration from SceneTextures.

class FViewInfo;
class FSceneView;
class FSceneViewFamily;
struct IPooledRenderTarget;

class IVariableRateShadingImageGenerator;

DECLARE_LOG_CATEGORY_EXTERN(LogVRS, Log, All);

class FVariableRateShadingImageManager : public FRenderResource
{
public:
	/**
	 * Pass type used to determine requested image type based on current CVar settings.
	 */
	enum EVRSPassType
	{
		BasePass,
		TranslucencyAll,
		NaniteEmitGBufferPass,
		SSAO,
		SSR,
		ReflectionEnvironmentAndSky,
		LightFunctions,
		Decals,
		Num
	};

	/**
	 * Image type to request from generator. Only the CAS generator currently distinguishes between Full and Conservative
	 */
	enum EVRSImageType : uint32
	{
		Disabled = 0,
		Full = 1,
		Conservative = 2
	};

	/**
	 * Explicit VRS source flags; each type of VRS image generated here will have it's own flag defined in this bitfield.
	 * These flags can be used for any particular rendering path to exclude specific types of VRS generation; e.g.
	 * Used to allow exclusion of certain types when getting image.
	 */
	enum class EVRSSourceType : uint32
	{
		/** Fixed Foveation: Shading rate is decreased near the periphery of the viewport */
		FixedFoveation = 0x1,

		/** Eye-Tracked Foveation: Shading rate is decreased in the periphery of the user's gaze */
		EyeTrackedFoveation = 0x2,

		/** Contrast Adaptive Shading: Shading rate is decreased in areas of low contrast (calculated based on the prior frame) */
		ContrastAdaptiveShading = 0x4,

		// @todo: Add more types here as they're implemented.

		/** Mask of all available VRS source types. */
		All = FixedFoveation | EyeTrackedFoveation | ContrastAdaptiveShading,

		/** Mask of XR foveated VRS gen only. */
		XRFoveation = FixedFoveation | EyeTrackedFoveation,

		None = 0x0,
	};

	FVariableRateShadingImageManager();
	virtual ~FVariableRateShadingImageManager();

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	/**
	 * Get the combined VRS image for the specified view setup, target size/configuration, and application flags.
	 * May be run multiple times from different passes.
	 */
	FRDGTextureRef GetVariableRateShadingImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, EVRSPassType PassType, bool bRequestSoftwareImage = false);

	/**
	 * Prepare VRS images and store them for later access.
	 * Should be run exactly once in Render(), before attempting to get any VRS images for that frame.
	 */
	void PrepareImageBasedVRS(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const FMinimalSceneTextures& SceneTextures);

	/**
	 * Returns true if any generator among the given types is enabled, false otherwise.
	 */
	bool IsTypeEnabledForView(const FSceneView& View, EVRSSourceType Type);

	RENDERER_API void RegisterExternalImageGenerator(IVariableRateShadingImageGenerator* ExternalGenerator);
	RENDERER_API void UnregisterExternalImageGenerator(IVariableRateShadingImageGenerator* ExternalGenerator);

	static bool IsHardwareVRSSupported();
	static bool IsSoftwareVRSSupported();

	RENDERER_API bool IsPipelineVRSEnabled() const;
	RENDERER_API bool IsAttachmentVRSEnabled() const;

	RENDERER_API bool IsVRSEnabledForFrame()  const;
	RENDERER_API bool IsHardwareVRSEnabledForFrame() const;
	RENDERER_API bool IsSoftwareVRSEnabledForFrame() const;

	UE_DEPRECATED(5.5, "IsHardwareVRSEnabled() is deprecated, please use IsHardwareVRSEnabledForFrame() instead.")
	bool IsHardwareVRSEnabled() { return IsHardwareVRSEnabledForFrame(); };

	UE_DEPRECATED(5.5, "IsSoftwareVRSEnabled() is deprecated, please use IsSoftwareVRSEnabledForFrame() instead.")
	bool IsSoftwareVRSEnabled() { return IsSoftwareVRSEnabledForFrame(); };

	bool GetNeedStaticMeshUpdate() const;
	void SetNeedStaticMeshUpdate(bool bInNeedStaticMeshUpdate);

	static bool IsVRSCompatibleWithView(const FViewInfo& View);
	static bool IsVRSCompatibleWithOutputType(const EDisplayOutputFormat& DisplayOutputFormat);

	static FIntPoint GetSRITileSize(bool bSoftwareSize = false);
	static FRDGTextureDesc GetSRIDesc(const FSceneViewFamily& ViewFamily, bool bSoftwareSize = false);
	static int32 GetNumberOfSupportedRates();

	void DrawDebugPreview(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, FRDGTextureRef OutputSceneColor);

private:
	TRefCountPtr<IPooledRenderTarget> MobileHMDFixedFoveationOverrideImage;
	// Used to access the generators functionalities.
	TArray<IVariableRateShadingImageGenerator*> ImageGenerators;
	TArray<IVariableRateShadingImageGenerator*> ActiveGenerators;
	// This is used only to own the memory of generators created in the FVariableRateShadingImageManager constructor.
	TArray<TUniquePtr<IVariableRateShadingImageGenerator>> InternalGenerators;
	// Guards ImageGenerators because most of the calls of the manager are on the render thread but the 
	// Register/UnregisterExternalImageGenerator could come from the game thread.
	FRWLock	GeneratorsMutex;

	FRDGTextureRef CombineShadingRateImages(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, TArray<FRDGTextureRef> Sources);
	FRDGTextureRef GetForceRateImage(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, int RateIndex = 0, bool bGetSoftwareImage = false);

	UE_DEPRECATED(5.5, "This version of GetForceRateImage is deprecated, please use GetForceRateImage(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, int RateIndex, bool bGetSoftwareImage) instead.")
	FRDGTextureRef GetForceRateImage(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, int RateIndex = 0,
		EVRSImageType ImageType = EVRSImageType::Full, bool bGetSoftwareImage = false) { return GetForceRateImage(GraphBuilder, ViewFamily, RateIndex, bGetSoftwareImage); };

	EVRSImageType GetImageTypeFromPassType(EVRSPassType PassType);

	bool bHardwareVRSEnabledForFrame = false;
	bool bSoftwareVRSEnabledForFrame = false;
	int32 VRSForceRateForFrame = -1;

	bool bNeedStaticMeshUpdate = false;
};

ENUM_CLASS_FLAGS(FVariableRateShadingImageManager::EVRSSourceType);

RENDERER_API extern TGlobalResource<FVariableRateShadingImageManager> GVRSImageManager;

class IVariableRateShadingImageGenerator
{
public:
	virtual ~IVariableRateShadingImageGenerator() = default;

	// Returns cached VRS image.
	virtual FRDGTextureRef GetImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage = false) = 0;

	// Generates image(s) and saves to generator cache. Should only be run once per view per frame, in Render().
	virtual void PrepareImages(FRDGBuilder& GraphBuilder, const FSceneViewFamily& ViewFamily, const FMinimalSceneTextures& SceneTextures, bool bPrepareHardwareImages, bool bPrepareSoftwareImages) = 0;

	// Returns whether or not generator is enabled - can change at runtime
	virtual bool IsEnabled() const { return false; };

	// Returns whether or not the given view supports this generator
	virtual bool IsSupportedByView(const FSceneView& View) const { return false; };

	// Return bitmask of generator type
	virtual FVariableRateShadingImageManager::EVRSSourceType GetType() const { return FVariableRateShadingImageManager::EVRSSourceType::None; };

	// Get VRS image to be used w/ debug overlay
	virtual FRDGTextureRef GetDebugImage(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, FVariableRateShadingImageManager::EVRSImageType ImageType, bool bGetSoftwareImage = false) = 0;
};
