// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/ScopeRWLock.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveComponentId.h"
#include "Components/ComponentInterfaces.h"
#include "GameFramework/Actor.h"

#if !UE_BUILD_SHIPPING // TODO: Decide whether or not the struct should be entirely stripped out of shipping

class UMaterialInterface;
class FScene;
class FViewInfo;
class FViewCommands;

DECLARE_MULTICAST_DELEGATE(FOnUpdateViewDebugInfo);

/**
 * A collection of debug data associated with the current on screen view.
 */
struct FViewDebugInfo
{
	friend class FDrawPrimitiveDebuggerModule;

private:

	static RENDERER_API FViewDebugInfo Instance;

	RENDERER_API FViewDebugInfo();
	
public:

	/**
	 * Gets a reference to the view debug information that is used by the renderer.
	 * @returns The debug information that is used by the renderer.
	 */
	static inline FViewDebugInfo& Get()
	{
		return Instance;
	}

	/**
	 * Data collected about a single primitive being drawn to the screen.
	 */
	struct FPrimitiveInfo
	{
		TWeakObjectPtr<UObject> Owner;
		FPrimitiveComponentId ComponentId;
		IPrimitiveComponent* ComponentInterface;
		TWeakObjectPtr<UObject> ComponentUObject;
		FPrimitiveSceneInfo* PrimitiveSceneInfo;
		FString Name;
		FPrimitiveStats Stats;
		TArray<TWeakObjectPtr<UMaterialInterface>> Materials;
		TWeakObjectPtr<UMaterialInterface> OverlayMaterial;
		int32 LODAtLastCapture;

		bool operator<(const FPrimitiveInfo& Other) const
		{
			// Sort by name to group similar assets together, then by exact primitives so we can ignore duplicates
			const int32 NameCompare = Name.Compare(Other.Name);
			if (NameCompare != 0)
			{
				return NameCompare < 0;
			}

			return PrimitiveSceneInfo < Other.PrimitiveSceneInfo;
		}

		bool IsPrimitiveValid() const
		{
			bool bValid = true;
			bValid &= Owner.IsValid();
			bValid &= ComponentInterface != nullptr;
			if (bValid)
			{
				bValid &= ComponentUObject.IsValid();
				if (bValid)
				{
					bValid &= !ComponentInterface->IsUnreachable();
				}
			}
			return bValid;
		}

		inline bool HasLODs() const
		{
			return !Stats.LODStats.IsEmpty();
		}

		inline bool IsLODIndexValid(int32 LOD) const
		{
			return LOD >= 0 && LOD < Stats.LODStats.Num();
		}

		RENDERER_API int32 ComputeCurrentLODIndex(int32 PlayerIndex = 0, int32 ViewIndex = 0) const;

		inline FPrimitiveLODStats* GetCurrentLOD(int32 PlayerIndex = 0, int32 ViewIndex = 0)
		{
			int32 LOD = ComputeCurrentLODIndex(PlayerIndex, ViewIndex);
			if (!IsLODIndexValid(LOD)) LOD = LODAtLastCapture;
			return IsLODIndexValid(LOD) ? &Stats.LODStats[LOD] : nullptr;
		}

		inline const FPrimitiveLODStats* GetCurrentLOD(int32 PlayerIndex = 0, int32 ViewIndex = 0) const
		{
			int32 LOD = ComputeCurrentLODIndex(PlayerIndex, ViewIndex);
			if (!IsLODIndexValid(LOD)) LOD = LODAtLastCapture;
			return IsLODIndexValid(LOD) ? &Stats.LODStats[LOD] : nullptr;
		}

		inline FPrimitiveLODStats* GetLOD(int32 LOD)
		{
			return IsLODIndexValid(LOD) ? &Stats.LODStats[LOD] : nullptr;
		}

		inline const FPrimitiveLODStats* GetLOD(int32 LOD) const
		{
			return IsLODIndexValid(LOD) ? &Stats.LODStats[LOD] : nullptr;
		}

		inline UMaterialInterface* GetMaterial(uint16 Index) const
		{
			return Index < Materials.Num() ? Materials[Index].Get() : nullptr;
		}

		inline int32 GetNumLODs() const
		{
			return Stats.LODStats.Num();
		}

		inline FString GetOwnerName() const
		{
			if (const AActor* Actor = Cast<AActor>(Owner))
			{
				return Actor->GetHumanReadableName();
			}
			return ComponentInterface->GetOwnerName();
		}

		inline FVector GetPrimitiveLocation() const
		{
			return ComponentInterface->GetTransform().GetLocation();
		}
	};

private:

	bool bHasEverUpdated;
	bool bIsOutdated;
	bool bShouldUpdate;
	bool bShouldCaptureSingleFrame;
	bool bShouldClearCapturedData;

	FOnUpdateViewDebugInfo OnUpdate;

	mutable FRWLock Lock;
	
	TMap<FPrimitiveComponentId, FPrimitiveInfo> Primitives;

	RENDERER_API void ProcessPrimitive(FPrimitiveSceneInfo* PrimitiveSceneInfo, const FViewInfo& View, FScene* Scene, IPrimitiveComponent* DebugComponent);

	RENDERER_API void CaptureNextFrame();

	RENDERER_API void EnableLiveCapture();

	RENDERER_API void DisableLiveCapture();
	
	RENDERER_API void ClearCaptureData();

public:
	RENDERER_API void ProcessPrimitives(FScene* Scene, const FViewInfo& View, const FViewCommands& ViewCommands);

	/**
	 * Writes the draw call count of all currently tracked primitives to a csv file.
	 * The file will be stored in /Saved/Profiling/Primitives/...
	 */
	RENDERER_API void DumpDrawCallsToCSV();

	/**
	 * Writes detailed information about all currently tracked primitives to a csv file.
	 * The file will be stored in /Saved/Profiling/Primitives/...
	 */
	RENDERER_API void DumpToCSV() const;

	/**
	 * Performs an operation for each primitive currently tracked.
	 * @param Action The action to perform for each primitive.
	 */
	template <typename CallableT>
	void ForEachPrimitive(CallableT Action) const
	{
		const FPrimitiveSceneInfo* LastPrimitiveSceneInfo = nullptr;
		FRWScopeLock ScopeLock(Lock, SLT_ReadOnly);
		for (const auto& [PrimitiveId, Primitive] : Primitives)
		{
			if (Primitive.PrimitiveSceneInfo != LastPrimitiveSceneInfo)
			{
				Invoke(Action, Primitive);
				LastPrimitiveSceneInfo = Primitive.PrimitiveSceneInfo;
			}
		}
	}

	/**
	 * Checks if this debug information has ever been updated.
	 * @returns True if the information has been updated at least once.
	 */
	RENDERER_API bool HasEverUpdated() const;

	/**
	 * Checks if current information is from an older frame.
	 * @returns True if the data in this object is outdated.
	 */
	RENDERER_API bool IsOutOfDate() const;

	template <typename UserClass>
	FDelegateHandle AddUpdateHandler(UserClass* UserObject, void (UserClass::*Func)())
	{
		return OnUpdate.AddRaw(UserObject, Func);
	}

	FDelegateHandle AddUpdateHandler(void (*Func)())
	{
		return OnUpdate.AddStatic(Func);
	}

	void RemoveUpdateHandler(const FDelegateHandle& Handle)
	{
		OnUpdate.Remove(Handle);
	}
};
#endif
