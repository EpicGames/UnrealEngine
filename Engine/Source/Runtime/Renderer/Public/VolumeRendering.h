// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumeRendering.h: Volume rendering definitions.
=============================================================================*/

#pragma once

#include "Shader.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameters.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "ScreenRendering.h"

/** Represents a subregion of a volume texture. */
struct FVolumeBounds
{
	int32 MinX, MinY, MinZ;
	int32 MaxX, MaxY, MaxZ;

	FVolumeBounds() :
		MinX(0),
		MinY(0),
		MinZ(0),
		MaxX(0),
		MaxY(0),
		MaxZ(0)
	{}

	FVolumeBounds(int32 Max) :
		MinX(0),
		MinY(0),
		MinZ(0),
		MaxX(Max),
		MaxY(Max),
		MaxZ(Max)
	{}

	bool IsValid() const
	{
		return MaxX > MinX && MaxY > MinY && MaxZ > MinZ;
	}
};

/** Vertex shader used to write to a range of slices of a 3d volume texture. */
class FWriteToSliceVS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FWriteToSliceVS, ENGINE_API);
public:
	FWriteToSliceVS();
	FWriteToSliceVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FVolumeBounds& VolumeBounds, const FIntVector& VolumeResolution)
	{
		const float InvVolumeResolutionX = 1.0f / VolumeResolution.X;
		const float InvVolumeResolutionY = 1.0f / VolumeResolution.Y;
		SetShaderValue(BatchedParameters, UVScaleBias, FVector4f(
			(VolumeBounds.MaxX - VolumeBounds.MinX) * InvVolumeResolutionX,
			(VolumeBounds.MaxY - VolumeBounds.MinY) * InvVolumeResolutionY,
			VolumeBounds.MinX * InvVolumeResolutionX,
			VolumeBounds.MinY * InvVolumeResolutionY));
		SetShaderValue(BatchedParameters, MinZ, VolumeBounds.MinZ);
	}

private:
	LAYOUT_FIELD(FShaderParameter, UVScaleBias);
	LAYOUT_FIELD(FShaderParameter, MinZ);
};

/** Geometry shader used to write to a range of slices of a 3d volume texture. */
class FWriteToSliceGS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FWriteToSliceGS, ENGINE_API);
public:
	FWriteToSliceGS();
	FWriteToSliceGS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, int32 MinZValue)
	{
		SetShaderValue(BatchedParameters, MinZ, MinZValue);
	}

private:
	LAYOUT_FIELD(FShaderParameter, MinZ);
};

// This function assumes the PSO had a PrimitiveType of PT_TriangleStrip
extern ENGINE_API void RasterizeToVolumeTexture(FRHICommandList& RHICmdList, FVolumeBounds VolumeBounds);

/** Vertex buffer used for rendering into a volume texture. */
class FVolumeRasterizeVertexBuffer : public FVertexBuffer
{
public:
	ENGINE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

extern ENGINE_API TGlobalResource<FVolumeRasterizeVertexBuffer> GVolumeRasterizeVertexBuffer;