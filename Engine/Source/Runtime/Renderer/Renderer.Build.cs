// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Renderer : ModuleRules
{
	public Renderer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePaths.AddRange(
			new string[] {
				EngineDirectory + "/Shaders/Private", // For HaltonUtilities.ush
			}
		);

		PrivateIncludePathModuleNames.Add("Shaders");

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine",
			}
		);

        if (Target.bBuildEditor == true)
        {
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"TargetPlatform",
					"GeometryCore",
					"NaniteUtilities",
				}
			);
        }

        // Renderer module builds faster without unity
        // Non-unity also provides faster iteration
		// Not enabled by default as it might harm full rebuild times without XGE
        //bFasterWithoutUnity = true;

        MinFilesUsingPrecompiledHeaderOverride = 4;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"ApplicationCore",
				"RenderCore",
				"ImageWriteQueue",
				"RHI",
				"MaterialShaderQualitySettings",
				"StateStream",
				"TraceLog",
			}
		);

        PrivateIncludePathModuleNames.AddRange(new string[] { "HeadMountedDisplay" });
        DynamicallyLoadedModuleNames.AddRange(new string[] { "HeadMountedDisplay" });
		PrivateIncludePathModuleNames.AddRange(new string[] { "EyeTracker" });
		DynamicallyLoadedModuleNames.AddRange(new string[] { "EyeTracker" });

		RuntimeDependencies.Add("$(EngineDir)/Content/Renderer/TessellationTable.bin");
	}
}
